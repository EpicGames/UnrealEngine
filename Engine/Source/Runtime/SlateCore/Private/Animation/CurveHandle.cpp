// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"


#define LOCTEXT_NAMESPACE "FCurveHandle"


/* FCurveHandle constructors
 *****************************************************************************/

FCurveHandle::FCurveHandle( const struct FCurveSequence* InOwnerSequence, int32 InCurveIndex )
	: OwnerSequence(InOwnerSequence)
	, CurveIndex(InCurveIndex)
{ }


/* FCurveHandle interface
 *****************************************************************************/

float FCurveHandle::GetLerp( ) const
{
	if (OwnerSequence == nullptr)
	{
		return 0.0f;
	}

	// How far we've played through the curve sequence so far.
	const float CurSequenceTime = OwnerSequence->GetSequenceTime();

	const FCurveSequence::FSlateCurve& TheCurve = OwnerSequence->GetCurve(CurveIndex);
	const float TimeSinceStarted = CurSequenceTime - TheCurve.StartTime;

	// How far we passed through the current curve scaled between 0 and 1.
	const float Time = FMath::Clamp(TimeSinceStarted / TheCurve.DurationSeconds, 0.0f, 1.0f);

	return ApplyEasing(Time, TheCurve.EaseFunction);
}


/* FCurveHandle static functions
 *****************************************************************************/

float FCurveHandle::ApplyEasing( float Time, ECurveEaseFunction EaseFunction )
{
	// Currently we always use normalized distances
	const float Distance = 1.0f;

	// Currently we don't support custom start offsets;
	const float Start = 0.0f;
	float CurveValue = 0.0f;

	switch( EaseFunction )
	{
		case ECurveEaseFunction::Linear:
			CurveValue = Start + Distance * Time;
			break;

		case ECurveEaseFunction::QuadIn:
			CurveValue = Start + Distance * Time * Time;
			break;

		case ECurveEaseFunction::QuadOut:
			CurveValue = Start + -Distance * Time * (Time - 2.0f);
			break;

		case ECurveEaseFunction::QuadInOut:
			{
				if( Time < 0.5f )
				{
					const float Scaled = Time * 2.0f;
					CurveValue = Start + Distance * 0.5f * Scaled * Scaled;
				}
				else
				{
					const float Scaled = (Time - 0.5f) * 2.0f;
					CurveValue = Start + -Distance * 0.5f * (Scaled * (Scaled - 2.0f) - 1.0f);
				}
			}
			break;

		case ECurveEaseFunction::CubicIn:
			CurveValue = Start + Distance * Time * Time * Time;
			break;

		case ECurveEaseFunction::CubicOut:
			{
				const float Offset = Time - 1.0f;
				CurveValue = Start + Distance * (Offset * Offset * Offset + 1.0f);
			}
			break;

		case ECurveEaseFunction::CubicInOut:
			{
				float Scaled = Time * 2.0f;
				if (Scaled < 1.0f)
				{
					CurveValue = Start + Distance / 2.0f * Scaled * Scaled * Scaled;
				}
				else
				{
					Scaled -= 2.0f;
					CurveValue = Start + Distance / 2.0f * (Scaled * Scaled * Scaled + 2.0f);
				}
			}
			break;

		default:
			// Unrecognized curve easing function type
			checkf(0, TEXT("Unrecognized curve easing function type [%i] for FCurveHandle"), (int)EaseFunction);
			break;
	}

	return CurveValue;
}


#undef LOCTEXT_NAMESPACE
