// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animation/CurveSequence.h"
#include "Types/SlateEnums.h"
#include "Application/SlateApplicationBase.h"
#include "Widgets/SWidget.h"
#include "Stats/Stats.h"

/* FCurveSequence constructors
 *****************************************************************************/

FCurveSequence::FCurveSequence( )
	: StartTime(0.0)
	, PauseTime(0.0)
	, TotalDuration(0.0)
	, bInReverse(true)
	, bIsLooping(false)
	, bIsPaused(false)
{ }

FCurveSequence::FCurveSequence( const float InStartTimeSeconds, const float InDurationSeconds, const ECurveEaseFunction InEaseFunction )
	: StartTime(0.0)
	, PauseTime(0.0)
	, TotalDuration(0.0)
	, bInReverse(true)
	, bIsLooping( false )
	, bIsPaused( false )
{
	const FCurveHandle IgnoredCurveHandle = AddCurve( InStartTimeSeconds, InDurationSeconds, InEaseFunction );
}

/* FCurveSequence interface
 *****************************************************************************/

FCurveSequence::~FCurveSequence()
{
	// If the curve sequence is destroyed before the owning widget, unregister the active timer
	if ( OwnerWidget.IsValid() && ActiveTimerHandle.IsValid() )
	{
		OwnerWidget.Pin()->UnRegisterActiveTimer(ActiveTimerHandle.Pin().ToSharedRef());
	}

	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
	}
}

FCurveHandle FCurveSequence::AddCurve( const float InStartTimeSeconds, const float InDurationSeconds, const ECurveEaseFunction InEaseFunction )
{
	// Keep track of how long this sequence is
	TotalDuration = FMath::Max(TotalDuration, (double)(InStartTimeSeconds + InDurationSeconds));
	// The initial state is to be at the end of the animation.
	StartTime = TotalDuration;

	// Actually make this curve and return a handle to it.
	Curves.Add( FSlateCurve(InStartTimeSeconds, InDurationSeconds, InEaseFunction) );
	return FCurveHandle(this, Curves.Num() - 1);
}


FCurveHandle FCurveSequence::AddCurveRelative( const float InOffset, const float InDurationSecond, const ECurveEaseFunction InEaseFunction )
{
	const float CurveStartTime = (float)TotalDuration + InOffset;
	return AddCurve(CurveStartTime, InDurationSecond, InEaseFunction);
}

void FCurveSequence::Play(const TSharedRef<SWidget>& InOwnerWidget, bool bPlayLooped, const float StartAtTime, bool bRequiresActiveTimer)
{
	if (bRequiresActiveTimer)
	{
		RegisterActiveTimerIfNeeded(InOwnerWidget);
	}

	bIsLooping = bPlayLooped;
	bIsPaused = false;
	
	// Playing forward
	bInReverse = false;

	// We start playing NOW.
	SetStartTime(FSlateApplicationBase::Get().GetCurrentTime() - StartAtTime);
}

void FCurveSequence::Play(const FTickerDelegate& InDelegate, bool bPlayLooped, const float StartAtTime)
{
	bIsLooping = bPlayLooped;
	bIsPaused = false;

	// Playing forward
	bInReverse = false;

	// We start playing NOW.
	SetStartTime(FSlateApplicationBase::Get().GetCurrentTime() - StartAtTime);

	// Register the core ticker so we can start giving callbacks
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FCurveSequence::TickPlay, InDelegate));
}

void FCurveSequence::Reverse( )
{	
	// We've played this far into the animation.
	const double FractionCompleted = FMath::Clamp((double)GetSequenceTime() / TotalDuration, 0.0, 1.0);

	// We're going the other way now.
	bInReverse = !bInReverse;

	// CurTime is now; we cannot change that, so everything happens relative to CurTime.
	const double CurTime = FSlateApplicationBase::Get().GetCurrentTime();

	// Assume CurTime is constant (now).
	// Figure out when the animation would need to have started in order to keep
	// its place if playing in reverse.
	const double NewStartTime = CurTime - TotalDuration * (bInReverse ? (1 - FractionCompleted) : FractionCompleted);
	SetStartTime(NewStartTime);
}

void FCurveSequence::PlayReverse(const TSharedRef<SWidget>& InOwnerWidget, bool bPlayLooped, const float StartAtTime, bool bRequiresActiveTimer)
{
	if (bRequiresActiveTimer)
	{
		RegisterActiveTimerIfNeeded(InOwnerWidget);
	}
	bIsLooping = bPlayLooped;
	bIsPaused = false;

	bInReverse = true;

	// We start reversing NOW.
	SetStartTime(FSlateApplicationBase::Get().GetCurrentTime() - StartAtTime);
}

void FCurveSequence::Pause()
{
	if ( IsPlaying() )
	{
		bIsPaused = true;
		PauseTime = FSlateApplicationBase::Get().GetCurrentTime();
	}
}

void FCurveSequence::Resume()
{
	if ( bIsPaused )
	{
		// Make sure the widget that owns the sequence is still valid
		auto PinnedOwner = OwnerWidget.Pin();
		if ( PinnedOwner.IsValid() )
		{
			bIsPaused = false;
			RegisterActiveTimerIfNeeded( PinnedOwner.ToSharedRef() );

			// Update the start time to be the same relative to the current time as it was when paused
			const double NewStartTime = FSlateApplicationBase::Get().GetCurrentTime() - ( PauseTime - StartTime );
			SetStartTime( NewStartTime );
		}
	}
}

void FCurveSequence::PlayRelative(const TSharedRef<SWidget>& InOwnerWidget, bool bForward)
{
	if (IsPlaying() == false)
	{
		if (bForward == true)
		{
			if (IsAtStart())
			{
				Play(InOwnerWidget);
			}
		}
		else
		{
			if (IsAtEnd())
			{
				PlayReverse(InOwnerWidget);
			}
		}
	}
	else
	{
		if (IsForward() != bForward)
		{
			Reverse();
		}
	}
}

bool FCurveSequence::IsPlaying( ) const
{
	return !bIsPaused && ( bIsLooping || ( FSlateApplicationBase::Get().GetCurrentTime() - StartTime ) <= TotalDuration );
}


void FCurveSequence::SetStartTime( double InStartTime )
{
	StartTime = InStartTime;
}

float FCurveSequence::GetSequenceTime( ) const
{
	const double CurrentTime = bIsPaused ? PauseTime : FSlateApplicationBase::Get().GetCurrentTime();
	const float SequenceTime = IsInReverse() ? (float)(TotalDuration - (CurrentTime - StartTime)) : (float)(CurrentTime - StartTime);

	return bIsLooping ? FMath::Fmod( SequenceTime, (float)TotalDuration ) : SequenceTime;
}


bool FCurveSequence::IsInReverse( ) const
{
	return bInReverse;
}


bool FCurveSequence::IsForward( ) const
{
	return !bInReverse;
}


void FCurveSequence::JumpToStart( )
{
	bInReverse = true;
	bIsLooping = false;
	bIsPaused = false;
	SetStartTime(FSlateApplicationBase::Get().GetCurrentTime() - TotalDuration);
}


void FCurveSequence::JumpToEnd( )
{
	bInReverse = false;
	bIsLooping = false;
	bIsPaused = false;
	SetStartTime(FSlateApplicationBase::Get().GetCurrentTime() - TotalDuration);
}


bool FCurveSequence::IsAtStart( ) const
{
	return ( IsInReverse() == true && IsPlaying() == false && !bIsLooping );
}


bool FCurveSequence::IsAtEnd( ) const
{
	return ( IsForward() == true && IsPlaying() == false && !bIsLooping );
}

bool FCurveSequence::IsLooping() const
{
	return bIsLooping;
}

float FCurveSequence::GetLerp( ) const
{
	// Only supported for sequences with a single curve.  If you have multiple curves, use your FCurveHandle to compute
	// interpolation alpha values.
	checkSlow(Curves.Num() == 1);

	return FCurveHandle( this, 0 ).GetLerp();
}

const FCurveSequence::FSlateCurve& FCurveSequence::GetCurve( int32 CurveIndex ) const
{
	return Curves[CurveIndex];
}

void FCurveSequence::RegisterActiveTimerIfNeeded(TSharedRef<SWidget> InOwnerWidget)
{
	// Register the active timer
	if ( !ActiveTimerHandle.IsValid() )
	{
		ActiveTimerHandle = InOwnerWidget->RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateRaw(this, &FCurveSequence::EnsureSlateTickDuringAnimation));

		// Save the reference in case we need to take care of unregistering the tick
		OwnerWidget = InOwnerWidget;
	}
}

bool FCurveSequence::TickPlay(float InDeltaTime, FTickerDelegate InUserDelegate)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FCurveSequence_TickPlay);
	return (InUserDelegate.IsBound() && InUserDelegate.Execute(InDeltaTime)) && IsPlaying();
}

EActiveTimerReturnType FCurveSequence::EnsureSlateTickDuringAnimation( double InCurrentTime, float InDeltaTime )
{
	// Keep Slate ticking until play stops
	return IsPlaying() ? EActiveTimerReturnType::Continue : EActiveTimerReturnType::Stop;
}
