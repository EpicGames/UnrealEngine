// Copyright Epic Games, Inc. All Rights Reserved.

#include "Application/ActiveTimerHandle.h"
#include "Types/SlateEnums.h"


FActiveTimerHandle::FActiveTimerHandle(float InExecutionPeriod, FWidgetActiveTimerDelegate InTimerFunction, double InNextExecutionTime)
	: ExecutionPeriod(InExecutionPeriod)
	, TimerFunction(InTimerFunction)
	, NextExecutionTime(InNextExecutionTime)
	, bExecutionPending(false)
{
}

bool FActiveTimerHandle::IsPendingExecution() const
{
	return bExecutionPending;
}

bool FActiveTimerHandle::UpdateExecutionPendingState( double CurrentTime )
{
	bExecutionPending = IsReady( CurrentTime );
	return bExecutionPending;
}

EActiveTimerReturnType FActiveTimerHandle::ExecuteIfPending( double CurrentTime, float DeltaTime )
{
	if ( bExecutionPending )
	{
		bExecutionPending = false;

		// before we do anything, check validity of the delegate.
		if ( !TimerFunction.IsBound() )
		{
			// if the handle is no longer valid, we must assume it is invalid (say, owning widget was destroyed) and needs to be removed.
			return EActiveTimerReturnType::Stop;
		}

		// update time in a while loop so we skip any times that we may have missed due to slow frames.
		if ( ExecutionPeriod > 0.0 )
		{
			while ( CurrentTime >= NextExecutionTime )
			{
				NextExecutionTime += ExecutionPeriod;
			}
		}
		// timer is updated, now call the delegate
		return TimerFunction.Execute( CurrentTime, DeltaTime );
	}
	// if execution is not pending, then we need to continue until it is
	return EActiveTimerReturnType::Continue;
}

bool FActiveTimerHandle::IsReady(double CurrentTime) const
{
	return CurrentTime >= NextExecutionTime;
}
