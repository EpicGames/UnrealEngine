// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/WidgetActiveTimerDelegate.h"

enum class EActiveTimerReturnType : uint8;

/** Stores info about an active timer delegate for a widget. */
class FActiveTimerHandle
{
public:
	/** Public ctor that initializes a new active timer handle. Not intended to be called by user code. */
	FActiveTimerHandle(float InExecutionPeriod, FWidgetActiveTimerDelegate InTimerFunction, double InNextExecutionTime);

	/** @return True if the active timer is pending execution */
	bool IsPendingExecution() const;

	/**
	* Updates the pending state of the active timer based on the current time
	* @return True if the active timer is pending execution
	*/
	bool UpdateExecutionPendingState( double CurrentTime );

	/**
	 * Execute the bound delegate if the active timer is bPendingExecution.
	 * @return EActiveTimerReturnType::Stop if the handle should be auto-unregistered
	 *         (either the TimerFunction indicated this or the TimerFunction is unbound).
	 */
	EActiveTimerReturnType ExecuteIfPending( double CurrentTime, float DeltaTime );

private:
	/** @return true if the tick handle is ready to have its delegate executed. */
	bool IsReady( double CurrentTime ) const;

private:
	/** The period between executions of the timer delegate */
	float ExecutionPeriod;

	/** The delegate to the active timer function */
	FWidgetActiveTimerDelegate TimerFunction;
	
	/** The next time at which the TimerFunction will execute */
	double NextExecutionTime;

	/** True if execution of the TimerFunction is pending */
	bool bExecutionPending;
};


