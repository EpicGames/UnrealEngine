// Copyright Epic Games, Inc. All Rights Reserved.

#include "Application/SlateApplicationBase.h"
#include "Widgets/SWindow.h"
#include "Layout/WidgetPath.h"
#include "Application/ActiveTimerHandle.h"
#include "Misc/ScopeLock.h"
#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#endif


/* Static initialization
 *****************************************************************************/

TSharedPtr<FSlateApplicationBase> FSlateApplicationBase::CurrentBaseApplication = nullptr;
TSharedPtr<GenericApplication> FSlateApplicationBase::PlatformApplication = nullptr;
// TODO: Identifier the cursor index in a smarter way.
const uint32 FSlateApplicationBase::CursorPointerIndex = ETouchIndex::CursorPointerIndex;
const uint32 FSlateApplicationBase::CursorUserIndex = 0;
const FPlatformUserId FSlateApplicationBase::SlateAppPrimaryPlatformUser = FPlatformUserId::CreateFromInternalId(0);

FWidgetPath FHitTesting::LocateWidgetInWindow(FVector2f ScreenspaceMouseCoordinate, const TSharedRef<SWindow>& Window, bool bIgnoreEnabledStatus, int32 UserIndex) const
{
	return SlateApp->LocateWidgetInWindow(ScreenspaceMouseCoordinate, Window, bIgnoreEnabledStatus, UserIndex);
}


FSlateApplicationBase::FSlateApplicationBase()
: Renderer()
, HitTesting(this)
#if WITH_ACCESSIBILITY
, AccessibleMessageHandler(new FSlateAccessibleMessageHandler())
#endif
, bIsSlateAsleep(false)
, CustomSafeZoneState(ECustomSafeZoneState::Unset)
{
	FTextLocalizationManager::Get().OnTextRevisionChangedEvent.AddLambda([]()
	{
		if (FSlateApplicationBase::IsInitialized())
		{
			// Redraw widgets when new localized text data is loaded
			FSlateApplicationBase::Get().InvalidateAllWidgets(false);
		}
	});
}

void FSlateApplicationBase::GetDisplayMetrics(FDisplayMetrics& OutDisplayMetrics) 
{ 
	FDisplayMetrics::RebuildDisplayMetrics(OutDisplayMetrics); 
	CachedDisplayMetrics = OutDisplayMetrics;
	CachedDebugTitleSafeRatio = FDisplayMetrics::GetDebugTitleSafeZoneRatio();
}

void FSlateApplicationBase::GetCachedDisplayMetrics(FDisplayMetrics& OutDisplayMetrics) const
{
	OutDisplayMetrics = CachedDisplayMetrics;
}

void FSlateApplicationBase::GetSafeZoneSize(FMargin& SafeZone, const UE::Slate::FDeprecateVector2DParameter& OverrideSize)
{
	FVector2f ContainerSize = FVector2f::ZeroVector;

#if WITH_EDITOR
	ContainerSize = OverrideSize;
#endif

	if (ContainerSize.IsZero())
	{
		FDisplayMetrics Metrics;
		GetCachedDisplayMetrics(Metrics);
		ContainerSize = FVector2f((float)Metrics.PrimaryDisplayWidth, (float)Metrics.PrimaryDisplayHeight);
	}

	FMargin SafeZoneRatio;
	GetSafeZoneRatio(SafeZoneRatio);
	SafeZone.Left = SafeZoneRatio.Left * ContainerSize.X / 2.0f;
	SafeZone.Right = SafeZoneRatio.Right * ContainerSize.X / 2.0f;
	SafeZone.Top = SafeZoneRatio.Top * ContainerSize.Y / 2.0f;
	SafeZone.Bottom = SafeZoneRatio.Bottom * ContainerSize.Y / 2.0f;
}

void FSlateApplicationBase::GetSafeZoneRatio(FMargin& SafeZoneRatio)
{
	if (IsCustomSafeZoneSet())
	{
		SafeZoneRatio = CustomSafeZoneRatio;
	}
	else
	{
		FDisplayMetrics Metrics;
		GetCachedDisplayMetrics(Metrics);
		float HalfWidth = (Metrics.PrimaryDisplayWidth * 0.5f);
		float HalfHeight = (Metrics.PrimaryDisplayHeight * 0.5f);
		SafeZoneRatio = Metrics.TitleSafePaddingSize;
		SafeZoneRatio.Left /= HalfWidth;
		SafeZoneRatio.Top /= HalfHeight;
		SafeZoneRatio.Right /= HalfWidth;
		SafeZoneRatio.Bottom /= HalfHeight;
	}
}

const FHitTesting& FSlateApplicationBase::GetHitTesting() const
{
	return HitTesting;
}

void FSlateApplicationBase::RegisterActiveTimer( const TSharedRef<FActiveTimerHandle>& ActiveTimerHandle )
{
	FScopeLock ActiveTimerLock(&ActiveTimerCS);
	ActiveTimerHandles.Add(ActiveTimerHandle);
}

void FSlateApplicationBase::UnRegisterActiveTimer( const TSharedRef<FActiveTimerHandle>& ActiveTimerHandle )
{
	FScopeLock ActiveTimerLock(&ActiveTimerCS);
	ActiveTimerHandles.Remove(ActiveTimerHandle);
}

void FSlateApplicationBase::UpdateAnyActiveTimersArePending()
{
	FScopeLock ActiveTimerLock(&ActiveTimerCS);

	const double CurrentTime = GetCurrentTime();
	bool bAnyTickReady = false;
	for (auto It = ActiveTimerHandles.CreateIterator(); It; ++It)
	{
		auto ActiveTimerInfoPinned = (*It).Pin();

		// remove any tick handles that may have become invalid.
		// If we didn't remove invalid handles here, they would never get removed because
		// we don't force widgets to UnRegister before they are destroyed.
		if(!ActiveTimerInfoPinned.IsValid())
		{
			It.RemoveCurrent();
			continue;
		}

		check(ActiveTimerInfoPinned.IsValid());

		// If an active timer is still pending execution from last frame, it is collapsed 
		// or otherwise blocked from ticking. Disregard until it executes.
		if (ActiveTimerInfoPinned->IsPendingExecution())
		{
			continue;
		}

		if (ActiveTimerInfoPinned->UpdateExecutionPendingState(CurrentTime))
		{
			bAnyTickReady = true;
		}
	}

	bAnyActiveTimersPending = bAnyTickReady;
}

bool FSlateApplicationBase::IsSlateAsleep()
{
	return bIsSlateAsleep;
}

void FSlateApplicationBase::UpdateCustomSafeZone(const FMargin& NewSafeZoneRatio, bool bShouldRecacheMetrics)
{
	if (bShouldRecacheMetrics)
	{
		FDisplayMetrics DisplayMetrics;
		GetDisplayMetrics(DisplayMetrics);
	}
	CustomSafeZoneRatio = NewSafeZoneRatio;

	// Allow for a custom margin of zero when explictly set
	if (CustomSafeZoneState != ECustomSafeZoneState::Set)
	{
		CustomSafeZoneState = NewSafeZoneRatio == FMargin() ? ECustomSafeZoneState::Unset : ECustomSafeZoneState::Debug;
	}
}

#if WITH_EDITOR
void FSlateApplicationBase::SwapSafeZoneTypes()
{
	FDisplayMetrics DisplayMetrics;
	GetDisplayMetrics(DisplayMetrics);

	if (FDisplayMetrics::GetDebugTitleSafeZoneRatio() < 1.0f)
	{
		ResetCustomSafeZone();
		CustomSafeZoneState = ECustomSafeZoneState::Debug;
		OnDebugSafeZoneChanged.Broadcast(FMargin(), false);
	}
}
#endif // WITH_EDITOR

void FSlateApplicationBase::ResetCustomSafeZone()
{
	CustomSafeZoneRatio = FMargin();
	CustomSafeZoneState = ECustomSafeZoneState::Unset;
}

bool FSlateApplicationBase::IsCustomSafeZoneSet() const
{
	return CustomSafeZoneState == ECustomSafeZoneState::Set 
		|| (CustomSafeZoneState == ECustomSafeZoneState::Debug && CustomSafeZoneRatio != FMargin());
}

void FSlateApplicationBase::SetCustomSafeZone(const FMargin& InSafeZone)
{
	CustomSafeZoneRatio = InSafeZone;
	CustomSafeZoneState = ECustomSafeZoneState::Set;
}

void FSlateApplicationBase::ToggleGlobalInvalidation(bool bIsGlobalInvalidationEnabled)
{
	if (GSlateEnableGlobalInvalidation != bIsGlobalInvalidationEnabled)
	{
		GSlateEnableGlobalInvalidation = bIsGlobalInvalidationEnabled;
		OnGlobalInvalidationToggledEvent.Broadcast(bIsGlobalInvalidationEnabled);
	}
}

void FSlateApplicationBase::InvalidateAllWidgets(bool bClearResourcesImmediately) const
{
	// Only invalidate things if the thread triggering the invalidate owns slate rendering
	if (DoesThreadOwnSlateRendering())
	{
		SCOPED_NAMED_EVENT(Slate_GlobalInvalidate, FColor::Red);
		UE_LOGF(LogSlate, Log, "InvalidateAllWidgets triggered.  All widgets were invalidated");
		OnInvalidateAllWidgetsEvent.Broadcast(bClearResourcesImmediately);
	}
}
