// Copyright Epic Games, Inc. All Rights Reserved.

#include "Application/SlateWindowHelper.h"
#include "Layout/ArrangedChildren.h"
#include "SlateGlobals.h"
#include "Layout/WidgetPath.h"
#include "Types/SlateAttributeMetaData.h"


DECLARE_CYCLE_STAT( TEXT("FindPathToWidget"), STAT_FindPathToWidget, STATGROUP_Slate );


/* FSlateWindowHelper static functions
 *****************************************************************************/

void FSlateWindowHelper::ArrangeWindowToFront( TArray< TSharedRef<SWindow> >& Windows, const TSharedRef<SWindow>& WindowToBringToFront )
{
	Windows.Remove(WindowToBringToFront);

	if ((Windows.Num() == 0) || WindowToBringToFront->IsTopmostWindow())
	{
		Windows.Add(WindowToBringToFront);
	}
	else
	{
		bool PerformedInsert = false;

		int32 WindowIndex = Windows.Num() - 1;
		for (; WindowIndex >= 0; --WindowIndex)
		{
			// Topmost windows first, then non-regular windows (popups), then regular windows.
			if (!Windows[WindowIndex]->IsTopmostWindow() && (!WindowToBringToFront->IsRegularWindow() || Windows[WindowIndex]->IsRegularWindow()))
			{
				break;
			}
		}

		Windows.Insert(WindowToBringToFront, WindowIndex + 1);
	}
}


void FSlateWindowHelper::BringWindowToFront( TArray<TSharedRef<SWindow>>& Windows, const TSharedRef<SWindow>& BringMeToFront )
{
	const TSharedRef<SWindow> TopLevelWindowToReorder = BringToFrontInParent(BringMeToFront);
#if PLATFORM_MAC
	if (TopLevelWindowToReorder == BringMeToFront)
#endif
	{
		FSlateWindowHelper::ArrangeWindowToFront(Windows, TopLevelWindowToReorder);
	}
}


bool FSlateWindowHelper::CheckWorkAreaForWindows( const TArray<TSharedRef<SWindow>>& WindowsToSearch, const FSlateRect& WorkAreaRect )
{
	for (TArray<TSharedRef<SWindow>>::TConstIterator CurrentWindowIt(WindowsToSearch); CurrentWindowIt; ++CurrentWindowIt)
	{
		const TSharedRef<SWindow>& CurrentWindow = *CurrentWindowIt;
		const FVector2f Position = CurrentWindow->GetPositionInScreen();
		const FVector2f Size = CurrentWindow->GetSizeInScreen();
		const FSlateRect WindowRect(Position.X, Position.Y, Size.X, Size.Y);

		if (FSlateRect::DoRectanglesIntersect(WorkAreaRect, WindowRect) || CheckWorkAreaForWindows(CurrentWindow->GetChildWindows(), WorkAreaRect))
		{
			return true;
		}
	}

	return false;
}


bool FSlateWindowHelper::ContainsWindow( const TArray<TSharedRef<SWindow>>& WindowsToSearch, const TSharedRef<SWindow>& WindowToFind )
{
	if (WindowsToSearch.Contains(WindowToFind))
	{
		return true;
	}

	for (int32 WindowIndex=0; WindowIndex < WindowsToSearch.Num(); ++WindowIndex)
	{
		if (ContainsWindow(WindowsToSearch[WindowIndex]->GetChildWindows(), WindowToFind))
		{
			return true;
		}
	}

	return false;
}


bool FSlateWindowHelper::FindPathToWidget( const TArray<TSharedRef<SWindow>>& WindowsToSearch, TSharedRef<const SWidget> InWidget, FWidgetPath& OutWidgetPath, EVisibility VisibilityFilter )
{
	SCOPE_CYCLE_COUNTER(STAT_FindPathToWidget);

	if (GSlateFastWidgetPath)
	{
		// We have to internally cast this anyway - because the constructed widget path will be of non-const widgets, so if you'll end up with
		// a mutable copy anyway.
		TSharedPtr<SWidget> CurWidget = ConstCastSharedRef<SWidget>(InWidget);
		OutWidgetPath.Widgets.SetFilter(VisibilityFilter);
		while (true)
		{
			// Update the Widgets visibility before getting the ArrangeChildren
			FSlateAttributeMetaData::UpdateOnlyVisibilityAttributes(*CurWidget.Get(), FSlateAttributeMetaData::EInvalidationPermission::DelayInvalidation);
			EVisibility CurWidgetVisibility = CurWidget->GetVisibility();
			if (OutWidgetPath.Widgets.Accepts(CurWidgetVisibility))
			{
				FArrangedWidget ArrangedWidget(CurWidget.ToSharedRef(), CurWidget->GetCachedGeometry());
				OutWidgetPath.Widgets.AddWidget(CurWidgetVisibility, ArrangedWidget);

				TSharedPtr<SWidget> CurWidgetParent = CurWidget->GetParentWidget();
				if (!CurWidgetParent.IsValid())
				{
					if (CurWidget->Advanced_IsWindow())
					{
						OutWidgetPath.TopLevelWindow = StaticCastSharedPtr<SWindow>(CurWidget);
						OutWidgetPath.Widgets.Reverse();
						// The path is complete, we found the top window.
						return true;
					}

					// This widget doesn't have a parent, abandon the tree.
					OutWidgetPath.Widgets.Empty();
					return false;
				}

				// Even if the parent pointer is valid, and even if the visibility is visible,
				// it's possible a widget shows and hides children without ever removing them
				// this is the case with widgets like SWidgetSwitcher.
				if (!CurWidgetParent->ValidatePathToChild(CurWidget.Get()))
				{
					OutWidgetPath.Widgets.Empty();
					return false;
				}

				CurWidget = CurWidgetParent;
			}
			else
			{
				// We ran into a widget that isn't accepted.
				OutWidgetPath.Widgets.Empty();
				return false;
			}
		}
	}
	else
	{
		// Iterate over our top level windows
		bool bFoundWidget = false;

		for (int32 WindowIndex = 0; !bFoundWidget && WindowIndex < WindowsToSearch.Num(); ++WindowIndex)
		{
			// Make a widget path that contains just the top-level window
			TSharedRef<SWindow> CurWindow = WindowsToSearch[WindowIndex];

			FArrangedChildren JustWindow(VisibilityFilter);
			{
				JustWindow.AddWidget(FArrangedWidget(CurWindow, CurWindow->GetWindowGeometryInScreen()));
			}

			FWidgetPath PathToWidget(CurWindow, JustWindow);

			// Attempt to extend it to the desired child widget; essentially a full-window search for 'InWidget
			if ((CurWindow == InWidget) || PathToWidget.ExtendPathTo(FWidgetMatcher(InWidget), VisibilityFilter))
			{
				OutWidgetPath = PathToWidget;
				bFoundWidget = true;
			}

			if (!bFoundWidget)
			{
				// Search this window's children
				bFoundWidget = FindPathToWidget(CurWindow->GetChildWindows(), InWidget, OutWidgetPath, VisibilityFilter);
			}
		}

		return bFoundWidget;
	}
}


TSharedPtr<SWindow> FSlateWindowHelper::FindWindowByPlatformWindow( const TArray< TSharedRef<SWindow> >& WindowsToSearch, const TSharedRef< FGenericWindow >& PlatformWindow )
{
	for (int32 WindowIndex = 0; WindowIndex < WindowsToSearch.Num(); ++WindowIndex)
	{
		TSharedRef<SWindow> SomeWindow = WindowsToSearch[WindowIndex];
		TSharedRef<FGenericWindow> SomeNativeWindow = StaticCastSharedRef<FGenericWindow>(SomeWindow->GetNativeWindow().ToSharedRef());
		
		if (SomeNativeWindow == PlatformWindow)
		{
			return SomeWindow;
		}

		// Search child windows
		TSharedPtr<SWindow> FoundChildWindow = FindWindowByPlatformWindow(SomeWindow->GetChildWindows(), PlatformWindow);

		if (FoundChildWindow.IsValid())
		{
			return FoundChildWindow;
		}
	}

	return TSharedPtr<SWindow>(nullptr);
}


void FSlateWindowHelper::RemoveWindowFromList( TArray<TSharedRef<SWindow>>& Windows, const TSharedRef<SWindow>& WindowToRemove )
{
	int32 NumRemoved = Windows.Remove(WindowToRemove);

	if (NumRemoved == 0)
	{
		for (int32 ChildIndex=0; ChildIndex < Windows.Num(); ++ChildIndex)
		{
			RemoveWindowFromList(Windows[ChildIndex]->GetChildWindows(), WindowToRemove) ;
		}	
	}
}


/* FSlateWindowHelper implementation
 *****************************************************************************/

TSharedRef<SWindow> FSlateWindowHelper::BringToFrontInParent( const TSharedRef<SWindow>& WindowToBringToFront )
{
	const TSharedPtr<SWindow> ParentWindow = WindowToBringToFront->GetParentWindow();

	if (!ParentWindow.IsValid())
	{
		return WindowToBringToFront;
	}

	FSlateWindowHelper::ArrangeWindowToFront(ParentWindow->GetChildWindows(), WindowToBringToFront);

	return BringToFrontInParent(ParentWindow.ToSharedRef());
}
