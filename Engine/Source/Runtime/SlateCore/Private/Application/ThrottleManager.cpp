// Copyright Epic Games, Inc. All Rights Reserved.

#include "Application/ThrottleManager.h"


static int32 GSlateAllowThrottling = 1;
static FAutoConsoleVariableRef CVarSlateAllowThrottling(
	TEXT("Slate.bAllowThrottling"),
	GSlateAllowThrottling,
	TEXT("Allow Slate to throttle parts of the engine to ensure the UI is responsive"));


/* FSlateThrottleManager structors
 *****************************************************************************/

FSlateThrottleManager::FSlateThrottleManager( )
	: ThrottleCount(0)
	, DisableThrottleCount(0)
{ }


/* FSlateThrottleManager interface
 *****************************************************************************/

FThrottleRequest FSlateThrottleManager::EnterResponsiveMode( )
{
	// Increase the number of active throttle requests
	++ThrottleCount;

	// Create a new handle for the request and return it to the user so they can close it later
	FThrottleRequest NewHandle;
	NewHandle.Index = ThrottleCount;

	return NewHandle;
}


bool FSlateThrottleManager::IsAllowingExpensiveTasks( ) const
{
	// Expensive tasks are allowed if the number of active throttle requests is zero
	// or we always always allow it due to a CVar
	return (ThrottleCount == 0) || !GSlateAllowThrottling || DisableThrottleCount > 0;
}


void FSlateThrottleManager::LeaveResponsiveMode( FThrottleRequest& InHandle )
{
	if( InHandle.IsValid() )
	{
		// Decrement throttle count. If it becomes zero we are no longer throttling
		check(ThrottleCount > 0);
		--ThrottleCount;

		InHandle.Index = INDEX_NONE;
	}
}

void FSlateThrottleManager::DisableThrottle(bool bState)
{
	DisableThrottleCount += bState ? 1 : -1;
	ensure(DisableThrottleCount >= 0);
}



/* FSlateThrottleManager static functions
 *****************************************************************************/

FSlateThrottleManager& FSlateThrottleManager::Get( )
{
	static FSlateThrottleManager* Instance = nullptr;

	if (Instance == nullptr)
	{
		Instance = new FSlateThrottleManager;
	}

	return *Instance;
}
