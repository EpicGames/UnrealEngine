// Copyright Epic Games, Inc. All Rights Reserved.

#include "Brushes/SlateDynamicImageBrush.h"
#include "Application/SlateApplicationBase.h"


/* FSlateDynamicImageBrush structors
 *****************************************************************************/


TSharedPtr<FSlateDynamicImageBrush> FSlateDynamicImageBrush::CreateWithImageData(
	const FName InTextureName,
	const UE::Slate::FDeprecateVector2DParameter& InImageSize,
	const TArray<uint8>& InImageData,
	const FLinearColor& InTint,
	ESlateBrushTileType::Type InTiling,
	ESlateBrushImageType::Type InImageType)
{
	TSharedPtr<FSlateDynamicImageBrush> Brush;
	if (FSlateApplicationBase::IsInitialized() &&
		InImageSize.X > 0.f && InImageSize.Y > 0.f &&
		FSlateApplicationBase::Get().GetRenderer()->GenerateDynamicImageResource(InTextureName, (uint32)InImageSize.X, (uint32)InImageSize.Y, InImageData))
	{
		Brush = MakeShareable(new FSlateDynamicImageBrush(
			InTextureName,
			InImageSize,
			InTint,
			InTiling,
			InImageType));
	}
	return Brush;
}

void FSlateDynamicImageBrush::ReleaseResource()
{
	ReleaseResourceInternal();	
}

void FSlateDynamicImageBrush::ReleaseResourceInternal()
{
	if (bIsInitalized)
	{
		bIsInitalized = false;
		if (FSlateApplicationBase::IsInitialized())
		{
			ResourceHandle = FSlateResourceHandle();

			UObject* Object = GetResourceObject();
			// Brush resource is no longer referenced by this object
			if (Object && bRemoveResourceFromRootSet)
			{
				Object->RemoveFromRoot();
			}

			if (FSlateRenderer* Renderer = FSlateApplicationBase::Get().GetRenderer())
			{
				Renderer->ReleaseDynamicResource(*this);
			}

		}
	}
}

FSlateDynamicImageBrush::~FSlateDynamicImageBrush( )
{
	ReleaseResourceInternal();
}
