// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleSlateDebugger.h"

#if WITH_SLATE_DEBUGGING

#include "Debugging/SlateDebugging.h"
#include "Widgets/SBoxPanel.h"
#include "Modules/ModuleManager.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "HAL/PlatformStackWalk.h"
#include "Layout/WidgetPath.h"
#include "Misc/StringBuilder.h"
#include "Types/ReflectionMetadata.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/Class.h"

#define LOCTEXT_NAMESPACE "ConsoleSlateDebugger"

FConsoleSlateDebugger::FConsoleSlateDebugger()
	: bEnabled(false)
	, bLogWarning(true)
	, bLogInputEvent(true)
	, bLogFocusEvent(true)
	, bLogAttemptNavigationEvent(true)
	, bLogExecuteNavigationEvent(true)
	, bLogCaptureStateChangeEvent(true)
	, bLogCursorChangeEvent(true)
	, bCaptureStack(false)
	, bInputRoutingModeEnabled(false)
	, ProcessInputCounter(0)
	, RouteInputCounter(0)
	, EnabledInputEvents(false, (uint8)ESlateDebuggingInputEvent::MAX)
	, EnabledFocusEvents(false, (uint8)ESlateDebuggingFocusEvent::MAX)
	, StartDebuggingCommand(
		TEXT("SlateDebugger.Event.Start"),
		*LOCTEXT("StartDebugger", "Starts the debugger.").ToString(),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebugger::StartDebugging))
	, StopDebuggingCommand(
		TEXT("SlateDebugger.Event.Stop"),
		*LOCTEXT("StopDebugger", "Stops the debugger.").ToString(),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebugger::StopDebugging))
	, StartDebuggingCommandAlias(
		TEXT("SlateDebugger.Start"),
		*LOCTEXT("StartDebuggerAlias", "Alias to 'SlateDebugger.Event.Start'.").ToString(),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebugger::StartDebugging))
	, StopDebuggingCommandAlias(
		TEXT("SlateDebugger.Stop"),
		*LOCTEXT("StopDebuggerAlias", "Alias to 'SlateDebugger.Event.Stop'.").ToString(),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebugger::StopDebugging))
	, EnableLogWarning(
		TEXT("SlateDebugger.Event.LogWarning"),
		bLogWarning,
		*LOCTEXT("LogWarning", "Log warning events").ToString(),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebugger::OnLogVariablesChanged))
	, EnableLogInputEvent(
		TEXT("SlateDebugger.Event.LogInputEvent"),
		bLogInputEvent,
		*LOCTEXT("LogInputEvent", "Log input events").ToString(),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebugger::OnLogVariablesChanged))
	, EnableLogFocusEvent(
		TEXT("SlateDebugger.Event.LogFocusEvent"),
		bLogFocusEvent,
		*LOCTEXT("LogFocusEvent", "Log focus events").ToString(),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebugger::OnLogVariablesChanged))
	, EnableLogAttemptNavigationEvent(
		TEXT("SlateDebugger.Event.LogAttemptNavigationEvent"),
		bLogAttemptNavigationEvent,
		*LOCTEXT("LogAttemptNavigationEvent", "Log attempt navigation events").ToString(),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebugger::OnLogVariablesChanged))
	, EnableLogExecuteNavigationEvent(
		TEXT("SlateDebugger.Event.LogExecuteNavigationEvent"),
		bLogExecuteNavigationEvent,
		*LOCTEXT("LogExecuteNavigationEvent", "Log execute navigation events").ToString(),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebugger::OnLogVariablesChanged))
	, EnableLogCaptureStateChangeEvent(
		TEXT("SlateDebugger.Event.LogCaptureStateChangeEvent"),
		bLogCaptureStateChangeEvent,
		*LOCTEXT("LogCaptureStateChangeEvent", "Log cursor state change events").ToString(),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebugger::OnLogVariablesChanged))
	, EnableLogCursorChangeEvent(
		TEXT("SlateDebugger.Event.LogCursorChangeEvent"),
		bLogCursorChangeEvent,
		*LOCTEXT("LogCursorChangeEvent", "Log cursor change events").ToString(),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebugger::OnLogVariablesChanged))
	, CaptureStackVariable(
		TEXT("SlateDebugger.Event.CaptureStack"),
		bCaptureStack,
		*LOCTEXT("CaptureStack", "Should we capture the stack when there are events?").ToString())
	, EnableInputRoutingMode(
		TEXT("SlateDebugger.Event.InputRoutingModeEnabled"),
		bInputRoutingModeEnabled,
		*LOCTEXT("InputRoutingModeEnabled", "Should we output the route that an input event took?").ToString(),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebugger::OnLogVariablesChanged))
	, SetInputFilterCommand(
		TEXT("SlateDebugger.Event.SetInputFilter"),
		*LOCTEXT("SetInputFilter", "Enable or Disable specific input filters").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FConsoleSlateDebugger::SetInputFilter))
	, ClearInputFiltersCommand(
		TEXT("SlateDebugger.Event.DisableAllInputFilters"),
		*LOCTEXT("DisableAllInputFilters", "Disable all input filters").ToString(),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebugger::ClearInputFilters))
	, EnableInputFiltersCommand(
		TEXT("SlateDebugger.Event.EnableAllInputFilters"),
		*LOCTEXT("EnableAllInputFilters", "Enable all input filters").ToString(),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebugger::EnableInputFilters))
	, SetFocusFilterCommand(
		TEXT("SlateDebugger.Event.SetFocusFilter"),
		*LOCTEXT("SetFocusFilter", "Enable or Disable specific focus filters").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FConsoleSlateDebugger::SetFocusFilter))
	, ClearFocusFiltersCommand(
		TEXT("SlateDebugger.Event.DisableAllFocusFilters"),
		*LOCTEXT("DisableAllFocusFilters", "Disable all focus filters").ToString(),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebugger::ClearFocusFilters))
	, EnableFocusFiltersCommand(
		TEXT("SlateDebugger.Event.EnableAllFocusFilters"),
		*LOCTEXT("EnableAllFocusFilters", "Enable all focus filters").ToString(),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebugger::EnableFocusFilters))
{
	//EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::MouseMove] = true;
	//EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::MouseEnter] = true;
	//EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::MouseLeave] = true;
	//EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::PreviewMouseButtonDown] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::MouseButtonDown] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::MouseButtonUp] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::MouseButtonDoubleClick] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::MouseWheel] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::TouchStart] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::TouchEnd] = true;
	//EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::TouchForceChanged] = true;
	//EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::TouchFirstMove] = true;
	//EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::TouchMoved] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::DragDetected] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::DragEnter] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::DragLeave] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::DragOver] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::DragDrop] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::DropMessage] = true;
	//EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::PreviewKeyDown] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::KeyDown] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::KeyUp] = true;
	//EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::KeyChar] = true;
	//EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::AnalogInput] = true;
	EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::TouchGesture] = true;
	//EnabledInputEvents[(uint8)ESlateDebuggingInputEvent::MotionDetected] = true;

	EnabledFocusEvents[(uint8)ESlateDebuggingFocusEvent::FocusChanging] = true;
	//EnabledFocusEvents[(uint8)ESlateDebuggingFocusEvent::FocusLost] = true;
	//EnabledFocusEvents[(uint8)ESlateDebuggingFocusEvent::FocusReceived] = true;
}

FConsoleSlateDebugger::~FConsoleSlateDebugger()
{
	RemoveListeners();
}

void FConsoleSlateDebugger::StartDebugging()
{
	UE_LOGF(LogSlateDebugger, Log, "Start Slate Debugger");
	bEnabled = true;
	UpdateListeners();
}

void FConsoleSlateDebugger::StopDebugging()
{
	UE_LOGF(LogSlateDebugger, Log, "Stop Slate Debugger");
	bEnabled = false;
	RemoveListeners();
}

void FConsoleSlateDebugger::OnLogVariablesChanged(IConsoleVariable*)
{
	UpdateListeners();
}

namespace SlateDebuggerUtils
{
	void SetEventFilter(const TArray< FString >& Params, const UEnum* EventEnum, TBitArray<>& Bitfield)
	{
		if (Params.Num() == 0)
		{
			// log the enabled values
			const int32 NumEnums = EventEnum->NumEnums();
			for (int32 Index = 0; Index < NumEnums; ++Index)
			{
				const int32 EnumValue = (int32)EventEnum->GetValueByIndex(Index);
				if (Bitfield.IsValidIndex(EnumValue))
				{
					const TCHAR* FilterValue = Bitfield[uint8(EnumValue)] ? TEXT("True") : TEXT("False");
					UE_LOGF(LogSlateDebugger, Log, "  %ls  %ls", *EventEnum->GetNameStringByIndex(Index), FilterValue);
				}
			}
			return;
		}
		else if ((Params.Num() % 2) == 0)
		{
			for (int32 Index = 0; Index < Params.Num(); Index += 2)
			{
				const int32 InputEventEnumValue = (int32)EventEnum->GetValueByNameString(Params[Index]);
				// We are casting ESlateDebuggingInputEvent::MAX to uint8. Let's make sure it has a valid value. Prefer to crash now than doing bad memory access.
				check(EventEnum->GetMaxEnumValue() < TNumericLimits<uint8>::Max());
				if (InputEventEnumValue == INDEX_NONE)
				{
					return;
				}

				bool bEnable = false;
				if (!LexTryParseString(bEnable, *Params[Index+1]))
				{
					return;
				}

				Bitfield[InputEventEnumValue] = bEnable;
			}
		}
	}
}

void FConsoleSlateDebugger::SetInputFilter(const TArray< FString >& Params)
{
	SlateDebuggerUtils::SetEventFilter(Params, StaticEnum<ESlateDebuggingInputEvent>(), EnabledInputEvents);
}

void FConsoleSlateDebugger::ClearInputFilters()
{
	EnabledInputEvents.Init(false, EnabledInputEvents.Num());
}

void FConsoleSlateDebugger::EnableInputFilters()
{
	EnabledInputEvents.Init(true, EnabledInputEvents.Num());
}

void FConsoleSlateDebugger::SetFocusFilter(const TArray< FString >& Params)
{
	SlateDebuggerUtils::SetEventFilter(Params, StaticEnum<ESlateDebuggingFocusEvent>(), EnabledFocusEvents);
}

void FConsoleSlateDebugger::ClearFocusFilters()
{
	EnabledFocusEvents.Init(false, EnabledFocusEvents.Num());
}

void FConsoleSlateDebugger::EnableFocusFilters()
{
	EnabledFocusEvents.Init(true, EnabledFocusEvents.Num());
}

void FConsoleSlateDebugger::RemoveListeners()
{
	FSlateDebugging::Warning.RemoveAll(this);
	FSlateDebugging::InputEvent.RemoveAll(this);
	FSlateDebugging::FocusEvent.RemoveAll(this);
	FSlateDebugging::AttemptNavigationEvent.RemoveAll(this);
	FSlateDebugging::MouseCaptureEvent.RemoveAll(this);
	FSlateDebugging::CursorChangedEvent.RemoveAll(this);

	FSlateDebugging::UnregisterWidgetInputRoutingEvent(this);
}

void FConsoleSlateDebugger::UpdateListeners()
{
	RemoveListeners();

	if (bEnabled)
	{
		if (bLogWarning)
		{
			FSlateDebugging::Warning.AddRaw(this, &FConsoleSlateDebugger::OnWarning);
		}
		if (bLogInputEvent)
		{
			if (bInputRoutingModeEnabled)
			{
				FSlateDebugging::RegisterWidgetInputRoutingEvent(this);
			}
			else
			{
				FSlateDebugging::InputEvent.AddRaw(this, &FConsoleSlateDebugger::OnInputEvent);
			}
		}
		if (bLogFocusEvent)
		{
			FSlateDebugging::FocusEvent.AddRaw(this, &FConsoleSlateDebugger::OnFocusEvent);
		}
		if (bLogAttemptNavigationEvent)
		{
			FSlateDebugging::AttemptNavigationEvent.AddRaw(this, &FConsoleSlateDebugger::OnAttemptNavigationEvent);
		}
		if (bLogExecuteNavigationEvent)
		{
			FSlateDebugging::ExecuteNavigationEvent.AddRaw(this, &FConsoleSlateDebugger::OnExecuteNavigationEvent);
		}
		if (bLogCaptureStateChangeEvent)
		{
			FSlateDebugging::MouseCaptureEvent.AddRaw(this, &FConsoleSlateDebugger::OnCaptureStateChangeEvent);
		}
		if (bLogCursorChangeEvent)
		{
			FSlateDebugging::CursorChangedEvent.AddRaw(this, &FConsoleSlateDebugger::OnCursorChangeEvent);
		}
	}
}

namespace ConsoleSlateDebugger
{
	using TString = TStringBuilder<64>;
	void BuildSpaces(TStringBuilder<64>& Message, int32 Amount)
	{
		const TCHAR* Space = TEXT("    ");
		for (int32 SpaceCounter = 0; SpaceCounter < Amount; ++SpaceCounter)
		{
			Message.Append(Space);
		}
	}
}
	
	
void FConsoleSlateDebugger::OnProcessInput(ESlateDebuggingInputEvent InputEventType, const FInputEvent& Event)
{
	if (EnabledInputEvents[(uint8)InputEventType])
	{
		ConsoleSlateDebugger::TString Message;
		ConsoleSlateDebugger::BuildSpaces(Message, ProcessInputCounter + RouteInputCounter);
	
		Message.Append(TEXT(" Process - "));
		Message.Append(StaticEnum<ESlateDebuggingInputEvent>()->GetDisplayNameTextByValue((int64)InputEventType).ToString());
		Message.Append(TEXT(" - "));
		Message.Append(Event.ToText().ToString());
		UE_LOGF(LogSlateDebugger, Log, "%ls", *Message);

		OptionallyDumpCallStack();
	}
	
	++ProcessInputCounter;
}
	
void FConsoleSlateDebugger::OnPreProcessInput(ESlateDebuggingInputEvent InputEventType, const TCHAR* InputPrecessor, bool bHandled)
{
	if (EnabledInputEvents[(uint8)InputEventType])
	{
		ConsoleSlateDebugger::TString Message;
		ConsoleSlateDebugger::BuildSpaces(Message, ProcessInputCounter + RouteInputCounter);
	
		Message.Append(TEXT(" InputProcessor - "));
		if (bHandled)
		{
			Message.Append(TEXT(" (Handled) - "));
		}
		Message.Append(InputPrecessor);
		UE_LOGF(LogSlateDebugger, Log, "%ls", *Message);
	}
}
	
void FConsoleSlateDebugger::OnRouteInput(ESlateDebuggingInputEvent InputEventType, const FName& RoutedType)
{
	if (EnabledInputEvents[(uint8)InputEventType])
	{
		ConsoleSlateDebugger::TString Message;
		ConsoleSlateDebugger::BuildSpaces(Message, ProcessInputCounter + RouteInputCounter);
	
		Message.Append(TEXT(" Route - "));
		Message.Append(StaticEnum<ESlateDebuggingInputEvent>()->GetDisplayNameTextByValue((int64)InputEventType).ToString());
		Message.Append(TEXT(" ("));
		Message.Append(RoutedType.ToString());
		Message.AppendChar(TEXT(')'));
		UE_LOGF(LogSlateDebugger, Log, "%ls", *Message);
	}
	
	++RouteInputCounter;
}
	
void FConsoleSlateDebugger::OnInputEvent(ESlateDebuggingInputEvent InputEventType, const FReply& InReply, const TSharedPtr<SWidget>& HandlerWidget)
{
	if (EnabledInputEvents[(uint8)InputEventType])
	{
		ConsoleSlateDebugger::TString Message;
		ConsoleSlateDebugger::BuildSpaces(Message, ProcessInputCounter + RouteInputCounter);
	
		Message.Append(TEXT(" Event - "));
		if (InReply.IsEventHandled())
		{
			Message.Append(TEXT(" (Handled) - "));
		}
		Message.Append(StaticEnum<ESlateDebuggingInputEvent>()->GetDisplayNameTextByValue((int64)InputEventType).ToString());
		Message.Append(TEXT(" - "));
		Message.Append(FReflectionMetaData::GetWidgetDebugInfo(HandlerWidget.Get()));
		UE_LOGF(LogSlateDebugger, Log, "%ls", *Message);
	}
}
	
void FConsoleSlateDebugger::OnInputRouted(ESlateDebuggingInputEvent InputEventType)
{
	// bInputRoutingModeEnabled could have been activated while routing an event.
	RouteInputCounter = FMath::Max(0, RouteInputCounter - 1);
}
	
void FConsoleSlateDebugger::OnInputProcessed(ESlateDebuggingInputEvent InputEventType)
{
	// bInputRoutingModeEnabled could have been activated while processing an event.
	ProcessInputCounter = FMath::Max(0, ProcessInputCounter-1);
}

void FConsoleSlateDebugger::OnWarning(const FSlateDebuggingWarningEventArgs& EventArgs)
{
	UE_LOGF(LogSlateDebugger, Warning, "%ls", *EventArgs.ToText().ToString());

	OptionallyDumpCallStack();
}

void FConsoleSlateDebugger::OnInputEvent(const FSlateDebuggingInputEventArgs& EventArgs)
{
	// If the input event isn't in the set we care about don't write it out.
	if (!EnabledInputEvents[(uint8)EventArgs.InputEventType])
	{
		return;
	}

	UE_LOGF(LogSlateDebugger, Log, "%ls", *EventArgs.ToText().ToString());

	OptionallyDumpCallStack();
}

void FConsoleSlateDebugger::OnFocusEvent(const FSlateDebuggingFocusEventArgs& EventArgs)
{
	if (!EnabledFocusEvents[(uint8)EventArgs.FocusEventType])
	{
		return;
	}

	UE_LOGF(LogSlateDebugger, Log, "%ls", *EventArgs.ToText().ToString());

	OptionallyDumpCallStack();
}

void FConsoleSlateDebugger::OnAttemptNavigationEvent(const FSlateDebuggingNavigationEventArgs& EventArgs)
{
	UE_LOGF(LogSlateDebugger, Log, "%ls", *EventArgs.ToText().ToString());

	OptionallyDumpCallStack();
}

void FConsoleSlateDebugger::OnExecuteNavigationEvent(const FSlateDebuggingExecuteNavigationEventArgs& EventArgs)
{
	OptionallyDumpCallStack();
}

void FConsoleSlateDebugger::OnCaptureStateChangeEvent(const FSlateDebuggingMouseCaptureEventArgs& EventArgs)
{
	UE_LOGF(LogSlateDebugger, Log, "%ls", *EventArgs.ToText().ToString());

	OptionallyDumpCallStack();
}

void FConsoleSlateDebugger::OnCursorChangeEvent(const FSlateDebuggingCursorQueryEventArgs& EventArgs)
{
	UE_LOGF(LogSlateDebugger, Log, "%ls", *EventArgs.ToText().ToString());

	OptionallyDumpCallStack();
}

void FConsoleSlateDebugger::OptionallyDumpCallStack()
{
	if (!bCaptureStack)
	{
		return;
	}

	PrintScriptCallstack();

	TArray<FProgramCounterSymbolInfo> Stack = FPlatformStackWalk::GetStack(7, 5);

	for (int i = 0; i < Stack.Num(); i++)
	{
		UE_LOGF(LogSlateDebugger, Log, "%ls", ANSI_TO_TCHAR(Stack[i].FunctionName));

		//ANSICHAR HumanReadableString[1024];
		//if (FPlatformStackWalk::SymbolInfoToHumanReadableString(Stack[i], HumanReadableString, 1024))
		//{
		//	UE_LOGF(LogSlateDebugger, Log, "%ls", ANSI_TO_TCHAR((const ANSICHAR*)&HumanReadableString[0]));
		//}
	}
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_SLATE_DEBUGGING
