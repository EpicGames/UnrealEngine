// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Debugging/SlateDebugging.h"

#if WITH_SLATE_DEBUGGING

#include "CoreMinimal.h"
#include "Debugging/ConsoleSlateDebuggerUtility.h"
#include "Delegates/Delegate.h"
#include "Input/Reply.h"
#include "HAL/IConsoleManager.h"

/**
 * Allows debugging the behavior of Slate from the console.
 * Basics:
 *   Start - SlateDebugger.Event.Start
 *   Stop  - SlateDebugger.Event.Stop
 *
 * Notes:
 *   If you need to begin debugging slate on startup do, -execcmds="SlateDebugger.Event.Start"
 */
class FConsoleSlateDebugger : public FSlateDebugging::IWidgetInputRoutingEvent
{
public:
	FConsoleSlateDebugger();
	virtual ~FConsoleSlateDebugger();

	void StartDebugging();
	void StopDebugging();
	bool IsEnabled() const { return bEnabled; }

protected:
	//~ Begin IWidgetInputRoutingEvent interface
	virtual void OnProcessInput(ESlateDebuggingInputEvent InputEventType, const FInputEvent& Event) override;
	virtual void OnPreProcessInput(ESlateDebuggingInputEvent InputEventType, const TCHAR* InputPrecessor, bool bHandled) override;
	virtual void OnRouteInput(ESlateDebuggingInputEvent InputEventType, const FName& RoutedType) override;
	virtual void OnInputEvent(ESlateDebuggingInputEvent InputEventType, const FReply& InReply, const TSharedPtr<SWidget>& HandlerWidget) override;
	virtual void OnInputRouted(ESlateDebuggingInputEvent InputEventType) override;
	virtual void OnInputProcessed(ESlateDebuggingInputEvent InputEventType) override;
	//~ End IWidgetInputRoutingEvent interface

private:

	void RemoveListeners();
	void UpdateListeners();

	void OnLogVariablesChanged(IConsoleVariable*);

	void SetInputFilter(const TArray< FString >& Parms);
	void ClearInputFilters();
	void EnableInputFilters();
	void SetFocusFilter(const TArray< FString >& Parms);
	void ClearFocusFilters();
	void EnableFocusFilters();

	void OnWarning(const FSlateDebuggingWarningEventArgs& EventArgs);
	void OnInputEvent(const FSlateDebuggingInputEventArgs& EventArgs);
	void OnFocusEvent(const FSlateDebuggingFocusEventArgs& EventArgs);
	void OnAttemptNavigationEvent(const FSlateDebuggingNavigationEventArgs& EventArgs);
	void OnExecuteNavigationEvent(const FSlateDebuggingExecuteNavigationEventArgs& EventArgs);
	void OnCaptureStateChangeEvent(const FSlateDebuggingMouseCaptureEventArgs& EventArgs);
	void OnCursorChangeEvent(const FSlateDebuggingCursorQueryEventArgs& EventArgs);

	void OptionallyDumpCallStack();

private:
	/** Is the Console Slate Debugger enabled. */
	bool bEnabled;

	/** */
	bool bLogWarning;
	bool bLogInputEvent;
	bool bLogFocusEvent;
	bool bLogAttemptNavigationEvent;
	bool bLogExecuteNavigationEvent;
	bool bLogCaptureStateChangeEvent;
	bool bLogCursorChangeEvent;
	
	/** Should we capture and dump the callstack when events happen? */
	bool bCaptureStack;

	/** Should we display the input event as routing or regular log */
	bool bInputRoutingModeEnabled;

	//* Routing Counter */
	int32 ProcessInputCounter;
	int32 RouteInputCounter;

	/** Which events should we log about. */
	TBitArray<> EnabledInputEvents;
	TBitArray<> EnabledFocusEvents;

	//~ Console objects
	FAutoConsoleCommand StartDebuggingCommand;
	FAutoConsoleCommand StopDebuggingCommand;
	FAutoConsoleCommand StartDebuggingCommandAlias;
	FAutoConsoleCommand StopDebuggingCommandAlias;
	FAutoConsoleVariableRef EnableLogWarning;
	FAutoConsoleVariableRef EnableLogInputEvent;
	FAutoConsoleVariableRef EnableLogFocusEvent;
	FAutoConsoleVariableRef EnableLogAttemptNavigationEvent;
	FAutoConsoleVariableRef EnableLogExecuteNavigationEvent;
	FAutoConsoleVariableRef EnableLogCaptureStateChangeEvent;
	FAutoConsoleVariableRef EnableLogCursorChangeEvent;
	FAutoConsoleVariableRef CaptureStackVariable;
	FAutoConsoleVariableRef EnableInputRoutingMode;
	FAutoConsoleCommand SetInputFilterCommand;
	FAutoConsoleCommand ClearInputFiltersCommand;
	FAutoConsoleCommand EnableInputFiltersCommand;
	FAutoConsoleCommand SetFocusFilterCommand;
	FAutoConsoleCommand ClearFocusFiltersCommand;
	FAutoConsoleCommand EnableFocusFiltersCommand;
};

#endif //WITH_SLATE_DEBUGGING