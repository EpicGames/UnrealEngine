// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugging/ConsoleSlateDebuggerBreak.h"
#include "Debugging/ConsoleSlateDebugger.h"

#if WITH_SLATE_DEBUGGING

#include "Application/SlateApplicationBase.h"
#include "CoreGlobals.h"
#include "Debugging/SlateDebugging.h"
#include "HAL/PlatformMisc.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/StringBuilder.h"
#include "Types/ReflectionMetadata.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "ConsoleSlateDebuggerBreak"


FConsoleSlateDebuggerBreak::FConsoleSlateDebuggerBreak()
	: WidgetInvalidationCommand(TEXT("SlateDebugger.Break.OnWidgetInvalidation"),
		TEXT("Break when the widget get invalidated (must be attached to a debugger).\n")
		TEXT("Usage: [WidgetPtr=0x1234567]|[WidgetId=12345] [Reason=Paint|Volatility|ChildOrder|RenderTransform|Visibility|AttributeRegistration|Prepass|All|]"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FConsoleSlateDebuggerBreak::HandleBreakOnWidgetInvalidation))
	, WidgetBeginPaintCommand(TEXT("SlateDebugger.Break.OnWidgetBeginPaint"),
		TEXT("Break before the widget get painted (must be attached to a debugger).\n")
		TEXT("Usage: [WidgetPtr=0x1234567]|[WidgetId=12345]"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FConsoleSlateDebuggerBreak::HandleBeginWidgetPaint))
	, WidgetEndPaintCommand(TEXT("SlateDebugger.Break.OnWidgetEndPaint"),
		TEXT("Break after the widget got painted (must be attached to a debugger).\n")
		TEXT("Usage: [WidgetPtr=0x1234567]|[WidgetId=12345]"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FConsoleSlateDebuggerBreak::HandleEndWidgetPaint))
	, RemoveAllCommand(TEXT("SlateDebugger.Break.RemoveAll"),
		TEXT("Remove all request to break.\n"),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerBreak::RemoveAll))
{
}


FConsoleSlateDebuggerBreak::~FConsoleSlateDebuggerBreak()
{
	RemoveAll();
}


void FConsoleSlateDebuggerBreak::AddInvalidation(const SWidget& Widget, EInvalidateWidgetReason Reason)
{
	const FConsoleSlateDebuggerUtility::TSWidgetId WidgetId = FConsoleSlateDebuggerUtility::GetId(Widget);
	AddInvalidation( {WidgetId, Reason} );
}


void FConsoleSlateDebuggerBreak::AddInvalidation(FInvalidationElement Element)
{
	if (Element.WidgetId == 0 || Element.Reason == EInvalidateWidgetReason::None)
	{
		return;
	}

	if (!FPlatformMisc::IsDebuggerPresent())
	{
		UE_LOGF(LogSlateDebugger, Warning, "The debugger is not attached");
	}


	if (BreakOnInvalidationElements.Num() == 0)
	{
		FSlateDebugging::WidgetInvalidateEvent.AddRaw(this, &FConsoleSlateDebuggerBreak::HandleWidgetInvalidated);
	}

	if (FInvalidationElement* FoundItem = BreakOnInvalidationElements.FindByPredicate([Element](const FInvalidationElement& Other)
		{
			return (Other.WidgetId == Element.WidgetId);
		}))
	{
		*FoundItem = MoveTemp(Element);
	}
	else
	{
		BreakOnInvalidationElements.Add(MoveTemp(Element));
	}
}


void FConsoleSlateDebuggerBreak::RemoveInvalidation(const SWidget& Widget)
{
	const FConsoleSlateDebuggerUtility::TSWidgetId WidgetId = FConsoleSlateDebuggerUtility::GetId(Widget);
	BreakOnInvalidationElements.RemoveAllSwap([WidgetId](const FInvalidationElement& Element)
		{
			return (Element.WidgetId == WidgetId);
		});
	if (BreakOnInvalidationElements.Num() == 0)
	{
		FSlateDebugging::WidgetInvalidateEvent.RemoveAll(this);
	}
}


void FConsoleSlateDebuggerBreak::AddBeginPaint(const SWidget& Widget)
{
	AddBeginPaint(FConsoleSlateDebuggerUtility::GetId(Widget));
}


void FConsoleSlateDebuggerBreak::AddBeginPaint(FConsoleSlateDebuggerUtility::TSWidgetId WidgetId)
{
	if (BreakOnBeginPaintElements.Num() == 0)
	{
		FSlateDebugging::BeginWidgetPaint.AddRaw(this, &FConsoleSlateDebuggerBreak::HandleBeginWidgetPaint);
	}
	BreakOnBeginPaintElements.AddUnique(WidgetId);
}


void FConsoleSlateDebuggerBreak::RemoveBeginPaint(const SWidget& Widget)
{
	BreakOnBeginPaintElements.RemoveSingleSwap(FConsoleSlateDebuggerUtility::GetId(Widget));
	if (BreakOnBeginPaintElements.Num() == 0)
	{
		FSlateDebugging::BeginWidgetPaint.RemoveAll(this);
	}
}


void FConsoleSlateDebuggerBreak::AddEndPaint(const SWidget& Widget)
{
	AddEndPaint(FConsoleSlateDebuggerUtility::GetId(Widget));
}


void FConsoleSlateDebuggerBreak::AddEndPaint(FConsoleSlateDebuggerUtility::TSWidgetId WidgetId)
{
	if (BreakOnEndPaintElements.Num() == 0)
	{
		FSlateDebugging::EndWidgetPaint.AddRaw(this, &FConsoleSlateDebuggerBreak::HandleEndWidgetPaint);
	}
	BreakOnEndPaintElements.AddUnique(WidgetId);
}


void FConsoleSlateDebuggerBreak::RemoveEndPaint(const SWidget& Widget)
{
	BreakOnEndPaintElements.RemoveSingleSwap(FConsoleSlateDebuggerUtility::GetId(Widget));
	if (BreakOnEndPaintElements.Num() == 0)
	{
		FSlateDebugging::EndWidgetPaint.RemoveAll(this);
	}
}


void FConsoleSlateDebuggerBreak::RemoveAll()
{
	FSlateDebugging::WidgetInvalidateEvent.RemoveAll(this);
	BreakOnInvalidationElements.Empty();
	FSlateDebugging::BeginWidgetPaint.RemoveAll(this);
	BreakOnBeginPaintElements.Empty();
	FSlateDebugging::EndWidgetPaint.RemoveAll(this);
	BreakOnEndPaintElements.Empty();
}


bool FConsoleSlateDebuggerBreak::ParseWidgetArgs(FConsoleSlateDebuggerBreak::FInvalidationElement& Out, const TArray<FString>& Params, bool bAllowInvalidationReason)
{
	const TCHAR* ReasonStr = TEXT("Reason=");
	const TCHAR* WidgetPtrStr = TEXT("WidgetPtr=");
	const TCHAR* WidgetIdStr = TEXT("WidgetId=");
	FStringView ReasonView{ ReasonStr };
	FStringView WidgetPtrView{ WidgetPtrStr };
	FStringView WidgetIdView{ WidgetIdStr };
	for (const FString& Param : Params)
	{
		FStringView ParamView{Param};
		if (ParamView.StartsWith(ReasonView, ESearchCase::IgnoreCase))
		{
			if (!bAllowInvalidationReason)
			{
				return false;
			}
			ParamView.RightChopInline(ReasonView.Len());
			if (!LexTryParseString(Out.Reason, ParamView.GetData()))
			{
				return false;
			}
		}
		else if (ParamView.StartsWith(WidgetPtrView, ESearchCase::IgnoreCase))
		{
			ParamView.RightChopInline(WidgetPtrView.Len());
			FConsoleSlateDebuggerUtility::TSWidgetId WidgetNumber;
			if (LexTryParseString(WidgetNumber, ParamView.GetData()) && WidgetNumber != 0)
			{
				Out.WidgetId = FConsoleSlateDebuggerUtility::GetId((SWidget*)WidgetNumber);
			}
			else
			{
				return false;
			}
		}
		else if (ParamView.StartsWith(WidgetIdView, ESearchCase::IgnoreCase))
		{
			ParamView.RightChopInline(WidgetIdView.Len());
			FConsoleSlateDebuggerUtility::TSWidgetId WidgetNumber;
			if (LexTryParseString(WidgetNumber, ParamView.GetData()) && WidgetNumber != 0)
			{
				Out.WidgetId = FConsoleSlateDebuggerUtility::GetId((SWidget*)WidgetNumber);
			}
			else
			{
				return false;
			}
		}
		else
		{
			return false;
		}
	}
	return Params.Num() != 0;
}

void FConsoleSlateDebuggerBreak::HandleBreakOnWidgetInvalidation(const TArray<FString>& Params)
{
	const TCHAR* UsageMessage = TEXT("Usage: [WidgetPtr=0x1234567]|[WidgetId=12345] [Reason=Paint|Volatility|ChildOrder|RenderTransform|Visibility|AttributeRegistration|Prepass|Any|]");

	FInvalidationElement InvalidationElement;
	if (ParseWidgetArgs(InvalidationElement, Params, true))
	{
		AddInvalidation(InvalidationElement);
	}
	else
	{
		UE_LOGF(LogSlateDebugger, Log, "%ls", UsageMessage);
	}
}


void FConsoleSlateDebuggerBreak::HandleBeginWidgetPaint(const TArray<FString>& Params)
{
	const TCHAR* UsageMessage = TEXT("Usage: [WidgetPtr=0x1234567]|[WidgetId=12345]");

	FInvalidationElement InvalidationElement;
	if (ParseWidgetArgs(InvalidationElement, Params, false))
	{
		AddBeginPaint(InvalidationElement.WidgetId);
	}
	else
	{
		UE_LOGF(LogSlateDebugger, Log, "%ls", UsageMessage);
	}
}


void FConsoleSlateDebuggerBreak::HandleEndWidgetPaint(const TArray<FString>& Params)
{
	const TCHAR* UsageMessage = TEXT("Usage: [WidgetPtr=0x1234567]|[WidgetId=12345]");

	FInvalidationElement InvalidationElement;
	if (ParseWidgetArgs(InvalidationElement, Params, false))
	{
		AddEndPaint(InvalidationElement.WidgetId);
	}
	else
	{
		UE_LOGF(LogSlateDebugger, Log, "%ls", UsageMessage);
	}
}


void FConsoleSlateDebuggerBreak::HandleWidgetInvalidated(const FSlateDebuggingInvalidateArgs& Args)
{
	if (Args.WidgetInvalidated == nullptr)
	{
		return;
	}

	const FConsoleSlateDebuggerUtility::TSWidgetId WidgetInvalidatedId = FConsoleSlateDebuggerUtility::GetId(Args.WidgetInvalidated);
	if (FInvalidationElement* FoundItem = BreakOnInvalidationElements.FindByPredicate([WidgetInvalidatedId](const FInvalidationElement& Element)
		{
			return (Element.WidgetId == WidgetInvalidatedId);
		}))
	{
		if (EnumHasAnyFlags(Args.InvalidateWidgetReason, FoundItem->Reason))
		{
			UE_LOGF(LogSlateDebugger, Log, "Widget '%ls' was invalidated.", *FReflectionMetaData::GetWidgetPath(Args.WidgetInvalidated));
			UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE();
		}
	}
}


void FConsoleSlateDebuggerBreak::HandleBeginWidgetPaint(const SWidget* Widget, const FPaintArgs& /*Args*/, const FGeometry& /*AllottedGeometry*/, const FSlateRect& /*MyCullingRect*/, const FSlateWindowElementList& /*OutDrawElements*/, int32 /*LayerId*/)
{
	if (Widget == nullptr)
	{
		return;
	}

	const FConsoleSlateDebuggerUtility::TSWidgetId WidgetId = FConsoleSlateDebuggerUtility::GetId(Widget);
	if (BreakOnBeginPaintElements.Contains(WidgetId))
	{
		UE_LOGF(LogSlateDebugger, Log, "Widget '%ls' was invalidated.", *FReflectionMetaData::GetWidgetPath(Widget));
		UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE();
	}
}


void FConsoleSlateDebuggerBreak::HandleEndWidgetPaint(const SWidget* Widget, const FSlateWindowElementList& /*OutDrawElements*/, int32 /*LayerId*/)
{
	if (Widget == nullptr)
	{
		return;
	}

	const FConsoleSlateDebuggerUtility::TSWidgetId WidgetId = FConsoleSlateDebuggerUtility::GetId(Widget);
	if (BreakOnEndPaintElements.Contains(WidgetId))
	{
		UE_LOGF(LogSlateDebugger, Log, "Widget '%ls' was invalidated.", *FReflectionMetaData::GetWidgetPath(Widget));
		UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE();
	}
}


#undef LOCTEXT_NAMESPACE

#endif // WITH_SLATE_DEBUGGING