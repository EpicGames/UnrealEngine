// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Debugging/SlateDebugging.h"

#if WITH_SLATE_DEBUGGING

#include "CoreMinimal.h"
#include "Debugging/ConsoleSlateDebuggerUtility.h"
#include "Delegates/Delegate.h"
#include "HAL/IConsoleManager.h"

class SWidget;

/**
 * Allows debugging events from the console.
 * Basics:
 *   SlateDebugger.Break.OnWidgetInvalidation [WidgetPtr=0x123456789] [WidgetId=123456] [Reason=All]
 *   SlateDebugger.Break.OnWidgetPaint [WidgetPointer=0x123456789] [WidgetId=123456]
 */
class FConsoleSlateDebuggerBreak
{
public:
	FConsoleSlateDebuggerBreak();
	~FConsoleSlateDebuggerBreak();

	void AddInvalidation(const SWidget& Widget, EInvalidateWidgetReason Reason);
	void RemoveInvalidation(const SWidget& Widget);

	void AddBeginPaint(const SWidget& Widget);
	void RemoveBeginPaint(const SWidget& Widget);

	void AddEndPaint(const SWidget& Widget);
	void RemoveEndPaint(const SWidget& Widget);

	void RemoveAll();

private:
	struct FInvalidationElement
	{
		FConsoleSlateDebuggerUtility::TSWidgetId WidgetId = FConsoleSlateDebuggerUtility::InvalidWidgetId;
		EInvalidateWidgetReason Reason = (EInvalidateWidgetReason)0xFF;
	};
	void AddInvalidation(FInvalidationElement Element);
	void AddBeginPaint(FConsoleSlateDebuggerUtility::TSWidgetId Element);
	void AddEndPaint(FConsoleSlateDebuggerUtility::TSWidgetId Element);

private:
	void HandleWidgetInvalidated(const FSlateDebuggingInvalidateArgs& /*Args*/);
	void HandleBeginWidgetPaint(const SWidget* /*Widget*/, const FPaintArgs& /*Args*/, const FGeometry& /*AllottedGeometry*/, const FSlateRect& /*MyCullingRect*/, const FSlateWindowElementList& /*OutDrawElements*/, int32 /*LayerId*/);
	void HandleEndWidgetPaint(const SWidget* /*Widget*/, const FSlateWindowElementList& /*OutDrawElements*/, int32 /*LayerId*/);

private:
	TArray<FInvalidationElement> BreakOnInvalidationElements;
	TArray<FConsoleSlateDebuggerUtility::TSWidgetId> BreakOnBeginPaintElements;
	TArray<FConsoleSlateDebuggerUtility::TSWidgetId> BreakOnEndPaintElements;

	//~ Console objects
private:
	void HandleBreakOnWidgetInvalidation(const TArray<FString>& Params);
	void HandleBeginWidgetPaint(const TArray<FString>& Params);
	void HandleEndWidgetPaint(const TArray<FString>& Params);

	bool ParseWidgetArgs(FInvalidationElement& Out, const TArray<FString>& Params, bool bAllowInvalidationReason);

private:
	FAutoConsoleCommand WidgetInvalidationCommand;
	FAutoConsoleCommand WidgetBeginPaintCommand;
	FAutoConsoleCommand WidgetEndPaintCommand;
	FAutoConsoleCommand RemoveAllCommand;
};

#endif //WITH_SLATE_DEBUGGING