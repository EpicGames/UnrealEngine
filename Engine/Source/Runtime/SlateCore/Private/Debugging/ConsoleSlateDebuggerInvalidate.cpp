// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugging/ConsoleSlateDebuggerInvalidate.h"
#include "Debugging/ConsoleSlateDebugger.h"

#if WITH_SLATE_DEBUGGING

#include "Application/SlateApplicationBase.h"
#include "CoreGlobals.h"
#include "Debugging/SlateDebugging.h"
#include "FastUpdate/SlateInvalidationRoot.h"
#include "FastUpdate/WidgetProxy.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/StringBuilder.h"
#include "Styling/CoreStyle.h"
#include "Types/ReflectionMetadata.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "ConsoleSlateDebuggerInvalidate"

FConsoleSlateDebuggerInvalidate::FConsoleSlateDebuggerInvalidate()
	: bEnabled(false)
	, bEnabledCVarValue(false)
	, bShowWidgetList(true)
	, bUseWidgetPathAsName(false)
	, bShowLegend(false)
	, bLogInvalidatedWidget(false)
	, bUsePerformanceThreshold(false)
	, InvalidateWidgetReasonFilter(static_cast<EInvalidateWidgetReason>(0xFF))
	, InvalidateRootReasonFilter(static_cast<ESlateDebuggingInvalidateRootReason>(0xFF))
	, DrawRootRootColor(FColorList::Red)
	, DrawRootChildOrderColor(FColorList::Blue)
	, DrawRootScreenPositionColor(FColorList::Green)
	, DrawWidgetPrepassColor(FColorList::Orange)
	, DrawWidgetLayoutColor(FColorList::Magenta)
	, DrawWidgetPaintColor(FColorList::Yellow)
	, DrawWidgetVolatilityColor(FColorList::Grey)
	, DrawWidgetChildOrderColor(FColorList::Cyan)
	, DrawWidgetRenderTransformColor(FColorList::Black)
	, DrawWidgetVisibilityColor(FColorList::White)
	, MaxNumberOfWidgetInList(20)
	, CacheDuration(2.0f)
	, ThresholdPerformanceMs(1.5f) //1.5ms
	, StartCommand(
		TEXT("SlateDebugger.Invalidate.Start"),
		TEXT("Start the Invalidation widget debug tool. It shows widgets that are invalidated."),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidate::StartDebugging))
	, StopCommand(
		TEXT("SlateDebugger.Invalidate.Stop"),
		TEXT("Stop the Invalidation widget debug tool."),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidate::StopDebugging))
	, EnabledRefCVar(
		TEXT("SlateDebugger.Invalidate.Enabled"),
		bEnabledCVarValue,
		TEXT("Start/Stop the Invalidation widget debug tool. It shows widgets that are invalidated."),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidate::HandleEnabled))
	, ShowLegendRefCVar(
		TEXT("SlateDebugger.Invalidate.bShowLegend"),
		bShowLegend,
		TEXT("Option to display the color legend."),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidate::HandledConfigChanged))
	, ShowWidgetsNameListRefCVar(
		TEXT("SlateDebugger.Invalidate.bShowWidgetList"),
		bShowWidgetList,
		TEXT("Option to display the names of invalidated widgets."),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidate::HandledConfigChanged))
	, LogInvalidatedWidgetRefCVar(
		TEXT("SlateDebugger.Invalidate.bLogInvalidatedWidget"),
		bLogInvalidatedWidget,
		TEXT("Option to log the invalidated widget to the console."),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidate::HandledConfigChanged))
	, UsePerformanceThresholdRefCVar(
		TEXT("SlateDebugger.Invalidate.bUsePerformanceThreshold"),
		bUsePerformanceThreshold,
		TEXT("Only display the invalidated widgets and/or log them if the performance are worst than the threshold (in millisecond)."),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidate::HandledConfigChanged))
	, ThresholdPerformanceRefCVar(
		TEXT("SlateDebugger.Invalidate.ThresholdPerformanceMS"),
		ThresholdPerformanceMs,
		TEXT("For bUsePerformanceThreshold, threshold in milliseconds to reach before logging and/or displaying the invalidated widgets."),
		FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidate::HandledConfigChanged))
	, SetInvalidateWidgetReasonFilterCommand(
		TEXT("SlateDebugger.Invalidate.SetInvalidateWidgetReasonFilter"),
		TEXT("Enable Invalidate Widget Reason filters. Usage: SetInvalidateWidgetReasonFilter None|Layout|Paint|Volatility|ChildOrder|RenderTransform|Visibility|Any"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidate::HandleSetInvalidateWidgetReasonFilter))
	, SetInvalidateRootReasonFilterCommand(
		TEXT("SlateDebugger.Invalidate.SetInvalidateRootReasonFilter"),
		TEXT("Enable Invalidate Root Reason filters. Usage: SetInvalidateRootReasonFilter None|ChildOrder|Root|ScreenPosition|Any"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidate::HandleSetInvalidateRootReasonFilter))
	, LastPerformanceThresholdFrameCount(0)
	, LastPerformanceThresholdSeconds(0.0)
{
	LoadConfig();
}

FConsoleSlateDebuggerInvalidate::~FConsoleSlateDebuggerInvalidate()
{
	StopDebugging();
}

void FConsoleSlateDebuggerInvalidate::LoadConfig()
{
	FColor TmpColor;
	auto GetColor = [&](const TCHAR* ColorText, FLinearColor& Color)
	{
		if (GConfig->GetColor(TEXT("SlateDebugger.Invalidate"), ColorText, TmpColor, *GEditorPerProjectIni))
		{
			Color = TmpColor;
		}
	};

	GConfig->GetBool(TEXT("SlateDebugger.Invalidate"), TEXT("bEnabled"), bEnabledCVarValue, *GEditorPerProjectIni);
	GConfig->GetBool(TEXT("SlateDebugger.Invalidate"), TEXT("bShowWidgetList"), bShowWidgetList, *GEditorPerProjectIni);
	GConfig->GetBool(TEXT("SlateDebugger.Invalidate"), TEXT("bUseWidgetPathAsName"), bUseWidgetPathAsName, *GEditorPerProjectIni);
	GConfig->GetBool(TEXT("SlateDebugger.Invalidate"), TEXT("bShowLegend"), bShowLegend, *GEditorPerProjectIni);
	GConfig->GetBool(TEXT("SlateDebugger.Invalidate"), TEXT("bLogInvalidatedWidget"), bLogInvalidatedWidget, *GEditorPerProjectIni);
	GConfig->GetBool(TEXT("SlateDebugger.Invalidate"), TEXT("bUsePerformanceThreshold"), bUsePerformanceThreshold, *GEditorPerProjectIni);
	GetColor(TEXT("DrawRootRootColor"), DrawRootRootColor);
	GetColor(TEXT("DrawRootChildOrderColor"), DrawRootChildOrderColor);
	GetColor(TEXT("DrawRootScreenPositionColor"), DrawRootScreenPositionColor);
	GetColor(TEXT("DrawWidgetPrepassColor"), DrawWidgetPrepassColor);
	GetColor(TEXT("DrawWidgetLayoutColor"), DrawWidgetLayoutColor);
	GetColor(TEXT("DrawWidgetPaintColor"), DrawWidgetPaintColor);
	GetColor(TEXT("DrawWidgetVolatilityColor"), DrawWidgetVolatilityColor);
	GetColor(TEXT("DrawWidgetChildOrderColor"), DrawWidgetChildOrderColor);
	GetColor(TEXT("DrawWidgetRenderTransformColor"), DrawWidgetRenderTransformColor);
	GetColor(TEXT("DrawWidgetVisibilityColor"), DrawWidgetVisibilityColor);
	GConfig->GetInt(TEXT("SlateDebugger.Invalidate"), TEXT("MaxNumberOfWidgetInList"), MaxNumberOfWidgetInList, *GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("SlateDebugger.Invalidate"), TEXT("CacheDuration"), CacheDuration, *GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("SlateDebugger.Invalidate"), TEXT("ThresholdPerformanceMs"), ThresholdPerformanceMs, *GEditorPerProjectIni);

	if (bEnabledCVarValue)
	{
		StartDebugging();
	}
}

void FConsoleSlateDebuggerInvalidate::SaveConfig()
{
	auto SetColor = [](const TCHAR* ColorText, const FLinearColor& Color)
	{
		FColor TmpColor = Color.ToFColor(true);
		GConfig->SetColor(TEXT("SlateDebugger.Invalidate"), ColorText, TmpColor, *GEditorPerProjectIni);
	};

	GConfig->SetBool(TEXT("SlateDebugger.Invalidate"), TEXT("bShowWidgetList"), bShowWidgetList, *GEditorPerProjectIni);
	GConfig->SetBool(TEXT("SlateDebugger.Invalidate"), TEXT("bUseWidgetPathAsName"), bUseWidgetPathAsName, *GEditorPerProjectIni);
	GConfig->SetBool(TEXT("SlateDebugger.Invalidate"), TEXT("bShowLegend"), bShowLegend, *GEditorPerProjectIni);
	GConfig->SetBool(TEXT("SlateDebugger.Invalidate"), TEXT("bLogInvalidatedWidget"), bLogInvalidatedWidget, *GEditorPerProjectIni);
	GConfig->SetBool(TEXT("SlateDebugger.Invalidate"), TEXT("bUsePerformanceThreshold"), bUsePerformanceThreshold, *GEditorPerProjectIni);
	SetColor(TEXT("DrawRootRootColor"), DrawRootRootColor);
	SetColor(TEXT("DrawRootChildOrderColor"), DrawRootChildOrderColor);
	SetColor(TEXT("DrawRootScreenPositionColor"), DrawRootScreenPositionColor);
	SetColor(TEXT("DrawWidgetPrepassColor"), DrawWidgetPrepassColor);
	SetColor(TEXT("DrawWidgetLayoutColor"), DrawWidgetLayoutColor);
	SetColor(TEXT("DrawWidgetPaintColor"), DrawWidgetPaintColor);
	SetColor(TEXT("DrawWidgetVolatilityColor"), DrawWidgetVolatilityColor);
	SetColor(TEXT("DrawWidgetChildOrderColor"), DrawWidgetChildOrderColor);
	SetColor(TEXT("DrawWidgetRenderTransformColor"), DrawWidgetRenderTransformColor);
	SetColor(TEXT("DrawWidgetVisibilityColor"), DrawWidgetVisibilityColor);
	GConfig->SetInt(TEXT("SlateDebugger.Invalidate"), TEXT("MaxNumberOfWidgetInList"), MaxNumberOfWidgetInList, *GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("SlateDebugger.Invalidate"), TEXT("CacheDuration"), CacheDuration, *GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("SlateDebugger.Invalidate"), TEXT("ThresholdPerformanceMs"), ThresholdPerformanceMs, *GEditorPerProjectIni);
}

void FConsoleSlateDebuggerInvalidate::StartDebugging()
{
	if (!bEnabled)
	{
		bEnabled = true;
		InvalidationInfos.Empty();
		FrameInvalidationInfos.Empty();
		FrameRootHandles.Empty();

		FSlateDebugging::PaintDebugElements.AddRaw(this, &FConsoleSlateDebuggerInvalidate::HandlePaintDebugInfo);
		FSlateDebugging::WidgetInvalidateEvent.AddRaw(this, &FConsoleSlateDebuggerInvalidate::HandleWidgetInvalidated);
		FCoreDelegates::OnEndFrame.AddRaw(this, &FConsoleSlateDebuggerInvalidate::HandleEndFrame);
	}
	bEnabledCVarValue = bEnabled;
}

void FConsoleSlateDebuggerInvalidate::StopDebugging()
{
	if (bEnabled)
	{
		FCoreDelegates::OnEndFrame.RemoveAll(this);
		FSlateDebugging::WidgetInvalidateEvent.RemoveAll(this);
		FSlateDebugging::PaintDebugElements.RemoveAll(this);

		InvalidationInfos.Empty();
		FrameInvalidationInfos.Empty();
		FrameRootHandles.Empty();
		bEnabled = false;
	}
	bEnabledCVarValue = bEnabled;
}

void FConsoleSlateDebuggerInvalidate::HandleEnabled(IConsoleVariable* Variable)
{
	if (bEnabledCVarValue)
	{
		StartDebugging();
	}
	else
	{
		StopDebugging();
	}
}

void FConsoleSlateDebuggerInvalidate::HandledConfigChanged(IConsoleVariable* Variable)
{
	SaveConfig();
}

void FConsoleSlateDebuggerInvalidate::HandleSetInvalidateWidgetReasonFilter(const TArray<FString>& Params)
{
	const TCHAR* UsageMessage = TEXT("Usage: SetInvalidateWidgetReasonFilter None|Layout|Paint|Volatility|ChildOrder|RenderTransform|Visibility|Any");
	if (Params.Num() == 0)
	{
		UE_LOGF(LogSlateDebugger, Log, "%ls", UsageMessage);

		TStringBuilder<512> MessageBuilder;
		MessageBuilder << TEXT("Current Invalidate Widget Reason set: ");
		MessageBuilder << LexToString(InvalidateWidgetReasonFilter);
		UE_LOGF(LogSlateDebugger, Log, "%ls", MessageBuilder.GetData());
	}
	else
	{
		TStringBuilder<512> MessageBuilder;
		MessageBuilder.Join(Params, TEXT('|'));
		EInvalidateWidgetReason NewInvalidateWidgetReasonFilter;
		if (!LexTryParseString(NewInvalidateWidgetReasonFilter, MessageBuilder.ToString()))
		{
			UE_LOGF(LogSlateDebugger, Log, "%ls", UsageMessage);
		}
		else
		{
			InvalidateWidgetReasonFilter = NewInvalidateWidgetReasonFilter;
			SaveConfig();
		}
	}
}

void FConsoleSlateDebuggerInvalidate::HandleSetInvalidateRootReasonFilter(const TArray<FString>& Params)
{
	const TCHAR* UsageMessage = TEXT("Usage: SetInvalidateRootReasonFilter None|ChildOrder|Root|ScreenPosition|Any");
	if (Params.Num() == 0)
	{
		UE_LOGF(LogSlateDebugger, Log, "%ls", UsageMessage);

		TStringBuilder<512> MessageBuilder;
		MessageBuilder << TEXT("Current Invalidate Root Reason set: ");
		MessageBuilder << LexToString(InvalidateRootReasonFilter);
		UE_LOGF(LogSlateDebugger, Log, "%ls", MessageBuilder.GetData());
	}
	else
	{
		TStringBuilder<512> MessageBuilder;
		MessageBuilder.Join(Params, TEXT('|'));
		ESlateDebuggingInvalidateRootReason NewInvalidateRootReasonFilter;
		if (!LexTryParseString(NewInvalidateRootReasonFilter, MessageBuilder.ToString()))
		{
			UE_LOGF(LogSlateDebugger, Log, "%ls", UsageMessage);
		}
		else
		{
			InvalidateRootReasonFilter = NewInvalidateRootReasonFilter;
			SaveConfig();
		}
	}
}

int32 FConsoleSlateDebuggerInvalidate::GetInvalidationPriority(EInvalidateWidgetReason InvalidationInfo, ESlateDebuggingInvalidateRootReason InvalidationRootReason) const
{
	InvalidationInfo &= InvalidateWidgetReasonFilter;
	InvalidationRootReason &= InvalidateRootReasonFilter;

	if (EnumHasAnyFlags(InvalidationRootReason, ESlateDebuggingInvalidateRootReason::Root))
	{
		return 100;
	}
	else if (EnumHasAnyFlags(InvalidationRootReason, ESlateDebuggingInvalidateRootReason::ChildOrder))
	{
		return 80;
	}
	else if (EnumHasAnyFlags(InvalidationRootReason, ESlateDebuggingInvalidateRootReason::ScreenPosition))
	{
		return 50;
	}

	if (EnumHasAnyFlags(InvalidationInfo, EInvalidateWidgetReason::ChildOrder))
	{
		return 45;
	}
	if (EnumHasAnyFlags(InvalidationInfo, EInvalidateWidgetReason::Prepass))
	{
		return 40;
	}
	if (EnumHasAnyFlags(InvalidationInfo, EInvalidateWidgetReason::Layout))
	{
		return 35;
	}
	if (EnumHasAnyFlags(InvalidationInfo, EInvalidateWidgetReason::Visibility | EInvalidateWidgetReason::RenderTransform))
	{
		return 30;
	}
	else if (EnumHasAnyFlags(InvalidationInfo, EInvalidateWidgetReason::Volatility))
	{
		return 25;
	}
	else if (EnumHasAnyFlags(InvalidationInfo, EInvalidateWidgetReason::Paint))
	{
		return 20;
	}
	return 0;
}

const FLinearColor& FConsoleSlateDebuggerInvalidate::GetColor(const FInvalidationInfo& InvalidationInfo) const
{
	if (EnumHasAnyFlags(InvalidationInfo.InvalidationRootReason, ESlateDebuggingInvalidateRootReason::Root))
	{
		return DrawRootRootColor;
	}
	else if (EnumHasAnyFlags(InvalidationInfo.InvalidationRootReason, ESlateDebuggingInvalidateRootReason::ChildOrder))
	{
		return DrawRootChildOrderColor;
	}
	else if (EnumHasAnyFlags(InvalidationInfo.InvalidationRootReason, ESlateDebuggingInvalidateRootReason::ScreenPosition))
	{
		return DrawRootScreenPositionColor;
	}

	if (EnumHasAnyFlags(InvalidationInfo.WidgetReason, EInvalidateWidgetReason::ChildOrder))
	{
		return DrawWidgetChildOrderColor;
	}
	else if (EnumHasAnyFlags(InvalidationInfo.WidgetReason, EInvalidateWidgetReason::Prepass))
	{
		return DrawWidgetPrepassColor;
	}
	else if (EnumHasAnyFlags(InvalidationInfo.WidgetReason, EInvalidateWidgetReason::Layout))
	{
		return DrawWidgetLayoutColor;
	}
	else if (EnumHasAnyFlags(InvalidationInfo.WidgetReason, EInvalidateWidgetReason::RenderTransform))
	{
		return DrawWidgetRenderTransformColor;
	}
	else if (EnumHasAnyFlags(InvalidationInfo.WidgetReason, EInvalidateWidgetReason::Visibility))
	{
		return DrawWidgetVisibilityColor;
	}
	else if (EnumHasAnyFlags(InvalidationInfo.WidgetReason, EInvalidateWidgetReason::Volatility))
	{
		return DrawWidgetVolatilityColor;
	}
	else if (EnumHasAnyFlags(InvalidationInfo.WidgetReason, EInvalidateWidgetReason::Paint))
	{
		return DrawWidgetPaintColor;
	}

	check(false);
	return FLinearColor::Black;
}

FConsoleSlateDebuggerInvalidate::FInvalidationInfo::FInvalidationInfo(const FSlateDebuggingInvalidateArgs& Args, int32 InInvalidationPriority, bool bBuildWidgetName, bool bUseWidgetPathAsName)
	: WidgetInvalidatedId(FConsoleSlateDebuggerUtility::GetId(Args.WidgetInvalidated))
	, WidgetInvalidatorId(FConsoleSlateDebuggerUtility::GetId(Args.WidgetInvalidateInvestigator))
	, WidgetInvalidated(Args.WidgetInvalidated->DoesSharedInstanceExist() ? Args.WidgetInvalidated->AsShared() : TWeakPtr<const SWidget>())
	, WidgetInvalidator(Args.WidgetInvalidateInvestigator && Args.WidgetInvalidateInvestigator->DoesSharedInstanceExist() ? Args.WidgetInvalidateInvestigator->AsShared() : TWeakPtr<const SWidget>())
	, WindowId(FConsoleSlateDebuggerUtility::InvalidWindowId)
	, WidgetReason(Args.InvalidateWidgetReason)
	, InvalidationRootReason(Args.InvalidateInvalidationRootReason)
	, InvalidationPriority(InInvalidationPriority)
	, InvalidationTime(0.0)
	, bIsInvalidatorPaintValid(false)
{
	if (bBuildWidgetName)
	{
		WidgetInvalidatedName = bUseWidgetPathAsName ? FReflectionMetaData::GetWidgetPath(Args.WidgetInvalidated) : FReflectionMetaData::GetWidgetDebugInfo(Args.WidgetInvalidated);
		WidgetInvalidatorName = bUseWidgetPathAsName ? FReflectionMetaData::GetWidgetPath(Args.WidgetInvalidateInvestigator) : FReflectionMetaData::GetWidgetDebugInfo(Args.WidgetInvalidateInvestigator);
	}
}

void FConsoleSlateDebuggerInvalidate::FInvalidationInfo::ReplaceInvalidated(const FSlateDebuggingInvalidateArgs& InArgs, int32 InInvalidationPriority, bool bInBuildWidgetName, bool bInUseWidgetPathAsName)
{
	if (WidgetInvalidatorId == FConsoleSlateDebuggerUtility::InvalidWidgetId)
	{
		WidgetInvalidatorId = WidgetInvalidatedId;
		WidgetInvalidator = MoveTemp(WidgetInvalidated);
		WidgetInvalidatorName = MoveTemp(WidgetInvalidatedName);
	}

	check(InArgs.WidgetInvalidated);
	WidgetInvalidatedId = FConsoleSlateDebuggerUtility::GetId(InArgs.WidgetInvalidated);
	WidgetInvalidated = InArgs.WidgetInvalidated->DoesSharedInstanceExist() ? InArgs.WidgetInvalidated->AsShared() : TWeakPtr<const SWidget>();
	if (bInBuildWidgetName)
	{
		WidgetInvalidatedName = bInUseWidgetPathAsName ? FReflectionMetaData::GetWidgetPath(InArgs.WidgetInvalidated) : FReflectionMetaData::GetWidgetDebugInfo(InArgs.WidgetInvalidated);
	}
	WidgetReason |= InArgs.InvalidateWidgetReason;
	InvalidationRootReason |= InArgs.InvalidateInvalidationRootReason;
	InvalidationPriority = InInvalidationPriority;

}

void FConsoleSlateDebuggerInvalidate::FInvalidationInfo::ReplaceInvalidator(const FSlateDebuggingInvalidateArgs& InArgs, int32 InInvalidationPriority, bool bInBuildWidgetName, bool bInUseWidgetPathAsName)
{
	WidgetInvalidatorId = FConsoleSlateDebuggerUtility::GetId(InArgs.WidgetInvalidateInvestigator);
	if (bInBuildWidgetName)
	{
		WidgetInvalidatorName = bInUseWidgetPathAsName ? FReflectionMetaData::GetWidgetPath(InArgs.WidgetInvalidateInvestigator) : FReflectionMetaData::GetWidgetDebugInfo(InArgs.WidgetInvalidateInvestigator);
	}
	WidgetReason |= InArgs.InvalidateWidgetReason;
	InvalidationRootReason |= InArgs.InvalidateInvalidationRootReason;
	InvalidationPriority = InInvalidationPriority;
}

void FConsoleSlateDebuggerInvalidate::FInvalidationInfo::UpdateInvalidationReason(const FSlateDebuggingInvalidateArgs& Args, int32 InInvalidationPriority)
{
	WidgetReason |= Args.InvalidateWidgetReason;
	InvalidationRootReason |= Args.InvalidateInvalidationRootReason;
	InvalidationPriority = InInvalidationPriority;
}

void FConsoleSlateDebuggerInvalidate::HandleEndFrame()
{
	const double LastTime = FSlateApplicationBase::Get().GetCurrentTime() - CacheDuration;
	for (int32 Index = InvalidationInfos.Num() - 1; Index >= 0; --Index)
	{
		if (InvalidationInfos[Index].InvalidationTime < LastTime)
		{
			InvalidationInfos.RemoveAtSwap(Index);
		}
	}
	if (LastPerformanceThresholdSeconds < LastTime)
	{
		LastPerformanceThresholdFrameCount = 0;
	}

	CleanFrameList();
	ProcessFrameList();
}

void FConsoleSlateDebuggerInvalidate::HandleWidgetInvalidated(const FSlateDebuggingInvalidateArgs& Args)
{
	// Reduce the invalidation tree to single child.
	//Tree:
	//A->B->C [Paint]
	//A->B->C->D [Layout]
	//Z->Y->C->D [Volatility]
	//X->W->C->D [Layout]
	//I->J->K [Paint]
	//Reduce to:
	//A->D [Layout] (ignore X->D because of the incoming order)
	//I->K [Paint]
	//~ depending of the incoming order, it's possible that we have A->C(Paint) and then A->D(Layout)

	if (Args.WidgetInvalidated == nullptr)
	{
		return;
	}

	if (!Args.WidgetInvalidated->GetProxyHandle().IsValid(Args.WidgetInvalidated))
	{
		return;
	}

	const FConsoleSlateDebuggerUtility::TSWidgetId WidgetInvalidatedId = FConsoleSlateDebuggerUtility::GetId(Args.WidgetInvalidated);
	const FConsoleSlateDebuggerUtility::TSWidgetId WidgetInvalidatorId = FConsoleSlateDebuggerUtility::GetId(Args.WidgetInvalidateInvestigator);

	const int32 InvalidationPriority = GetInvalidationPriority(Args.InvalidateWidgetReason, Args.InvalidateInvalidationRootReason);
	if (InvalidationPriority == 0)
	{
		// The invalidation is filtered.
		return;
	}

	FInvalidationInfo* FoundInvalidated = FrameInvalidationInfos.FindByPredicate([WidgetInvalidatedId, WidgetInvalidatorId](const FInvalidationInfo& InvalidationInfo)
		{
			return InvalidationInfo.WidgetInvalidatedId == WidgetInvalidatedId
				&& InvalidationInfo.WidgetInvalidatorId == WidgetInvalidatorId;
		});

	// Is the same invalidation couple already in the list
	if (FoundInvalidated)
	{
		// Is this invalidation is more important for the display?
		//Z->C[RenderTransform] to B->C[Layout]
		//NB we use < instead of <= so only the first incoming invalidation will be considered 
		if (FoundInvalidated->InvalidationPriority < InvalidationPriority)
		{
			FoundInvalidated->UpdateInvalidationReason(Args, InvalidationPriority);
		}
	}
	else
	{
		FoundInvalidated = FrameInvalidationInfos.FindByPredicate([WidgetInvalidatedId](const FInvalidationInfo& InvalidationInfo)
			{
				return InvalidationInfo.WidgetInvalidatedId == WidgetInvalidatedId;
			});

		if (FoundInvalidated)
		{
			// Same invalidated with a better priority, replace the invalidator
			//A->D [Paint] to A->D [Layout]. 
			if (FoundInvalidated->InvalidationPriority < InvalidationPriority)
			{
				FoundInvalidated->ReplaceInvalidator(Args, InvalidationPriority, bShowWidgetList, bUseWidgetPathAsName);
			}
		}
		else
		{
			FoundInvalidated = FrameInvalidationInfos.FindByPredicate([WidgetInvalidatorId](const FInvalidationInfo& InvalidationInfo)
				{
					return InvalidationInfo.WidgetInvalidatedId == WidgetInvalidatorId;
				});

			if (FoundInvalidated)
			{
				// is this a continuation of an existing chain
				if (FoundInvalidated->InvalidationPriority <= InvalidationPriority)
				{
					FoundInvalidated->ReplaceInvalidated(Args, InvalidationPriority, bShowWidgetList, bUseWidgetPathAsName);
				}
			}
			else
			{
				// New element in the chain
				FrameInvalidationInfos.Emplace(Args, InvalidationPriority, bShowWidgetList, bUseWidgetPathAsName);
				FrameRootHandles.AddUnique(Args.WidgetInvalidated->GetProxyHandle().GetInvalidationRootHandle());
			}
		}
	}
}

void FConsoleSlateDebuggerInvalidate::CleanFrameList()
{
	if (bUsePerformanceThreshold)
	{
		const double ThresholdPerformanceSeconds = double(ThresholdPerformanceMs)/1000.0;
		bool bFirstItem = true;
		for (int32 Index = FrameRootHandles.Num() - 1; Index >= 0; --Index)
		{
			FSlateInvalidationRootHandle RootHandle = FrameRootHandles[Index];
			if (RootHandle.GetInvalidationRoot() == nullptr)
			{
				FrameRootHandles.RemoveAtSwap(Index);
			}
			else if (RootHandle.Advanced_GetInvalidationRootNoCheck()->GetPerformanceStat().InvalidationProcessing < ThresholdPerformanceSeconds)
			{
				FrameRootHandles.RemoveAtSwap(Index);
			}
			else
			{
				if (bFirstItem)
				{
					UE_LOG(LogSlateDebugger, Log, TEXT("Slate Performance Threshold reached at frame %" UINT64_FMT), GFrameCounter);
					UE_TRACE_SLATE_BOOKMARK(TEXT("Slate Performance Threshold %" UINT64_FMT), GFrameCounter);
					LastPerformanceThresholdFrameCount = GFrameCounter;
					LastPerformanceThresholdSeconds = FSlateApplicationBase::Get().GetCurrentTime();
				}

				if (bLogInvalidatedWidget)
				{
					if (const SWidget* InvalidationRootAsWidget = RootHandle.Advanced_GetInvalidationRootNoCheck()->GetInvalidationRootWidget())
					{
						const FSlateInvalidationRoot::FPerformanceStat PerformanceStat = RootHandle.Advanced_GetInvalidationRootNoCheck()->GetPerformanceStat();

						UE_LOGF(LogSlateDebugger, Log, "InvalidationRoot: '%ls' Total: %f"
							"   PreUpdate: %f  Attribute: %f  Prepass: %f  Update: %f"
							, *(bUseWidgetPathAsName ? FReflectionMetaData::GetWidgetPath(InvalidationRootAsWidget) : FReflectionMetaData::GetWidgetDebugInfo(InvalidationRootAsWidget))
							, PerformanceStat.InvalidationProcessing
							, PerformanceStat.WidgetsPreUpdate
							, PerformanceStat.WidgetsAttribute
							, PerformanceStat.WidgetsPrepass
							, PerformanceStat.WidgetsUpdate
							);
					}
				}
				bFirstItem = false;
			}
		}
	}
}

void FConsoleSlateDebuggerInvalidate::ProcessFrameList()
{
	const double CurrentTime = FSlateApplicationBase::Get().GetCurrentTime();

	bool bLogOnce = false;
	for (FInvalidationInfo& FrameInvalidationInfo : FrameInvalidationInfos)
	{
		TSharedPtr<const SWidget> InvalidatedWidget = FrameInvalidationInfo.WidgetInvalidated.Pin();
		if (bUsePerformanceThreshold)
		{
			if (InvalidatedWidget == nullptr)
			{
				continue;
			}

			// is the invalidation root reach the performance threshold to be displayed
			FSlateInvalidationRootHandle RootHandle = InvalidatedWidget->GetProxyHandle().GetInvalidationRootHandle();
			if (!FrameRootHandles.Contains(RootHandle))
			{
				continue;
			}
		}

		if (bLogInvalidatedWidget)
		{
			TStringBuilder<512> MessageBuilder;
			MessageBuilder << TEXT("Invalidator: '");
			MessageBuilder << FrameInvalidationInfo.WidgetInvalidatorName;
			MessageBuilder << TEXT("' Invalidated: '");
			MessageBuilder << FrameInvalidationInfo.WidgetInvalidatedName;
			MessageBuilder << TEXT("' Root Reason: '");
			MessageBuilder << LexToString(FrameInvalidationInfo.InvalidationRootReason);
			MessageBuilder << TEXT("' Widget Reason: '");
			MessageBuilder << LexToString(FrameInvalidationInfo.WidgetReason);
			MessageBuilder << TEXT("'");

			UE_LOGF(LogSlateDebugger, Log, "%ls", MessageBuilder.ToString());
		}

		if (InvalidatedWidget)
		{
			FrameInvalidationInfo.WindowId = FConsoleSlateDebuggerUtility::FindWindowId(InvalidatedWidget.Get());
			if (FrameInvalidationInfo.WindowId != FConsoleSlateDebuggerUtility::InvalidWindowId)
			{
				FrameInvalidationInfo.DisplayColor = GetColor(FrameInvalidationInfo);
				FrameInvalidationInfo.InvalidationTime = CurrentTime;
				FrameInvalidationInfo.InvalidatedPaintLocation = InvalidatedWidget->GetPersistentState().AllottedGeometry.GetAbsolutePosition();
				FrameInvalidationInfo.InvalidatedPaintSize = InvalidatedWidget->GetPersistentState().AllottedGeometry.GetAbsoluteSize();

				if (TSharedPtr<const SWidget> Invalidator = FrameInvalidationInfo.WidgetInvalidator.Pin())
				{
					FrameInvalidationInfo.bIsInvalidatorPaintValid = true;
					FrameInvalidationInfo.InvalidatorPaintLocation = Invalidator->GetPersistentState().AllottedGeometry.GetAbsolutePosition();
					FrameInvalidationInfo.InvalidatorPaintSize = Invalidator->GetPersistentState().AllottedGeometry.GetAbsoluteSize();
				}
				InvalidationInfos.Emplace(MoveTemp(FrameInvalidationInfo));
			}
		}
	}
	FrameInvalidationInfos.Reset();
	FrameRootHandles.Reset();
}

void FConsoleSlateDebuggerInvalidate::HandlePaintDebugInfo(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId)
{
	++InOutLayerId;

	const FConsoleSlateDebuggerUtility::TSWindowId PaintWindow = FConsoleSlateDebuggerUtility::GetId(InOutDrawElements.GetPaintWindow());
	FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("SmallFont");
	const FSlateBrush* BoxBrush = FCoreStyle::Get().GetBrush("WhiteBrush");
	const FSlateBrush* CheckerboardBrush = FCoreStyle::Get().GetBrush("Checkerboard");
	FontInfo.OutlineSettings.OutlineSize = 1;

	CacheDuration = FMath::Max(CacheDuration, 0.01f);
	const double SlateApplicationCurrentTime = FSlateApplicationBase::Get().GetCurrentTime();

	FVector2f TextElementLocation {16.f, 64.f};
	auto MakeText = [&](const FString& Text, const FLinearColor& Color)
	{
		FSlateDrawElement::MakeText(
			InOutDrawElements
			, InOutLayerId
			, InAllottedGeometry.ToPaintGeometry(FVector2f(1.f, 1.f), FSlateLayoutTransform(TextElementLocation))
			, Text
			, FontInfo
			, ESlateDrawEffect::None
			, Color);
		TextElementLocation.Y += 12.f;
	};

	TArray<FConsoleSlateDebuggerUtility::TSWidgetId, TInlineAllocator<32>> AlreadyProcessedInvalidatedId;
	TArray<FConsoleSlateDebuggerUtility::TSWidgetId, TInlineAllocator<32>> InvalidationInfoIndexToPaint;
	for (int32 Index = 0; Index < InvalidationInfos.Num(); ++Index)
	{
		const FInvalidationInfo& InvalidationInfo = InvalidationInfos[Index];
		if (InvalidationInfo.WindowId != PaintWindow)
		{
			continue;
		}
		if (AlreadyProcessedInvalidatedId.Contains(InvalidationInfo.WidgetInvalidatedId))
		{
			continue;
		}
		if (InvalidationInfo.WidgetInvalidatorId != FConsoleSlateDebuggerUtility::InvalidWidgetId && AlreadyProcessedInvalidatedId.Contains(InvalidationInfo.WidgetInvalidatorId))
		{
			continue;
		}
		AlreadyProcessedInvalidatedId.Add(InvalidationInfo.WidgetInvalidatedId);
		if (InvalidationInfo.WidgetInvalidatorId != FConsoleSlateDebuggerUtility::InvalidWidgetId)
		{
			AlreadyProcessedInvalidatedId.Add(InvalidationInfo.WidgetInvalidatorId);
		}

		const float LerpValue = FMath::Clamp((float)(SlateApplicationCurrentTime - InvalidationInfo.InvalidationTime) / CacheDuration, 0.0f, 1.0f);
		const FLinearColor ColorWithOpacity = InvalidationInfo.DisplayColor.CopyWithNewOpacity(FMath::InterpExpoOut(1.0f, 0.2f, LerpValue));

		{
			const FGeometry InvalidatedGeometry = FGeometry::MakeRoot(InvalidationInfo.InvalidatedPaintSize, FSlateLayoutTransform(1.f, InvalidationInfo.InvalidatedPaintLocation));
			const FPaintGeometry InvalidatedPaintGeometry = InvalidatedGeometry.ToPaintGeometry();

			FSlateDrawElement::MakeBox(
				InOutDrawElements,
				InOutLayerId,
				InvalidatedPaintGeometry,
				BoxBrush,
				ESlateDrawEffect::None,
				ColorWithOpacity);
		}

		if (InvalidationInfo.bIsInvalidatorPaintValid)
		{
			const FGeometry InvalidatorGeometry = FGeometry::MakeRoot(InvalidationInfo.InvalidatorPaintSize, FSlateLayoutTransform(1.f, InvalidationInfo.InvalidatorPaintLocation));
			const FPaintGeometry InvalidatorPaintGeometry = InvalidatorGeometry.ToPaintGeometry();

			FSlateDrawElement::MakeDebugQuad(
				InOutDrawElements,
				InOutLayerId,
				InvalidatorPaintGeometry,
				ColorWithOpacity);
			FSlateDrawElement::MakeBox(
				InOutDrawElements,
				InOutLayerId,
				InvalidatorPaintGeometry,
				CheckerboardBrush,
				ESlateDrawEffect::None,
				ColorWithOpacity);
		}

		if (bShowWidgetList)
		{
			InvalidationInfoIndexToPaint.Add(Index);
		}
	}


	if (bShowLegend)
	{
		MakeText(TEXT("Invalidation Root - Root"), DrawRootRootColor);
		MakeText(TEXT("Invalidation Root - Child Order"), DrawRootChildOrderColor);
		MakeText(TEXT("Invalidation Root - Screen Position"), DrawRootScreenPositionColor);
		MakeText(TEXT("Widget - Layout"), DrawWidgetLayoutColor);
		MakeText(TEXT("Widget - Prepass"), DrawWidgetPrepassColor);
		MakeText(TEXT("Widget - Paint"), DrawWidgetPaintColor);
		MakeText(TEXT("Widget - Volatility"), DrawWidgetVolatilityColor);
		MakeText(TEXT("Widget - Child Order"), DrawWidgetChildOrderColor);
		MakeText(TEXT("Widget - Render Transform"), DrawWidgetRenderTransformColor);
		MakeText(TEXT("Widget - Visibility"), DrawWidgetVisibilityColor);
		TextElementLocation.Y += 20.f;
	}

	if (LastPerformanceThresholdFrameCount > 0)
	{
		FSlateFontInfo NormalFontInfo = FCoreStyle::Get().GetFontStyle("NormalFont");
		FSlateDrawElement::MakeText(
			InOutDrawElements
			, InOutLayerId
			, InAllottedGeometry.ToPaintGeometry(FVector2f(1.f, 1.f), FSlateLayoutTransform(TextElementLocation))
			, FString::Printf(TEXT("Slate Performance Threshold Reached: %" UINT64_FMT), LastPerformanceThresholdFrameCount)
			, NormalFontInfo
			, ESlateDrawEffect::None
			, FLinearColor::Red);
		TextElementLocation.Y += 20.f;
	}

	if (bShowWidgetList)
	{
		int32 MaxCount = FMath::Min(InvalidationInfoIndexToPaint.Num(), MaxNumberOfWidgetInList);
		for (int32 Index = 0; Index < MaxCount; ++Index)
		{
			const int32 IndexValue = InvalidationInfoIndexToPaint[Index];
			const FInvalidationInfo& InvalidationInfo = InvalidationInfos[IndexValue];

			FString WidgetDisplayName = FString::Printf(TEXT("'%s' -> '%s'"), *InvalidationInfo.WidgetInvalidatorName, *InvalidationInfo.WidgetInvalidatedName);
			MakeText(WidgetDisplayName, InvalidationInfo.DisplayColor);
		}

		if (InvalidationInfoIndexToPaint.Num() > MaxNumberOfWidgetInList)
		{
			TextElementLocation.Y += 12.f;
			FString WidgetDisplayName = FString::Printf(TEXT("   %d more invalidations"), InvalidationInfoIndexToPaint.Num() - MaxNumberOfWidgetInList);
			MakeText(WidgetDisplayName, FLinearColor::White);
		}
	}
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_SLATE_DEBUGGING