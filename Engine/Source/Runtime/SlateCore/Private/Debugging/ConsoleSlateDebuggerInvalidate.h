// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Debugging/SlateDebugging.h"

#if WITH_SLATE_DEBUGGING

#include "CoreMinimal.h"
#include "Debugging/ConsoleSlateDebuggerUtility.h"
#include "Delegates/Delegate.h"
#include "FastUpdate/SlateInvalidationRootHandle.h"
#include "HAL/IConsoleManager.h"

struct FGeometry;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class SWidget;
class SWindow;

/**
 * Allows debugging the invalidation from the console.
 * Basics:
 *   Start - SlateDebugger.Invalidate.Start
 *   Stop  - SlateDebugger.Invalidate.Stop
 */
class FConsoleSlateDebuggerInvalidate
{
public:
	FConsoleSlateDebuggerInvalidate();
	~FConsoleSlateDebuggerInvalidate();

	void StartDebugging();
	void StopDebugging();
	bool IsEnabled() const { return bEnabled; }

	void LoadConfig();
	void SaveConfig();

private:	
	struct FInvalidationInfo
	{
		FInvalidationInfo(const FSlateDebuggingInvalidateArgs& Args, int32 InvalidationPriority, bool bBuildWidgetName, bool bUseWidgetPathAsName);

		void ReplaceInvalidated(const FSlateDebuggingInvalidateArgs& Args, int32 InvalidationPriority, bool bBuildWidgetName, bool bUseWidgetPathAsName);
		void ReplaceInvalidator(const FSlateDebuggingInvalidateArgs& Args, int32 InvalidationPriority, bool bBuildWidgetName, bool bUseWidgetPathAsName);
		void UpdateInvalidationReason(const FSlateDebuggingInvalidateArgs& Args, int32 InInvalidationPriority);

		FConsoleSlateDebuggerUtility::TSWidgetId WidgetInvalidatedId;
		FConsoleSlateDebuggerUtility::TSWidgetId WidgetInvalidatorId;
		TWeakPtr<const SWidget> WidgetInvalidated;
		TWeakPtr<const SWidget> WidgetInvalidator;
		FConsoleSlateDebuggerUtility::TSWindowId WindowId;
		FString WidgetInvalidatedName;
		FString WidgetInvalidatorName;
		FVector2f InvalidatedPaintLocation;
		FVector2f InvalidatedPaintSize;
		FVector2f InvalidatorPaintLocation;
		FVector2f InvalidatorPaintSize;
		EInvalidateWidgetReason WidgetReason;
		ESlateDebuggingInvalidateRootReason InvalidationRootReason;
		int32 InvalidationPriority;
		FLinearColor DisplayColor;
		double InvalidationTime;
		bool bIsInvalidatorPaintValid;
	};

	void HandleEnabled(IConsoleVariable* Variable);
	void HandledConfigChanged(IConsoleVariable* Variable);
	void HandleSetInvalidateWidgetReasonFilter(const TArray<FString>& Params);
	void HandleSetInvalidateRootReasonFilter(const TArray<FString>& Params);

	void HandleEndFrame();
	void HandleWidgetInvalidated(const FSlateDebuggingInvalidateArgs& Args);
	void HandlePaintDebugInfo(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId);

	int32 GetInvalidationPriority(EInvalidateWidgetReason InvalidationInfo, ESlateDebuggingInvalidateRootReason InvalidationRootReason) const;
	const FLinearColor& GetColor(const FInvalidationInfo& InvalidationInfo) const;
	void CleanFrameList();
	void ProcessFrameList();

private:
	bool bEnabled;
	bool bEnabledCVarValue;

	//~ Settings
	bool bShowWidgetList;
	bool bUseWidgetPathAsName;
	bool bShowLegend;
	bool bLogInvalidatedWidget;
	bool bUsePerformanceThreshold;
	EInvalidateWidgetReason InvalidateWidgetReasonFilter;
	ESlateDebuggingInvalidateRootReason InvalidateRootReasonFilter;
	FLinearColor DrawRootRootColor;
	FLinearColor DrawRootChildOrderColor;
	FLinearColor DrawRootScreenPositionColor;
	FLinearColor DrawWidgetPrepassColor;
	FLinearColor DrawWidgetLayoutColor;
	FLinearColor DrawWidgetPaintColor;
	FLinearColor DrawWidgetVolatilityColor;
	FLinearColor DrawWidgetChildOrderColor;
	FLinearColor DrawWidgetRenderTransformColor;
	FLinearColor DrawWidgetVisibilityColor;
	int32 MaxNumberOfWidgetInList;
	float CacheDuration;
	float ThresholdPerformanceMs;

	//~ Console objects
	FAutoConsoleCommand StartCommand;
	FAutoConsoleCommand StopCommand;
	FAutoConsoleVariableRef EnabledRefCVar;
	FAutoConsoleVariableRef ShowLegendRefCVar;
	FAutoConsoleVariableRef ShowWidgetsNameListRefCVar;
	FAutoConsoleVariableRef LogInvalidatedWidgetRefCVar;
	FAutoConsoleVariableRef UsePerformanceThresholdRefCVar;
	FAutoConsoleVariableRef ThresholdPerformanceRefCVar;
	FAutoConsoleCommand SetInvalidateWidgetReasonFilterCommand;
	FAutoConsoleCommand SetInvalidateRootReasonFilterCommand;
	
	uint64 LastPerformanceThresholdFrameCount;
	double LastPerformanceThresholdSeconds;
	TArray<FInvalidationInfo> InvalidationInfos;
	TArray<FInvalidationInfo> FrameInvalidationInfos;
	TArray<FSlateInvalidationRootHandle> FrameRootHandles;
};

#endif //WITH_SLATE_DEBUGGING