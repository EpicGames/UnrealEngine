// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleSlateDebuggerInvalidationRoot.h"
#include "ConsoleSlateDebugger.h"

#if WITH_SLATE_DEBUGGING

#include "Application/SlateApplicationBase.h"
#include "CoreGlobals.h"
#include "Debugging/SlateDebugging.h"
#include "FastUpdate/SlateInvalidationRootList.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/StringBuilder.h"
#include "Styling/CoreStyle.h"
#include "Types/ReflectionMetadata.h"
#include "Widgets/SWidget.h"


#define LOCTEXT_NAMESPACE "ConsoleSlateDebuggerInvalidationRoot"

FConsoleSlateDebuggerInvalidationRoot::FConsoleSlateDebuggerInvalidationRoot()
	: bEnabled(false)
	, bEnabledCVarValue(false)
	, bDisplayInvalidationRootList(true)
	, bUseWidgetPathAsName(false)
	, bShowLegend(false)
	, bShowQuad(true)
	, DrawSlowPathColor(FColorList::Red)
	, DrawFastPathColor(FColorList::Green)
	, DrawNoneColor(FColorList::Blue)
	, MaxNumberOfWidgetInList(20)
	, CacheDuration(2.0f)
	, StartCommand(
		TEXT("SlateDebugger.InvalidationRoot.Start"),
		TEXT("Start the Invalidation Root widget debug tool. It shows when Invalidation Roots are using the slow or the fast path."),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidationRoot::StartDebugging))
	, StopCommand(
		TEXT("SlateDebugger.InvalidationRoot.Stop"),
		TEXT("Stop the Invalidation Root widget debug tool."),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidationRoot::StopDebugging))
	, EnabledRefCVar(
		TEXT("SlateDebugger.InvalidationRoot.Enable")
		, bEnabledCVarValue
		, TEXT("Start/Stop the Invalidation Root widget debug tool. It shows when Invalidation Roots are using the slow or the fast path.")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidationRoot::HandleEnabled))
	, ToggleLegendCommand(
		TEXT("SlateDebugger.InvalidationRoot.ToggleLegend"),
		TEXT("Option to display the color legend."),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidationRoot::ToggleLegend))
	, ToogleWidgetsNameListCommand(
		TEXT("SlateDebugger.InvalidationRoot.ToggleWidgetNameList"),
		TEXT("Option to display the name of the Invalidation Root."),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerInvalidationRoot::ToggleWidgetNameList))

{
	GConfig->GetBool(TEXT("SlateDebugger.InvalidationRoot"), TEXT("bDisplayInvalidationRootList"), bDisplayInvalidationRootList, *GEditorPerProjectIni);
	GConfig->GetBool(TEXT("SlateDebugger.InvalidationRoot"), TEXT("bUseWidgetPathAsName"), bUseWidgetPathAsName, *GEditorPerProjectIni);
	GConfig->GetBool(TEXT("SlateDebugger.InvalidationRoot"), TEXT("bShowLegend"), bShowLegend, *GEditorPerProjectIni);
	GConfig->GetBool(TEXT("SlateDebugger.InvalidationRoot"), TEXT("bShowQuad"), bShowQuad, *GEditorPerProjectIni);
	FColor TmpColor;
	if (GConfig->GetColor(TEXT("SlateDebugger.InvalidationRoot"), TEXT("DrawSlowPathColor"), TmpColor, *GEditorPerProjectIni))
	{
		DrawSlowPathColor = TmpColor;
	}
	if (GConfig->GetColor(TEXT("SlateDebugger.InvalidationRoot"), TEXT("DrawFastPathColor"), TmpColor, *GEditorPerProjectIni))
	{
		DrawFastPathColor = TmpColor;
	}
	if (GConfig->GetColor(TEXT("SlateDebugger.InvalidationRoot"), TEXT("DrawNoneColor"), TmpColor, *GEditorPerProjectIni))
	{
		DrawNoneColor = TmpColor;
	}
	GConfig->GetInt(TEXT("SlateDebugger.InvalidationRoot"), TEXT("MaxNumberOfWidgetInList"), MaxNumberOfWidgetInList, *GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("SlateDebugger.InvalidationRoot"), TEXT("CacheDuration"), CacheDuration, *GEditorPerProjectIni);
}

FConsoleSlateDebuggerInvalidationRoot::~FConsoleSlateDebuggerInvalidationRoot()
{
	StopDebugging();
}

void FConsoleSlateDebuggerInvalidationRoot::SaveConfig()
{
	GConfig->SetBool(TEXT("SlateDebugger.InvalidationRoot"), TEXT("bDisplayInvalidationRootList"), bDisplayInvalidationRootList, *GEditorPerProjectIni);
	GConfig->SetBool(TEXT("SlateDebugger.InvalidationRoot"), TEXT("bUseWidgetPathAsName"), bUseWidgetPathAsName, *GEditorPerProjectIni);
	GConfig->SetBool(TEXT("SlateDebugger.InvalidationRoot"), TEXT("bShowQuad"), bShowQuad, *GEditorPerProjectIni);
	FColor TmpColor = DrawSlowPathColor.ToFColor(true);
	GConfig->SetColor(TEXT("SlateDebugger.InvalidationRoot"), TEXT("DrawSlowPathColor"), TmpColor, *GEditorPerProjectIni);
	TmpColor = DrawFastPathColor.ToFColor(true);
	GConfig->SetColor(TEXT("SlateDebugger.InvalidationRoot"), TEXT("DrawFastPathColor"), TmpColor, *GEditorPerProjectIni);
	TmpColor = DrawNoneColor.ToFColor(true);
	GConfig->SetColor(TEXT("SlateDebugger.InvalidationRoot"), TEXT("DrawNoneColor"), TmpColor, *GEditorPerProjectIni);
	GConfig->SetInt(TEXT("SlateDebugger.InvalidationRoot"), TEXT("MaxNumberOfWidgetInList"), MaxNumberOfWidgetInList, *GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("SlateDebugger.InvalidationRoot"), TEXT("CacheDuration"), CacheDuration, *GEditorPerProjectIni);
}

void FConsoleSlateDebuggerInvalidationRoot::StartDebugging()
{
	if (!bEnabled)
	{
		bEnabled = true;
		InvaliadatedRoots.Empty();

		FSlateDebugging::PaintDebugElements.AddRaw(this, &FConsoleSlateDebuggerInvalidationRoot::HandlePaintDebugInfo);
	}
	bEnabledCVarValue = bEnabled;
}

void FConsoleSlateDebuggerInvalidationRoot::StopDebugging()
{
	if (bEnabled)
	{
		FSlateDebugging::PaintDebugElements.RemoveAll(this);

		InvaliadatedRoots.Empty();
		bEnabled = false;
	}
	bEnabledCVarValue = bEnabled;
}

void FConsoleSlateDebuggerInvalidationRoot::HandleEnabled(IConsoleVariable* Variable)
{
	if (bEnabledCVarValue)
	{
		StartDebugging();
	}
	else
	{
		StopDebugging();
	}
}

void FConsoleSlateDebuggerInvalidationRoot::ToggleLegend()
{
	bShowLegend = !bShowLegend;
	SaveConfig();
}

void FConsoleSlateDebuggerInvalidationRoot::ToggleWidgetNameList()
{
	bDisplayInvalidationRootList = !bDisplayInvalidationRootList;
	SaveConfig();
}

const FLinearColor& FConsoleSlateDebuggerInvalidationRoot::GetColor(ESlateInvalidationPaintType PaintType) const
{
	switch (PaintType)
	{
	case ESlateInvalidationPaintType::None:
		return DrawNoneColor;
	case ESlateInvalidationPaintType::Slow:
		return DrawSlowPathColor;
	case ESlateInvalidationPaintType::Fast:
		return DrawFastPathColor;
	}
	check(false);
	return FLinearColor::Black;
};

void FConsoleSlateDebuggerInvalidationRoot::HandlePaintDebugInfo(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId)
{
	++InOutLayerId;
	
	const TMap<int32, FSlateInvalidationRoot*>& AllInvalidationRootInstance = GSlateInvalidationRootListInstance.GetInvalidationRoots();
	const FConsoleSlateDebuggerUtility::TSWindowId PaintWindow = FConsoleSlateDebuggerUtility::GetId(InOutDrawElements.GetPaintWindow());
	const FSlateBrush* QuadBrush = FCoreStyle::Get().GetBrush(TEXT("FocusRectangle"));
	FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("SmallFont");
	FontInfo.OutlineSettings.OutlineSize = 1;

	int32 NumberOfWidget = 0;
	CacheDuration = FMath::Max(CacheDuration, 0.01f);
	const double SlateApplicationCurrentTime = FSlateApplicationBase::Get().GetCurrentTime();


	float TextElementY = 48.f;
	auto MakeText = [&](const FString& Text, const FVector2f& Location, const FLinearColor& Color)
	{
		FSlateDrawElement::MakeText(
			InOutDrawElements
			, InOutLayerId
			, InAllottedGeometry.ToPaintGeometry(FVector2f(1.f, 1.f), FSlateLayoutTransform(Location))
			, Text
			, FontInfo
			, ESlateDrawEffect::None
			, Color);
	};

	if (bShowLegend)
	{
		//const TSharedRef<FSlateFontMeasure>& FontMeasureService = FSlateApplicationBase::Get().GetRenderer()->GetFontMeasureService();
		//FVector2f FontSize = FontMeasureService->Measure(TEXT("No Paint occurred"), FontInfo);
		//FontSize.Y *= 3;
		//const FSlateBrush* BoxBrush = FCoreStyle::Get().GetBrush("WhiteBrush");
		//const FGeometry Geometry = FGeometry::MakeRoot(FontSize, FSlateLayoutTransform(1.f, FVector2f(10.f, 10.f + 0.f)));

		//FSlateDrawElement::MakeBox(
		//	InOutDrawElements,
		//	InOutLayerId,
		//	Geometry.ToPaintGeometry(),
		//	BoxBrush,
		//	ESlateDrawEffect::None,
		//	FLinearColor::Black);

		MakeText(TEXT("Slow Path"), FVector2f(10.f, 10.f+0.f), DrawSlowPathColor);
		MakeText(TEXT("Fast Path"), FVector2f(10.f, 10.f+12.f), DrawFastPathColor);
		MakeText(TEXT("No Paint occurred"), FVector2f(10.f, 10.f+24.f), DrawNoneColor);
		TextElementY += 36.f;
	}

	for (const auto& Itt : AllInvalidationRootInstance)
	{
		// Do not show the invalidation root that are not enabled
		if (!Itt.Value->GetInvalidationRootWidget()->Advanced_IsInvalidationRoot())
		{
			continue;
		}

		const ESlateInvalidationPaintType LastPaintType = Itt.Value->GetLastPaintType();
		FConsoleSlateDebuggerUtility::TSWindowId WindowId = FConsoleSlateDebuggerUtility::InvalidWindowId;
		{
			FInvalidatedInfo* FoundInvalidationInfo = InvaliadatedRoots.Find(Itt.Key);
			if (FoundInvalidationInfo)
			{
				WindowId = FoundInvalidationInfo->WindowId;
			}
			else
			{
				// Fetch and cache the window id for the invalidation root
				WindowId = FConsoleSlateDebuggerUtility::FindWindowId(Itt.Value->GetInvalidationRootWidget());
				if (WindowId != FConsoleSlateDebuggerUtility::InvalidWindowId)
				{
					FInvalidatedInfo Info;
					Info.WindowId = WindowId;
					Info.PaintType = LastPaintType;
					Info.FlashingSeconds = 0.0;
					InvaliadatedRoots.Add(Itt.Key, Info);
				}
			}	
		}
		
		if (WindowId == PaintWindow)
		{
			FLinearColor DrawColor = GetColor(LastPaintType);
			float LerpValue = 1.0f;

			FInvalidatedInfo& FoundInvalidationInfo = InvaliadatedRoots[Itt.Key];
			// If we went from fast to slow or to none, flash it on screen
			if (LastPaintType != FoundInvalidationInfo.PaintType)
			{
				if (LastPaintType == ESlateInvalidationPaintType::Slow || LastPaintType == ESlateInvalidationPaintType::Fast)
				{
					if (FoundInvalidationInfo.PaintType != ESlateInvalidationPaintType::Slow)
					{
						FoundInvalidationInfo.FlashingSeconds = SlateApplicationCurrentTime;
						FoundInvalidationInfo.FlashingColor = DrawColor;
					}
				}
			}
			FoundInvalidationInfo.PaintType = LastPaintType;

			const float DeltaTime = (float)(SlateApplicationCurrentTime - FoundInvalidationInfo.FlashingSeconds);
			const bool bLearp = DeltaTime <= CacheDuration;
			if (bLearp)
			{
				DrawColor = FoundInvalidationInfo.FlashingColor;
				LerpValue = FMath::Clamp(DeltaTime / CacheDuration, 0.1f, 1.0f);
			}

			const SWidget* Widget = Itt.Value->GetInvalidationRootWidget();
			const FGeometry& AllottedGeometry = Widget->GetPersistentState().AllottedGeometry;
			const FGeometry Geometry = FGeometry::MakeRoot(AllottedGeometry.GetAbsoluteSize(), FSlateLayoutTransform(1.f, AllottedGeometry.GetAbsolutePosition()));
			const FPaintGeometry PaintGeometry = Geometry.ToPaintGeometry();
			const FLinearColor ColorWithOpacity = DrawColor.CopyWithNewOpacity(FMath::InterpExpoOut(1.0f, 0.2f, LerpValue));

			if (bShowQuad)
			{
				FSlateDrawElement::MakeDebugQuad(
					InOutDrawElements,
					InOutLayerId,
					PaintGeometry,
					ColorWithOpacity);
			}

			FSlateDrawElement::MakeBox(
				InOutDrawElements,
				InOutLayerId,
				PaintGeometry,
				QuadBrush,
				ESlateDrawEffect::None,
				ColorWithOpacity);
				
			FString WidgetId =  FString::Printf(TEXT("Id:('%d')"), Itt.Key);
			FSlateDrawElement::MakeText(
				InOutDrawElements,
				InOutLayerId,
				PaintGeometry,
				*WidgetId,
				FontInfo,
				ESlateDrawEffect::None,
				DrawColor);

			if (bDisplayInvalidationRootList)
			{
				if (NumberOfWidget < MaxNumberOfWidgetInList)
				{
					FString WidgetDisplayName = FString::Printf(TEXT("Id:('%d') - %s"), Itt.Key, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
					MakeText(WidgetDisplayName, FVector2f(0.f, (12.f * NumberOfWidget) + TextElementY), DrawColor);
				}
			}
			++NumberOfWidget;
		}
	}

	if (bDisplayInvalidationRootList && NumberOfWidget == MaxNumberOfWidgetInList)
	{
		FString WidgetDisplayName = FString::Printf(TEXT("   %d more invalidation root"), NumberOfWidget - MaxNumberOfWidgetInList);
		MakeText(WidgetDisplayName, FVector2f(0.f, (12.f * NumberOfWidget) + TextElementY), FLinearColor::White);
	}
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_SLATE_DEBUGGING
