// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Debugging/SlateDebugging.h"

#if WITH_SLATE_DEBUGGING

#include "CoreMinimal.h"
#include "Debugging/ConsoleSlateDebuggerUtility.h"
#include "Delegates/Delegate.h"
#include "FastUpdate/SlateInvalidationRoot.h"
#include "HAL/IConsoleManager.h"

struct FGeometry;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class SWidget;
class SWindow;

/**
 * Allows debugging the SlateInvalidationRoot from the console.
 * Basics:
 *   Start - SlateDebugger.InvalidationRoot.Start
 *   Stop  - SlateDebugger.InvalidationRoot.Stop
 */
class FConsoleSlateDebuggerInvalidationRoot
{
public:
	FConsoleSlateDebuggerInvalidationRoot();
	~FConsoleSlateDebuggerInvalidationRoot();

	void StartDebugging();
	void StopDebugging();
	bool IsEnabled() const { return bEnabled; }

	void SaveConfig();

private:
	void ToggleLegend();
	void ToggleWidgetNameList();
	void HandleEnabled(IConsoleVariable* Variable);
	void HandlePaintDebugInfo(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId);

	const FLinearColor& GetColor(ESlateInvalidationPaintType PaintType) const;

private:
	bool bEnabled;
	bool bEnabledCVarValue;

	//~ Settings
	bool bDisplayInvalidationRootList;
	bool bUseWidgetPathAsName;
	bool bShowLegend;
	bool bShowQuad;
	FLinearColor DrawSlowPathColor;
	FLinearColor DrawFastPathColor;
	FLinearColor DrawNoneColor;
	int32 MaxNumberOfWidgetInList;
	float CacheDuration;

	//~ Console objects
	FAutoConsoleCommand StartCommand;
	FAutoConsoleCommand StopCommand;
	FAutoConsoleVariableRef EnabledRefCVar;
	FAutoConsoleCommand ToggleLegendCommand;
	FAutoConsoleCommand ToogleWidgetsNameListCommand;

	struct FInvalidatedInfo
	{
		FConsoleSlateDebuggerUtility::TSWindowId WindowId;
		ESlateInvalidationPaintType PaintType;
		FLinearColor FlashingColor;
		double FlashingSeconds;
	};

	using TInvalidatedMap = TMap<int32, FInvalidatedInfo>;
	TInvalidatedMap InvaliadatedRoots;
};

#endif //WITH_SLATE_DEBUGGING