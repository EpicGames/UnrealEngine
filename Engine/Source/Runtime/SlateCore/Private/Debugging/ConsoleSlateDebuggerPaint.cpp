// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleSlateDebuggerPaint.h"

#if WITH_SLATE_DEBUGGING

#include "CoreGlobals.h"
#include "Debugging/SlateDebugging.h"
#include "Layout/WidgetPath.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"

#define LOCTEXT_NAMESPACE "ConsoleSlateDebuggerPaint"

FConsoleSlateDebuggerPaint::FConsoleSlateDebuggerPaint()
	: FConsoleSlateDebuggerPassBase()
	, bLogWarningIfWidgetIsPaintedMoreThanOnce(true)
	, EnabledRefCVar(
		TEXT("SlateDebugger.Paint.Enable")
		, bEnabled
		, TEXT("Start/Stop the painted widget debug tool. It shows when widgets are painted.")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPaint::HandleEnabled))
	, ShowPaintWidgetCommand(
		TEXT("SlateDebugger.Paint.Start")
		, TEXT("Start the painted widget debug tool. Use to show widget that have been painted this frame.")
		, FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerPaint::StartDebugging))
	, HidePaintWidgetCommand(
		TEXT("SlateDebugger.Paint.Stop")
		, TEXT("Stop the painted widget debug tool.")
		, FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerPaint::StopDebugging))
	, LogPaintedWidgetOnceCommand(
		TEXT("SlateDebugger.Paint.LogOnce")
		, TEXT("Log the names of all widgets that were painted during the last update.")
		, FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerPaint::HandleLogOnce))
	, EnableWidgetsNameListRefCVar(
		TEXT("SlateDebugger.Paint.EnableWidgetNameList")
		, bDisplayWidgetsNameList
		, TEXT("Start/Stop displaying the name of the widgets that have been painted.")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPaint::SaveConfigOnVariableChanged))
	, ToggleWidgetsNameListCommand(
		TEXT("SlateDebugger.Paint.ToggleWidgetNameList")
		, TEXT("Option displaying the name of the widgets that have been painted.")
		, FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerPaint::HandleToggleWidgetNameList))
	, MaxNumberOfWidgetInListRefCVar(
		TEXT("SlateDebugger.Paint.MaxNumberOfWidgetDisplayedInList")
		, MaxNumberOfWidgetInList
		, TEXT("The max number of widgets that will be displayed when DisplayWidgetNameList is active.")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPaint::SaveConfigOnVariableChanged))
	, LogWarningIfWidgetIsPaintedMoreThanOnceRefCVar(
		TEXT("SlateDebugger.Paint.LogWarningIfWidgetIsPaintedMoreThanOnce")
		, bLogWarningIfWidgetIsPaintedMoreThanOnce
		, TEXT("Option to log a warning if a widget is painted more than once in a single frame.")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPaint::SaveConfigOnVariableChanged))
	, OnlyProjectContentRefCVar(
		TEXT("SlateDebugger.Paint.OnlyProjectContent")
		, bDebugProjectContentOnly
		, TEXT("Only debug the project's contents (exclude editor, chrome, diagnostic output)")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPaint::RebuildWidgetListOnVariableChanged))
	, OnlyGameWindowDepCVar(
		TEXT("SlateDebugger.Paint.OnlyGameWindow"),
		TEXT("SlateDebugger.Paint.OnlyProjectContent"),
		TEXT("5.8"),
		EShadowCVarBehavior::Warn)
	, OnlyInvalidationRootRefCVar(
		TEXT("SlateDebugger.Paint.OnlyInvalidationRoot")
		, bInvalidationRootOnly
		, TEXT("Option to only debug widgets within an Invalidation Root")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPaint::RebuildWidgetListOnVariableChanged))
	, DrawBorderEnabledRefCVar(
		TEXT("SlateDebugger.Paint.DrawBorder")
		, bDrawBorder
		, TEXT("Draw a border around the widgets being painted")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPaint::SaveConfigOnVariableChanged))
	, DrawFillEnabledRefCVar(
		TEXT("SlateDebugger.Paint.DrawFill")
		, bDrawBox
		, TEXT("Fill the widgets being painted")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPaint::SaveConfigOnVariableChanged))
{
}

FConsoleSlateDebuggerPaint::~FConsoleSlateDebuggerPaint()
{
	// StopDebugging will end up calling the virtual function GetEnabledCVar()
	StopDebugging();
}

void FConsoleSlateDebuggerPaint::LoadConfig()
{
	FConsoleSlateDebuggerPassBase::LoadConfig();
	
	const FString Section = *GetConfigSection();
	GConfig->GetBool(*Section, TEXT("bLogWarningIfWidgetIsPaintedMoreThanOnce"), bLogWarningIfWidgetIsPaintedMoreThanOnce, *GEditorPerProjectIni);

	OnlyProjectContentRefCVar->Set(bDebugProjectContentOnly, ECVF_SetByProjectSetting);
	OnlyInvalidationRootRefCVar->Set(bInvalidationRootOnly, ECVF_SetByProjectSetting);
	EnableWidgetsNameListRefCVar->Set(bDisplayWidgetsNameList, ECVF_SetByProjectSetting);
	DrawFillEnabledRefCVar->Set(bDrawBox, ECVF_SetByProjectSetting);
	DrawBorderEnabledRefCVar->Set(bDrawBorder, ECVF_SetByProjectSetting);
}

void FConsoleSlateDebuggerPaint::SaveConfig()
{
	FConsoleSlateDebuggerPassBase::SaveConfig();
	
	const FString Section = *GetConfigSection();
	GConfig->SetBool(*Section, TEXT("bLogWarningIfWidgetIsPaintedMoreThanOnce"), bLogWarningIfWidgetIsPaintedMoreThanOnce, *GEditorPerProjectIni);
}

void FConsoleSlateDebuggerPaint::StartDebugging_Internal()
{
	FConsoleSlateDebuggerPassBase::StartDebugging_Internal();
	
	FSlateDebugging::EndWidgetPaint.AddRaw(this, &FConsoleSlateDebuggerPaint::HandleEndWidgetPaint);
}

void FConsoleSlateDebuggerPaint::StopDebugging_Internal()
{
	FSlateDebugging::EndWidgetPaint.RemoveAll(this);
	
	FConsoleSlateDebuggerPassBase::StopDebugging_Internal();
}

void FConsoleSlateDebuggerPaint::AddUpdatedWidget(const SWidget& Widget, const FConsoleSlateDebuggerUtility::TSWindowId WindowId, bool bIncrementUpdateCount)
{
	FWidgetInfo& WidgetInfo = AddUpdatedWidget_Internal(Widget, WindowId, Widget.Debug_GetLastPaintFrame());

	if (bLogWarningIfWidgetIsPaintedMoreThanOnce && WidgetInfo.UpdateCount != 0)
	{
		UE_LOGF(LogSlateDebugger, Warning, "'%ls' got painted more than once on frame %u.", *(WidgetInfo.WidgetName), GFrameNumber);
	}

	if (bIncrementUpdateCount)
	{
		++WidgetInfo.UpdateCount;
	}
}

void FConsoleSlateDebuggerPaint::HandleEndWidgetPaint(const SWidget* Widget, const FSlateWindowElementList& OutDrawElements, int32 LayerId)
{
	if (bDebugProjectContentOnly && !Widget->IsProjectContent())
	{
		return;
	}

	const FConsoleSlateDebuggerUtility::TSWindowId WindowId = FConsoleSlateDebuggerUtility::GetId(OutDrawElements.GetPaintWindow());
	AddUpdatedWidget(*Widget, WindowId);
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_SLATE_DEBUGGING
