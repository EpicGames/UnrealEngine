// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Debugging/SlateDebugging.h"

#if WITH_SLATE_DEBUGGING

#include "CoreMinimal.h"
#include "ConsoleSlateDebuggerPassBase.h"
#include "Debugging/ConsoleSlateDebuggerUtility.h"
#include "Delegates/Delegate.h"
#include "HAL/IConsoleManager.h"
#include "Rendering/DrawElements.h"

/**
 * Allows debugging the behavior of SWidget::Paint from the console.
 * Basics:
 *   Start - SlateDebugger.Paint.Start
 *   Stop  - SlateDebugger.Paint.Stop
 */
class FConsoleSlateDebuggerPaint final : public FConsoleSlateDebuggerPassBase
{
public:
	FConsoleSlateDebuggerPaint();
	virtual ~FConsoleSlateDebuggerPaint() override ;

	virtual void LoadConfig() override;
	virtual void SaveConfig() override;

protected:
	virtual FString GetConfigSection() const override { return TEXT("SlateDebugger.Paint"); }
	virtual FString GetNumberOfWidgetsUpdatedLogString(uint32 NumberOfWidgetsUpdatedThisFrame) const override
	{
		return FString::Printf(TEXT("%d widgets painted"), NumberOfWidgetsUpdatedThisFrame);
	}
	virtual FAutoConsoleVariableRef& GetEnabledCVar() override { return EnabledRefCVar; }
	virtual void StartDebugging_Internal() override;
	virtual void StopDebugging_Internal() override;
	
	virtual void AddUpdatedWidget(const SWidget& Widget, const FConsoleSlateDebuggerUtility::TSWindowId WindowId, bool bIncrementUpdateCount = true) override;

private:
	void HandleEndWidgetPaint(const SWidget* Widget, const FSlateWindowElementList& OutDrawElements, int32 LayerId);

private:
	bool bLogWarningIfWidgetIsPaintedMoreThanOnce;
	
	//~ Console objects
	FAutoConsoleVariableRef EnabledRefCVar;
	FAutoConsoleCommand ShowPaintWidgetCommand;
	FAutoConsoleCommand HidePaintWidgetCommand;
	FAutoConsoleCommand LogPaintedWidgetOnceCommand;
	FAutoConsoleVariableRef EnableWidgetsNameListRefCVar;
	FAutoConsoleCommand ToggleWidgetsNameListCommand;
	FAutoConsoleVariableRef MaxNumberOfWidgetInListRefCVar;
	FAutoConsoleVariableRef LogWarningIfWidgetIsPaintedMoreThanOnceRefCVar;
	FAutoConsoleVariableRef OnlyProjectContentRefCVar;
	FAutoConsoleVariableDeprecated OnlyGameWindowDepCVar;
	FAutoConsoleVariableRef OnlyInvalidationRootRefCVar;
	FAutoConsoleVariableRef DrawBorderEnabledRefCVar;
	FAutoConsoleVariableRef DrawFillEnabledRefCVar;
};

#endif //WITH_SLATE_DEBUGGING