// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleSlateDebuggerPassBase.h"

#if WITH_SLATE_DEBUGGING

#include "ConsoleSlateDebugger.h"
#include "CoreGlobals.h"
#include "Application/SlateApplicationBase.h"
#include "Debugging/SlateDebugging.h"
#include "Layout/WidgetPath.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Styling/CoreStyle.h"
#include "Types/ReflectionMetadata.h"

FConsoleSlateDebuggerPassBase::FConsoleSlateDebuggerPassBase()
	: bEnabled(false)
	, bDisplayWidgetsNameList(false)
	, bUseWidgetPathAsName(false)
	, bDrawBox(false)
	, bDrawBorder(true)
	, bLogWidgetName(false)
	, bLogWidgetNameOnce(false)
	, bDebugProjectContentOnly(true)
	, bInvalidationRootOnly(false)
	, MostRecentColor(0.75f, 0.0f, 0.0f, 0.02f)
	, LeastRecentColor(0.0f, 0.75f, 0.0f, 0.5f)
	, DrawWidgetNameColor(FColorList::SpicyPink)
	, MaxNumberOfWidgetInList(20)
	, FadeDuration(2.0f)
{
}

FConsoleSlateDebuggerPassBase::~FConsoleSlateDebuggerPassBase()
{
}

const FConsoleSlateDebuggerPassBase::TUpdatedWidgetMap& FConsoleSlateDebuggerPassBase::GetWidgets()
{
	return Widgets;
}

void FConsoleSlateDebuggerPassBase::LoadConfig()
{
	const FString Section = *GetConfigSection();

	// UE_DEPRECATED(5.8)
	FString Text;
	if (GConfig->GetString(*Section, TEXT("bDebugGameWindowOnly"), Text, *GEditorPerProjectIni))
	{
		bDebugProjectContentOnly = FCString::ToBool(*Text);
	}

	GConfig->GetBool(*Section, TEXT("bDebugProjectContentOnly"), bDebugProjectContentOnly, *GEditorPerProjectIni);
	GConfig->GetBool(*Section, TEXT("bInvalidationRootOnly"), bInvalidationRootOnly, *GEditorPerProjectIni);
	GConfig->GetBool(*Section, TEXT("bDisplayWidgetsNameList"), bDisplayWidgetsNameList, *GEditorPerProjectIni);
	GConfig->GetBool(*Section, TEXT("bUseWidgetPathAsName"), bUseWidgetPathAsName, *GEditorPerProjectIni);
	GConfig->GetBool(*Section, TEXT("bDrawBox"), bDrawBox, *GEditorPerProjectIni);
	GConfig->GetBool(*Section, TEXT("bDrawBorder"), bDrawBorder, *GEditorPerProjectIni);
	GConfig->GetBool(*Section, TEXT("bLogWidgetName"), bLogWidgetName, *GEditorPerProjectIni);
	FColor TmpColor;
	if (GConfig->GetColor(*Section, TEXT("MostRecentPaintColor"), TmpColor, *GEditorPerProjectIni))
	{
		MostRecentColor = TmpColor;
	}
	if (GConfig->GetColor(*Section, TEXT("LeastRecentPaintColor"), TmpColor, *GEditorPerProjectIni))
	{
		LeastRecentColor = TmpColor;
	}
	if (GConfig->GetColor(*Section, TEXT("DrawWidgetNameColor"), TmpColor, *GEditorPerProjectIni))
	{
		DrawWidgetNameColor = TmpColor;
	}
	GConfig->GetInt(*Section, TEXT("MaxNumberOfWidgetInList"), MaxNumberOfWidgetInList, *GEditorPerProjectIni);
	GConfig->GetFloat(*Section, TEXT("FadeDuration"), FadeDuration, *GEditorPerProjectIni);
}

void FConsoleSlateDebuggerPassBase::SaveConfig()
{
	const FString Section = *GetConfigSection();
	GConfig->SetBool(*Section, TEXT("bDebugProjectContentOnly"), bDebugProjectContentOnly, *GEditorPerProjectIni);
	GConfig->RemoveKeyFromSection(*Section, TEXT("bDebugGameWindowOnly"), *GEditorPerProjectIni); // UE_DEPRECATED(5.8)
	GConfig->SetBool(*Section, TEXT("bInvalidationRootOnly"), bInvalidationRootOnly, *GEditorPerProjectIni);
	GConfig->SetBool(*Section, TEXT("bDisplayWidgetsNameList"), bDisplayWidgetsNameList, *GEditorPerProjectIni);
	GConfig->SetBool(*Section, TEXT("bUseWidgetPathAsName"), bUseWidgetPathAsName, *GEditorPerProjectIni);
	GConfig->SetBool(*Section, TEXT("bDrawBox"), bDrawBox, *GEditorPerProjectIni);
	GConfig->SetBool(*Section, TEXT("bDrawBorder"), bDrawBorder, *GEditorPerProjectIni);
	GConfig->SetBool(*Section, TEXT("bLogWidgetName"), bLogWidgetName, *GEditorPerProjectIni);
	FColor TmpColor = MostRecentColor.ToFColor(true);
	GConfig->SetColor(*Section, TEXT("MostRecentPaintColor"), TmpColor, *GEditorPerProjectIni);
	TmpColor = LeastRecentColor.ToFColor(true);
	GConfig->SetColor(*Section, TEXT("LeastRecentPaintColor"), TmpColor, *GEditorPerProjectIni);
	TmpColor = DrawWidgetNameColor.ToFColor(true);
	GConfig->SetColor(*Section, TEXT("DrawWidgetNameColor"), TmpColor, *GEditorPerProjectIni);
	GConfig->SetInt(*Section, TEXT("MaxNumberOfWidgetInList"), MaxNumberOfWidgetInList, *GEditorPerProjectIni);
	GConfig->SetFloat(*Section, TEXT("FadeDuration"), FadeDuration, *GEditorPerProjectIni);
}

void FConsoleSlateDebuggerPassBase::StartDebugging()
{
	if (!bEnabled)
	{
		// This wil end up calling HandleEnabled > StartDebugging_Internal
		GetEnabledCVar()->Set(true, EConsoleVariableFlags::ECVF_SetByCode);
	}
}

void FConsoleSlateDebuggerPassBase::StopDebugging()
{
	if (bEnabled)
	{
		// This wil end up calling HandleEnabled > StopDebugging_Internal
		GetEnabledCVar()->Set(false, EConsoleVariableFlags::ECVF_SetByCode);
	}
}

void FConsoleSlateDebuggerPassBase::HandleEnabled(IConsoleVariable* Variable)
{
	// The value has already been changed by the CVar
	if (bEnabled)
	{
		StartDebugging_Internal();
	}
	else
	{
		StopDebugging_Internal();
	}
}

void FConsoleSlateDebuggerPassBase::StartDebugging_Internal()
{
	bEnabled = true;
	Widgets.Empty();

	BuildInitialWidgetList();

	FSlateDebugging::PaintDebugElements.AddRaw(this, &FConsoleSlateDebuggerPassBase::HandlePaintDebugInfo);
	FCoreDelegates::OnEndFrame.AddRaw(this, &FConsoleSlateDebuggerPassBase::HandleEndFrame);
}

void FConsoleSlateDebuggerPassBase::StopDebugging_Internal()
{
	FCoreDelegates::OnEndFrame.RemoveAll(this);
	FSlateDebugging::PaintDebugElements.RemoveAll(this);

	Widgets.Empty();
	bEnabled = false;
}

void FConsoleSlateDebuggerPassBase::HandleLogOnce()
{
	bLogWidgetNameOnce = true;
}

void FConsoleSlateDebuggerPassBase::SaveConfigOnVariableChanged(IConsoleVariable* Variable)
{
	SaveConfig();
}

void FConsoleSlateDebuggerPassBase::RebuildWidgetListOnVariableChanged(IConsoleVariable* Variable)
{
	SaveConfig();
	if (bEnabled)
	{
		// Recreate the list
		Widgets.Empty();
		BuildInitialWidgetList();
	}
}

void FConsoleSlateDebuggerPassBase::HandleToggleWidgetNameList()
{
	bDisplayWidgetsNameList = !bDisplayWidgetsNameList;
	SaveConfig();
}

void FConsoleSlateDebuggerPassBase::HandleEndFrame()
{
	for (TUpdatedWidgetMap::TIterator It(Widgets); It; ++It)
	{
		if (!It.Value().Widget.IsValid())
		{
			It.RemoveCurrent();
		}
		else
		{
			It.Value().UpdateCount = 0;
		}
	}
}

void FConsoleSlateDebuggerPassBase::HandlePaintDebugInfo(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId)
{
	++InOutLayerId;

	FConsoleSlateDebuggerUtility::TSWindowId PaintWindow = FConsoleSlateDebuggerUtility::GetId(InOutDrawElements.GetPaintWindow());

	uint32 NumberOfWidgetsUpdatedThisFrame = 0;
	uint32 NumberOfWidgetsLoggedThisFrame = 0; // We might log widgets updated on previous frames
	const float TextElementY = 36.f;
	const FSlateBrush* BoxBrush = FCoreStyle::Get().GetBrush("WhiteBrush");
	const FSlateBrush* QuadBrush = FCoreStyle::Get().GetBrush("Border");
	FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("SmallFont");
	FontInfo.OutlineSettings.OutlineSize = 1;

	FadeDuration = FMath::Max(FadeDuration, 0.01f);
	const double SlateApplicationCurrentTime = FSlateApplicationBase::Get().GetCurrentTime();

	auto MakeText = [&](const FString& Text, const FVector2f& Location, const FLinearColor& Color)
	{
		FSlateDrawElement::MakeText(
			InOutDrawElements
			, InOutLayerId
			, InAllottedGeometry.ToPaintGeometry(FVector2f(1.f, 1.f), FSlateLayoutTransform(Location))
			, Text
			, FontInfo
			, ESlateDrawEffect::None
			, Color
		);
	};

	for (TPair<FConsoleSlateDebuggerUtility::TSWidgetId, FWidgetInfo>& Pair : Widgets)
	{
		FWidgetInfo& WidgetInfo = Pair.Value;
		if (WidgetInfo.Window != PaintWindow)
		{
			continue;
		}

		if (bInvalidationRootOnly & !WidgetInfo.bInInvalidationRoot)
		{
			continue;
		}

		if (TSharedPtr<const SWidget> Widget = WidgetInfo.Widget.Pin())
		{
			// Skip the widget if collapsed or not in a window anymore (ex: in an unactive tab)
			if ( (Widget->GetProxyHandle().IsValid(Widget.Get()) && !Widget->GetProxyHandle().GetWidgetVisibility(Widget.Get()).IsVisible())
				|| !FSlateApplicationBase::Get().FindWidgetWindow(Widget.ToSharedRef()))
			{
				continue;
			}
			// Update with latest size and location
			WidgetInfo.PaintLocation = Widget->GetPersistentState().AllottedGeometry.GetAbsolutePosition();
			WidgetInfo.PaintSize = Widget->GetPersistentState().AllottedGeometry.GetAbsoluteSize();
		}

		if (WidgetInfo.LastUpdatedFrame == GFrameNumber)
		{
			++NumberOfWidgetsUpdatedThisFrame;
			if (bLogWidgetNameOnce)
			{
				UE_LOGF(LogSlateDebugger, Log, "%ls", *WidgetInfo.WidgetName);
			}
		}

		// LerpValue of 0 represents a widget painted this frame, 1 a Widget painted FadeDuration ago (or more)
		const float LerpValue = FMath::Clamp((float)(SlateApplicationCurrentTime - WidgetInfo.LastUpdatedTime) / FadeDuration, 0.0f, 1.0f);
		const FLinearColor FinalColor = FMath::Lerp(MostRecentColor, LeastRecentColor, LerpValue);
		const FGeometry Geometry = FGeometry::MakeRoot(WidgetInfo.PaintSize, FSlateLayoutTransform(1.f, WidgetInfo.PaintLocation));
		const FPaintGeometry PaintGeometry = Geometry.ToPaintGeometry();
		
		if (bDrawBox)
		{
			FSlateDrawElement::MakeBox(
				InOutDrawElements,
				InOutLayerId,
				PaintGeometry,
				BoxBrush,
				ESlateDrawEffect::None,
				FinalColor
			);
		}

		if (bDrawBorder)
		{
			FSlateDrawElement::MakeBox(
				InOutDrawElements,
				InOutLayerId,
				PaintGeometry,
				QuadBrush,
				ESlateDrawEffect::None,
				FinalColor.CopyWithNewOpacity(1.0f)
			);
		}
		
		if (bDisplayWidgetsNameList && LerpValue < 1.0f) // Only display the name of recent widgets
		{
			if (NumberOfWidgetsLoggedThisFrame < (uint32)MaxNumberOfWidgetInList)
			{
				const FString LoggedName = WidgetInfo.LastUpdatedFrame == GFrameNumber ? // Show differently a widget that was updated on a previous frame
					WidgetInfo.WidgetName :
					FString::Printf(TEXT("%s ( %3d frames ago )"), *WidgetInfo.WidgetName, GFrameNumber - WidgetInfo.LastUpdatedFrame);

				MakeText(LoggedName, FVector2f(0.f, (12.f * NumberOfWidgetsLoggedThisFrame) + TextElementY), DrawWidgetNameColor);
				++NumberOfWidgetsLoggedThisFrame;
			}
		}
	}
	bLogWidgetNameOnce = false;

	{
		FString NumberOfWidgetsDrawn = GetNumberOfWidgetsUpdatedLogString(NumberOfWidgetsUpdatedThisFrame);
		MakeText(NumberOfWidgetsDrawn, WidgetLogLocation, DrawWidgetNameColor);
	}

	if (bDisplayWidgetsNameList && NumberOfWidgetsUpdatedThisFrame > (uint32)MaxNumberOfWidgetInList)
	{
		FString WidgetDisplayName = FString::Printf(TEXT("   %d more widgets"), NumberOfWidgetsUpdatedThisFrame - MaxNumberOfWidgetInList);
		MakeText(WidgetDisplayName, FVector2f(0.f, (12.f * NumberOfWidgetsLoggedThisFrame) + TextElementY), FLinearColor::White);
	}
}

void FConsoleSlateDebuggerPassBase::BuildInitialWidgetList()
{
	// Do not log the widget names on initial build
	TGuardValue LogWidgetNameGuard(bLogWidgetName, false);

	TArray<TSharedRef<SWindow>> SlateWindows = FSlateApplicationBase::Get().GetTopLevelWindows();
	for (TArray<TSharedRef<SWindow>>::TIterator It(SlateWindows); It; ++It)
	{
		const TSharedRef<SWindow>& Window = *It;

		const FConsoleSlateDebuggerUtility::TSWindowId WindowId = FConsoleSlateDebuggerUtility::GetId(*Window);
		AddInitialVisibleWidget(*StaticCastSharedRef<SWidget>(Window), WindowId);

		SlateWindows.Append(Window->GetChildWindows());
	}
}

void FConsoleSlateDebuggerPassBase::AddInitialVisibleWidget(const SWidget& Widget, const FConsoleSlateDebuggerUtility::TSWindowId WindowId)
{
	if (!Widget.GetVisibility().IsVisible() || Widget.Debug_GetLastPaintFrame() == 0)
	{
		return;
	}

	if (!(bDebugProjectContentOnly && !Widget.IsProjectContent()))
	{
		constexpr bool bIncrementUpdateCount = false; // This is called from BuildInitialWidgetList, we therefore do not want to record an update event
		AddUpdatedWidget(Widget, WindowId, bIncrementUpdateCount);
	}

	const FChildren* Children = const_cast<SWidget&>(Widget).Debug_GetChildrenForReflector();
	if (ensure(Children))
	{
		Children->ForEachWidget([&](const SWidget& ChildWidget)
		{
			AddInitialVisibleWidget(ChildWidget, WindowId);
		});
	}
}

FConsoleSlateDebuggerPassBase::FWidgetInfo& FConsoleSlateDebuggerPassBase::AddUpdatedWidget_Internal(const SWidget& Widget, const FConsoleSlateDebuggerUtility::TSWindowId WindowId, uint32 LastUpdatedFrame)
{
	// Use the Widget pointer for the id.
	//That may introduce bug when a widget is destroyed and the same memory is reused for another widget. We do not care for this debug tool.
	//We do not keep the widget alive or reuse it later, cache all the info that we need.
	const FConsoleSlateDebuggerUtility::TSWidgetId WidgetId = FConsoleSlateDebuggerUtility::GetId(Widget);

	FWidgetInfo* FoundItem = Widgets.Find(WidgetId);
	if (FoundItem == nullptr)
	{
		FoundItem = &Widgets.Add(WidgetId);
		FoundItem->UpdateCount = 0;
		FoundItem->Window = WindowId;
		FoundItem->WidgetName = bUseWidgetPathAsName ? FReflectionMetaData::GetWidgetPath(Widget) : FReflectionMetaData::GetWidgetDebugInfo(Widget);
	}
	else if (FoundItem->Window != WindowId)
	{
		UE_LOGF(LogSlateDebugger, Log, "The WindowId of this widget '%ls' has changed since the last time it was seen.", *FoundItem->WidgetName);
		FoundItem->Window = WindowId;
	}
	
	if (bLogWidgetName)
	{
		UE_LOGF(LogSlateDebugger, Log, "%ls", *(FoundItem->WidgetName));
	}
	
	FoundItem->Widget = Widget.AsWeak();
	FoundItem->PaintLocation = Widget.GetPersistentState().AllottedGeometry.GetAbsolutePosition();
	FoundItem->PaintSize = Widget.GetPersistentState().AllottedGeometry.GetAbsoluteSize();
	FoundItem->bInInvalidationRoot = Widget.GetProxyHandle().GetInvalidationRootHandle().GetInvalidationRoot() != nullptr;
	if (!FoundItem->bInInvalidationRoot)
	{
		// If the widget was added to the Invalidation Root after it stopped caching, the widget would not be inside an invalidation root
		TSharedPtr<const SWidget> InvalidationRoot = Widget.AsShared();
		while (InvalidationRoot && !InvalidationRoot->Advanced_IsInvalidationRoot())
		{
			InvalidationRoot = InvalidationRoot->GetParentWidget();
		}
		FoundItem->bInInvalidationRoot = InvalidationRoot.IsValid();
	}
	FoundItem->LastUpdatedFrame = LastUpdatedFrame;
	FoundItem->LastUpdatedTime = FSlateApplicationBase::Get().GetCurrentTime();
	if (GFrameNumber != FoundItem->LastUpdatedFrame) // if we do not have the time at which it was drawn, we estimate
	{
		FoundItem->LastUpdatedTime -= (GFrameNumber - FoundItem->LastUpdatedFrame) * FApp::GetDeltaTime(); 
	}

	return *FoundItem;
}

#endif // WITH_SLATE_DEBUGGING