// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Debugging/SlateDebugging.h"

#if WITH_SLATE_DEBUGGING

#include "CoreMinimal.h"
#include "Debugging/ConsoleSlateDebuggerUtility.h"
#include "Delegates/Delegate.h"
#include "HAL/IConsoleManager.h"
#include "Rendering/DrawElements.h"

class FConsoleSlateDebuggerPassBase
{
public:
	struct FWidgetInfo 
	{
		FConsoleSlateDebuggerUtility::TSWindowId Window = FConsoleSlateDebuggerUtility::InvalidWindowId;
		TWeakPtr<const SWidget> Widget; // used to check if the widget has been destroyed
		FVector2f PaintLocation;
		FVector2f PaintSize;
		FString WidgetName;

		uint32 LastUpdatedFrame = 0;
		double LastUpdatedTime = 0.0;
		int32 UpdateCount = 0;
		bool bInInvalidationRoot = false;
	};

	using TUpdatedWidgetMap = TMap<FConsoleSlateDebuggerUtility::TSWidgetId, FWidgetInfo>;

	FConsoleSlateDebuggerPassBase();
	virtual ~FConsoleSlateDebuggerPassBase();

	/* Starts debugging by setting the enabled CVar to true. See GetEnabledCVar */
	void StartDebugging();
	/* Stops debugging by setting the enabled CVar to false. See GetEnabledCVar */
	void StopDebugging();
	
	bool IsEnabled() const { return bEnabled; }

	const TUpdatedWidgetMap& GetWidgets();

	virtual void LoadConfig();
	virtual void SaveConfig();

protected:
	/* Return the name of the config section used to load and save the config */
	virtual FString GetConfigSection() const = 0;
	/* Return the text to be shown on screen indicating the number of widgets updated when bDisplayWidgetsNameList is true */
	virtual FString GetNumberOfWidgetsUpdatedLogString(uint32 NumberOfWidgetsUpdatedThisFrame) const = 0;
	/* Return the CVar used to enable/disable the debugger pass. Used by Start/StopDebugging */
	virtual FAutoConsoleVariableRef& GetEnabledCVar() = 0;
	/* Function called when the debugging starts. Used to attach delegates */
	virtual void StartDebugging_Internal();
	/* Function called when the debugging stops. Used to remove delegates */
	virtual void StopDebugging_Internal();

	/**
	 * Called when a widget should be added to the internal widget list.
	 * @param Widget The Widget to add to the list
	 * @param WindowId The ID of the window owning the widget
	 * @param bIncrementUpdateCount If true, the UpdateCount of the widget should be increased. This can ve false when building the initial widget list
	 */
	virtual void AddUpdatedWidget(const SWidget& Widget, const FConsoleSlateDebuggerUtility::TSWindowId WindowId, bool bIncrementUpdateCount = true) = 0;

	/**
	 * The internal function that should be called from AddUpdatedWidget to update the internal widget list.
	 * This internal version has a parameter LastUpdatedFrame used to set when the widget was last updated.
	 * @param Widget The Widget to add to the list
	 * @param WindowId The ID of the window owning the widget
	 * @param LastUpdatedFrame The Frame at which the widget was last updated
	 * @return Returns the internal FWidgetInfo allowing additional properties to be set, like the widget's UpdateCount
	 */
	FWidgetInfo& AddUpdatedWidget_Internal(const SWidget& Widget, const FConsoleSlateDebuggerUtility::TSWindowId WindowId, uint32 LastUpdatedFrame);

	/* Should be called by the GetEnabledCVar when it is updated. Will end up calling Start/StopDebugging_Internal */
	void HandleEnabled(IConsoleVariable* Variable);
	/* Should be called by the CVar enabling a one time log of the updated widgets */
	void HandleLogOnce();
	/* Should be called by the CVar enabling the updated widget list to be shown on screen */
	void HandleToggleWidgetNameList();
	/* Should be called by CVars that should only save the current config */
	void SaveConfigOnVariableChanged(IConsoleVariable* Variable);
	/* Should be called by the CVar that changes the scope of the widgets retrieved. Ends up rebuilding the widget list */
	void RebuildWidgetListOnVariableChanged(IConsoleVariable* Variable);
	/* Called on EndFrame to reset the counts and remove deleted widgets */
	void HandleEndFrame();
	/* Called by Slate to draw additional elements on screen. */
	void HandlePaintDebugInfo(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId);

private:
	/* Not all widgets are accessible from the PaintEvents, so we need to loop through all of them to add all current ones */
	void BuildInitialWidgetList();

	void AddInitialVisibleWidget(const SWidget& Widget, const FConsoleSlateDebuggerUtility::TSWindowId WindowId);
	
protected:
	//~ Settings
	bool bEnabled;
	bool bDisplayWidgetsNameList;
	bool bUseWidgetPathAsName;
	bool bDrawBox;
	bool bDrawBorder;
	bool bLogWidgetName;
	bool bLogWidgetNameOnce;
	bool bDebugProjectContentOnly;
	bool bInvalidationRootOnly;
	FLinearColor MostRecentColor;
	FLinearColor LeastRecentColor;
	FLinearColor DrawWidgetNameColor;
	int32 MaxNumberOfWidgetInList;
	float FadeDuration;

	FVector2f WidgetLogLocation{10.f, 10.f};

	TUpdatedWidgetMap Widgets;
};

#endif //WITH_SLATE_DEBUGGING
