// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleSlateDebuggerPrepass.h"

#if WITH_SLATE_DEBUGGING

#include "CoreGlobals.h"
#include "Application/SlateApplicationBase.h"
#include "Debugging/SlateDebugging.h"
#include "Layout/WidgetPath.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"

#define LOCTEXT_NAMESPACE "ConsoleSlateDebuggerPrepass"

FConsoleSlateDebuggerPrepass::FConsoleSlateDebuggerPrepass()
	: FConsoleSlateDebuggerPassBase()
	, EnabledRefCVar(
		TEXT("SlateDebugger.Prepass.Enable")
		, bEnabled
		, TEXT("Start/Stop the prepassed widget debug tool. It shows when widgets are prepassed.")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPrepass::HandleEnabled))
	, ShowPrepassWidgetCommand(
		TEXT("SlateDebugger.Prepass.Start")
		, TEXT("Start the prepassed widget debug tool. Use to show widget that have been prepassed this frame.")
		, FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerPrepass::StartDebugging))
	, HidePrepassWidgetCommand(
		TEXT("SlateDebugger.Prepass.Stop")
		, TEXT("Stop the prepassed widget debug tool.")
		, FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerPrepass::StopDebugging))
	, LogPrepassedWidgetOnceCommand(
		TEXT("SlateDebugger.Prepass.LogOnce")
		, TEXT("Log the names of all widgets that were prepassed during the last update.")
		, FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerPrepass::HandleLogOnce))
	, EnableWidgetsNameListRefCVar(
		TEXT("SlateDebugger.Prepass.EnableWidgetNameList")
		, bDisplayWidgetsNameList
		, TEXT("Start/Stop displaying the name of the widgets that have been prepassed.")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPrepass::SaveConfigOnVariableChanged))
	, ToggleWidgetsNameListCommand(
		TEXT("SlateDebugger.Prepass.ToggleWidgetNameList")
		, TEXT("Option displaying the name of the widgets that have been prepassed.")
		, FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerPrepass::HandleToggleWidgetNameList))
	, MaxNumberOfWidgetInListRefCVar(
		TEXT("SlateDebugger.Prepass.MaxNumberOfWidgetDisplayedInList")
		, MaxNumberOfWidgetInList
		, TEXT("The max number of widgets that will be displayed when DisplayWidgetNameList is active.")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPrepass::SaveConfigOnVariableChanged))
	, OnlyProjectContentRefCVar(
		TEXT("SlateDebugger.Prepass.OnlyProjectContent")
		, bDebugProjectContentOnly
		, TEXT("Only debug the project's contents (exclude editor, chrome, diagnostic output)")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPrepass::RebuildWidgetListOnVariableChanged))
	, OnlyGameWindowDepCVar(
		TEXT("SlateDebugger.Prepass.OnlyGameWindow"),
		TEXT("SlateDebugger.Prepass.OnlyProjectContent"),
		TEXT("5.8"),
		EShadowCVarBehavior::Warn)
	, OnlyInvalidationRootRefCVar(
		TEXT("SlateDebugger.Prepass.OnlyInvalidationRoot")
		, bInvalidationRootOnly
		, TEXT("Option to only debug widgets within an Invalidation Root")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPrepass::RebuildWidgetListOnVariableChanged))
	, DrawBorderEnabledRefCVar(
		TEXT("SlateDebugger.Prepass.DrawBorder")
		, bDrawBorder
		, TEXT("Draw a border around the widgets being prepassed")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPrepass::SaveConfigOnVariableChanged))
	, DrawFillEnabledRefCVar(
		TEXT("SlateDebugger.Prepass.DrawFill")
		, bDrawBox
		, TEXT("Fill the widgets being prepassed")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerPrepass::SaveConfigOnVariableChanged))
{
	WidgetLogLocation = FVector2f(10.f, 20.f); // Different from FConsoleSlateDebuggerPaint so the texts do not overlap
}

FConsoleSlateDebuggerPrepass::~FConsoleSlateDebuggerPrepass()
{
	// StopDebugging will end up calling the virtual function GetEnabledCVar()
	StopDebugging();
}

void FConsoleSlateDebuggerPrepass::LoadConfig()
{
	FConsoleSlateDebuggerPassBase::LoadConfig();
	
	OnlyProjectContentRefCVar->Set(bDebugProjectContentOnly, ECVF_SetByProjectSetting);
	OnlyInvalidationRootRefCVar->Set(bInvalidationRootOnly, ECVF_SetByProjectSetting);
	EnableWidgetsNameListRefCVar->Set(bDisplayWidgetsNameList, ECVF_SetByProjectSetting);
	DrawFillEnabledRefCVar->Set(bDrawBox, ECVF_SetByProjectSetting);
	DrawBorderEnabledRefCVar->Set(bDrawBorder, ECVF_SetByProjectSetting);
}

void FConsoleSlateDebuggerPrepass::StartDebugging_Internal()
{
	FConsoleSlateDebuggerPassBase::StartDebugging_Internal();
	
	FSlateDebugging::EndWidgetPrepass.AddRaw(this, &FConsoleSlateDebuggerPrepass::HandleEndWidgetPrepass);
}

void FConsoleSlateDebuggerPrepass::StopDebugging_Internal()
{
	FSlateDebugging::EndWidgetPrepass.RemoveAll(this);
	
	FConsoleSlateDebuggerPassBase::StopDebugging_Internal();
}

void FConsoleSlateDebuggerPrepass::AddUpdatedWidget(const SWidget& Widget, const FConsoleSlateDebuggerUtility::TSWindowId WindowId, bool bIncrementUpdateCount)
{
	FWidgetInfo& WidgetInfo = AddUpdatedWidget_Internal(Widget, WindowId, Widget.Debug_GetLastPrepassFrame());

	if (bIncrementUpdateCount)
	{
		++WidgetInfo.UpdateCount;
	}
}

void FConsoleSlateDebuggerPrepass::HandleEndWidgetPrepass(const SWidget* Widget)
{
	const TSharedPtr<SWindow> WindowToDrawIn = FSlateApplicationBase::Get().FindWidgetWindow(Widget->AsShared());

	if (!WindowToDrawIn || (bDebugProjectContentOnly && !Widget->IsProjectContent()))
	{
		return;
	}

	const FConsoleSlateDebuggerUtility::TSWindowId WindowId = FConsoleSlateDebuggerUtility::GetId(*WindowToDrawIn);
	AddUpdatedWidget(*Widget, WindowId);
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_SLATE_DEBUGGING
