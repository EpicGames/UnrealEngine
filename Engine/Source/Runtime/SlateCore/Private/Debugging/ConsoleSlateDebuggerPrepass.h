// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Debugging/SlateDebugging.h"

#if WITH_SLATE_DEBUGGING

#include "CoreMinimal.h"
#include "ConsoleSlateDebuggerPassBase.h"
#include "Debugging/ConsoleSlateDebuggerUtility.h"
#include "Delegates/Delegate.h"
#include "HAL/IConsoleManager.h"
#include "Rendering/DrawElements.h"

/**
 * Allows debugging the behavior of SWidget::Prepass from the console.
 * Basics:
 *   Start - SlateDebugger.Prepass.Start
 *   Stop  - SlateDebugger.Prepass.Stop
 */
class FConsoleSlateDebuggerPrepass final : public FConsoleSlateDebuggerPassBase
{
public:
	FConsoleSlateDebuggerPrepass();
	virtual ~FConsoleSlateDebuggerPrepass() override;

	virtual void LoadConfig() override;

protected:
	virtual FString GetConfigSection() const override { return TEXT("SlateDebugger.Prepass"); }
	virtual FString GetNumberOfWidgetsUpdatedLogString(uint32 NumberOfWidgetsUpdatedThisFrame) const override
	{
		return FString::Printf(TEXT("%d widgets prepassed"), NumberOfWidgetsUpdatedThisFrame);
	}
	virtual FAutoConsoleVariableRef& GetEnabledCVar() override { return EnabledRefCVar; }
	virtual void StartDebugging_Internal() override;
	virtual void StopDebugging_Internal() override;
	
	virtual void AddUpdatedWidget(const SWidget& Widget, const FConsoleSlateDebuggerUtility::TSWindowId WindowId, bool bIncrementUpdateCount = true) override;

private:
	void HandleEndWidgetPrepass(const SWidget* Widget);

private:
	//~ Console objects
	FAutoConsoleVariableRef EnabledRefCVar;
	FAutoConsoleCommand ShowPrepassWidgetCommand;
	FAutoConsoleCommand HidePrepassWidgetCommand;
	FAutoConsoleCommand LogPrepassedWidgetOnceCommand;
	FAutoConsoleVariableRef EnableWidgetsNameListRefCVar;
	FAutoConsoleCommand ToggleWidgetsNameListCommand;
	FAutoConsoleVariableRef MaxNumberOfWidgetInListRefCVar;
	FAutoConsoleVariableRef OnlyProjectContentRefCVar;
	FAutoConsoleVariableDeprecated OnlyGameWindowDepCVar;
	FAutoConsoleVariableRef OnlyInvalidationRootRefCVar;
	FAutoConsoleVariableRef DrawBorderEnabledRefCVar;
	FAutoConsoleVariableRef DrawFillEnabledRefCVar;
};

#endif //WITH_SLATE_DEBUGGING