// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleSlateDebuggerUpdate.h"
#include "ConsoleSlateDebugger.h"

#if WITH_SLATE_DEBUGGING

#include "Application/SlateApplicationBase.h"
#include "CoreGlobals.h"
#include "Debugging/SlateDebugging.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/StringBuilder.h"
#include "Styling/CoreStyle.h"
#include "Types/ReflectionMetadata.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "ConsoleSlateDebuggerUpdate"

FConsoleSlateDebuggerUpdate::FConsoleSlateDebuggerUpdate()
	: bEnabled(false)
	, bEnabledCVarValue(false)
	, bDisplayWidgetsNameList(false)
	, bUseWidgetPathAsName(false)
	, bDisplayUpdateFromPaint(false)
	, bShowLegend(false)
	, bShowQuad(false)
	, bDebugProjectContentOnly(true)
	, WidgetUpdateFlagsFilter(EWidgetUpdateFlags::AnyUpdate)
	, DrawVolatilePaintColor(FColorList::Red)
	, DrawRepaintColor(FColorList::Yellow)
	, DrawTickColor(FColorList::Blue)
	, DrawActiveTimerColor(FColorList::Green)
	, DrawWidgetNameColor(FColorList::Red)
	, MaxNumberOfWidgetInList(20)
	, InvalidationRootIdFilter(-1)
	, CacheDuration(2.0f)
	, StartCommand(
		TEXT("SlateDebugger.Update.Start"),
		TEXT("Start the update widget debug tool. It shows when widgets are updated."),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerUpdate::StartDebugging))
	, StopCommand(
		TEXT("SlateDebugger.Update.Stop"),
		TEXT("Stop the update widget debug tool."),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerUpdate::StopDebugging))
	, EnabledRefCVar(
		TEXT("SlateDebugger.Update.Enable")
		, bEnabledCVarValue
		, TEXT("Start/Stop the painted widget debug tool. It shows when widgets are updated.")
		, FConsoleVariableDelegate::CreateRaw(this, &FConsoleSlateDebuggerUpdate::HandleEnabled))
	, ToggleLegendCommand(
		TEXT("SlateDebugger.Update.ToggleLegend"),
		TEXT("Option to display the color legend."),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerUpdate::ToggleDisplayLegend))
	, ToogleWidgetsNameListCommand(
		TEXT("SlateDebugger.Update.ToggleWidgetNameList"),
		TEXT("Option to display the name of the widgets that have been updated."),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerUpdate::ToogleDisplayWidgetNameList))
	, ToogleDisplayUpdateFromPaintCommand(
		TEXT("SlateDebugger.Update.ToggleUpdateFromPaint"),
		TEXT("Option to also display the widgets that do not have an update flag but are updated as a side effect of an other widget."),
		FConsoleCommandDelegate::CreateRaw(this, &FConsoleSlateDebuggerUpdate::ToogleDisplayUpdateFromPaint))
	, SetWidgetUpdateFlagsFilterCommand(
		TEXT("SlateDebugger.Update.SetWidgetUpdateFlagsFilter"),
		TEXT("Enable or Disable specific Widget Update Flags filters. Usage: SetWidgetUpdateFlagsFilter [None] [Tick] [ActiveTimer] [Repaint] [VolatilePaint] [Any]"),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FConsoleSlateDebuggerUpdate::HandleSetWidgetUpdateFlagsFilter))
	, InvalidationRootFilterRefCVar(
		TEXT("SlateDebugger.Update.SetInvalidationRootIdFilter"),
		InvalidationRootIdFilter,
		TEXT("Option to show only the widgets that are part of an invalidation root."))
	, OnlyProjectContentRefCVar(
		TEXT("SlateDebugger.Update.OnlyProjectContent"),
		bDebugProjectContentOnly,
		TEXT("Only debug content widgets (exclude editor, chrome, diagnostic output)"))
	, OnlyGameWindowDepCVar(
		TEXT("SlateDebugger.Update.OnlyGameWindow"),
		TEXT("SlateDebugger.Update.OnlyProjectContent"),
		TEXT("5.8"),
		EShadowCVarBehavior::Warn)
{
	GConfig->GetBool(TEXT("SlateDebugger.Update"), TEXT("bDisplayWidgetsNameList"), bDisplayWidgetsNameList, *GEditorPerProjectIni);
	GConfig->GetBool(TEXT("SlateDebugger.Update"), TEXT("bUseWidgetPathAsName"), bUseWidgetPathAsName, *GEditorPerProjectIni);
	GConfig->GetBool(TEXT("SlateDebugger.Update"), TEXT("bDisplayUpdateFromPaint"), bDisplayUpdateFromPaint, *GEditorPerProjectIni);
	GConfig->GetBool(TEXT("SlateDebugger.Update"), TEXT("bShowQuad"), bShowQuad, *GEditorPerProjectIni);
	int32 TmpInt = 0;
	if (GConfig->GetInt(TEXT("SlateDebugger.Update"), TEXT("WidgetUpdateFlagsFilter"), TmpInt, *GEditorPerProjectIni))
	{
		WidgetUpdateFlagsFilter = static_cast<EWidgetUpdateFlags>(TmpInt);
	}
	FColor TmpColor;
	if (GConfig->GetColor(TEXT("SlateDebugger.Update"), TEXT("DrawVolatilePaintColor"), TmpColor, *GEditorPerProjectIni))
	{
		DrawVolatilePaintColor = TmpColor;
	}
	if (GConfig->GetColor(TEXT("SlateDebugger.Update"), TEXT("DrawRepaintColor"), TmpColor, *GEditorPerProjectIni))
	{
		DrawRepaintColor = TmpColor;
	}
	if (GConfig->GetColor(TEXT("SlateDebugger.Update"), TEXT("DrawTickColor"), TmpColor, *GEditorPerProjectIni))
	{
		DrawTickColor = TmpColor;
	}
	if (GConfig->GetColor(TEXT("SlateDebugger.Update"), TEXT("DrawActiveTimerColor"), TmpColor, *GEditorPerProjectIni))
	{
		DrawActiveTimerColor = TmpColor;
	}
	if (GConfig->GetColor(TEXT("SlateDebugger.Update"), TEXT("DrawWidgetNameColor"), TmpColor, *GEditorPerProjectIni))
	{
		DrawWidgetNameColor = TmpColor;
	}	
	GConfig->GetInt(TEXT("SlateDebugger.Update"), TEXT("MaxNumberOfWidgetInList"), MaxNumberOfWidgetInList, *GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("SlateDebugger.Update"), TEXT("CacheDuration"), CacheDuration, *GEditorPerProjectIni);
}

FConsoleSlateDebuggerUpdate::~FConsoleSlateDebuggerUpdate()
{
	StopDebugging();
}

void FConsoleSlateDebuggerUpdate::SaveConfig()
{
	GConfig->SetBool(TEXT("SlateDebugger.Update"), TEXT("bDisplayWidgetsNameList"), bDisplayWidgetsNameList, *GEditorPerProjectIni);
	GConfig->SetBool(TEXT("SlateDebugger.Update"), TEXT("bUseWidgetPathAsName"), bUseWidgetPathAsName, *GEditorPerProjectIni);
	GConfig->SetBool(TEXT("SlateDebugger.Update"), TEXT("bDisplayUpdateFromPaint"), bDisplayUpdateFromPaint, *GEditorPerProjectIni);
	int32 TmpInt = static_cast<int32>(WidgetUpdateFlagsFilter);
	GConfig->SetInt(TEXT("SlateDebugger.Update"), TEXT("WidgetUpdateFlagsFilter"), TmpInt, *GEditorPerProjectIni);
	FColor TmpColor = DrawVolatilePaintColor.ToFColor(true);
	GConfig->SetColor(TEXT("SlateDebugger.Update"), TEXT("DrawVolatilePaintColor"), TmpColor, *GEditorPerProjectIni);
	TmpColor = DrawRepaintColor.ToFColor(true);
	GConfig->SetColor(TEXT("SlateDebugger.Update"), TEXT("DrawRepaintColor"), TmpColor, *GEditorPerProjectIni);
	TmpColor = DrawTickColor.ToFColor(true);
	GConfig->SetColor(TEXT("SlateDebugger.Update"), TEXT("DrawTickColor"), TmpColor, *GEditorPerProjectIni);
	TmpColor = DrawActiveTimerColor.ToFColor(true);
	GConfig->SetColor(TEXT("SlateDebugger.Update"), TEXT("DrawActiveTimerColor"), TmpColor, *GEditorPerProjectIni);
	TmpColor = DrawWidgetNameColor.ToFColor(true);
	GConfig->SetColor(TEXT("SlateDebugger.Update"), TEXT("DrawWidgetNameColor"), TmpColor, *GEditorPerProjectIni);
	GConfig->SetInt(TEXT("SlateDebugger.Update"), TEXT("MaxNumberOfWidgetInList"), MaxNumberOfWidgetInList, *GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("SlateDebugger.Update"), TEXT("CacheDuration"), CacheDuration, *GEditorPerProjectIni);
}

void FConsoleSlateDebuggerUpdate::StartDebugging()
{
	if (!bEnabled)
	{
		bEnabled = true;
		UpdatedWidgets.Empty();

		FSlateDebugging::PaintDebugElements.AddRaw(this, &FConsoleSlateDebuggerUpdate::HandlePaintDebugInfo);
		FSlateDebugging::WidgetUpdatedEvent.AddRaw(this, &FConsoleSlateDebuggerUpdate::HandleWidgetUpdate);
		FCoreDelegates::OnEndFrame.AddRaw(this, &FConsoleSlateDebuggerUpdate::HandleEndFrame);
	}
	bEnabledCVarValue = bEnabled;
}

void FConsoleSlateDebuggerUpdate::StopDebugging()
{
	if (bEnabled)
	{
		FCoreDelegates::OnEndFrame.RemoveAll(this);
		FSlateDebugging::WidgetUpdatedEvent.RemoveAll(this);
		FSlateDebugging::PaintDebugElements.RemoveAll(this);

		UpdatedWidgets.Empty();
		bEnabled = false;
	}
	bEnabledCVarValue = bEnabled;
}

void FConsoleSlateDebuggerUpdate::HandleEnabled(IConsoleVariable* Variable)
{
	if (bEnabledCVarValue)
	{
		StartDebugging();
	}
	else
	{
		StopDebugging();
	}
}

void FConsoleSlateDebuggerUpdate::HandleSetWidgetUpdateFlagsFilter(const TArray<FString>& Params)
{
	const TCHAR* UsageMessage = TEXT("Usage: SetWidgetUpdateFlagsFilter [None] [Tick] [ActiveTimer] [Repaint] [VolatilePaint] [Any]");
	if (Params.Num() == 0)
	{
		UE_LOGF(LogSlateDebugger, Log, "%ls", UsageMessage);

		TStringBuilder<128> Message;
		Message << TEXT("Current Widget Update Flags set: ");
		bool bFirstFlag = true;
		auto AddPipeChar = [&]()
		{
			if (bFirstFlag)
			{
				Message << TEXT("|");
			}
			bFirstFlag = false;
		};

		if (EnumHasAnyFlags(WidgetUpdateFlagsFilter, EWidgetUpdateFlags::NeedsTick))
		{ 
			AddPipeChar();
			Message << TEXT("Tick");
		}
		if (EnumHasAnyFlags(WidgetUpdateFlagsFilter, EWidgetUpdateFlags::NeedsTick))
		{
			AddPipeChar();
			Message << TEXT("ActiveTimer");
		}
		if (EnumHasAnyFlags(WidgetUpdateFlagsFilter, EWidgetUpdateFlags::NeedsTick))
		{
			AddPipeChar();
			Message << TEXT("Repaint");
		}
		if (EnumHasAnyFlags(WidgetUpdateFlagsFilter, EWidgetUpdateFlags::NeedsTick))
		{
			AddPipeChar();
			Message << TEXT("VolatilePaint");
		}
		UE_LOGF(LogSlateDebugger, Log, "%ls", Message.GetData());
	}
	else
	{
		EWidgetUpdateFlags NewWidgetUpdateFlagsFilter = EWidgetUpdateFlags::None;
		bool bHasValidFlags = false;
		for (const FString& Param : Params)
		{
			if (Param == TEXT("Tick"))
			{
				NewWidgetUpdateFlagsFilter |= EWidgetUpdateFlags::NeedsTick;
				bHasValidFlags = true;
			}
			else if (Param == TEXT("ActiveTimer"))
			{
				NewWidgetUpdateFlagsFilter |= EWidgetUpdateFlags::NeedsActiveTimerUpdate;
				bHasValidFlags = true;
			}
			else if (Param == TEXT("Repaint"))
			{
				NewWidgetUpdateFlagsFilter |= EWidgetUpdateFlags::NeedsRepaint;
				bHasValidFlags = true;
			}
			else if (Param == TEXT("VolatilePaint"))
			{
				NewWidgetUpdateFlagsFilter |= EWidgetUpdateFlags::NeedsVolatilePaint;
				bHasValidFlags = true;
			}
			else if (Param == TEXT("Any"))
			{
				NewWidgetUpdateFlagsFilter |= EWidgetUpdateFlags::AnyUpdate;
				bHasValidFlags = true;
			}
			else if (Param == TEXT("None"))
			{
				NewWidgetUpdateFlagsFilter = EWidgetUpdateFlags::None;
				bHasValidFlags = true;
			}
			else
			{
				bHasValidFlags = false;
				UE_LOGF(LogSlateDebugger, Warning, "Param '%ls' is invalid.", *Param);
			}
		}

		if (!bHasValidFlags)
		{
			UE_LOGF(LogSlateDebugger, Log, "%ls", UsageMessage);
		}
		else
		{
			WidgetUpdateFlagsFilter = NewWidgetUpdateFlagsFilter;
			SaveConfig();
		}
	}
}

void FConsoleSlateDebuggerUpdate::ToggleDisplayLegend()
{
	bShowLegend = !bShowLegend;
	SaveConfig();
}

void FConsoleSlateDebuggerUpdate::ToogleDisplayWidgetNameList()
{
	bDisplayWidgetsNameList = !bDisplayWidgetsNameList;
	SaveConfig();
}

void FConsoleSlateDebuggerUpdate::ToogleDisplayUpdateFromPaint()
{
	bDisplayUpdateFromPaint = !bDisplayUpdateFromPaint;
	SaveConfig();
}

void FConsoleSlateDebuggerUpdate::HandleEndFrame()
{
	double LastTime = FSlateApplicationBase::Get().GetCurrentTime() - CacheDuration;
	for (TWidgetMap::TIterator It(UpdatedWidgets); It; ++It)
	{
		if (It.Value().LastInvalidationTime < LastTime)
		{
			It.RemoveCurrent();
		}
	}
}

FConsoleSlateDebuggerUpdate::FWidgetInfo::FWidgetInfo(const SWidget* Widget, EWidgetUpdateFlags InUpdateFlags)
	: WindowId(0)
	, UpdateFlags(InUpdateFlags)
	, LastInvalidationTime(FSlateApplicationBase::Get().GetCurrentTime())
	, bIsProjectContent(Widget->IsProjectContent())
{
	PaintLocation = Widget->GetPersistentState().AllottedGeometry.GetAbsolutePosition();
	PaintSize = Widget->GetPersistentState().AllottedGeometry.GetAbsoluteSize();
	WidgetName = FReflectionMetaData::GetWidgetDebugInfo(Widget);

	WindowId = FConsoleSlateDebuggerUtility::FindWindowId(Widget);
}

void FConsoleSlateDebuggerUpdate::FWidgetInfo::Update(const SWidget* Widget, EWidgetUpdateFlags InUpdateFlags)
{
	UpdateFlags = InUpdateFlags;
	LastInvalidationTime = FSlateApplicationBase::Get().GetCurrentTime();
	PaintLocation = Widget->GetPersistentState().AllottedGeometry.GetAbsolutePosition();
	PaintSize = Widget->GetPersistentState().AllottedGeometry.GetAbsoluteSize();
}

void FConsoleSlateDebuggerUpdate::HandleWidgetUpdate(const FSlateDebuggingWidgetUpdatedEventArgs& Args)
{
	if (Args.Widget) // can become nullptr in fast path when a Tick or an ActiveTimer remove it from the list
	{
		const FConsoleSlateDebuggerUtility::TSWidgetId WidgetId = FConsoleSlateDebuggerUtility::GetId(Args.Widget);

		EWidgetUpdateFlags UpdateFlags = Args.UpdateFlags;
		if (Args.Widget->Advanced_IsInvalidationRoot())
		{
			// InvalidationRoot should always be volatile paint, let hide them to do confused the user.
			UpdateFlags &= (~EWidgetUpdateFlags::NeedsVolatilePaint);
		}

		if (Args.bFromPaint && bDisplayUpdateFromPaint)
		{
			UpdateFlags |= EWidgetUpdateFlags::NeedsRepaint;
		}

		if (EnumHasAnyFlags(UpdateFlags, WidgetUpdateFlagsFilter))
		{
			if (InvalidationRootIdFilter < 0 || Args.Widget->GetProxyHandle().GetInvalidationRootHandle().GetUniqueId() == InvalidationRootIdFilter)
			{
				if (FWidgetInfo* WidgetInfo = UpdatedWidgets.Find(WidgetId))
				{
					WidgetInfo->Update(Args.Widget, Args.UpdateFlags);
				}
				else
				{
					UpdatedWidgets.Emplace(WidgetId, FWidgetInfo{ Args.Widget, Args.UpdateFlags });
				}
			}
		}
	}
}

void FConsoleSlateDebuggerUpdate::HandlePaintDebugInfo(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId)
{

	++InOutLayerId;

	const FConsoleSlateDebuggerUtility::TSWindowId PaintWindow = FConsoleSlateDebuggerUtility::GetId(InOutDrawElements.GetPaintWindow());
	int32 NumberOfWidget = 0;
	TArray<const FString*, TInlineAllocator<100>> NamesToDisplay;
	const FSlateBrush* FocusBrush = FCoreStyle::Get().GetBrush(TEXT("FocusRectangle"));
	FSlateFontInfo FontInfo = FCoreStyle::Get().GetFontStyle("SmallFont");
	FontInfo.OutlineSettings.OutlineSize = 1;

	CacheDuration = FMath::Max(CacheDuration, 0.01f);
	const double SlateApplicationCurrentTime = FSlateApplicationBase::Get().GetCurrentTime();

	float TextElementY = 48.f;
	auto MakeText = [&](const FString& Text, const FVector2f& Location, const FLinearColor& Color)
	{
		FSlateDrawElement::MakeText(
			InOutDrawElements
			, InOutLayerId
			, InAllottedGeometry.ToPaintGeometry(FVector2f(1.f, 1.f), FSlateLayoutTransform(Location))
			, Text
			, FontInfo
			, ESlateDrawEffect::None
			, Color);
	};

	if (bShowLegend)
	{
		MakeText(TEXT("Repaint"), FVector2f(10.f, 10.f + 0.f), DrawRepaintColor);
		MakeText(TEXT("Volatile Paint"), FVector2f(10.f, 10.f + 12.f), DrawVolatilePaintColor);
		MakeText(TEXT("Tick"), FVector2f(10.f, 10.f + 24.f), DrawTickColor);
		MakeText(TEXT("Active Timer"), FVector2f(10.f, 10.f + 36.f), DrawActiveTimerColor);
		TextElementY += 48.f;
	}

	for (const auto& Itt : UpdatedWidgets)
	{
		if (Itt.Value.WindowId == PaintWindow)
		{
			if (bDebugProjectContentOnly && !Itt.Value.bIsProjectContent)
			{
				continue;
			}

			const float LerpValue = FMath::Clamp((float)(SlateApplicationCurrentTime - Itt.Value.LastInvalidationTime) / CacheDuration, 0.f, 1.f);
			const FGeometry Geometry = FGeometry::MakeRoot(Itt.Value.PaintSize, FSlateLayoutTransform(1.f, Itt.Value.PaintLocation));
			const FPaintGeometry PaintGeometry = Geometry.ToPaintGeometry();
			FLinearColor Color = DrawVolatilePaintColor;

			if (EnumHasAnyFlags(Itt.Value.UpdateFlags, EWidgetUpdateFlags::NeedsRepaint))
			{
				Color = DrawRepaintColor;
			}
			else if (EnumHasAnyFlags(Itt.Value.UpdateFlags, EWidgetUpdateFlags::NeedsVolatilePaint))
			{
				Color = DrawVolatilePaintColor;
			}
			else if (EnumHasAnyFlags(Itt.Value.UpdateFlags, EWidgetUpdateFlags::NeedsTick))
			{
				Color = DrawTickColor;
			}
			else if (EnumHasAnyFlags(Itt.Value.UpdateFlags, EWidgetUpdateFlags::NeedsActiveTimerUpdate))
			{
				Color = DrawActiveTimerColor;
			}

			const FLinearColor ColorWithOpacity = Color.CopyWithNewOpacity(FMath::InterpExpoOut(1.0f, 0.2f, LerpValue));

			if (bShowQuad)
			{
				FSlateDrawElement::MakeDebugQuad(
					InOutDrawElements,
					InOutLayerId,
					PaintGeometry,
					ColorWithOpacity);
			}

			FSlateDrawElement::MakeBox(
				InOutDrawElements,
				InOutLayerId,
				PaintGeometry,
				FocusBrush,
				ESlateDrawEffect::None,
				ColorWithOpacity);

			if (bDisplayWidgetsNameList)
			{
				if (NumberOfWidget < MaxNumberOfWidgetInList)
				{
					MakeText(Itt.Value.WidgetName, FVector2f(0.f, (12.f * NumberOfWidget) + TextElementY), Color);
				}
			}
			++NumberOfWidget;
		}
	}

	if (bDisplayWidgetsNameList && NumberOfWidget > MaxNumberOfWidgetInList)
	{
		FString WidgetDisplayName = FString::Printf(TEXT("   %d more updates"), NumberOfWidget - MaxNumberOfWidgetInList);
		MakeText(WidgetDisplayName, FVector2f(0.f, (12.f * NumberOfWidget) + TextElementY), FLinearColor::White);
	}
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_SLATE_DEBUGGING
