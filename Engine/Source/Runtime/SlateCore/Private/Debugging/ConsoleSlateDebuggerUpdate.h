// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Debugging/SlateDebugging.h"

#if WITH_SLATE_DEBUGGING

#include "CoreMinimal.h"
#include "Debugging/ConsoleSlateDebuggerUtility.h"
#include "Delegates/Delegate.h"
#include "Input/Reply.h"
#include "HAL/IConsoleManager.h"
#include "Rendering/DrawElements.h"
#include "FastUpdate/WidgetUpdateFlags.h"

struct FGeometry;
class FPaintArgs;
class FSlateWindowElementList;
class SWidget;

/**
 * Allows debugging the behavior of SWidget::Paint from the console.
 * Basics:
 *   Start - SlateDebugger.Update.Start
 *   Stop  - SlateDebugger.Update.Stop
 */
class FConsoleSlateDebuggerUpdate
{
public:
	FConsoleSlateDebuggerUpdate();
	virtual ~FConsoleSlateDebuggerUpdate();

	void StartDebugging();
	void StopDebugging();
	bool IsEnabled() const { return bEnabled; }

	void ToggleDisplayLegend();
	bool IsLegendShown() const { return bShowLegend; }
	void ToogleDisplayWidgetNameList();
	void ToogleDisplayUpdateFromPaint();

	void SaveConfig();

private:
	void HandleEnabled(IConsoleVariable* Variable);
	void HandleSetWidgetUpdateFlagsFilter(const TArray<FString>& Params);

	void HandleEndFrame();
	void HandlePaintDebugInfo(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, FSlateWindowElementList& InOutDrawElements, int32& InOutLayerId);
	void HandleWidgetUpdate(const FSlateDebuggingWidgetUpdatedEventArgs& Args);

private:
	bool bEnabled;
	bool bEnabledCVarValue;

	//~ Settings
	bool bDisplayWidgetsNameList;
	bool bUseWidgetPathAsName;
	bool bDisplayUpdateFromPaint;
	bool bShowLegend;
	bool bShowQuad;
	bool bDebugProjectContentOnly;
	EWidgetUpdateFlags WidgetUpdateFlagsFilter;
	FLinearColor DrawVolatilePaintColor;
	FLinearColor DrawRepaintColor;
	FLinearColor DrawTickColor;
	FLinearColor DrawActiveTimerColor;
	FLinearColor DrawWidgetNameColor;
	int32 MaxNumberOfWidgetInList;
	int32 InvalidationRootIdFilter;
	float CacheDuration;

	//~ Console objects
	FAutoConsoleCommand StartCommand;
	FAutoConsoleCommand StopCommand;
	FAutoConsoleVariableRef EnabledRefCVar;
	FAutoConsoleCommand ToggleLegendCommand;
	FAutoConsoleCommand ToogleWidgetsNameListCommand;
	FAutoConsoleCommand ToogleDisplayUpdateFromPaintCommand;
	FAutoConsoleCommand SetWidgetUpdateFlagsFilterCommand;
	FAutoConsoleVariableRef InvalidationRootFilterRefCVar;
	FAutoConsoleVariableRef OnlyProjectContentRefCVar;
	FAutoConsoleVariableDeprecated OnlyGameWindowDepCVar;

	struct FWidgetInfo
	{
		FWidgetInfo(const SWidget* Widget, EWidgetUpdateFlags InUpdateFlags);
		void Update(const SWidget* Widget, EWidgetUpdateFlags InUpdateFlags);

		FConsoleSlateDebuggerUtility::TSWindowId WindowId;
		FVector2f PaintLocation;
		FVector2f PaintSize;
		FString WidgetName;
		EWidgetUpdateFlags UpdateFlags;
		double LastInvalidationTime;
		bool bIsProjectContent;
	};

	using TWidgetMap = TMap<FConsoleSlateDebuggerUtility::TSWidgetId, FWidgetInfo>;
	TWidgetMap UpdatedWidgets;
};

#endif //WITH_SLATE_DEBUGGING