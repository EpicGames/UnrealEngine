// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleSlateDebuggerUtility.h"

#if WITH_SLATE_DEBUGGING

#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

DEFINE_LOG_CATEGORY(LogSlateDebugger);

FConsoleSlateDebuggerUtility::TSWidgetId FConsoleSlateDebuggerUtility::GetId(const SWidget& Widget)
{
#if UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
	return Widget.GetId();
#else
	return reinterpret_cast<TSWidgetId>(&Widget);
#endif
}

FConsoleSlateDebuggerUtility::TSWidgetId FConsoleSlateDebuggerUtility::GetId(const SWidget* Widget)
{
// We prefer to use the widget id but if it's not available, then use the widget address.
//The address can only be compare and it is not ideal.
//A new widget can be created at the same location of a destroyed widget.
//Hopefully, no clash will occur.
#if UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
	return Widget ? Widget->GetId() : InvalidWidgetId;
#else
	return reinterpret_cast<TSWidgetId>(Widget);
#endif
}

FConsoleSlateDebuggerUtility::TSWindowId FConsoleSlateDebuggerUtility::GetId(const SWindow& Widget)
{
#if UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
	return Widget.GetId();
#else
	return reinterpret_cast<TSWindowId>(&Widget);
#endif
}

FConsoleSlateDebuggerUtility::TSWindowId FConsoleSlateDebuggerUtility::GetId(const SWindow* Widget)
{
#if UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
	return Widget ? Widget->GetId() : InvalidWindowId;
#else
	return reinterpret_cast<TSWindowId>(Widget);
#endif
}

FConsoleSlateDebuggerUtility::TSWindowId FConsoleSlateDebuggerUtility::FindWindowId(const SWidget& Widget)
{
	return FindWindowId(&Widget);
}

FConsoleSlateDebuggerUtility::TSWindowId FConsoleSlateDebuggerUtility::FindWindowId(const SWidget* Widget)
{
	while (Widget)
	{
		if (Widget->Advanced_IsWindow())
		{
			return GetId(static_cast<const SWindow*>(Widget));
		}
		Widget = Widget->GetParentWidget().Get();
	}
	return InvalidWindowId;
}

#endif // WITH_SLATE_DEBUGGING
