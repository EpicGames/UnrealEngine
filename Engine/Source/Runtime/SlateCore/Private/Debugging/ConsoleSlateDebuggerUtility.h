// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Debugging/SlateDebugging.h"
#include "SlateGlobals.h"

#if WITH_SLATE_DEBUGGING

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSlateDebugger, Log, All);


class SWidget;
class SWindow;

struct FConsoleSlateDebuggerUtility
{
	FConsoleSlateDebuggerUtility() = delete;

#if UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
	using TSWidgetId = UPTRINT;
	using TSWindowId = UPTRINT;
#else
	using TSWidgetId = uint64;
	using TSWindowId = uint64;
#endif
	static const TSWidgetId InvalidWidgetId = 0;
	static const TSWindowId InvalidWindowId = 0;

	static TSWidgetId GetId(const SWidget& Widget);
	static TSWidgetId GetId(const SWidget* Widget);
	static TSWindowId GetId(const SWindow& Widget);
	static TSWindowId GetId(const SWindow* Widget);
	static TSWindowId FindWindowId(const SWidget& Widget);
	static TSWindowId FindWindowId(const SWidget* Widget);
};

#endif //WITH_SLATE_DEBUGGING