// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugging/SlateCrashReporterHandler.h"

#include "HAL/ThreadManager.h"
#include "Layout/ChildrenBase.h"
#include "Misc/ScopeLock.h"

#include "Widgets/SWidget.h"

#if UE_WITH_SLATE_CRASHREPORTER

namespace UE::Slate::Private
{

static bool GSlateEnableCrashHandler = true;
static FAutoConsoleVariableRef CVarEnableNiagaraCRHandler(
	TEXT("Slate.EnableCrashHandler"),
	GSlateEnableCrashHandler,
	TEXT("Slate will add states into the crash reporter."),
	ECVF_Default);


struct FSlateCrashReporterHandler
{
public:
	enum EThread
	{
		Thread_Slate = 0,	// The Slate thread when preloading occurs
		Thread_Game = 1,	// The Main Game thread
		Thread_Other = 2,	// All other threads. Slate should never runs on other threads.
	};
	static const uint8 NumberOfThread = 3;

private:
	struct FContext
	{
		FContext(const SWidget& Widget)
			: WidgetWeak(Widget.AsWeak())
			, Type(Widget.GetType())
			, Tag(Widget.GetTag())
			, CreatedInLocation(Widget.GetCreatedInLocation())
			, ContextName(NAME_None)
		{
		}

		FContext(const SWidget& Widget, FName InContextName)
			: FContext(Widget)
		{
			ContextName = InContextName;
		}

		TWeakPtr<const SWidget> WidgetWeak;
		FName Type;
		FName Tag;
		FName CreatedInLocation;
		FName ContextName;
	};

	FSlateCrashReporterHandler() = default;
	FSlateCrashReporterHandler(const FSlateCrashReporterHandler&) = delete;
	FSlateCrashReporterHandler& operator=(const FSlateCrashReporterHandler&) = delete;

public:
	static EThread GetThreadId()
	{
		return IsInGameThread() ? EThread::Thread_Game : (IsInSlateThread() ? EThread::Thread_Slate : EThread::Thread_Other);
	}

	static const TCHAR* LexToString(EThread Thread)
	{
		switch(Thread)
		{
		case EThread::Thread_Game:
			return TEXT("Game");
		case EThread::Thread_Slate:
			return TEXT("Slate");
		}
		return TEXT("Other");
	}

	static FSlateCrashReporterHandler& Get()
	{
		struct FLocal
		{
			FSlateCrashReporterHandler* Instance;
			FLocal()
			{
				Instance = new FSlateCrashReporterHandler();
				FGenericCrashContext::OnAdditionalCrashContextDelegate().AddRaw(Instance, &FSlateCrashReporterHandler::DumpInfo);
			}
			~FLocal()
			{
				FGenericCrashContext::OnAdditionalCrashContextDelegate().RemoveAll(Instance);
				delete Instance;
			}
		};
		static FLocal Local;
		return *Local.Instance;
	}

	EThread PushInfo(const SWidget& Widget, FName ContextName)
	{
		const EThread Thread = GetThreadId();
		if (Thread == EThread::Thread_Other)
		{
			FScopeLock LockGuard(&CriticalSection);
			Contexts[EThread::Thread_Other].Emplace(Widget, ContextName);
		}
		else
		{
			Contexts[Thread].Emplace(Widget, ContextName);
		}
		return Thread;
	}

	void PopInfo(EThread InThread)
	{
		if (InThread == EThread::Thread_Other)
		{
			FScopeLock LockGuard(&CriticalSection);
			Contexts[EThread::Thread_Other].Pop();
		}
		else
		{
			Contexts[InThread].Pop();
		}
	}

	void DumpInfo(FCrashContextExtendedWriter& Writer)
	{
		FScopeLock LockGuard(&CriticalSection);

		Builder.Reset();
		for (int32 ThreadIndex = 0; ThreadIndex < NumberOfThread; ++ThreadIndex)
		{
			if (Contexts[ThreadIndex].Num())
			{
				Builder.Append(TEXT("Thread: "));
				Builder.Append(LexToString((EThread)ThreadIndex));
				Builder.AppendChar(TEXT('\n'));
				for (const FContext& Context : Contexts[ThreadIndex])
				{
					Builder.Append(TEXT("Context:'"));
					Context.ContextName.AppendString(Builder);
					Builder.Append(TEXT("' Type:'"));
					Context.Type.AppendString(Builder);
					Builder.Append(TEXT("' Tag:'"));
					Context.Tag.AppendString(Builder);
					Builder.Append(TEXT("' Location:'"));
					Context.CreatedInLocation.AppendString(Builder);
					if (TSharedPtr<const SWidget> Widget = Context.WidgetWeak.Pin())
					{
						Builder.Append(TEXT("' Path:'"));
						AddWidgetPath(Widget);
					}
					Builder.Append(TEXT("'\n"));
				}
			}
		}
		
		if (Builder.Len())
		{
			Writer.AddString(TEXT("Slate"), Builder.ToString());
		}
	}

private:
	FCriticalSection CriticalSection;
	TArray<FContext> Contexts[NumberOfThread];
	TStringBuilder<16384> Builder;

	void AddWidgetPath(TSharedPtr<const SWidget> InLeafNode)
	{
		constexpr int32 StackSize = 500;

		SWidget* Stack[StackSize];
		int32 StackPtr = 0;

		Stack[StackPtr] = const_cast<SWidget*>(InLeafNode.Get());

		while (Stack[StackPtr] && StackPtr < (StackSize - 1)) //-V781
		{
			++StackPtr;
			Stack[StackPtr] = Stack[StackPtr - 1]->GetParentWidget().Get();
		}

		// If the final parent was null, decrease the ptr to a valid widget.
		if (!Stack[StackPtr])
		{
			--StackPtr;
		}

		auto GetChildIndex = [](SWidget* InParent, SWidget* InChild) -> int32
			{
				if (FChildren* Children = InParent->GetChildren())
				{
					for (int32 Index = 0; Index < Children->Num(); ++Index)
					{
						if (&*Children->GetChildAt(Index) == InChild)
						{
							return Index;
						}
					}
				}

				return -1;
			};

		while (StackPtr >= 0)
		{
			Builder.Append(*Stack[StackPtr]->GetTypeAsString());

			const FName WidgetTag = Stack[StackPtr]->GetTag();

			if (!WidgetTag.IsNone())
			{
				Builder.Appendf(TEXT(" (%s)"), *WidgetTag.ToString());
			}

			if (StackPtr > 0)
			{
				if (FChildren* Children = Stack[StackPtr]->GetChildren())
				{
					if (Children->Num() > 1)
					{
						Builder.Appendf(TEXT(" [%i]"), GetChildIndex(Stack[StackPtr], Stack[StackPtr - 1]));
					}
				}

				Builder.Append(TEXT(": "));
			}

			--StackPtr;
		}
	}
};

} // namespace

namespace UE::Slate
{

FName FSlateCrashReporterScope::NAME_Paint = TEXT("Paint");
FName FSlateCrashReporterScope::NAME_Prepass = TEXT("Prepass");

FSlateCrashReporterScope::FSlateCrashReporterScope(const SWidget& Widget, FName Tag)
{
	bWasEnabled = Private::GSlateEnableCrashHandler != 0;
	if (bWasEnabled)
	{
		ThreadId = Private::FSlateCrashReporterHandler::Get().PushInfo(Widget, Tag);
	}
}


FSlateCrashReporterScope::~FSlateCrashReporterScope()
{
	if (bWasEnabled)
	{
		Private::FSlateCrashReporterHandler::Get().PopInfo((Private::FSlateCrashReporterHandler::EThread)ThreadId);
	}
}

} //namespace

#endif