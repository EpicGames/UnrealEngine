// Copyright Epic Games, Inc. All Rights Reserved.

#include "Debugging/SlateDebugging.h"

#include "Animation/CurveSequence.h"
#include "Application/SlateApplicationBase.h"
#include "Debugging/WidgetList.h"
#include "FastUpdate/WidgetProxy.h"
#include "Layout/WidgetPath.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Rendering/DrawElements.h"
#include "Rendering/DrawElementPayloads.h"
#include "SlateGlobals.h"
#include "Styling/CoreStyle.h"
#include "Types/ReflectionMetadata.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateDebugging)

#if UE_BUILD_SHIPPING
CSV_DEFINE_CATEGORY_MODULE(SLATECORE_API, Slate, false);
#else
CSV_DEFINE_CATEGORY_MODULE(SLATECORE_API, Slate, true);
#endif

#if WITH_SLATE_DEBUGGING

#define LOCTEXT_NAMESPACE "SlateDebugger"

FSlateDebuggingInputEventArgs::FSlateDebuggingInputEventArgs(ESlateDebuggingInputEvent InInputEventType, const FInputEvent* InInputEvent, const FReply& InReply, const TSharedPtr<SWidget>& InHandlerWidget, const FString& InAdditionalContent)
	: InputEventType(InInputEventType)
	, InputEvent(InInputEvent)
	, Reply(InReply)
	, HandlerWidget(InHandlerWidget)
	, AdditionalContent(InAdditionalContent)
{
}

FText FSlateDebuggingInputEventArgs::ToText() const
{
	static const FText InputEventFormat = LOCTEXT("InputEventFormat", "{0} - ({1}) - ({2}) - [{3}]");

	const UEnum* SlateDebuggingInputEventEnum = StaticEnum<ESlateDebuggingInputEvent>();
	const FText InputEventTypeText = SlateDebuggingInputEventEnum->GetDisplayNameTextByValue((int64)InputEventType);
	const FText InputEventText = InputEvent ? InputEvent->ToText() : LOCTEXT("NullEvent", "<null event>");
	const FText AdditionalContentText = FText::FromString(AdditionalContent);
	const FText HandlerWidgetText = FText::FromString(FReflectionMetaData::GetWidgetDebugInfo(HandlerWidget.Get()));

	return FText::Format(
		InputEventFormat,
		InputEventTypeText,
		HandlerWidgetText,
		InputEventText,
		AdditionalContentText
	);
}

FSlateDebuggingFocusEventArgs::FSlateDebuggingFocusEventArgs(
	ESlateDebuggingFocusEvent InFocusEventType,
	const FFocusEvent& InFocusEvent,
	const FWeakWidgetPath& InOldFocusedWidgetPath,
	const TSharedPtr<SWidget>& InOldFocusedWidget,
	const FWidgetPath& InNewFocusedWidgetPath,
	const TSharedPtr<SWidget>& InNewFocusedWidget)
	: FocusEventType(InFocusEventType)
	, FocusEvent(InFocusEvent)
	, OldFocusedWidgetPath(InOldFocusedWidgetPath)
	, OldFocusedWidget(InOldFocusedWidget)
	, NewFocusedWidgetPath(InNewFocusedWidgetPath)
	, NewFocusedWidget(InNewFocusedWidget)
{
} 

FText FSlateDebuggingFocusEventArgs::ToText() const
{
	static const FText FocusEventFormat = LOCTEXT("FocusEventFormat", "{0}({1}:{2}) - {3} -> {4}");

	FText FocusEventText;
	switch (FocusEventType)
	{
	case ESlateDebuggingFocusEvent::FocusChanging:
		FocusEventText = LOCTEXT("FocusChanging", "Focus Changing");
		break;
	case ESlateDebuggingFocusEvent::FocusLost:
		FocusEventText = LOCTEXT("FocusLost", "Focus Lost");
		break;
	case ESlateDebuggingFocusEvent::FocusReceived:
		FocusEventText = LOCTEXT("FocusReceived", "Focus Received");
		break;
	default:
		ensureMsgf(false, TEXT("A focus event was not handled."));
	}

	FText CauseText;
	switch (FocusEvent.GetCause())
	{
	case EFocusCause::Mouse:
		CauseText = LOCTEXT("FocusCause_Mouse", "Mouse");
		break;
	case EFocusCause::Navigation:
		CauseText = LOCTEXT("FocusCause_Navigation", "Navigation");
		break;
	case EFocusCause::SetDirectly:
		CauseText = LOCTEXT("FocusCause_SetDirectly", "SetDirectly");
		break;
	case EFocusCause::Cleared:
		CauseText = LOCTEXT("FocusCause_Cleared", "Cleared");
		break;
	case EFocusCause::OtherWidgetLostFocus:
		CauseText = LOCTEXT("FocusCause_OtherWidgetLostFocus", "OtherWidgetLostFocus");
		break;
	case EFocusCause::WindowActivate:
		CauseText = LOCTEXT("FocusCause_WindowActivate", "WindowActivate");
		break;
	default:
		ensureMsgf(false, TEXT("A focus case was not handled."));
	}

	const int32 UserIndex = FocusEvent.GetUser();

	const FText OldFocusedWidgetText = FText::FromString(FReflectionMetaData::GetWidgetDebugInfo(OldFocusedWidget.Get()));
	const FText NewFocusedWidgetText = FText::FromString(FReflectionMetaData::GetWidgetDebugInfo(NewFocusedWidget.Get()));

	return FText::Format(
		FocusEventFormat,
		FocusEventText,
		UserIndex,
		CauseText,
		OldFocusedWidgetText,
		NewFocusedWidgetText
	);
}

FSlateDebuggingNavigationEventArgs::FSlateDebuggingNavigationEventArgs(
	const FNavigationEvent& InNavigationEvent,
	const FNavigationReply& InNavigationReply,
	const FWidgetPath& InNavigationSource,
	const TSharedPtr<SWidget>& InDestinationWidget,
	const ESlateDebuggingNavigationMethod InNavigationMethod)
	: NavigationEvent(InNavigationEvent)
	, NavigationReply(InNavigationReply)
	, NavigationSource(InNavigationSource)
	, DestinationWidget(InDestinationWidget)
	, NavigationMethod(InNavigationMethod)
{
}

FText FSlateDebuggingNavigationEventArgs::ToText() const
{
	static const FText NavEventFormat = LOCTEXT("NavEventFormat", "Navigation User({4}) Source({0}:{1}) | {5} | {2} -> {3}");

	const FText SourceWidgetText = FText::FromString(FReflectionMetaData::GetWidgetDebugInfo(&NavigationSource.GetLastWidget().Get()));
	const FText DestinationWidgetText = FText::FromString(FReflectionMetaData::GetWidgetDebugInfo(DestinationWidget.Get()));
	const FText NavigationTypeText = StaticEnum<EUINavigation>()->GetDisplayNameTextByValue((int64)NavigationEvent.GetNavigationType());
	const FText NavigationGenesisText = StaticEnum<ENavigationGenesis>()->GetDisplayNameTextByValue((int64)NavigationEvent.GetNavigationGenesis());
	const FText NavigationMethodText = StaticEnum<ESlateDebuggingNavigationMethod>()->GetDisplayNameTextByValue((int64)NavigationMethod);

	return FText::Format(
		NavEventFormat,
		NavigationGenesisText,
		NavigationTypeText,
		SourceWidgetText,
		DestinationWidgetText,
		NavigationEvent.GetUserIndex(),
		NavigationMethodText
	);
}

FSlateDebuggingWarningEventArgs::FSlateDebuggingWarningEventArgs(
	const FText& InWarning,
	const TSharedPtr<SWidget>& InOptionalContextWidget)
	: Warning(InWarning)
	, OptionalContextWidget(InOptionalContextWidget)
{
}

FText FSlateDebuggingWarningEventArgs::ToText() const
{
	static const FText InputEventFormat = LOCTEXT("WarningEventFormat", "{0} (Widget: {1})");

	const FText ContextWidget = FText::FromString(FReflectionMetaData::GetWidgetDebugInfo(OptionalContextWidget.Get()));

	return FText::Format(
		InputEventFormat,
		Warning,
		ContextWidget
	);
}

FSlateDebuggingMouseCaptureEventArgs::FSlateDebuggingMouseCaptureEventArgs(
	bool InCaptured,
	uint32 InUserIndex,
	uint32 InPointerIndex,
	const TSharedPtr<const SWidget>& InCapturingWidget)
	: Captured(InCaptured)
	, UserIndex(InUserIndex)
	, PointerIndex(InPointerIndex)
	, CaptureWidget(InCapturingWidget)
{
}

FText FSlateDebuggingMouseCaptureEventArgs::ToText() const
{
	static const FText StateChangeEventFormat = LOCTEXT("StateChangeEventFormat", "{0}({1}:{2}) : {3}");

	const FText StateText = Captured ? LOCTEXT("MouseCaptured", "Mouse Captured") : LOCTEXT("MouseCaptureLost", "Mouse Capture Lost");
	const FText SourceWidget = FText::FromString(FReflectionMetaData::GetWidgetDebugInfo(CaptureWidget.Get()));

	return FText::Format(
		StateChangeEventFormat,
		StateText,
		UserIndex,
		PointerIndex,
		SourceWidget
	);
}

FSlateDebuggingCursorQueryEventArgs::FSlateDebuggingCursorQueryEventArgs(const TSharedPtr<const SWidget>& InWidgetOverridingCursor, const FCursorReply& InReply)
	: WidgetOverridingCursor(InWidgetOverridingCursor)
	, Reply(InReply)
{
}

FText FSlateDebuggingCursorQueryEventArgs::ToText() const
{
	const FText ContextWidget = FText::FromString(FReflectionMetaData::GetWidgetDebugInfo(WidgetOverridingCursor.Get()));

	FText EventText;
	if (Reply.GetCursorWidget().IsValid())
	{
		static const FText InputEventFormat = LOCTEXT("CursorChangedToWidget", "{0} To Widget: {0} (Widget: {1})");
		EventText = FText::Format(
			InputEventFormat,
			FText::FromString(FReflectionMetaData::GetWidgetDebugInfo(Reply.GetCursorWidget().Get())),
			ContextWidget
		);
	}
	else
	{
		static const FText InputEventFormat = LOCTEXT("CursorChangedToCursor", "Cursor Changed: To Type: {0} (By Widget: {1})");
		EventText = FText::Format(
			InputEventFormat,
			FText::FromString(StaticEnum<EMouseCursor::Type>()->GetNameStringByValue((int64)Reply.GetCursorType())),
			ContextWidget
		);
	}
	return EventText;
}

FString LexToString(ESlateDebuggingInvalidateRootReason InValue)
{
	if (InValue == ESlateDebuggingInvalidateRootReason::None)
	{
		return TEXT("None");
	}
	if (InValue == (ESlateDebuggingInvalidateRootReason)0xFF)
	{
		return TEXT("All");
	}

	TStringBuilder<512> Result;
#define ENUM_CASE_TO_STRING(Enum) if (EnumHasAnyFlags(InValue, ESlateDebuggingInvalidateRootReason::Enum)) { if (Result.Len() != 0) { Result.AppendChar(TEXT('|')); } Result.Append(TEXT(#Enum)); }
	ENUM_CASE_TO_STRING(ChildOrder);
	ENUM_CASE_TO_STRING(Root);
	ENUM_CASE_TO_STRING(ScreenPosition);
#undef ENUM_CASE_TO_STRING

	return Result.ToString();
}

bool LexTryParseString(ESlateDebuggingInvalidateRootReason& OutValue, const TCHAR* Buffer)
{
	bool bResult = false;
	ESlateDebuggingInvalidateRootReason Value = ESlateDebuggingInvalidateRootReason::None;
	auto ParseResult = [&bResult, &Value](FStringView& SubString)
	{
		SubString.TrimStartAndEndInline();

		if (SubString.Equals(TEXT("All"), ESearchCase::IgnoreCase)) { Value = (ESlateDebuggingInvalidateRootReason)0xFF; return; }
		if (SubString.Equals(TEXT("Any"), ESearchCase::IgnoreCase)) { Value = (ESlateDebuggingInvalidateRootReason)0xFF; return; }

#define ENUM_CASE_FROM_STRING(Enum) if (SubString.Equals(TEXT(#Enum), ESearchCase::IgnoreCase)) { Value |= ESlateDebuggingInvalidateRootReason::Enum; return; }
		ENUM_CASE_FROM_STRING(None)
		ENUM_CASE_FROM_STRING(ChildOrder)
		ENUM_CASE_FROM_STRING(Root)
		ENUM_CASE_FROM_STRING(ScreenPosition)
		bResult = false;
#undef ENUM_CASE_FROM_STRING
	};

	if (Buffer && *Buffer)
	{
		bResult = true;
		while (const TCHAR* At = FCString::Strchr(Buffer, TEXT('|')))
		{
			FStringView SubString{ Buffer, UE_PTRDIFF_TO_INT32(At - Buffer) };
			ParseResult(SubString);
			Buffer = At + 1;
		}
		if (*Buffer)
		{
			FStringView SubString{ Buffer };
			ParseResult(SubString);
		}
	}

	if (bResult)
	{
		OutValue = Value;
	}
	return bResult;
}

void LexFromString(ESlateDebuggingInvalidateRootReason& OutValue, const TCHAR* Buffer)
{
	OutValue = ESlateDebuggingInvalidateRootReason::None;
	LexTryParseString(OutValue, Buffer);
}

FSlateDebuggingInvalidateArgs::FSlateDebuggingInvalidateArgs(
	const SWidget* InWidgetInvalidated,
	const SWidget* InWidgetInvalidateInvestigator,
	EInvalidateWidgetReason InInvalidateReason)
	: WidgetInvalidated(InWidgetInvalidated)
	, WidgetInvalidateInvestigator(InWidgetInvalidateInvestigator)
	, InvalidateWidgetReason(InInvalidateReason)
	, InvalidateInvalidationRootReason(ESlateDebuggingInvalidateRootReason::None)
{
}

FSlateDebuggingInvalidateArgs::FSlateDebuggingInvalidateArgs(
	const SWidget* InWidgetInvalidated,
	const SWidget* InWidgetInvalidateInvestigator,
	ESlateDebuggingInvalidateRootReason InInvalidateReason)
	: WidgetInvalidated(InWidgetInvalidated)
	, WidgetInvalidateInvestigator(InWidgetInvalidateInvestigator)
	, InvalidateWidgetReason(EInvalidateWidgetReason::None)
	, InvalidateInvalidationRootReason(InInvalidateReason)
{
}

FSlateDebuggingWidgetUpdatedEventArgs::FSlateDebuggingWidgetUpdatedEventArgs(
	const SWidget* InWidget,
	EWidgetUpdateFlags InUpdateFlags,
	bool bInFromPaint)
	: Widget(InWidget)
	, UpdateFlags(InUpdateFlags)
	, bFromPaint(bInFromPaint)
{
}

FSlateDebuggingElementTypeAddedEventArgs::FSlateDebuggingElementTypeAddedEventArgs(
	const FSlateWindowElementList& InElementList,
	int32 InElementIndex,
	EElementType InElementType)
	: ElementList(InElementList)
	, ElementIndex(InElementIndex)
	, ElementType(InElementType)
{
}

FText FSlateDebuggingWidgetUpdatedEventArgs::ToText() const
{
	TArray<FText> UpdateText;
	if (EnumHasAnyFlags(UpdateFlags, EWidgetUpdateFlags::NeedsVolatilePaint))
	{
		UpdateText.Add( LOCTEXT("NeedsVolatilePaint", "Volatile Repaint"));
	}
	else if (EnumHasAnyFlags(UpdateFlags, EWidgetUpdateFlags::NeedsRepaint))
	{
		UpdateText.Add(LOCTEXT("NeedsRepaint", "Repaint"));
	}
	else if (EnumHasAllFlags(UpdateFlags, EWidgetUpdateFlags::NeedsActiveTimerUpdate|EWidgetUpdateFlags::NeedsTick ))
	{
		UpdateText.Add(LOCTEXT("NeedsTickNeedsActiveTimerUpdate", "Active Timer and Tick"));
	}
	else if (EnumHasAnyFlags(UpdateFlags, EWidgetUpdateFlags::NeedsActiveTimerUpdate))
	{
		UpdateText.Add(LOCTEXT("NeedsActiveTimerUpdate", "Active Timer"));
	}
	else if (EnumHasAnyFlags(UpdateFlags, EWidgetUpdateFlags::NeedsTick))
	{
		UpdateText.Add(LOCTEXT("NeedsTick", "Tick"));
	}

	return FText::Format(
		LOCTEXT("WidgetUpdatedEventFormat", "{0} {1}"),
		FText::Join(FText::FromString(TEXT("|")), UpdateText),
		FText::FromString(FReflectionMetaData::GetWidgetDebugInfo(Widget)));
}

FSlateDebugging::FBeginWindow FSlateDebugging::BeginWindow;

FSlateDebugging::FEndWindow FSlateDebugging::EndWindow;

FSlateDebugging::FBeginWidgetPaint FSlateDebugging::BeginWidgetPaint;

FSlateDebugging::FEndWidgetPaint FSlateDebugging::EndWidgetPaint;

FSlateDebugging::FBeginWidgetPrepass FSlateDebugging::BeginWidgetPrepass;

FSlateDebugging::FEndWidgetPrepass FSlateDebugging::EndWidgetPrepass;

FSlateDebugging::FPaintDebugElements FSlateDebugging::PaintDebugElements;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FSlateDebugging::FDrawElement FSlateDebugging::ElementAdded;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FSlateDebugging::FDrawElementType FSlateDebugging::ElementTypeAdded;

FSlateDebugging::FWidgetWarningEvent FSlateDebugging::Warning;

FSlateDebugging::FWidgetInputEvent FSlateDebugging::InputEvent;

FSlateDebugging::FWidgetFocusEvent FSlateDebugging::FocusEvent;

FSlateDebugging::FWidgetAttemptNavigationEvent FSlateDebugging::AttemptNavigationEvent;

FSlateDebugging::FWidgetExecuteNavigationEvent FSlateDebugging::ExecuteNavigationEvent;

FSlateDebugging::FWidgetMouseCaptureEvent FSlateDebugging::MouseCaptureEvent;

FSlateDebugging::FWidgetCursorQuery FSlateDebugging::CursorChangedEvent;

FSlateDebugging::FWidgetInvalidate FSlateDebugging::WidgetInvalidateEvent;
 
FSlateDebugging::FWidgetUpdatedEvent FSlateDebugging::WidgetUpdatedEvent;

FSlateDebugging::FUICommandRun FSlateDebugging::CommandRun;

DECLARE_MULTICAST_DELEGATE_OneParam(FWidgetMouseCaptureEvent, const FSlateDebuggingMouseCaptureEventArgs& /*EventArgs*/);

FSlateDebugging::FLastCursorQuery FSlateDebugging::LastCursorQuery;

TArray<FSlateDebugging::IWidgetInputRoutingEvent*> FSlateDebugging::RoutingEvents;

void FSlateDebugging::BroadcastWarning(const FText& WarningText, const TSharedPtr<SWidget>& OptionalContextWidget)
{
	if (Warning.IsBound())
	{
		Warning.Broadcast(FSlateDebuggingWarningEventArgs(WarningText, OptionalContextWidget));
	}
}

void FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const FReply& InReply)
{
	if (InReply.IsEventHandled() && InputEvent.IsBound())
	{
		InputEvent.Broadcast(FSlateDebuggingInputEventArgs(InputEventType, InInputEvent, InReply, TSharedPtr<SWidget>(), FString()));
	}
	for (IWidgetInputRoutingEvent* Event : RoutingEvents)
	{
		Event->OnInputEvent(InputEventType, InReply, TSharedPtr<SWidget>());
	}
}

void FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const TSharedPtr<SWidget>& HandlerWidget)
{
	if (InputEvent.IsBound())
	{
		InputEvent.Broadcast(FSlateDebuggingInputEventArgs(InputEventType, InInputEvent, FReply::Handled(), HandlerWidget, FString()));
	}
	for (IWidgetInputRoutingEvent* Event : RoutingEvents)
	{
		Event->OnInputEvent(InputEventType, FReply::Handled(), HandlerWidget);
	}
}

void FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const FReply& InReply, const TSharedPtr<SWidget>& HandlerWidget)
{
	if (InReply.IsEventHandled() && InputEvent.IsBound())
	{
		InputEvent.Broadcast(FSlateDebuggingInputEventArgs(InputEventType, InInputEvent, InReply, HandlerWidget, FString()));
	}
	for (IWidgetInputRoutingEvent* Event : RoutingEvents)
	{
		Event->OnInputEvent(InputEventType, InReply, HandlerWidget);
	}
}

void FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const FReply& InReply, const TSharedPtr<SWidget>& HandlerWidget, const FString& AdditionalContent)
{
	if (InReply.IsEventHandled() && InputEvent.IsBound())
	{
		InputEvent.Broadcast(FSlateDebuggingInputEventArgs(InputEventType, InInputEvent, InReply, HandlerWidget, AdditionalContent));
	}
	for (IWidgetInputRoutingEvent* Event : RoutingEvents)
	{
		Event->OnInputEvent(InputEventType, InReply, HandlerWidget);
	}
}

void FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const FReply& InReply, const TSharedPtr<SWidget>& HandlerWidget, const FName& AdditionalContent)
{
	if (InReply.IsEventHandled() && InputEvent.IsBound())
	{
		InputEvent.Broadcast(FSlateDebuggingInputEventArgs(InputEventType, InInputEvent, InReply, HandlerWidget, AdditionalContent.ToString()));
	}
	for (IWidgetInputRoutingEvent* Event : RoutingEvents)
	{
		Event->OnInputEvent(InputEventType, InReply, HandlerWidget);
	}
}

void FSlateDebugging::BroadcastInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const FReply& InReply, const TSharedPtr<SWidget>& HandlerWidget, const TCHAR AdditionalContent)
{
	if (InReply.IsEventHandled() && InputEvent.IsBound())
	{
		InputEvent.Broadcast(FSlateDebuggingInputEventArgs(InputEventType, InInputEvent, InReply, HandlerWidget, FString::ConstructFromPtrSize(&AdditionalContent, 1)));
	}
	for (IWidgetInputRoutingEvent* Event : RoutingEvents)
	{
		Event->OnInputEvent(InputEventType, InReply, HandlerWidget);
	}
}

void FSlateDebugging::BroadcastNoReplyInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const TSharedPtr<SWidget>& HandlerWidget)
{
	if (InputEvent.IsBound())
	{
		InputEvent.Broadcast(FSlateDebuggingInputEventArgs(InputEventType, InInputEvent, FReply::Unhandled(), HandlerWidget, TEXT("")));
	}
	for (IWidgetInputRoutingEvent* Event : RoutingEvents)
	{
		Event->OnInputEvent(InputEventType, FReply::Unhandled(), HandlerWidget);
	}
}

void FSlateDebugging::BroadcastPreProcessInputEvent(ESlateDebuggingInputEvent InputEventType, const TCHAR* InputPrecessorName, bool bHandled)
{
	for (IWidgetInputRoutingEvent* Event : RoutingEvents)
	{
		Event->OnPreProcessInput(InputEventType, InputPrecessorName, bHandled);
	}
}

FSlateDebugging::FScopeProcessInputEvent::FScopeProcessInputEvent(ESlateDebuggingInputEvent InInputEvent, const FInputEvent& Event)
	: InputEvent(InInputEvent)
{
	for (IWidgetInputRoutingEvent* RoutingEvent : FSlateDebugging::RoutingEvents)
	{
		RoutingEvent->OnProcessInput(InputEvent, Event);
	}
}

FSlateDebugging::FScopeProcessInputEvent::~FScopeProcessInputEvent()
{
	for (IWidgetInputRoutingEvent* RoutingEvent : FSlateDebugging::RoutingEvents)
	{
		RoutingEvent->OnInputProcessed(InputEvent);
	}
}

FSlateDebugging::FScopeRouteInputEvent::FScopeRouteInputEvent(ESlateDebuggingInputEvent InInputEvent, const FName& RoutingType)
	: InputEvent(InInputEvent)
{
	for (IWidgetInputRoutingEvent* RoutingEvent : FSlateDebugging::RoutingEvents)
	{
		RoutingEvent->OnRouteInput(InputEvent, RoutingType);
	}
}

FSlateDebugging::FScopeRouteInputEvent::~FScopeRouteInputEvent()
{
	for (IWidgetInputRoutingEvent* RoutingEvent : FSlateDebugging::RoutingEvents)
	{
		RoutingEvent->OnInputRouted(InputEvent);
	}
}

void FSlateDebugging::RegisterWidgetInputRoutingEvent(IWidgetInputRoutingEvent* Event)
{
	check(Event);
	RoutingEvents.Add(Event);
}

void FSlateDebugging::UnregisterWidgetInputRoutingEvent(IWidgetInputRoutingEvent* Event)
{
	check(Event);
	RoutingEvents.RemoveSwap(Event);
}

void FSlateDebugging::BroadcastFocusChanging(const FFocusEvent& InFocusEvent, const FWeakWidgetPath& InOldFocusedWidgetPath, const TSharedPtr<SWidget>& InOldFocusedWidget, const FWidgetPath& InNewFocusedWidgetPath, const TSharedPtr<SWidget>& InNewFocusedWidget)
{
	if (FocusEvent.IsBound())
	{
		FocusEvent.Broadcast(FSlateDebuggingFocusEventArgs(ESlateDebuggingFocusEvent::FocusChanging, InFocusEvent, InOldFocusedWidgetPath, InOldFocusedWidget, InNewFocusedWidgetPath, InNewFocusedWidget));
	}
}

void FSlateDebugging::BroadcastFocusLost(const FFocusEvent& InFocusEvent, const FWeakWidgetPath& InOldFocusedWidgetPath, const TSharedPtr<SWidget>& InOldFocusedWidget, const FWidgetPath& InNewFocusedWidgetPath, const TSharedPtr<SWidget>& InNewFocusedWidget)
{
	if (FocusEvent.IsBound())
	{
		FocusEvent.Broadcast(FSlateDebuggingFocusEventArgs(ESlateDebuggingFocusEvent::FocusLost, InFocusEvent, InOldFocusedWidgetPath, InOldFocusedWidget, InNewFocusedWidgetPath, InNewFocusedWidget));
	}
}

void FSlateDebugging::BroadcastFocusReceived(const FFocusEvent& InFocusEvent, const FWeakWidgetPath& InOldFocusedWidgetPath, const TSharedPtr<SWidget>& InOldFocusedWidget, const FWidgetPath& InNewFocusedWidgetPath, const TSharedPtr<SWidget>& InNewFocusedWidget)
{
	if (FocusEvent.IsBound())
	{
		FocusEvent.Broadcast(FSlateDebuggingFocusEventArgs(ESlateDebuggingFocusEvent::FocusReceived, InFocusEvent, InOldFocusedWidgetPath, InOldFocusedWidget, InNewFocusedWidgetPath, InNewFocusedWidget));
	}
}

void FSlateDebugging::BroadcastAttemptNavigation(const FNavigationEvent& InNavigationEvent, const FNavigationReply& InNavigationReply, const FWidgetPath& InNavigationSource, const TSharedPtr<SWidget>& InDestinationWidget, ESlateDebuggingNavigationMethod InNavigationMethod)
{
	if (AttemptNavigationEvent.IsBound())
	{
		AttemptNavigationEvent.Broadcast(FSlateDebuggingNavigationEventArgs(InNavigationEvent, InNavigationReply, InNavigationSource, InDestinationWidget, InNavigationMethod));
	}
}

void FSlateDebugging::BroadcastExecuteNavigation()
{
	if (ExecuteNavigationEvent.IsBound())
	{
		ExecuteNavigationEvent.Broadcast(FSlateDebuggingExecuteNavigationEventArgs());
	}
}

void FSlateDebugging::BroadcastMouseCapture(uint32 UserIndex, uint32 PointerIndex, TSharedPtr<const SWidget> InCapturingWidget)
{
	if (MouseCaptureEvent.IsBound())
	{
		MouseCaptureEvent.Broadcast(FSlateDebuggingMouseCaptureEventArgs(true, UserIndex, PointerIndex, InCapturingWidget));
	}
}

void FSlateDebugging::BroadcastMouseCaptureLost(uint32 UserIndex, uint32 PointerIndex, TSharedPtr<const SWidget> InWidgetLostCapture)
{
	if (MouseCaptureEvent.IsBound())
	{
		MouseCaptureEvent.Broadcast(FSlateDebuggingMouseCaptureEventArgs(false, UserIndex, PointerIndex, InWidgetLostCapture));
	}
}

void FSlateDebugging::BroadcastCursorQuery(TSharedPtr<const SWidget> InWidgetOverridingCursor, const FCursorReply& InReply)
{
	if (LastCursorQuery.WidgetThatOverrideCursorLast_UnsafeToUseForAnythingButCompare != InWidgetOverridingCursor.Get() ||
		LastCursorQuery.MouseCursor != InReply.GetCursorType() ||
		LastCursorQuery.CursorWidget.Pin() != InReply.GetCursorWidget())
	{
		LastCursorQuery.WidgetThatOverrideCursorLast_UnsafeToUseForAnythingButCompare = InWidgetOverridingCursor.Get();
		LastCursorQuery.MouseCursor = InReply.GetCursorType();
		LastCursorQuery.CursorWidget = InReply.GetCursorWidget();

		CursorChangedEvent.Broadcast(FSlateDebuggingCursorQueryEventArgs(InWidgetOverridingCursor, InReply));
	}
}

void FSlateDebugging::BroadcastWidgetInvalidate(const SWidget* InWidgetInvalidated, const SWidget* InWidgetInvalidateInvestigator, EInvalidateWidgetReason InInvalidateReason)
{
	if (WidgetInvalidateEvent.IsBound())
	{
		WidgetInvalidateEvent.Broadcast(FSlateDebuggingInvalidateArgs(InWidgetInvalidated, InWidgetInvalidateInvestigator, InInvalidateReason));
	}
}

void FSlateDebugging::BroadcastInvalidationRootInvalidate(const SWidget* InWidgetInvalidated, const SWidget* InWidgetInvalidateInvestigator, ESlateDebuggingInvalidateRootReason InInvalidateReason)
{
	if (WidgetInvalidateEvent.IsBound())
	{
		WidgetInvalidateEvent.Broadcast(FSlateDebuggingInvalidateArgs(InWidgetInvalidated, InWidgetInvalidateInvestigator, InInvalidateReason));
	}
}

void FSlateDebugging::BroadcastWidgetUpdated(const SWidget* Invalidated, EWidgetUpdateFlags UpdateFlags)
{
	if (WidgetUpdatedEvent.IsBound())
	{
		WidgetUpdatedEvent.Broadcast(FSlateDebuggingWidgetUpdatedEventArgs(Invalidated, UpdateFlags, false));
	}
}

void FSlateDebugging::BroadcastWidgetUpdatedByPaint(const SWidget* Invalidated, EWidgetUpdateFlags UpdateFlags)
{
	if (WidgetUpdatedEvent.IsBound())
	{
		WidgetUpdatedEvent.Broadcast(FSlateDebuggingWidgetUpdatedEventArgs(Invalidated, UpdateFlags, true));
	}
}

const TArray<const SWidget*>& FSlateDebugging::GetAllWidgets()
{
#if UE_WITH_SLATE_DEBUG_WIDGETLIST
	return UE::Slate::FWidgetList::GetAllWidgets();
#else
	static const TArray<const SWidget*> EmptyArray;
	return EmptyArray;
#endif
}

void FSlateDebugging::ExportWidgetList(FStringView Filename)
{
#if UE_WITH_SLATE_DEBUG_WIDGETLIST
	UE::Slate::FWidgetList::ExportToCSV(*WriteToString<256>(Filename));
#endif
}

uint32 FSlateDebugging::NoInvalidationPanels = 0;
uint32 FSlateDebugging::NoDynamicInvalidationPanels = 0;
uint32 FSlateDebugging::NoDynamicInvalidationPanelsSupportingInvalidation = 0;

void FSlateDebugging::IncrementInvalidationPanelCount(bool bCanCache, bool bCanUseDynamic, bool bSupportsInvalidation)
{
	if (!bCanCache)
	{
		return;
	}

	++NoInvalidationPanels;
	if (bCanUseDynamic)
	{
		++NoDynamicInvalidationPanels;
		if (bSupportsInvalidation)
		{
			++NoDynamicInvalidationPanelsSupportingInvalidation;
		}
	}
}

void FSlateDebugging::ResetInvalidationPanelCount()
{
	NoInvalidationPanels = 0;
	NoDynamicInvalidationPanels = 0;
	NoDynamicInvalidationPanelsSupportingInvalidation = 0;
}

#undef LOCTEXT_NAMESPACE

#endif
