// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetList.h"

#if UE_WITH_SLATE_DEBUG_WIDGETLIST

#include "Application/SlateApplicationBase.h"
#include "CoreGlobals.h"
#include "FastUpdate/WidgetProxy.h"
#include "FastUpdate/SlateInvalidationRoot.h"
#include "GenericPlatform/ICursor.h"
#include "Layout/Children.h"
#include "Misc/OutputDeviceFile.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "Types/SlateAttributeMetaData.h"
#include "Types/SlateCursorMetaData.h"
#include "Types/SlateMouseEventsMetaData.h"
#include "Types/ReflectionMetadata.h"
#include "Widgets/IToolTip.h"
#include "Widgets/SWidget.h"

namespace UE
{
namespace Slate
{

// ------------------------------------------------

TArray<const SWidget*> FWidgetList::AllWidgets;

// ------------------------------------------------

struct FLogAllWidgetsDebugInfoFlags
{
	bool bDebug = false;
	bool bPaint = false;
	bool bProxy = false;
	bool bChildren = false;
	bool bParent = false;
	bool bToolTip = false;
	bool bCursor = false;
	bool bSlateAttribute = false;
	bool bMouseEventsHandler = false;
	bool bActiveTimers = false;
	bool bRenderTransform = false;

	void Parse(const FString& Arg)
	{
		if (FParse::Bool(*Arg, TEXT("Debug="), bDebug))
		{
			return;
		}

		if (FParse::Bool(*Arg, TEXT("Paint="), bPaint))
		{
			return;
		}

		if (FParse::Bool(*Arg, TEXT("Proxy="), bProxy))
		{
			return;
		}

		if (FParse::Bool(*Arg, TEXT("Children="), bChildren))
		{
			return;
		}

		if (FParse::Bool(*Arg, TEXT("Parent="), bParent))
		{
			return;
		}

		if (FParse::Bool(*Arg, TEXT("ToolTip="), bToolTip))
		{
			return;
		}

		if (FParse::Bool(*Arg, TEXT("Cursor="), bCursor))
		{
			return;
		}
		
		if (FParse::Bool(*Arg, TEXT("SlateAttribute="), bSlateAttribute))
		{
			return;
		}

		if (FParse::Bool(*Arg, TEXT("MouseEvents="), bMouseEventsHandler))
		{
			return;
		}
		
		if (FParse::Bool(*Arg, TEXT("ActiveTimers="), bActiveTimers))
		{
			return;
		}
		
		if (FParse::Bool(*Arg, TEXT("RenderTransform="), bRenderTransform))
		{
			return;
		}
	}
};

void LogAllWidgetsDebugInfoImpl(FOutputDevice& Ar, const FLogAllWidgetsDebugInfoFlags& DebugInfoFlags)
{
	TStringBuilder<1024> MessageBuilder;

	MessageBuilder << TEXT("Pointer;DebugInfo");
	if (DebugInfoFlags.bDebug)
	{
		MessageBuilder << TEXT(";Type;WidgetPath;ReadableLocation");
	}
	if (DebugInfoFlags.bPaint)
	{
		MessageBuilder << TEXT(";LastPaintFrame;LayerId;AllottedGeometryAbsoluteSizeX;AllottedGeometryAbsoluteSizeY");
	}
	if (DebugInfoFlags.bProxy)
	{
		MessageBuilder << TEXT(";InvalidationRootPointer;InvalidationRootDebugInfo;ProxyIndex");
	}
	if (DebugInfoFlags.bChildren)
	{
		MessageBuilder << TEXT(";NumAllChildren;NumChildren");
	}
	if (DebugInfoFlags.bParent)
	{
		MessageBuilder << TEXT(";ParentPointer;ParentDebugInfo");
	}
	if (DebugInfoFlags.bToolTip)
	{
		MessageBuilder << TEXT(";ToolTipIsSet;ToolTipIsEmpty");
	}
	if (DebugInfoFlags.bCursor)
	{
		MessageBuilder << TEXT(";CursorIsSet;CursorValue");
	}
	if (DebugInfoFlags.bSlateAttribute)
	{
		MessageBuilder << TEXT(";NumSlateAttribute;NumSlateCollaspedAttribute");
	}
	if (DebugInfoFlags.bMouseEventsHandler)
	{
		MessageBuilder << TEXT(";MouseButtonDown;MouseButtonUp;MouseMove;MouseDblClick;MouseEnter;MouseLeave");
	}
	if (DebugInfoFlags.bActiveTimers)
	{
		MessageBuilder << TEXT(";HasActiveTimers");
	}
	if (DebugInfoFlags.bRenderTransform)
	{
		MessageBuilder << TEXT(";RenderTransformSet;RenderTransformPivot");
	}

	Ar.Log(MessageBuilder.ToString());

	const TArray<const SWidget*>& WidgetList = FWidgetList::GetAllWidgets();
	for (const SWidget* Widget : WidgetList)
	{
		MessageBuilder.Reset();

		MessageBuilder.Appendf(TEXT("%p"), (void*)Widget);
		MessageBuilder << TEXT(";");
		MessageBuilder << FReflectionMetaData::GetWidgetDebugInfo(Widget);

		if (DebugInfoFlags.bDebug)
		{
			MessageBuilder << TEXT(";");
			MessageBuilder << Widget->GetTypeAsString();
			MessageBuilder << TEXT(";");
			MessageBuilder << FReflectionMetaData::GetWidgetPath(Widget);
			MessageBuilder << TEXT(";");
			MessageBuilder << Widget->GetReadableLocation();
		}

		if (DebugInfoFlags.bPaint)
		{
			FVector2f AbsoluteSize = Widget->GetPersistentState().AllottedGeometry.GetAbsoluteSize();
			MessageBuilder << TEXT(";");
			MessageBuilder << Widget->Debug_GetLastPaintFrame();
			MessageBuilder << TEXT(";");
			MessageBuilder << Widget->GetPersistentState().LayerId;
			MessageBuilder << TEXT(";");
			MessageBuilder.Appendf(TEXT("%f"), AbsoluteSize.X);
			MessageBuilder << TEXT(";");
			MessageBuilder.Appendf(TEXT("%f"), AbsoluteSize.Y);
		}

		if (DebugInfoFlags.bProxy)
		{
			const FWidgetProxyHandle& ProxyHandle = Widget->GetProxyHandle();
			if (ProxyHandle.IsValid(Widget))
			{
				MessageBuilder << TEXT(";");
				MessageBuilder.Appendf(TEXT("%p"), (void*)ProxyHandle.GetInvalidationRootHandle().Advanced_GetInvalidationRootNoCheck());
				MessageBuilder << TEXT(";");
				MessageBuilder << FReflectionMetaData::GetWidgetDebugInfo(ProxyHandle.GetInvalidationRootHandle().Advanced_GetInvalidationRootNoCheck()->GetInvalidationRootWidget());
				MessageBuilder << TEXT(";");
				MessageBuilder << ProxyHandle.GetWidgetIndex().ToString();
			}
			else
			{
				MessageBuilder << TEXT(";;;");
			}
		}

		if (DebugInfoFlags.bChildren)
		{
			MessageBuilder << TEXT(";");
			MessageBuilder << const_cast<SWidget*>(Widget)->GetAllChildren()->Num();
			MessageBuilder << TEXT(";");
			MessageBuilder << const_cast<SWidget*>(Widget)->GetChildren()->Num();
		}

		if (DebugInfoFlags.bParent)
		{
			const SWidget* ParentWidget = Widget->GetParentWidget().Get();
			MessageBuilder << TEXT(";");
			MessageBuilder.Appendf(TEXT("%p"), (void*)ParentWidget);
			MessageBuilder << TEXT(";");
			MessageBuilder << FReflectionMetaData::GetWidgetDebugInfo(ParentWidget);
		}

		if (DebugInfoFlags.bToolTip)
		{
			if (TSharedPtr<IToolTip> ToolTip = const_cast<SWidget*>(Widget)->GetToolTip())
			{
				MessageBuilder << TEXT(";true");
				if (ToolTip->IsEmpty())
				{
					MessageBuilder << TEXT(";true");
				}
				else
				{
					MessageBuilder << TEXT(";false");
				}
			}
			else
			{
				MessageBuilder << TEXT(";false;false");
			}
		}
		
		if (DebugInfoFlags.bCursor)
		{
			if (TSharedPtr<FSlateCursorMetaData> Data = Widget->GetMetaData<FSlateCursorMetaData>())
			{
				if (Data->Cursor.IsSet())
				{
					TOptional<EMouseCursor::Type> Cursor = Data->Cursor.Get();
					if (Cursor.IsSet())
					{
						MessageBuilder << TEXT(";Set;");
						MessageBuilder << static_cast<int32>(Data->Cursor.Get().GetValue());
					}
					else
					{
						MessageBuilder << TEXT(";Optional;0");
					}
				}
				else
				{
					MessageBuilder << TEXT(";MetaData;0");
				}
			}
			else
			{
				MessageBuilder << TEXT(";None;0");
			}
		}

		if (DebugInfoFlags.bSlateAttribute)
		{
			if (FSlateAttributeMetaData* MetaData = FSlateAttributeMetaData::FindMetaData(*Widget))
			{
				MessageBuilder << TEXT(";");
				MessageBuilder << MetaData->GetRegisteredAttributeCount();
				MessageBuilder << TEXT(";");
				MessageBuilder << MetaData->GetRegisteredAffectVisibilityAttributeCount();
			}
			else
			{
				MessageBuilder << TEXT(";0;0");
			}
		}

		if (DebugInfoFlags.bMouseEventsHandler)
		{
			if (TSharedPtr<FSlateMouseEventsMetaData> Data = Widget->GetMetaData<FSlateMouseEventsMetaData>())
			{
				MessageBuilder << (Data->MouseButtonDownHandle.IsBound() ? TEXT(";bound") : TEXT(";"));
				MessageBuilder << (Data->MouseButtonUpHandle.IsBound() ? TEXT(";bound") : TEXT(";"));
				MessageBuilder << (Data->MouseMoveHandle.IsBound() ? TEXT(";bound") : TEXT(";"));
				MessageBuilder << (Data->MouseDoubleClickHandle.IsBound() ? TEXT(";bound") : TEXT(";"));
				MessageBuilder << (Data->MouseEnterHandler.IsBound() ? TEXT(";bound") : TEXT(";"));
				MessageBuilder << (Data->MouseLeaveHandler.IsBound() ? TEXT(";bound") : TEXT(";"));
			}
			else
			{
				MessageBuilder << TEXT(";;;;;;");
			}
		}

		if (DebugInfoFlags.bActiveTimers)
		{
			MessageBuilder << (Widget->HasActiveTimers() ? TEXT(";true") : TEXT(";false"));
		}
		
		if (DebugInfoFlags.bRenderTransform)
		{
			MessageBuilder << (Widget->GetRenderTransform().IsSet() ? TEXT(";true") : TEXT(";false"));
			MessageBuilder.Appendf(TEXT("%f,%f;"), Widget->GetRenderTransformPivot().X, Widget->GetRenderTransformPivot().Y);
		}

		Ar.Log(MessageBuilder.ToString());
	}
}

void LogAllWidgetsDebugInfo(const TArray< FString >& Args, UWorld*, FOutputDevice& Ar)
{
	FLogAllWidgetsDebugInfoFlags DebugInfoFlags;
	FString OutputFilename;
	for (const FString& Arg : Args)
	{
		if (FParse::Value(*Arg, TEXT("File="), OutputFilename))
		{
			continue;
		}

		DebugInfoFlags.Parse(Arg);
	}

	if (OutputFilename.Len() > 0)
	{
		FOutputDeviceFile OutputDeviceFile {*FPaths::Combine(FPaths::ProjectSavedDir(), OutputFilename), true};
		OutputDeviceFile.SetSuppressEventTag(true);
		LogAllWidgetsDebugInfoImpl(OutputDeviceFile, DebugInfoFlags);
	}
	else
	{
		LogAllWidgetsDebugInfoImpl(Ar, DebugInfoFlags);
	}
}

FAutoConsoleCommandWithWorldArgsAndOutputDevice ConsoleCommandLogAllWidgets(
	TEXT("Slate.Debug.LogAllWidgets"),
	TEXT("Prints all the SWidgets type, debug info, path, painted, ...\n")
	TEXT("If a file name is not provided, it will output to the log console.\n")
	TEXT("Slate.Debug.LogAllWidgets [File=MyFile.csv] [Debug=true] [Paint=false] [Proxy=false] [Children=false] [Parent=false] [ToolTip=false] [Cursor=false] [MouseEvents=false]"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(&LogAllWidgetsDebugInfo)
);

void FWidgetList::ExportToCSV(FStringView OutputFilename)
{
	if (OutputFilename.Len() == 0)
	{
		return;
	}

	FLogAllWidgetsDebugInfoFlags Flags;
	Flags.bDebug = true;
	Flags.bPaint = true;
	Flags.bProxy = true;
	Flags.bChildren = true;
	Flags.bParent = true;
	Flags.bToolTip = true;
	Flags.bCursor = true;
	Flags.bSlateAttribute = true;
	Flags.bMouseEventsHandler = true;
	Flags.bActiveTimers = true;

	FOutputDeviceFile OutputDeviceFile(*WriteToString<256>(OutputFilename), true /*bDisableBackup*/);
	OutputDeviceFile.SetSuppressEventTag(true);

	LogAllWidgetsDebugInfoImpl(OutputDeviceFile, Flags);
}

} //Slate
} //UE

#endif //UE_WITH_SLATE_DEBUG_WIDGETLIST
