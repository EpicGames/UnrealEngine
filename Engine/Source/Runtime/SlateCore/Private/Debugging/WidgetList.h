// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/SlateAttribute.h"
#include "Debugging/SlateDebugging.h"

#if UE_WITH_SLATE_DEBUG_WIDGETLIST && (UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#error "UE_WITH_SLATE_DEBUG_WIDGETLIST is defined in a non-debug build"
#endif

#if UE_WITH_SLATE_DEBUG_WIDGETLIST

class SWidget;

namespace UE
{
namespace Slate
{

class FWidgetList 
{
public:
	static void AddWidget(const SWidget* Widget) { AllWidgets.Add(Widget); }
	static void RemoveWidget(const SWidget* Widget) { AllWidgets.RemoveSingleSwap(Widget); }

	static const TArray<const SWidget*>& GetAllWidgets() { return AllWidgets; }

	static void ExportToCSV(FStringView FileName);

private:
	FWidgetList() = delete;

	static TArray<const SWidget*> AllWidgets;
};

#define UE_SLATE_DEBUG_WIDGETLIST_ADD_WIDGET(Widget) { ::UE::Slate::FWidgetList::AddWidget(Widget); }
#define UE_SLATE_DEBUG_WIDGETLIST_REMOVE_WIDGET(Widget) { ::UE::Slate::FWidgetList::RemoveWidget(Widget); }

} //Slate
} //UE

#else
#define UE_SLATE_DEBUG_WIDGETLIST_ADD_WIDGET(Widget)
#define UE_SLATE_DEBUG_WIDGETLIST_REMOVE_WIDGET(Widget)
#endif //UE_WITH_SLATE_DEBUG_WIDGETLIST