// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "Containers/Array.h"
#include "Templates/UnrealTypeTraits.h"

/**
 * An Array of elements that are always sorted.
 */
template <typename InElementType, typename InSortPredicate = TLess<>>
class TSlateElementSortedArray
{
	using DataType = TArray<InElementType>;
	DataType Data;

public:
	using SizeType = typename DataType::SizeType;
	using ElementType = InElementType;
	using ElementParamType = typename TCallTraits<ElementType>::ParamType;
	using SortPredicate = TLess<>;

public:
	/** @returns Reference to indexed element. */
	FORCEINLINE ElementParamType operator[](SizeType Index) const
	{
		return Data.GetData()[Index];
	}

public:
	/** Removes all elements from the array, potentially leaving space allocated for an expected number of elements about to be added. */
	FORCEINLINE void Empty(SizeType ExpectedNumElements = 0)
	{
		Data.Empty(ExpectedNumElements);
	}

	/** @returns True if index is valid. False otherwise. */
	FORCEINLINE bool IsValidIndex(SizeType Index) const
	{
		return Data.IsValidIndex(Index);
	}

	/** @return The number of elements in the array. */
	FORCEINLINE SizeType Num() const
	{
		return Data.Num();
	}

public:
	/**
	 * Binary searches the element within the array.
	 *
	 * @param Item Item to look for.
	 * @returns Index of the found element. INDEX_NONE otherwise.
	 */
	SizeType Find(ElementParamType Item) const
	{
		return Algo::BinarySearch(Data, Item, SortPredicate());
	}

	/**
	 * Checks if this array contains the element.
	 *
	 * @param Item Item to look for.
	 * @returns	True if found. False otherwise.
	 */
	bool Contains(ElementParamType Item) const
	{
		return Find(Item) != INDEX_NONE;
	}

	/**
	 * @return the index of the first element >= value, or INDEX_NONE if no such element is found.
	 */
	SizeType FindLowerBound(ElementParamType Value) const
	{
		const SizeType FoundIndex = Algo::LowerBound(Data, Value, SortPredicate());
		return Data.IsValidIndex(FoundIndex) ? FoundIndex : INDEX_NONE;
	}

	/**
	 * @returns the index of the first element > value, or INDEX_NONE if no such element if found.
	 */
	SizeType FindUpperBound(ElementParamType Value) const
	{
		const SizeType FoundIndex = Algo::UpperBound(Data, Value, SortPredicate());
		return Data.IsValidIndex(FoundIndex) ? FoundIndex : INDEX_NONE;
	}

public:
	/**
	 * Adds a new item to the array at the sorted location if it doesn't exist, possibly reallocating the whole array to fit.
	 *
	 * @param Item The item to add
	 * @return Index to the new item
	 */
	SizeType InsertUnique(ElementParamType Item)
	{
		SizeType LowerBoundIndex = Algo::LowerBound(Data, Item, SortPredicate());
		if (LowerBoundIndex >= Num() || SortPredicate()(Item, Data.GetData()[LowerBoundIndex]))
		{
			return Data.Insert(Item, LowerBoundIndex);
		}
		return 0;
	}

	/**
	 * Adds a new item to the end of the array. The array needs to be sorted manually after.
	 * Use when multiple insertion is needed and it would be faster to sort the array once.
	 *
	 * @param Item The item to add
	 * @return Index to the new item
	*/
	FORCEINLINE SizeType AddUnsorted(ElementType Item)
	{
		return Data.Add(Item);
	}

	/**
	 * Removes the first occurrence of the specified item in the array, maintaining order but not indices.
	 *
	 * @param Item The item to remove.
	 * @returns The number of items removed. For RemoveSingle, this is always either 0 or 1.
	 */
	SizeType RemoveSingle(ElementParamType Item)
	{
		SizeType Index = Find(Item);
		if (Index != INDEX_NONE)
		{
			Data.RemoveAt(Index, EAllowShrinking::No);
			return 1;
		}
		return 0;
	}

	/**
	 * Removes an element (or elements) at given location optionally shrinking
	 * the array.
	 *
	 * @param Index Location in array of the element to remove.
	 * @param Count (Optional) Number of elements to remove. Default is 1.
	 * @param bAllowShrinking (Optional) Tells if this call can shrink array if suitable after remove. Default is true.
	 */
	template <typename CountType>
	FORCEINLINE void RemoveAt(SizeType Index, CountType Count)
	{
		Data.RemoveAt(Index, Count, EAllowShrinking::No);
	}

	/** Sorts the array using the SortPredicate. */
	void Sort()
	{
		Algo::Sort(Data, SortPredicate());
	}

public:
	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	using RangedForConstIteratorType = typename DataType::RangedForConstIteratorType;
	FORCEINLINE RangedForConstIteratorType	begin() const { return Data.begin(); }
	FORCEINLINE RangedForConstIteratorType	end() const { return Data.end(); }
};
