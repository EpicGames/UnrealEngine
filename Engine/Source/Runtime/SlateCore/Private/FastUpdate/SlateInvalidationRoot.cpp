// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastUpdate/SlateInvalidationRoot.h"
#include "FastUpdate/SlateInvalidationRootHandle.h"
#include "FastUpdate/SlateInvalidationRootList.h"
#include "FastUpdate/SlateInvalidationWidgetHeap.h"
#include "FastUpdate/SlateInvalidationWidgetList.h"
#include "Async/TaskGraphInterfaces.h"
#include "Application/SlateApplicationBase.h"
#include "Widgets/SWidget.h"
#include "Input/HittestGrid.h"
#include "Layout/Children.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Trace/SlateTrace.h"
#include "Types/ReflectionMetadata.h"
#include "Types/SlateAttributeMetaData.h"
#include "Rendering/DrawElementPayloads.h"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(SLATECORE_API, Slate);


#if WITH_SLATE_DEBUGGING
bool GSlateInvalidationRootDumpUpdateList = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootDumpUpdateList(
	TEXT("Slate.InvalidationRoot.DumpUpdateList"),
	GSlateInvalidationRootDumpUpdateList,
	TEXT("Each frame, log the widgets that will be updated.")
);
void DumpUpdateList(const FSlateInvalidationWidgetList& FastWidgetPathList, const TArray<FSlateInvalidationWidgetHeapElement>&);

bool GSlateInvalidationRootDumpUpdateListOnce = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootDumpUpdateListOnce(
	TEXT("Slate.InvalidationRoot.DumpUpdateListOnce"),
	GSlateInvalidationRootDumpUpdateListOnce,
	TEXT("Log the widgets that will be updated this frame.")
);

static FAutoConsoleCommand CVarHandleDumpUpdateListCommand_Deprecated(
	TEXT("Slate.DumpUpdateList"),
	TEXT("(Deprecated) use Slate.InvalidationRoot.DumpUpdateListOnce"),
	FConsoleCommandDelegate::CreateStatic([](){ GSlateInvalidationRootDumpUpdateListOnce = true; })
);

bool GSlateInvalidationRootDumpPreInvalidationList = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootDumpPreInvalidationList(
	TEXT("Slate.InvalidationRoot.DumpPreInvalidationList"),
	GSlateInvalidationRootDumpPreInvalidationList,
	TEXT("Each frame, log the widgets that are processed in the pre update phase.")
);
void LogPreInvalidationItem(const FSlateInvalidationWidgetList& FastWidgetPathList, FSlateInvalidationWidgetIndex WidgetIndex);

bool GSlateInvalidationRootDumpPrepassInvalidationList = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootDumpPrepassInvalidationList(
	TEXT("Slate.InvalidationRoot.DumpPrepassInvalidationList"),
	GSlateInvalidationRootDumpPrepassInvalidationList,
	TEXT("Each frame, log the widgets that are processed in the prepass update phase.")
);
void LogPrepassInvalidationItem(const FSlateInvalidationWidgetList& FastWidgetPathList, FSlateInvalidationWidgetIndex WidgetIndex);

bool GSlateInvalidationRootDumpPostInvalidationList = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootDumpPostInvalidationList(
	TEXT("Slate.InvalidationRoot.DumpPostInvalidationList"),
	GSlateInvalidationRootDumpPostInvalidationList,
	TEXT("Each frame, log the widgets that are processed in the post update phase.")
);
void LogPostInvalidationItem(const FSlateInvalidationWidgetList& FastWidgetPathList, FSlateInvalidationWidgetIndex WidgetIndex);
#endif

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
bool GSlateInvalidationRootVerifyWidgetList = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootVerifyWidgetList(
	TEXT("Slate.InvalidationRoot.VerifyWidgetList"),
	GSlateInvalidationRootVerifyWidgetList,
	TEXT("Every tick, verify that the updated list does match a newly created list.")
);
void VerifyWidgetList(TSharedRef<SWidget> RootWidget, FSlateInvalidationRootHandle InvalidationRootHandle, FSlateInvalidationWidgetList& WidgetList);

bool GSlateInvalidationRootVerifyWidgetsIndex = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootVerifyWidgetsIndex(
	TEXT("Slate.InvalidationRoot.VerifyWidgetsIndex"),
	GSlateInvalidationRootVerifyWidgetsIndex,
	TEXT("Every tick, verify that every widgets has the correct corresponding index.")
);

bool GSlateInvalidationRootVerifyValidWidgets = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootVerifyValidWidgets(
	TEXT("Slate.InvalidationRoot.VerifyValidWidgets"),
	GSlateInvalidationRootVerifyValidWidgets,
	TEXT("Every tick, verify that every WidgetProxy has a valid SWidget.")
);

bool GSlateInvalidationRootVerifyHittestGrid = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootVerifyHittestGrid(
	TEXT("Slate.InvalidationRoot.VerifyHittestGrid"),
	GSlateInvalidationRootVerifyHittestGrid,
	TEXT("Every tick, verify the hittest grid.")
);
void VerifyHittest(SWidget* InvalidationRootWidget, FSlateInvalidationWidgetList& WidgetList, FHittestGrid* HittestGrid);

bool GSlateInvalidationRootVerifyWidgetVisibility = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootVerifyWidgetVisibility(
	TEXT("Slate.InvalidationRoot.VerifyWidgetVisibility"),
	GSlateInvalidationRootVerifyWidgetVisibility,
	TEXT("Every tick, verify that the cached visibility of the widgets is properly set.")
);
void VerifyWidgetVisibility(FSlateInvalidationWidgetList& WidgetList);

bool GSlateInvalidationRootVerifyWidgetVolatile = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootVerifyWidgetVolatile(
	TEXT("Slate.InvalidationRoot.VerifyWidgetVolatile"),
	GSlateInvalidationRootVerifyWidgetVolatile,
	TEXT("Every tick, verify that volatile widgets are mark properly and are in the correct list.")
);
void VerifyWidgetVolatile(FSlateInvalidationWidgetList& WidgetList, TArray<FSlateInvalidationWidgetHeapElement>& FinalUpdateList);

bool GSlateInvalidationRootVerifyWidgetsUpdateList = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootVerifyWidgetsUpdateList(
	TEXT("Slate.InvalidationRoot.VerifyWidgetUpdateList"),
	GSlateInvalidationRootVerifyWidgetsUpdateList,
	TEXT("Every tick, verify that pre and post update list contains the correct information and they are sorted.")
);
void VerifyWidgetsUpdateList_BeforeProcessPreUpdate(const TSharedRef<SWidget>&, FSlateInvalidationWidgetList*, FSlateInvalidationWidgetPreHeap*, FSlateInvalidationWidgetPostHeap*, TArray<FSlateInvalidationWidgetHeapElement>&);
void VerifyWidgetsUpdateList_ProcessPrepassUpdate(FSlateInvalidationWidgetList*, FSlateInvalidationWidgetPrepassHeap*, FSlateInvalidationWidgetPostHeap*);
void VerifyWidgetsUpdateList_BeforeProcessPostUpdate(const TSharedRef<SWidget>&, FSlateInvalidationWidgetList*, FSlateInvalidationWidgetPreHeap*, FSlateInvalidationWidgetPostHeap*, TArray<FSlateInvalidationWidgetHeapElement>&);
void VerifyWidgetsUpdateList_AfterProcessPostUpdate(const TSharedRef<SWidget>&, FSlateInvalidationWidgetList*, FSlateInvalidationWidgetPreHeap*, FSlateInvalidationWidgetPostHeap*, TArray<FSlateInvalidationWidgetHeapElement>&);

bool GSlateInvalidationRootVerifySlateAttribute = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootVerifySlateAttribute(
	TEXT("Slate.InvalidationRoot.VerifySlateAttribute"),
	GSlateInvalidationRootVerifySlateAttribute,
	TEXT("Every tick, verify that the widgets that have registered attribute are correctly updated once and the list contains all the widgets.")
);
void VerifySlateAttribute_BeforeUpdate(FSlateInvalidationWidgetList& FastWidgetPathList);
void VerifySlateAttribute_AfterUpdate(const FSlateInvalidationWidgetList& FastWidgetPathList);

bool GSlateInvalidationRootVerifyWidgetsAreUpdatedOnce = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootVerifyWidgetsAreUpdatedOnce(
	TEXT("Slate.InvalidationRoot.VerifyWidgetsAreUpdatedOnce"),
	GSlateInvalidationRootVerifyWidgetsAreUpdatedOnce,
	TEXT("Verify that the widgets are painted only once per tick.")
);

bool GSlateInvalidationRootVerifyCachedElementDataList = false;
static FAutoConsoleVariableRef CVarSlateInvalidationRootVerifyCachedElementDataList(
	TEXT("Slate.InvalidationRoot.VerifyCachedElementDataList"),
	GSlateInvalidationRootVerifyCachedElementDataList,
	TEXT("Verify that the widgets cached are valid.")
);
void VerifyCachedElementDataList(const FSlateInvalidationRoot* InvalidationRoot);

#endif //UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING



int32 GSlateInvalidationWidgetListMaxArrayElements = 64;
FAutoConsoleVariableRef CVarSlateInvalidationWidgetListMaxArrayElements(
	TEXT("Slate.InvalidationList.MaxArrayElements"),
	GSlateInvalidationWidgetListMaxArrayElements,
	TEXT("With the invalidation system, the preferred size of the elements array."));

int32 GSlateInvalidationWidgetListNumberElementLeftBeforeSplitting = 40;
FAutoConsoleVariableRef CVarSlateInvalidationWidgetListNumElementLeftBeforeSplitting(
	TEXT("Slate.InvalidationList.NumberElementLeftBeforeSplitting"),
	GSlateInvalidationWidgetListNumberElementLeftBeforeSplitting,
	TEXT("With the invalidation system, when splitting, only split the array when the number of element left is under X."));

bool GSlateInvalidationEnableReindexLayerId = true;
FAutoConsoleVariableRef CVarSlateInvalidationEnableReindexLayerId(
	TEXT("Slate.InvalidationList.EnableReindexLayerId"),
	GSlateInvalidationEnableReindexLayerId,
	TEXT("With invalidation system, when a painted widget returns a bigger LayerId that it used to, re-index the other widgets."));

/**
 *
 */
 namespace UE::Slate::Private
 {
	 constexpr EInvalidateWidgetReason PreInvalidationReason = EInvalidateWidgetReason::Layout
		 | EInvalidateWidgetReason::AttributeRegistration
		 | EInvalidateWidgetReason::Visibility
		 | EInvalidateWidgetReason::ChildOrder;
	 constexpr EInvalidateWidgetReason PostInvalidationReason = EInvalidateWidgetReason::Layout
		 | EInvalidateWidgetReason::Paint
		 | EInvalidateWidgetReason::Volatility
		 | EInvalidateWidgetReason::RenderTransform
		 | EInvalidateWidgetReason::Prepass;

	bool EInvalidateWidgetReason_HasPreUpdateFlag(EInvalidateWidgetReason InvalidateReason)
	{
		return EnumHasAnyFlags(InvalidateReason, PreInvalidationReason);
	}

	bool EInvalidateWidgetReason_HasPostUpdateFlag(EInvalidateWidgetReason InvalidateReason)
	{
		static_assert(std::is_same<std::underlying_type_t<EInvalidateWidgetReason>, uint8>::value, "EInvalidateWidgetReason is not a uint8");
		const uint8 AnyPostUpdate = (0xFF & ~(uint8)EInvalidateWidgetReason::AttributeRegistration);
		return (((uint8)InvalidateReason & AnyPostUpdate) != 0);
	}
 }


/**
 *
 */
FSlateInvalidationRootList GSlateInvalidationRootListInstance;

FSlateInvalidationRoot::FSlateInvalidationRoot()
	: CachedViewOffset(0.0f, 0.0f)
	, CachedElementData(new FSlateCachedElementData)
	, InvalidationRootWidget(nullptr)
	, RootHittestGrid(nullptr)
	, CachedMaxLayerId(0)
	, bNeedsSlowPath(true)
	, bNeedScreenPositionShift(false)
	, bProcessingPreUpdate(false)
	, bProcessingAttributeUpdate(false)
	, bProcessingPrepassUpdate(false)
	, bProcessingPostUpdate(false)
	, bBuildingWidgetList(false)
	, bProcessingChildOrderInvalidation(false)
	, bDeferInvalidateAllWidgetsAndClearResources(TOptional<bool>{})
#if WITH_SLATE_DEBUGGING
	, LastPaintType(ESlateInvalidationPaintType::None)
	, PerformanceStat()
#endif
{
	InvalidationRootHandle = FSlateInvalidationRootHandle(GSlateInvalidationRootListInstance.AddInvalidationRoot(this));
	FSlateApplicationBase::Get().OnInvalidateAllWidgets().AddRaw(this, &FSlateInvalidationRoot::HandleInvalidateAllWidgets);

	const FSlateInvalidationWidgetList::FArguments Arg = { GSlateInvalidationWidgetListMaxArrayElements, GSlateInvalidationWidgetListNumberElementLeftBeforeSplitting };
	FastWidgetPathList = MakeUnique<FSlateInvalidationWidgetList>(InvalidationRootHandle, Arg);
	WidgetsNeedingPreUpdate = MakeUnique<FSlateInvalidationWidgetPreHeap>(*FastWidgetPathList);
	WidgetsNeedingPrepassUpdate = MakeUnique<FSlateInvalidationWidgetPrepassHeap>(*FastWidgetPathList);
	WidgetsNeedingPostUpdate = MakeUnique<FSlateInvalidationWidgetPostHeap>(*FastWidgetPathList);

#if WITH_SLATE_DEBUGGING
	SetLastPaintType(ESlateInvalidationPaintType::None);
#endif
}

FSlateInvalidationRoot::~FSlateInvalidationRoot()
{
	ClearAllFastPathData(true);

#if UE_SLATE_DEBUGGING_CLEAR_ALL_FAST_PATH_DATA
	ensure(FastWidgetPathToClearedBecauseOfDelay.Num() == 0);
#endif

	if (FSlateApplicationBase::IsInitialized())
	{
		FSlateApplicationBase::Get().OnInvalidateAllWidgets().RemoveAll(this);

		FSlateApplicationBase::Get().GetRenderer()->DestroyCachedFastPathElementData(CachedElementData);
	}
	else
	{
		delete CachedElementData;
	}

	GSlateInvalidationRootListInstance.RemoveInvalidationRoot(InvalidationRootHandle.GetUniqueId());
}

void FSlateInvalidationRoot::AddReferencedObjects(FReferenceCollector& Collector)
{
	CachedElementData->AddReferencedObjects(Collector);
}

FString FSlateInvalidationRoot::GetReferencerName() const
{
	return TEXT("FSlateInvalidationRoot");
}

void FSlateInvalidationRoot::InvalidateRootChildOrder(const SWidget* Investigator)
{
	// Invalidate all proxy handles
	FastWidgetPathList->Reset();
	WidgetsNeedingPreUpdate->Reset(false);
	WidgetsNeedingPrepassUpdate->Reset(false);
	WidgetsNeedingPostUpdate->Reset(false);
	InvalidationRootWidget->Invalidate(EInvalidateWidgetReason::Prepass);
	bNeedsSlowPath = true;

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::BroadcastInvalidationRootInvalidate(InvalidationRootWidget, Investigator, ESlateDebuggingInvalidateRootReason::ChildOrder);
#endif
	UE_TRACE_SLATE_ROOT_CHILDORDER_INVALIDATED(InvalidationRootWidget, Investigator);
}

void FSlateInvalidationRoot::InvalidateRootLayout(const SWidget* Investigator)
{
	InvalidationRootWidget->Invalidate(EInvalidateWidgetReason::Prepass);
	bNeedsSlowPath = true; // with the loop before it should only do one slateprepass

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::BroadcastInvalidationRootInvalidate(InvalidationRootWidget, Investigator, ESlateDebuggingInvalidateRootReason::Root);
#endif
	UE_TRACE_SLATE_ROOT_INVALIDATED(InvalidationRootWidget, Investigator);
}

void FSlateInvalidationRoot::InvalidateWidget(FWidgetProxy& Proxy, EInvalidateWidgetReason InvalidateReason)
{
	ensureMsgf(bProcessingChildOrderInvalidation == false, TEXT("A widget got invalidated while building the childorder."));

	if (bProcessingAttributeUpdate)
	{
		if (!FSlateAttributeBase::IsInvalidateWidgetReasonSupported(InvalidateReason))
		{
			ensureMsgf(false, TEXT("An invalid invalidation occurred while processing the widget attributes. That may result in an infinite loop."));
			return;
		}
	}

	if (!bNeedsSlowPath)
	{
		Proxy.CurrentInvalidateReason |= InvalidateReason;
		if (UE::Slate::Private::EInvalidateWidgetReason_HasPreUpdateFlag(InvalidateReason))
		{
			WidgetsNeedingPreUpdate->HeapPushUnique(Proxy);
		}

		if (!bProcessingPrepassUpdate && !bProcessingPostUpdate && EnumHasAnyFlags(InvalidateReason, EInvalidateWidgetReason::Prepass))
		{
			ensureMsgf(Proxy.GetWidget() && Proxy.GetWidget()->NeedsPrepass(), TEXT("A Prepass invalidation occurs and the widget doesn't have the NeedsPrepass flag."));
			ensureMsgf(EnumHasAnyFlags(InvalidateReason, EInvalidateWidgetReason::Layout), TEXT("The Prepass invalidation should include a Layout invalidation."));
			WidgetsNeedingPrepassUpdate->PushBackUnique(Proxy);
		}

		if (UE::Slate::Private::EInvalidateWidgetReason_HasPostUpdateFlag(InvalidateReason))
		{
			WidgetsNeedingPostUpdate->PushBackOrHeapUnique(Proxy);
		}

		{
			SWidget* WidgetPtr = Proxy.GetWidget();
#if WITH_SLATE_DEBUGGING
			FSlateDebugging::BroadcastWidgetInvalidate(WidgetPtr, nullptr, InvalidateReason);
#endif
			UE_TRACE_SLATE_WIDGET_INVALIDATED(WidgetPtr, nullptr, InvalidateReason);
		}
	}
}

void FSlateInvalidationRoot::InvalidateScreenPosition(const SWidget* Investigator)
{
	bNeedScreenPositionShift = true;

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::BroadcastInvalidationRootInvalidate(InvalidationRootWidget, Investigator, ESlateDebuggingInvalidateRootReason::ScreenPosition);
#endif
}

const TSharedPtr<SWidget> FSlateInvalidationRoot::GetFastPathWidgetListRoot() const
{
	return GetFastPathWidgetList().GetRoot().Pin();
}

FSlateInvalidationResult FSlateInvalidationRoot::PaintInvalidationRoot(const FSlateInvalidationContext& Context)
{
	const int32 LayerId = 0;

	check(InvalidationRootWidget);
	check(RootHittestGrid);

#if WITH_SLATE_DEBUGGING
	SetLastPaintType(ESlateInvalidationPaintType::None);
#endif

	FSlateInvalidationResult Result;

	if (Context.bAllowFastPathUpdate)
	{
		Context.WindowElementList->PushCachedElementData(*CachedElementData);
	}

	TSharedRef<SWidget> RootWidget = GetRootWidget();

	if (bNeedScreenPositionShift)
	{
		SCOPED_NAMED_EVENT(Slate_InvalidateScreenPosition, FColor::Red);
		AdjustWidgetsDesktopGeometry(Context.PaintArgs->GetWindowToDesktopTransform());
		bNeedScreenPositionShift = false;
	}

	EFlowDirection NewFlowDirection = GSlateFlowDirection;
	if (RootWidget->GetFlowDirectionPreference() == EFlowDirectionPreference::Inherit)
	{
		NewFlowDirection = GSlateFlowDirectionShouldFollowCultureByDefault ? FLayoutLocalization::GetLocalizedLayoutDirection() : RootWidget->ComputeFlowDirection();
	}
	TGuardValue<EFlowDirection> FlowGuard(GSlateFlowDirection, NewFlowDirection);
	if (!Context.bAllowFastPathUpdate || bNeedsSlowPath || GSlateIsInInvalidationSlowPath)
	{
		SCOPED_NAMED_EVENT(Slate_PaintSlowPath, FColor::Red);
		
		// Clears existing cached element lists
		ClearAllFastPathData(!Context.bAllowFastPathUpdate);

		GSlateIsOnFastUpdatePath = false;
		bNeedsSlowPath = false;

		CachedViewOffset = Context.ViewOffset;

		{
			if (Context.bAllowFastPathUpdate)
			{
				TGuardValue<bool> InSlowPathGuard(GSlateIsInInvalidationSlowPath, true);

				BuildFastPathWidgetList(RootWidget);
			}

			// Repopulates cached element lists
			CachedMaxLayerId = PaintSlowPath(Context);
#if WITH_SLATE_DEBUGGING
			SetLastPaintType(ESlateInvalidationPaintType::Slow);
#endif
		}

		Result.bRepaintedWidgets = true;
	}
	else if (!FastWidgetPathList->IsEmpty())
	{
		// We should not have been supplied a different root than the one we generated a path to
		check(RootWidget == FastWidgetPathList->GetRoot().Pin());

		Result.bRepaintedWidgets = PaintFastPath(Context);
	}

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	if (!bNeedsSlowPath)
	{
		// In case widgets were added during Slow Path or PrePass, we need to recheck if we can call ProcessPreUpdate
		bNeedsSlowPath = !CanCacheThisFrame();
	}
#endif

	// Once everything is painted and ticked, new widget (SListView, SRichBox, SScrollBox) might have created new widget in both SlowPath and FastPath
	// It might create a 1 frame hitch because the widget is in the cached and should not.
	if (!bNeedsSlowPath && WidgetsNeedingPreUpdate->Num() > 0)
	{
		SCOPED_NAMED_EVENT(Slate_InvalidationProcessing_PreUpdate, FColor::Blue);
		ProcessPreUpdate();
	}

	if (Context.bAllowFastPathUpdate)
	{
		Context.WindowElementList->PopCachedElementData();
	}

	FinalUpdateList.Reset();

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (GSlateInvalidationRootVerifyCachedElementDataList && !bNeedsSlowPath)
	{
		VerifyCachedElementDataList(this);
	}
	if (GSlateInvalidationRootVerifyHittestGrid && Context.bAllowFastPathUpdate)
	{
		VerifyHittest(InvalidationRootWidget, GetFastPathWidgetList(), GetHittestGrid());
	}
#endif

	Result.ViewOffset = CachedViewOffset;
	Result.MaxLayerIdPainted = CachedMaxLayerId;
	return Result;
}

void FSlateInvalidationRoot::OnWidgetDestroyed(const SWidget* Widget)
{
	// We need the index even if we've invalidated this root.  We need to clear out its proxy regardless
	const FSlateInvalidationWidgetIndex ProxyIndex = Widget->GetProxyHandle().GetWidgetIndex();
	if (FastWidgetPathList->IsValidIndex(ProxyIndex))
	{
		FSlateInvalidationWidgetList::InvalidationWidgetType& Proxy = (*FastWidgetPathList)[ProxyIndex];
		if (Proxy.IsSameWidget(Widget))
		{
			Proxy.ResetWidget();
		}
	}
}

void FSlateInvalidationRoot::PaintFastPath_AddUniqueSortedToFinalUpdateList(const FSlateInvalidationWidgetIndex InvalidationWidgetIndex)
{
	const FSlateInvalidationWidgetSortOrder SortIndex{ *FastWidgetPathList, InvalidationWidgetIndex };
	int32 BackwardIndex = FinalUpdateList.Num() - 1;
	for (; BackwardIndex >= 0; --BackwardIndex)
	{
		const FSlateInvalidationWidgetHeapElement NextWidgetElement = FinalUpdateList[BackwardIndex];
		if (NextWidgetElement.GetWidgetIndex() == InvalidationWidgetIndex)
		{
			// No need to insert
			return;
		}

		if (NextWidgetElement.GetWidgetSortOrder() > SortIndex)
		{
			break;
		}
	}

	FinalUpdateList.EmplaceAt(BackwardIndex + 1, InvalidationWidgetIndex, SortIndex);
}

namespace UE::Slate::Private
{
	struct FSlateInvalidationReindexHeap
	{
		FSlateInvalidationReindexHeap() = default;

		struct FWidgetOrderGreater
		{
			FORCEINLINE bool operator()(const FSlateInvalidationWidgetHeapElement& A, const FSlateInvalidationWidgetHeapElement& B) const
			{
				return A.GetWidgetSortOrder() < B.GetWidgetSortOrder();
			}
		};

		void HeapPush(FSlateInvalidationWidgetHeapElement Element)
		{
			Heap.HeapPush(Element, FWidgetOrderGreater());
		}

		[[nodiscard]] FSlateInvalidationWidgetHeapElement HeapPeek()
		{
			return Heap.HeapTop();
		}

		void HeapPopDiscard()
		{
			Heap.HeapPopDiscard(FWidgetOrderGreater(), EAllowShrinking::No);
		}

		[[nodiscard]] bool IsEmpty() const
		{
			return Heap.Num() == 0;
		}

	private:
		TArray<FSlateInvalidationWidgetHeapElement, TInlineAllocator<16>> Heap;
	};

	struct FSlateInvalidationPaintFastPathContext
	{
		FSlateInvalidationReindexHeap ReindexUpdateList;
	};
}

void FSlateInvalidationRoot::PaintFastPath_FixupParentLayerId(UE::Slate::Private::FSlateInvalidationPaintFastPathContext& FastPathContext, const FWidgetProxy& InvalidationWidget, const int32 NewOutgoingLayerId)
{
	// Update the parent outgoinglayerid and rerun the algo for it... we don't seems to need it maybe we can remove it for good.
	const FSlateInvalidationWidgetIndex ParentInvalidationWidgetIndex = InvalidationWidget.ParentIndex;
	if (ParentInvalidationWidgetIndex != FSlateInvalidationWidgetIndex::Invalid)
	{
		const FSlateInvalidationWidgetList::InvalidationWidgetType& ParentInvalidationWidget = (*FastWidgetPathList)[ParentInvalidationWidgetIndex];
		SWidget* ParentWidgetPtr = ParentInvalidationWidget.GetWidget();

		check(ParentWidgetPtr); // if the child is valid, then the parent must be valid
		if (ParentWidgetPtr->GetPersistentState().OutgoingLayerId < NewOutgoingLayerId)
		{
			const_cast<FSlateWidgetPersistentState&>(ParentWidgetPtr->GetPersistentState()).OutgoingLayerId = NewOutgoingLayerId;
			FastPathContext.ReindexUpdateList.HeapPush(FSlateInvalidationWidgetHeapElement{ ParentInvalidationWidgetIndex, FSlateInvalidationWidgetSortOrder{*FastWidgetPathList, ParentInvalidationWidgetIndex} });
		}
	}
}

void FSlateInvalidationRoot::PaintFastPath_FixupLayerId(UE::Slate::Private::FSlateInvalidationPaintFastPathContext& FastPaintContext, const FWidgetProxy& InvalidationWidget, const int32 NewOutgoingLayerId)
{
	// We ignore all deferred painting for now.
	if (InvalidationWidget.GetWidget()->GetPersistentState().bDeferredPainting)
	{
		return;
	}

	// For all sibling update the LayerId.
	//Option 1: The next sibling already needs a paint update. If the next sibling incremented the LayerId (most Widget), then just make sure the next LayerId is big enough. It will update the parent on its turn.
	//Option 2: The next sibling already needs a paint update. If the next sibling has the same LayerId (SVerticalBox/SConstraintCanvas), then we need to update the parent outgoing.
	//Option 3: The next sibling does not need a paint update. If the next sibling incremented the LayerId (most Widget), then re index the sibling (repaint it). It will update the parent on its turn.
	//Option 4: The next sibling does not need a paint update. If the next sibling has the same LayerId (SVerticalBox/SConstraintCanvas), then we need to update the parent outgoing and continue to the next sibling until one needs a paint update.

	bool bFoundAtLeastOneSibling = false;
	bool bFixupParentLayerId = false;
	const int32 LayerId = InvalidationWidget.GetWidget()->GetPersistentState().LayerId;

	FSlateInvalidationWidgetList::FIndexRange ParentWidgetRange;
	{
		if (InvalidationWidget.ParentIndex != FSlateInvalidationWidgetIndex::Invalid)
		{
			const FSlateInvalidationWidgetList::InvalidationWidgetType& ParentInvalidationWidget = (*FastWidgetPathList)[InvalidationWidget.ParentIndex];
			ParentWidgetRange = { *FastWidgetPathList, ParentInvalidationWidget.Index, ParentInvalidationWidget.LeafMostChildIndex };
		}
	}

	FSlateInvalidationWidgetIndex NextSiblingIndex = FastWidgetPathList->FindNextSibling(InvalidationWidget.Index);
	while (NextSiblingIndex != FSlateInvalidationWidgetIndex::Invalid)
	{
		const FSlateInvalidationWidgetList::InvalidationWidgetType& NextSiblingInvalidationWidget = (*FastWidgetPathList)[NextSiblingIndex];
		SWidget* NextSiblingWidgetPtr = NextSiblingInvalidationWidget.GetWidget();
		if (NextSiblingInvalidationWidget.Visibility.IsVisible() && NextSiblingWidgetPtr)
		{
			// We ignore all deferred painting for now.
			if (!NextSiblingWidgetPtr->GetPersistentState().bDeferredPainting)
			{
				bFoundAtLeastOneSibling = true;

				// Is this widget going to be painted anyway (already in the list). If so, just update it's internal LayerId.
				//Because of the Visible and GetWidget are test, we cannot only test the last item from the FinaUpdateList.
				bool bNextSiblingWillBePainted = false;
				for (int32 BackwardIndex = FinalUpdateList.Num() - 1; BackwardIndex >= 0; --BackwardIndex)
				{
					const FSlateInvalidationWidgetHeapElement WidgetElement = FinalUpdateList[BackwardIndex];
					if (WidgetElement.GetWidgetIndex() == NextSiblingIndex)
					{
						bNextSiblingWillBePainted = NextSiblingWidgetPtr->HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsRepaint | EWidgetUpdateFlags::NeedsVolatilePaint);
						break;
					}
					// We can stop if they are not in the same hierarchy.
					if (ParentWidgetRange.IsValid() && !ParentWidgetRange.Include(WidgetElement.GetWidgetSortOrder()))
					{
						break;
					}
				}

				if (bNextSiblingWillBePainted)
				{
					// Need to update the parent OuterLayerId in case we are using SVerticalBox/SConstraintCanvas (that reuse the same LayerId to improve rendering performance).
					if (NextSiblingWidgetPtr->GetPersistentState().LayerId == LayerId)
					{
						// Push the parent update, the new LayerId. The algo will paint the next sibling. Then confirm (on the next loop that the parent is ok).
					}
					else if (NextSiblingWidgetPtr->GetPersistentState().LayerId < NewOutgoingLayerId + 1)
					{
						// The next sibling should also update the parent but in case the parent do a FMath::Max() logic, we do not take any chances.
						// Make sure the LayerId is correct for the next sibling.
						const_cast<FSlateWidgetPersistentState&>(NextSiblingWidgetPtr->GetPersistentState()).LayerId = NewOutgoingLayerId + 1;
					}
					break;
				}
				else if (NextSiblingWidgetPtr->GetPersistentState().LayerId == LayerId)
				{
					// We cannot assume that every other siblings will also have the same LayerId. We need to loop and update for every next sibling.
					// Keep looping for the next sibling.
				}
				else if (NextSiblingWidgetPtr->GetPersistentState().LayerId < NewOutgoingLayerId + 1)
				{
					// We are re-indexing this sibling. It should also update the parent but in case the parent do a FMath::Max() logic, we do not take any chances.
					// Re-indexing the this sibling (for now we are just painting it).
					//NB, if we do a property re-indexing algo, we need to update this sibling LayerId and all the children (recursively) of this sibling.
					const_cast<FSlateWidgetPersistentState&>(NextSiblingWidgetPtr->GetPersistentState()).LayerId = NewOutgoingLayerId + 1;
					NextSiblingWidgetPtr->UpdateFlags |= EWidgetUpdateFlags::NeedsRepaint;

					PaintFastPath_AddUniqueSortedToFinalUpdateList(NextSiblingIndex);
					break;
				}
				else
				{
					// The Parent put gabs in the LayerId (like SOverlay) we can stop looking for issues.
					break;
				}
			}
		}
		NextSiblingIndex = FastWidgetPathList->FindNextSibling(NextSiblingIndex);
	}

	// Update or no update. Check if the parent needs an update.
	PaintFastPath_FixupParentLayerId(FastPaintContext, InvalidationWidget, NewOutgoingLayerId);
}

bool FSlateInvalidationRoot::PaintFastPath_UpdateNextWidget(const FSlateInvalidationContext& Context, UE::Slate::Private::FSlateInvalidationPaintFastPathContext& FastPaintContext)
{
	bool bNeedsPaint = false;
	const FSlateInvalidationWidgetIndex MyIndex = FinalUpdateList.Pop(EAllowShrinking::No).GetWidgetIndex();

	FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget = (*FastWidgetPathList)[MyIndex];

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (GSlateInvalidationRootVerifyWidgetsAreUpdatedOnce)
	{
		ensureAlwaysMsgf(!InvalidationWidget.bDebug_Updated, TEXT("VerifyWidgetsAreUpdatedOnce failed. Widget '%s' is going to be updated more than once"), *FReflectionMetaData::GetWidgetDebugInfo(InvalidationWidget.GetWidget()));
	}
#endif

	// Check visibility, it was tested before adding it to the list but another widget may have change while updating.
	if (InvalidationWidget.Visibility.IsVisible() && InvalidationWidget.GetWidget())
	{
		const FWidgetProxy::FUpdateResult UpdateResult = InvalidationWidget.Update(*Context.PaintArgs, *Context.WindowElementList);

		// NB WidgetPtr can now be invalid. The widget can now be collapsed.

		if (UpdateResult.bPainted)
		{
			bNeedsPaint = true;

			{
				// Remove from the update list elements that are processed by this paint
				FSlateInvalidationWidgetList::FIndexRange PaintedWidgetRange{ *FastWidgetPathList, InvalidationWidget.Index, InvalidationWidget.LeafMostChildIndex };
				while (FinalUpdateList.Num() > 0)
				{
					const int32 LastIndex = FinalUpdateList.Num() - 1;
					// It's already been processed by the previous draw
					if (!PaintedWidgetRange.Include(FinalUpdateList[LastIndex].GetWidgetSortOrder()))
					{
						break;
					}

					// It's already been processed by the previous draw
					FinalUpdateList.RemoveAt(LastIndex, EAllowShrinking::No);
				}
			}

			// Did it painted more elements than it previously had
			if (UpdateResult.NewOutgoingLayerId > UpdateResult.PreviousOutgoingLayerId && GSlateInvalidationEnableReindexLayerId)
			{
				if (InvalidationWidget.Visibility.IsVisible() && InvalidationWidget.GetWidget())
				{
					PaintFastPath_FixupLayerId(FastPaintContext, InvalidationWidget, UpdateResult.NewOutgoingLayerId);
				}
			}

			CachedMaxLayerId = FMath::Max(UpdateResult.NewOutgoingLayerId, CachedMaxLayerId);
		}
	}

	return bNeedsPaint;
}

bool FSlateInvalidationRoot::PaintFastPath(const FSlateInvalidationContext& Context)
{
	SCOPED_NAMED_EVENT(SWidget_FastPathUpdate, FColor::Green);
	CSV_SCOPED_TIMING_STAT(Slate, PaintFastPath);

	check(!bNeedsSlowPath);

#if WITH_SLATE_DEBUGGING
	if (GSlateInvalidationRootDumpUpdateList || GSlateInvalidationRootDumpUpdateListOnce)
	{
		DumpUpdateList(*FastWidgetPathList, FinalUpdateList);
	}
#endif
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (GSlateInvalidationRootVerifyWidgetsAreUpdatedOnce)
	{
		FastWidgetPathList->ForEachInvalidationWidget([](FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
			{
				InvalidationWidget.bDebug_Updated = false;
			});
	}
#endif

	bool bWidgetsNeededRepaint = false;
	{
		TGuardValue<bool> OnFastPathGuard(GSlateIsOnFastUpdatePath, true);

		// Widgets that needs re-indexing while updating the FinalUpdateList.
		UE::Slate::Private::FSlateInvalidationPaintFastPathContext FastPaintContext;

		// Is the new LayerId bigger than the previous we received.
		{
			const FSlateInvalidationWidgetIndex FirstWidgetIndex = FastWidgetPathList->FirstIndex();
			if (FirstWidgetIndex != FSlateInvalidationWidgetIndex::Invalid)
			{
				const FSlateInvalidationWidgetList::InvalidationWidgetType& FirstWidgetInvalidationWidget = (*FastWidgetPathList)[FirstWidgetIndex];
				// Are we going to already process it anyway
				if (FinalUpdateList.Num() == 0 || FinalUpdateList.Last().GetWidgetIndex() != FirstWidgetIndex)
				{
					if (SWidget* FirstWidgetPtr = FirstWidgetInvalidationWidget.GetWidget())
					{
						// The new LayerId is bigger than previously. Because of some widget optimization, see PaintFastPath_FixupLayerId, we need to repaint the widgets.
						if (Context.IncomingLayerId > FirstWidgetPtr->GetPersistentState().LayerId)
						{
							// +1 is enough but we add a buffer just in case.
							const int32 LayerIdBuffer = 100;
							const_cast<FSlateWidgetPersistentState&>(FirstWidgetPtr->GetPersistentState()).LayerId = Context.IncomingLayerId + LayerIdBuffer;
							FirstWidgetPtr->UpdateFlags |= EWidgetUpdateFlags::NeedsRepaint;

							const FSlateInvalidationWidgetSortOrder FirstWidgetSortIndex{ *FastWidgetPathList, FirstWidgetIndex };
							FinalUpdateList.Emplace(FirstWidgetIndex, FirstWidgetSortIndex);
						}
					}
				}
			}
		}

		// The update list is put in reverse order by ProcessInvalidation
		while (FinalUpdateList.Num() > 0 || !FastPaintContext.ReindexUpdateList.IsEmpty())
		{
			if (FinalUpdateList.Num() > 0 && !FastPaintContext.ReindexUpdateList.IsEmpty())
			{
				const FSlateInvalidationWidgetIndex MyReindex = FastPaintContext.ReindexUpdateList.HeapPeek().GetWidgetIndex();
				const FSlateInvalidationWidgetList::InvalidationWidgetType& ReindexInvalidationWidget = (*FastWidgetPathList)[MyReindex];

				const FSlateInvalidationWidgetList::FIndexRange ReindexWidgetRange{ *FastWidgetPathList, ReindexInvalidationWidget.Index, ReindexInvalidationWidget.LeafMostChildIndex };
				if (ReindexWidgetRange.Include(FinalUpdateList[FinalUpdateList.Num() - 1].GetWidgetSortOrder()))
				{
					// If the next widget to update is inside the widget range of the re-index widget, then update the child widget first.
					const bool bPainted = PaintFastPath_UpdateNextWidget(Context, FastPaintContext);
					bWidgetsNeededRepaint = bWidgetsNeededRepaint || bPainted;
					if (bNeedsSlowPath)
					{
						break;
					}
				}
				else
				{
					// The next widget to update is not a children/grand children of the re-index. Reindex it.
					FastPaintContext.ReindexUpdateList.HeapPopDiscard();
					SWidget* ReindexWidgetPtr = ReindexInvalidationWidget.GetWidget();
					if (ReindexInvalidationWidget.Visibility.IsVisible() && ReindexWidgetPtr)
					{
						PaintFastPath_FixupLayerId(FastPaintContext, ReindexInvalidationWidget, ReindexWidgetPtr->GetPersistentState().OutgoingLayerId);
					}
				}
			}
			else if (FinalUpdateList.Num() > 0)
			{
				const bool bPainted = PaintFastPath_UpdateNextWidget(Context, FastPaintContext);
				bWidgetsNeededRepaint = bWidgetsNeededRepaint || bPainted;
				if (bNeedsSlowPath)
				{
					break;
				}
			}
			else
			{
				const FSlateInvalidationWidgetIndex MyReindex = FastPaintContext.ReindexUpdateList.HeapPeek().GetWidgetIndex();
				const FSlateInvalidationWidgetList::InvalidationWidgetType& ReindexInvalidationWidget = (*FastWidgetPathList)[MyReindex];

				FastPaintContext.ReindexUpdateList.HeapPopDiscard();
				SWidget* ReindexWidgetPtr = ReindexInvalidationWidget.GetWidget();
				if (ReindexInvalidationWidget.Visibility.IsVisible() && ReindexWidgetPtr)
				{
					PaintFastPath_FixupLayerId(FastPaintContext, ReindexInvalidationWidget, ReindexWidgetPtr->GetPersistentState().OutgoingLayerId);
				}
			}
		}

		FinalUpdateList.Reset();
	}

	bool bExecuteSlowPath = bNeedsSlowPath;
	if (bExecuteSlowPath)
	{
		SCOPED_NAMED_EVENT(Slate_PaintSlowPath, FColor::Red);
		Context.WindowElementList->PopCachedElementData(); // to ensure we are not caching anymore
		ClearAllFastPathData(true);
		CachedMaxLayerId = PaintSlowPath(Context);
	}

#if WITH_SLATE_DEBUGGING
	SetLastPaintType(bExecuteSlowPath ? ESlateInvalidationPaintType::Slow : ESlateInvalidationPaintType::Fast);
#endif

	return bWidgetsNeededRepaint;
}

void FSlateInvalidationRoot::AdjustWidgetsDesktopGeometry(UE::Slate::FDeprecateVector2DParameter WindowToDesktopTransform)
{
	FSlateLayoutTransform WindowToDesktop(WindowToDesktopTransform);

	FastWidgetPathList->ForEachWidget([WindowToDesktopTransform, &WindowToDesktop](SWidget& Widget)
		{
			Widget.PersistentState.DesktopGeometry = Widget.PersistentState.AllottedGeometry;
			Widget.PersistentState.DesktopGeometry.AppendTransform(WindowToDesktop);
		});
}

void FSlateInvalidationRoot::BuildFastPathWidgetList(const TSharedRef<SWidget>& RootWidget)
{
	TGuardValue<bool> Tmp(bBuildingWidgetList, true);

	// We do not care if update are requested. We need to redo all the data.
	WidgetsNeedingPreUpdate->Reset(false);
	WidgetsNeedingPrepassUpdate->Reset(false);
	WidgetsNeedingPostUpdate->Reset(false);
	FinalUpdateList.Reset();

	// Rebuild the list and update SlateAttribute
	FastWidgetPathList->BuildWidgetList(RootWidget);
}

void FSlateInvalidationRoot::ProcessPreUpdate()
{
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (GSlateInvalidationRootVerifyWidgetsUpdateList)
	{
		VerifyWidgetsUpdateList_BeforeProcessPreUpdate(GetRootWidget(), FastWidgetPathList.Get(), WidgetsNeedingPreUpdate.Get(), WidgetsNeedingPostUpdate.Get(), FinalUpdateList);
	}
#endif

	TGuardValue<bool> Tmp(bProcessingPreUpdate, true);

	TSharedRef<SWidget> RootWidget = GetRootWidget();
	if (FastWidgetPathList->GetRoot().Pin() != RootWidget)
	{
		BuildFastPathWidgetList(RootWidget);

		// Add the root to the update list (to prepass and paint it)
		check(RootWidget->GetProxyHandle().IsValid(&RootWidget.Get()));
		WidgetsNeedingPostUpdate->Reset(true); // we can clear the post list, because all widgets will be updated
		RootWidget->Invalidate(EInvalidateWidgetReason::Prepass);
	}
	else
	{
		{
#if WITH_SLATE_DEBUGGING
			if (GSlateInvalidationRootDumpPreInvalidationList)
			{
				UE_LOGF(LogSlate, Log, "Dumping Pre Invalidation List");
				UE_LOGF(LogSlate, Log, "-------------------");
			}
#endif

			/** */
			struct FChildOrderInvalidationCallbackImpl : FSlateInvalidationWidgetList::IProcessChildOrderInvalidationCallback
			{
				FChildOrderInvalidationCallbackImpl(
					const FSlateInvalidationWidgetList& InWidgetList
					, FSlateInvalidationWidgetPreHeap& InPreUpdate
					, FSlateInvalidationWidgetPrepassHeap& InPrepassUpdate
					, FSlateInvalidationWidgetPostHeap& InPostUpdate)
					: WidgetList(InWidgetList)
					, PreUpdate(InPreUpdate)
					, PrepassUpdate(InPrepassUpdate)
					, PostUpdate(InPostUpdate)
				{}
				virtual ~FChildOrderInvalidationCallbackImpl() = default;
				const FSlateInvalidationWidgetList& WidgetList;
				FSlateInvalidationWidgetPreHeap& PreUpdate;
				FSlateInvalidationWidgetPrepassHeap& PrepassUpdate;
				FSlateInvalidationWidgetPostHeap& PostUpdate;
				TArray<FSlateInvalidationWidgetPreHeap::FElement*, FConcurrentLinearArrayAllocator> WidgetToResort;

				virtual void PreChildRemove(const FSlateInvalidationWidgetList::FIndexRange& Range) override
				{
					// The widgets got removed from the list. There is no need to update them anymore.
					//Also, their index will not be valid after this function.
					PreUpdate.RemoveRange(Range);
					PostUpdate.RemoveRange(Range);
					PrepassUpdate.RemoveRange(Range);
				}
				using FReIndexOperation = FSlateInvalidationWidgetList::IProcessChildOrderInvalidationCallback::FReIndexOperation;
				virtual void ProxiesReIndexed(const FReIndexOperation& Operation) override
				{
					// Re-index in Pre and Post list (modify the index and the sort value)
					FChildOrderInvalidationCallbackImpl const* Self = this;
					auto ReIndexIfNeeded = [&Operation, Self](FSlateInvalidationWidgetPreHeap::FElement& Element)
					{
						if (Operation.GetRange().Include(Element.GetWidgetSortOrder()))
						{
							FSlateInvalidationWidgetIndex NewIndex = Operation.ReIndex(Element.GetWidgetIndex());
							Element = FSlateInvalidationWidgetPreHeap::FElement{ NewIndex, FSlateInvalidationWidgetSortOrder{ Self->WidgetList, NewIndex } };
						}
					};
					PreUpdate.ForEachIndexes(ReIndexIfNeeded);
					PostUpdate.ForEachIndexes(ReIndexIfNeeded);
					PrepassUpdate.ForEachIndexes(ReIndexIfNeeded);
				}
				using FReSortOperation = FSlateInvalidationWidgetList::IProcessChildOrderInvalidationCallback::FReSortOperation;
				virtual void ProxiesPreResort(const FReSortOperation& Operation) override
				{
					// The sort order value will change but the order (operator<) is still valid.
					FChildOrderInvalidationCallbackImpl* Self = this;
					auto ReSortIfNeeded = [&Operation, Self](FSlateInvalidationWidgetPreHeap::FElement& Element)
					{
						if (Operation.GetRange().Include(Element.GetWidgetSortOrder()))
						{
							Self->WidgetToResort.Add(&Element);
						}
					};
					PreUpdate.ForEachIndexes(ReSortIfNeeded);
					PostUpdate.ForEachIndexes(ReSortIfNeeded);
					PrepassUpdate.ForEachIndexes(ReSortIfNeeded);
				}
				virtual void ProxiesPostResort()
				{
					for (FSlateInvalidationWidgetPreHeap::FElement* Element : WidgetToResort)
					{
						FSlateInvalidationWidgetIndex Index = Element->GetWidgetIndex();
						(*Element) = FSlateInvalidationWidgetPreHeap::FElement{ Index, FSlateInvalidationWidgetSortOrder{ WidgetList, Index } };
					}
					WidgetToResort.Reset();
				}
			};

			FChildOrderInvalidationCallbackImpl ChildOrderInvalidationCallback{ *FastWidgetPathList, *WidgetsNeedingPreUpdate, *WidgetsNeedingPrepassUpdate, *WidgetsNeedingPostUpdate };

			while(WidgetsNeedingPreUpdate->Num() > 0 && !bNeedsSlowPath)
			{
				// Process ChildOrder && AttributeRegistration invalidation.

				const FSlateInvalidationWidgetIndex WidgetIndex = WidgetsNeedingPreUpdate->HeapPop();
				SWidget* WidgetPtr = nullptr;
				EInvalidateWidgetReason CurrentInvalidateReason = EInvalidateWidgetReason::None;
				{
					// After an ChildOrder, the index may not have changed but the internal array may have grow.
					//Growing may invalidate the array address.
					const FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget = (*FastWidgetPathList)[WidgetIndex];
					WidgetPtr = InvalidationWidget.GetWidget();
					CurrentInvalidateReason = InvalidationWidget.CurrentInvalidateReason;
				}

				// It could have been destroyed
				if (WidgetPtr)
				{
#if WITH_SLATE_DEBUGGING
					if (GSlateInvalidationRootDumpPreInvalidationList)
					{
						LogPreInvalidationItem(*FastWidgetPathList, WidgetIndex);
					}
#endif

					bool bIsInvalidationWidgetValid = true;
					if (EnumHasAnyFlags(CurrentInvalidateReason, EInvalidateWidgetReason::ChildOrder))
					{
// Uncomment to see to be able to compare the list before and after when debugging
#if 0
						TArray<TTuple<FSlateInvalidationWidgetIndex, FSlateInvalidationWidgetSortOrder, TWeakPtr<SWidget>>, FConcurrentLinearArrayAllocator> PreviousPreUpdate;
						TArray<TTuple<FSlateInvalidationWidgetIndex, FSlateInvalidationWidgetSortOrder, TWeakPtr<SWidget>>, FConcurrentLinearArrayAllocator> PreviousPrepassUpdate;
						TArray<TTuple<FSlateInvalidationWidgetIndex, FSlateInvalidationWidgetSortOrder, TWeakPtr<SWidget>>, FConcurrentLinearArrayAllocator> PreviousPostUpdate;
						PreviousPreUpdate.Reserve(WidgetsNeedingPreUpdate->Num());
						PreviousPrepassUpdate.Reserve(WidgetsNeedingPrepassUpdate->Num());
						PreviousPostUpdate.Reserve(WidgetsNeedingPostUpdate->Num());
						for (const auto& Element : WidgetsNeedingPreUpdate->GetRaw())
						{
							ensureAlwaysMsgf(FastWidgetPathList->IsValidIndex(Element.GetWidgetIndex()), TEXT("The element is invalid"));
							PreviousPreUpdate.Emplace(Element.GetWidgetIndex(), Element.GetWidgetSortOrder(), (*FastWidgetPathList)[Element.GetWidgetIndex()].GetWidgetAsShared());
						}
						for (const auto& Element : WidgetsNeedingPrepassUpdate->GetRaw())
						{
							ensureAlwaysMsgf(FastWidgetPathList->IsValidIndex(Element.GetWidgetIndex()), TEXT("The element is invalid."));
							PreviousPrepassUpdate.Emplace(Element.GetWidgetIndex(), Element.GetWidgetSortOrder(), (*FastWidgetPathList)[Element.GetWidgetIndex()].GetWidgetAsShared());
						}
						for (const auto& Element : WidgetsNeedingPostUpdate->GetRaw())
						{
							ensureAlwaysMsgf(FastWidgetPathList->IsValidIndex(Element.GetWidgetIndex()), TEXT("The element is invalid."));
							PreviousPostUpdate.Emplace(Element.GetWidgetIndex(), Element.GetWidgetSortOrder(), (*FastWidgetPathList)[Element.GetWidgetIndex()].GetWidgetAsShared());
						}
#endif

						TGuardValue<bool> ProcessChildOrderInvalidationGuardValue(bProcessingChildOrderInvalidation, true);
						bIsInvalidationWidgetValid = FastWidgetPathList->ProcessChildOrderInvalidation(WidgetIndex, ChildOrderInvalidationCallback);
						if (bIsInvalidationWidgetValid)
						{
							FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget = (*FastWidgetPathList)[WidgetIndex];
							EnumRemoveFlags(InvalidationWidget.CurrentInvalidateReason, EInvalidateWidgetReason::ChildOrder);
						}
					}

					if (bIsInvalidationWidgetValid && EnumHasAnyFlags(CurrentInvalidateReason, EInvalidateWidgetReason::AttributeRegistration))
					{
						FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget = (*FastWidgetPathList)[WidgetIndex];
						// Process the attribute invalidation even if it could be process in a ChildOrder invalidation. The attribute is added with "Unique".
						FastWidgetPathList->ProcessAttributeRegistrationInvalidation(InvalidationWidget);
						EnumRemoveFlags(InvalidationWidget.CurrentInvalidateReason, EInvalidateWidgetReason::AttributeRegistration);
					}

					if (bIsInvalidationWidgetValid && EnumHasAnyFlags(CurrentInvalidateReason, EInvalidateWidgetReason::Visibility))
					{
						FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget = (*FastWidgetPathList)[WidgetIndex];
						WidgetPtr->UpdateFastPathVisibility(InvalidationWidget.Visibility.MimicAsParent(), GetHittestGrid());
						// We should Keep the flag for the layout process (visibility changes affect the layout and paint) and it will be removed then.
						//This phase only updated the internal flags and internal states.
						//But if we are in the layout or paint process, and we add the Visibility flag, then the visibility flag will be remove and the UpdateFastPathVisibility won't execute.
						//The EInvalidationWidgetReason is already full (8bits). We can add a new bit in the InvalidationWidgetType but that would mean more flag management.
						//Easy solution is to use the "RenderTransform" flag because it does the same job as the "Visibility" in the layout and paint phase.
						EnumRemoveFlags(InvalidationWidget.CurrentInvalidateReason, EInvalidateWidgetReason::Visibility);
						EnumAddFlags(InvalidationWidget.CurrentInvalidateReason, EInvalidateWidgetReason::RenderTransform);
						WidgetsNeedingPostUpdate->PushBackOrHeapUnique(InvalidationWidget);
					}
				}
			}
		}
	}


#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (GSlateInvalidationRootVerifyWidgetList)
	{
		VerifyWidgetList(RootWidget, InvalidationRootHandle, GetFastPathWidgetList());
	}

	if (GSlateInvalidationRootVerifyWidgetsIndex)
	{
		ensureMsgf(FastWidgetPathList->VerifyWidgetsIndex(), TEXT("We failed to verify that every widgets has the correct index."));
	}

	if (GSlateInvalidationRootVerifyValidWidgets)
	{
		ensureMsgf(FastWidgetPathList->VerifyProxiesWidget(), TEXT("We failed to verify that every WidgetProxy has a valid SWidget"));
	}
#endif //UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
}

void FSlateInvalidationRoot::ProcessAttributeUpdate()
{
	TGuardValue<bool> Tmp(bProcessingAttributeUpdate, true);

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (GSlateInvalidationRootVerifySlateAttribute)
	{
		VerifySlateAttribute_BeforeUpdate(*FastWidgetPathList);
	}
#endif

	FSlateInvalidationWidgetList::FWidgetAttributeIterator AttributeItt = FastWidgetPathList->CreateWidgetAttributeIterator();
	while (AttributeItt.IsValid())
	{
		FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget = (*FastWidgetPathList)[AttributeItt.GetCurrentIndex()];
		SWidget* WidgetPtr = InvalidationWidget.GetWidget();
		if (ensureAlways(WidgetPtr))
		{
			if (!InvalidationWidget.Visibility.IsCollapseIndirectly())
			{
				// if my parent is not collapse, then update my visible state
				FSlateAttributeMetaData::UpdateOnlyVisibilityAttributes(*WidgetPtr, FSlateAttributeMetaData::EInvalidationPermission::AllowInvalidation);
				if (!InvalidationWidget.Visibility.IsCollapsed())
				{
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
					ensureAlwaysMsgf(!GSlateInvalidationRootVerifySlateAttribute || InvalidationWidget.bDebug_AttributeUpdated == false, TEXT("Attribute should only be updated once per frame."));
					InvalidationWidget.bDebug_AttributeUpdated = true;
#endif

#if WITH_SLATE_DEBUGGING
					FSlateInvalidationWidgetVisibility PreviousVisibility = InvalidationWidget.Visibility;
					FSlateInvalidationWidgetIndex PreviousLeafMostChildIndex = InvalidationWidget.LeafMostChildIndex;
#endif

					FSlateAttributeMetaData::UpdateExceptVisibilityAttributes(*WidgetPtr, FSlateAttributeMetaData::EInvalidationPermission::AllowInvalidation);
					AttributeItt.Advance();

#if WITH_SLATE_DEBUGGING
					// visibility is updated with FSlateAttributeMetaData::UpdateOnlyVisibilityAttributes
					ensureMsgf(PreviousVisibility == InvalidationWidget.Visibility, TEXT("The visibility of widget '%s' doesn't match the previous visibility after the attribute update."), *FReflectionMetaData::GetWidgetDebugInfo(WidgetPtr));
					// childorder is not support as an invalidation with attribute
					ensureMsgf(PreviousLeafMostChildIndex == InvalidationWidget.LeafMostChildIndex, TEXT("The number of child of widget '%s' doesn't match the previous count after the attribute update."), *FReflectionMetaData::GetWidgetDebugInfo(WidgetPtr));
#endif
				}
				else
				{
					FSlateAttributeMetaData::ApplyDelayedInvalidation(*WidgetPtr);
					AttributeItt.Advance();
				}
			}
			else
			{
				FSlateAttributeMetaData::ApplyDelayedInvalidation(*WidgetPtr);
				AttributeItt.Advance();
			}
		}
		else
		{
			AttributeItt.Advance();
		}
	}

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (GSlateInvalidationRootVerifySlateAttribute)
	{
		VerifySlateAttribute_AfterUpdate(*FastWidgetPathList);
	}
#endif //UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
}

void FSlateInvalidationRoot::ProcessPrepassUpdate()
{
	TGuardValue<bool> Tmp(bProcessingPrepassUpdate, true);

#if WITH_SLATE_DEBUGGING
	if (GSlateInvalidationRootDumpPostInvalidationList)
	{
		UE_LOGF(LogSlate, Log, "Dumping Prepass Invalidation List");
		UE_LOGF(LogSlate, Log, "-------------------");
	}
#endif

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (GSlateInvalidationRootVerifyWidgetsUpdateList)
	{
		VerifyWidgetsUpdateList_ProcessPrepassUpdate(FastWidgetPathList.Get(), WidgetsNeedingPrepassUpdate.Get(), WidgetsNeedingPostUpdate.Get());
	}
#endif

	FSlateInvalidationWidgetList::FIndexRange PreviousInvalidationWidgetRange;

	// It update forward (smallest index to biggest )
	while (WidgetsNeedingPrepassUpdate->Num())
	{
		const FSlateInvalidationWidgetPrepassHeap::FElement WidgetElement = WidgetsNeedingPrepassUpdate->HeapPop();
		if (PreviousInvalidationWidgetRange.IsValid())
		{
			// It's already been processed by the previous slate prepass
			if (PreviousInvalidationWidgetRange.Include(WidgetElement.GetWidgetSortOrder()))
			{
				continue;
			}
		}
		FWidgetProxy& WidgetProxy = (*FastWidgetPathList)[WidgetElement.GetWidgetIndex()];
		PreviousInvalidationWidgetRange = FSlateInvalidationWidgetList::FIndexRange(*FastWidgetPathList, WidgetProxy.Index, WidgetProxy.LeafMostChildIndex);

		// Widget could be null if it was removed and we are on the slow path
		if (SWidget* WidgetPtr = WidgetProxy.GetWidget())
		{
#if WITH_SLATE_DEBUGGING
			if (GSlateInvalidationRootDumpPrepassInvalidationList)
			{
				LogPrepassInvalidationItem(*FastWidgetPathList, WidgetElement.GetWidgetIndex());
			}
#endif

			if (!WidgetProxy.Visibility.IsCollapsed() && WidgetProxy.bIsVolatilePrepass)
			{
				WidgetPtr->MarkPrepassAsDirty();
			}

			// Run the invalidation, it will also be run in ProcessPostUpdate.
			//If there is no new invalidation, then the Widget's CurrentInvalidation should be none and nothing will be execute.
			//Then, if needed, ProcessPostUpdate will add the widget to the FinalUpdateList in the update correct order.
			WidgetProxy.ProcessPostInvalidation(*WidgetsNeedingPostUpdate, *FastWidgetPathList, *this);
			EnumRemoveFlags(WidgetProxy.CurrentInvalidateReason, UE::Slate::Private::PostInvalidationReason);
		}
	}
	WidgetsNeedingPrepassUpdate->Reset(true);
}

bool FSlateInvalidationRoot::ProcessPostUpdate()
{
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (GSlateInvalidationRootVerifyWidgetsUpdateList)
	{
		VerifyWidgetsUpdateList_BeforeProcessPostUpdate(GetRootWidget(), FastWidgetPathList.Get(), WidgetsNeedingPreUpdate.Get(), WidgetsNeedingPostUpdate.Get(), FinalUpdateList);
	}
#endif

	TGuardValue<bool> Tmp(bProcessingPostUpdate, true);
	bool bWidgetsNeedRepaint = false;

#if WITH_SLATE_DEBUGGING
	if (GSlateInvalidationRootDumpPostInvalidationList)
	{
		UE_LOGF(LogSlate, Log, "Dumping Post Invalidation List");
		UE_LOGF(LogSlate, Log, "-------------------");
	}
#endif

	// It update backward (biggest index to smallest)
	while (WidgetsNeedingPostUpdate->Num() && !bNeedsSlowPath)
	{
		const FSlateInvalidationWidgetPostHeap::FElement WidgetElement = WidgetsNeedingPostUpdate->HeapPop();
		FWidgetProxy& WidgetProxy = (*FastWidgetPathList)[WidgetElement.GetWidgetIndex()];

		// Widget could be null if it was removed and we are on the slow path
		if (SWidget* WidgetPtr = WidgetProxy.GetWidget())
		{
#if WITH_SLATE_DEBUGGING
			if (GSlateInvalidationRootDumpPostInvalidationList)
			{
				LogPostInvalidationItem(*FastWidgetPathList, WidgetElement.GetWidgetIndex());
			}
#endif

			bWidgetsNeedRepaint |= WidgetProxy.ProcessPostInvalidation(*WidgetsNeedingPostUpdate, *FastWidgetPathList, *this);
			EnumRemoveFlags(WidgetProxy.CurrentInvalidateReason, UE::Slate::Private::PostInvalidationReason);

			if (WidgetPtr->HasAnyUpdateFlags(EWidgetUpdateFlags::AnyUpdate) && WidgetProxy.Visibility.IsVisible())
			{
				FinalUpdateList.Emplace(WidgetElement.GetWidgetIndex(), WidgetElement.GetWidgetSortOrder());
			}
		}
	}
	WidgetsNeedingPostUpdate->Reset(true);

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (GSlateInvalidationRootVerifyWidgetsUpdateList && !bNeedsSlowPath)
	{
		VerifyWidgetsUpdateList_AfterProcessPostUpdate(GetRootWidget(), FastWidgetPathList.Get(), WidgetsNeedingPreUpdate.Get(), WidgetsNeedingPostUpdate.Get(), FinalUpdateList);
	}
#endif

	return bWidgetsNeedRepaint;
}

bool FSlateInvalidationRoot::ProcessInvalidation()
{
	SCOPED_NAMED_EVENT(Slate_InvalidationProcessing, FColor::Blue);
#if WITH_SLATE_DEBUGGING
	PerformanceStat = FPerformanceStat();
	FScopedDurationTimer TmpPerformance_ProcessInvalidation(PerformanceStat.InvalidationProcessing);
#endif

	TGuardValue<bool> OnFastPathGuard(GSlateIsOnFastProcessInvalidation, true);

	bool bWidgetsNeedRepaint = false;

	// Process ChildOrder | Visibility | AttributeRegistration invalidations.
	if (!bNeedsSlowPath)
	{
		check(WidgetsNeedingPreUpdate);
		check(WidgetsNeedingPrepassUpdate);
		check(WidgetsNeedingPostUpdate);

		SCOPED_NAMED_EVENT(Slate_InvalidationProcessing_PreUpdate, FColor::Blue);
#if WITH_SLATE_DEBUGGING
		FScopedDurationTimer TmpPerformance(PerformanceStat.WidgetsPreUpdate);
#endif

		ProcessPreUpdate();
	}

	// Execute the attributes delegate.
	if (!bNeedsSlowPath)
	{
		SCOPED_NAMED_EVENT(Slate_InvalidationProcessing_AttributeUpdate, FColor::Blue);
#if WITH_SLATE_DEBUGGING
		FScopedDurationTimer TmpPerformance(PerformanceStat.WidgetsAttribute);
#endif

		ProcessAttributeUpdate();
	}

	// Re-process new ChildOrder | Visibility | AttributeRegistration invalidations. Attributes may have added new invalidation.
	if (!bNeedsSlowPath && WidgetsNeedingPreUpdate->Num() > 0)
	{
		SCOPED_NAMED_EVENT(Slate_InvalidationProcessing_PreUpdate, FColor::Blue);

		ProcessPreUpdate();
	}

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (GSlateInvalidationRootVerifyWidgetVisibility && !bNeedsSlowPath)
	{
		VerifyWidgetVisibility(GetFastPathWidgetList());
	}
#endif

	if (!bNeedsSlowPath)
	{
		// Put all widgets in the VolatileUpdate (Paint, Tick, ActiveTimer) list in the WidgetsNeedingPostUpdate
		WidgetsNeedingPrepassUpdate->Heapify();
		WidgetsNeedingPostUpdate->Heapify();
		{
			for (FSlateInvalidationWidgetList::FWidgetVolatileUpdateIterator Iterator = FastWidgetPathList->CreateWidgetVolatileUpdateIterator(true)
				; Iterator.IsValid()
				; Iterator.Advance())
			{
				FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget = (*FastWidgetPathList)[Iterator.GetCurrentIndex()];
				WidgetsNeedingPostUpdate->HeapPushUnique(InvalidationWidget);
				if (InvalidationWidget.bIsVolatilePrepass)
				{
					InvalidationWidget.CurrentInvalidateReason |= EInvalidateWidgetReason::Layout;
					WidgetsNeedingPrepassUpdate->HeapPushUnique(InvalidationWidget);
				}
			}
		}
	}

	// Process widget that needs Prepass and PrepassVolatile.
	//They will execute forward and then executed again in ProcessPostUpdate in backward.
	//That save time when the big widget needs Prepass because we can skip all the child widgets.
	if (!bNeedsSlowPath)
	{
		SCOPED_NAMED_EVENT(Slate_InvalidationProcessing_PrepassUpdate, FColor::Blue);
#if WITH_SLATE_DEBUGGING
		FScopedDurationTimer TmpPerformance(PerformanceStat.WidgetsPrepass);
#endif

		ProcessPrepassUpdate();
	}

	// Process all the layout widgets (RenderTransform | Layout | Prepass)
	if (!bNeedsSlowPath)
	{
		FinalUpdateList.Reset(WidgetsNeedingPostUpdate->Num());

		SCOPED_NAMED_EVENT(Slate_InvalidationProcessing_PostUpdate, FColor::Blue);
#if WITH_SLATE_DEBUGGING
		FScopedDurationTimer TmpPerformance(PerformanceStat.WidgetsUpdate);
#endif

		bWidgetsNeedRepaint = ProcessPostUpdate();
	}

	if (bDeferInvalidateAllWidgetsAndClearResources.IsSet())
	{
		HandleInvalidateAllWidgets(bDeferInvalidateAllWidgetsAndClearResources.GetValue());
		bDeferInvalidateAllWidgetsAndClearResources = {};
	}

	if (bNeedsSlowPath)
	{
		WidgetsNeedingPreUpdate->Reset(true);
		WidgetsNeedingPrepassUpdate->Reset(true);
		WidgetsNeedingPostUpdate->Reset(true);
		FinalUpdateList.Reset();
		CachedElementData->Empty();
		bWidgetsNeedRepaint = true;
	}

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (GSlateInvalidationRootVerifyWidgetVolatile && !bNeedsSlowPath)
	{
		VerifyWidgetVolatile(GetFastPathWidgetList(), FinalUpdateList);
	}
#endif //UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING

	return bWidgetsNeedRepaint;
}

void FSlateInvalidationRoot::ClearAllFastPathData(bool bClearResourcesImmediately)
{
	FastWidgetPathList->ForEachWidget([bClearResourcesImmediately](SWidget& Widget)
		{
			Widget.PersistentState.CachedElementHandle = FSlateCachedElementsHandle::Invalid;
			if (bClearResourcesImmediately)
			{
				Widget.FastPathProxyHandle = FWidgetProxyHandle();
			}
		});
	FastWidgetPathList->Reset();

#if UE_SLATE_DEBUGGING_CLEAR_ALL_FAST_PATH_DATA
	if (!bClearResourcesImmediately)
	{
		for (const FWidgetProxy& Proxy : FastWidgetPathList)
		{
			if (SWidget* Widget = Proxy.GetWidget())
			{
				if (Widget->FastPathProxyHandle.IsValid())
				{
					FastWidgetPathToClearedBecauseOfDelay.Add(Widget);
				}
			}
		}
	}
	else
	{
		for (const FWidgetProxy& Proxy : FastWidgetPathList)
		{
			FastWidgetPathToClearedBecauseOfDelay.RemoveSingleSwap(Proxy.GetWidget());
		}
	}
#endif

	WidgetsNeedingPreUpdate->Reset(false);
	WidgetsNeedingPrepassUpdate->Reset(false);
	WidgetsNeedingPostUpdate->Reset(false);
	FastWidgetPathList->Empty();
	CachedElementData->Empty();
	FinalUpdateList.Empty();
}

void FSlateInvalidationRoot::SetNeedsSlowPath(bool InNeedsSlowPath)
{
	bNeedsSlowPath = InNeedsSlowPath;
}

void FSlateInvalidationRoot::HandleInvalidateAllWidgets(bool bClearResourcesImmediately)
{
	if (bProcessingPreUpdate | bProcessingAttributeUpdate | bProcessingPrepassUpdate | bProcessingPostUpdate)
	{
		bDeferInvalidateAllWidgetsAndClearResources = bClearResourcesImmediately;
	}
	else
	{
		Advanced_ResetInvalidation(bClearResourcesImmediately);
		OnRootInvalidated();
	}
}

void FSlateInvalidationRoot::Advanced_ResetInvalidation(bool bClearResourcesImmediately)
{
	InvalidateRootChildOrder();

	if (bClearResourcesImmediately)
	{
		ClearAllFastPathData(true);
	}

	bNeedsSlowPath = true;
}

#if WITH_SLATE_DEBUGGING
void DumpUpdateList(const FSlateInvalidationWidgetList& FastWidgetPathList, const TArray<FSlateInvalidationWidgetHeapElement>& FinalUpdateList)
{
	UE_LOGF(LogSlate, Log, "Dumping Update List");
	UE_LOGF(LogSlate, Log, "-------------------");
	// The update list is put in reverse order 
	for (int32 ListIndex = FinalUpdateList.Num() - 1; ListIndex >= 0; --ListIndex)
	{
		const FSlateInvalidationWidgetIndex MyIndex = FinalUpdateList[ListIndex].GetWidgetIndex();

		const FWidgetProxy& WidgetProxy = FastWidgetPathList[MyIndex];
		const SWidget* WidgetPtr = WidgetProxy.GetWidget();
		if (WidgetProxy.Visibility.IsVisible() && WidgetPtr)
		{
			if (WidgetPtr->HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePaint))
			{
				UE_LOGF(LogSlate, Log, "Volatile Repaint %ls", *FReflectionMetaData::GetWidgetDebugInfo(WidgetPtr));
			}
			else if (WidgetPtr->HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsRepaint))
			{
				UE_LOGF(LogSlate, Log, "Repaint %ls", *FReflectionMetaData::GetWidgetDebugInfo(WidgetPtr));
			}
			else
			{
				if (WidgetPtr->HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsActiveTimerUpdate))
				{
					UE_LOGF(LogSlate, Log, "ActiveTimer %ls", *FReflectionMetaData::GetWidgetDebugInfo(WidgetPtr));
				}

				if (WidgetPtr->HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsTick))
				{
					UE_LOGF(LogSlate, Log, "Tick %ls", *FReflectionMetaData::GetWidgetDebugInfo(WidgetPtr));
				}
			}
		}
	}
	UE_LOGF(LogSlate, Log, "-------------------");

	GSlateInvalidationRootDumpUpdateListOnce = false;
}


void LogPreInvalidationItem(const FSlateInvalidationWidgetList& FastWidgetPathList, FSlateInvalidationWidgetIndex WidgetIndex)
{
	const FWidgetProxy& Proxy = FastWidgetPathList[WidgetIndex];

	if (EnumHasAnyFlags(Proxy.CurrentInvalidateReason, EInvalidateWidgetReason::AttributeRegistration))
	{
		UE_LOGF(LogSlate, Log, "  AttributeRegistration %ls", *FReflectionMetaData::GetWidgetDebugInfo(Proxy.GetWidget()));
	}
	else if (EnumHasAnyFlags(Proxy.CurrentInvalidateReason, EInvalidateWidgetReason::ChildOrder))
	{
		UE_LOGF(LogSlate, Log, "  Child Order %ls", *FReflectionMetaData::GetWidgetDebugInfo(Proxy.GetWidget()));
	}
	else
	{
		UE_LOGF(LogSlate, Log, "  [?] %ls", *FReflectionMetaData::GetWidgetDebugInfo(Proxy.GetWidget()));
	}
}

void LogPrepassInvalidationItem(const FSlateInvalidationWidgetList& FastWidgetPathList, FSlateInvalidationWidgetIndex WidgetIndex)
{
	const FWidgetProxy& Proxy = FastWidgetPathList[WidgetIndex];
	UE_LOGF(LogSlate, Log, "  Prepass %ls", *FReflectionMetaData::GetWidgetDebugInfo(Proxy.GetWidget()));
}

void LogPostInvalidationItem(const FSlateInvalidationWidgetList& FastWidgetPathList, FSlateInvalidationWidgetIndex WidgetIndex)
{
	const FWidgetProxy& Proxy = FastWidgetPathList[WidgetIndex];

	if (EnumHasAnyFlags(Proxy.CurrentInvalidateReason, EInvalidateWidgetReason::Layout))
	{
		UE_LOGF(LogSlate, Log, "  Layout %ls", *FReflectionMetaData::GetWidgetDebugInfo(Proxy.GetWidget()));
	}
	else if (EnumHasAnyFlags(Proxy.CurrentInvalidateReason, EInvalidateWidgetReason::Visibility))
	{
		UE_LOGF(LogSlate, Log, "  Visibility %ls", *FReflectionMetaData::GetWidgetDebugInfo(Proxy.GetWidget()));
	}
	else if (EnumHasAnyFlags(Proxy.CurrentInvalidateReason, EInvalidateWidgetReason::Volatility))
	{
		UE_LOGF(LogSlate, Log, "  Volatility %ls", *FReflectionMetaData::GetWidgetDebugInfo(Proxy.GetWidget()));
	}
	else if (EnumHasAnyFlags(Proxy.CurrentInvalidateReason, EInvalidateWidgetReason::RenderTransform))
	{
		UE_LOGF(LogSlate, Log, "  RenderTransform %ls", *FReflectionMetaData::GetWidgetDebugInfo(Proxy.GetWidget()));
	}
	else if (EnumHasAnyFlags(Proxy.CurrentInvalidateReason, EInvalidateWidgetReason::Paint))
	{
		UE_LOGF(LogSlate, Log, "  Paint %ls", *FReflectionMetaData::GetWidgetDebugInfo(Proxy.GetWidget()));
	}
	else if (!Proxy.GetWidget()->HasAnyUpdateFlags(EWidgetUpdateFlags::AnyUpdate))
	{
		UE_LOGF(LogSlate, Log, "  [?] %ls", *FReflectionMetaData::GetWidgetDebugInfo(Proxy.GetWidget()));
	}
}
#endif

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING

#define UE_SLATE_LOG_ERROR_IF_FALSE(Test, FlagToReset, Message, ...) \
	ensureAlwaysMsgf((Test), Message, ##__VA_ARGS__); \
	if (!(Test)) \
	{ \
		UE_LOG(LogSlate, Error, Message, ##__VA_ARGS__); \
		FlagToReset->Set(false, FlagToReset->GetFlags()); \
		return; \
	}

void VerifyWidgetList(TSharedRef<SWidget> RootWidget, FSlateInvalidationRootHandle InvalidationRootHandle, FSlateInvalidationWidgetList& WidgetList)
{
	FSlateInvalidationWidgetList List(InvalidationRootHandle, FSlateInvalidationWidgetList::FArguments{ 128, 128, 1000, false });
	List.BuildWidgetList(RootWidget);
	bool bIsIdentical = (List.DeepCompare(WidgetList));
	if (!bIsIdentical)
	{
		UE_LOGF(LogSlate, Log, "**-- New Build List --**");
		List.LogWidgetsList(true);
		UE_LOGF(LogSlate, Log, "**-- Invaliation Root List --**");
		WidgetList.LogWidgetsList(true);

		UE_SLATE_LOG_ERROR_IF_FALSE(false, CVarSlateInvalidationRootVerifyWidgetList, TEXT("The updated list doesn't match a newly created list."));
	}
}

void VerifyHittest(SWidget* InvalidationRootWidget, FSlateInvalidationWidgetList& WidgetList, FHittestGrid* HittestGrid)
{
	check(InvalidationRootWidget);
	check(HittestGrid);

	UE_SLATE_LOG_ERROR_IF_FALSE(WidgetList.VerifySortOrder()
		, CVarSlateInvalidationRootVerifyHittestGrid
		, TEXT("The array's sort order for InvalidationRoot '%s' is not respected.")
		, *FReflectionMetaData::GetWidgetPath(InvalidationRootWidget));

	TArray<FHittestGrid::FWidgetSortData> WeakHittestGridSortDatas = HittestGrid->GetAllWidgetSortDatas();

	struct FHittestWidgetSortData
	{
		const SWidget* Widget;
		int64 PrimarySort;
		FSlateInvalidationWidgetSortOrder SecondarySort;
	};

	TArray<FHittestWidgetSortData, FConcurrentLinearArrayAllocator> HittestGridSortDatas;
	HittestGridSortDatas.Reserve(WeakHittestGridSortDatas.Num());

	// Widgets need to be valid in the hittestgrid
	for (const FHittestGrid::FWidgetSortData& Data : WeakHittestGridSortDatas)
	{
		TSharedPtr<SWidget> Widget = Data.WeakWidget.Pin();
		UE_SLATE_LOG_ERROR_IF_FALSE(Widget, CVarSlateInvalidationRootVerifyHittestGrid, TEXT("A widget is invalid in the HittestGrid"));

		FHittestWidgetSortData SortData = { Widget.Get(), Data.PrimarySort, Data.SecondarySort };
		HittestGridSortDatas.Add(MoveTemp(SortData));
	}

	// The order in the WidgetList is sorted. It's not the case of the HittestGrid.

	FSlateInvalidationWidgetSortOrder PreviousSecondarySort;
	const SWidget* LastWidget = nullptr;
	WidgetList.ForEachWidget([&HittestGridSortDatas, &PreviousSecondarySort, &LastWidget](const SWidget& Widget)
		{
			if (Widget.GetVisibility().IsHitTestVisible())
			{
				// Is the widget in the hittestgrid
				const int32 FoundHittestIndex = HittestGridSortDatas.IndexOfByPredicate([&Widget](const FHittestWidgetSortData& HittestGrid)
					{
						return HittestGrid.Widget == &Widget;
					});
				const bool bHasFoundWidget = HittestGridSortDatas.IsValidIndex(FoundHittestIndex);
				if (!bHasFoundWidget)
				{
					return;
				}

				UE_SLATE_LOG_ERROR_IF_FALSE(Widget.GetProxyHandle().GetWidgetSortOrder() == HittestGridSortDatas[FoundHittestIndex].SecondarySort
					, CVarSlateInvalidationRootVerifyHittestGrid
					, TEXT("The SecondarySort of widget '%s' doesn't match the SecondarySort inside the HittestGrid.")
					, *FReflectionMetaData::GetWidgetPath(Widget));

				LastWidget = &Widget;
				PreviousSecondarySort = HittestGridSortDatas[FoundHittestIndex].SecondarySort;

				HittestGridSortDatas.RemoveAtSwap(FoundHittestIndex);
			}
		});

	const int32 FoundHittestIndex = HittestGridSortDatas.IndexOfByPredicate([InvalidationRootWidget](const FHittestWidgetSortData& HittestGrid)
		{
			return HittestGrid.Widget == InvalidationRootWidget;
		});
	if (HittestGridSortDatas.IsValidIndex(FoundHittestIndex))
	{
		HittestGridSortDatas.RemoveAtSwap(FoundHittestIndex);
	}

	// New Widget could be added with an ChildOrder invalidation while Painting or SlatePrepass.
	//They wouldn't be in the WidgetList yet. Make sure a parent has the ChildOrder Invalidation (so they get repainted to the hittest grid).
	for (const FHittestWidgetSortData& HittestGridSortData : HittestGridSortDatas)
	{
		UE_SLATE_LOG_ERROR_IF_FALSE(HittestGridSortData.Widget != nullptr
			, CVarSlateInvalidationRootVerifyHittestGrid
			, TEXT("A widget in the HittestGrid is invalid and shouldn't be in the HittestGrid."));

		if (!HittestGridSortData.Widget->GetProxyHandle().IsValid(HittestGridSortData.Widget))
		{
			// Is the parent have ChildOrder invalidation
			FSlateInvalidationWidgetIndex ParentWidetIndex = FSlateInvalidationWidgetIndex::Invalid;
			SWidget* ParentWidget = HittestGridSortData.Widget->GetParentWidget().Get();
			while(ParentWidget)
			{
				if (ParentWidget->GetProxyHandle().IsValid(ParentWidget))
				{
					ParentWidetIndex = ParentWidget->GetProxyHandle().GetWidgetIndex();
					break;
				}
				ParentWidget = ParentWidget->GetParentWidget().Get();
			}

			if (ParentWidetIndex != FSlateInvalidationWidgetIndex::Invalid)
			{
				UE_SLATE_LOG_ERROR_IF_FALSE(WidgetList.IsValidIndex(ParentWidetIndex) && EnumHasAllFlags(WidgetList[ParentWidetIndex].CurrentInvalidateReason, EInvalidateWidgetReason::ChildOrder)
					, CVarSlateInvalidationRootVerifyHittestGrid
					, TEXT("The parent widget doesn't have ChildOrder invalidation. The hittest grid has an invalid widget."));
			}
		}
	}
}

void VerifyWidgetVisibility(FSlateInvalidationWidgetList& WidgetList)
{
	WidgetList.ForEachInvalidationWidget([&WidgetList](FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
		{
			if (SWidget* Widget = InvalidationWidget.GetWidget())
			{
				{
					const EVisibility WidgetVisibility = Widget->GetVisibility();
					bool bParentIsVisible = true;
					bool bParentIsCollapsed = false;

					const TSharedPtr<SWidget> ParentWidget = Widget->GetParentWidget();
					if (InvalidationWidget.ParentIndex != FSlateInvalidationWidgetIndex::Invalid)
					{
						// Confirm that we have the correct parent
						UE_SLATE_LOG_ERROR_IF_FALSE(WidgetList.IsValidIndex(InvalidationWidget.ParentIndex)
							, CVarSlateInvalidationRootVerifyWidgetVisibility
							, TEXT("Widget '%s' Parent index is invalid.")
							, *FReflectionMetaData::GetWidgetDebugInfo(Widget));

						const FSlateInvalidationWidgetList::InvalidationWidgetType& ParentInvalidationWidget = WidgetList[InvalidationWidget.ParentIndex];
						UE_SLATE_LOG_ERROR_IF_FALSE(ParentWidget.Get() == ParentInvalidationWidget.GetWidget()
							, CVarSlateInvalidationRootVerifyWidgetVisibility
							, TEXT("Widget '%s' Parent is not '%s'.")
							, *FReflectionMetaData::GetWidgetDebugInfo(Widget)
							, *FReflectionMetaData::GetWidgetDebugInfo(ParentWidget.Get()));

						bParentIsVisible = ParentInvalidationWidget.Visibility.IsVisible();
						bParentIsCollapsed = ParentInvalidationWidget.Visibility.IsCollapsed();
					}
					else
					{
						UE_SLATE_LOG_ERROR_IF_FALSE(ParentWidget == nullptr || ParentWidget->Advanced_IsInvalidationRoot()
							, CVarSlateInvalidationRootVerifyWidgetVisibility
							, TEXT("Widget '%s' Parent is valid and is not an invalidation root.")
							, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
					}

					UE_SLATE_LOG_ERROR_IF_FALSE(InvalidationWidget.Visibility.AreAncestorsVisible() == bParentIsVisible
						, CVarSlateInvalidationRootVerifyWidgetVisibility
						, TEXT("Widget '%s' AreAncestorsVisible flag is wrong.")
						, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
					UE_SLATE_LOG_ERROR_IF_FALSE(InvalidationWidget.Visibility.IsVisible() == (bParentIsVisible && WidgetVisibility.IsVisible())
						, CVarSlateInvalidationRootVerifyWidgetVisibility
						, TEXT("Widget '%s' IsVisible flag is wrong.")
						, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
					UE_SLATE_LOG_ERROR_IF_FALSE(InvalidationWidget.Visibility.IsCollapsed() == bParentIsCollapsed || WidgetVisibility == EVisibility::Collapsed
						, CVarSlateInvalidationRootVerifyWidgetVisibility
						, TEXT("Widget '%s' IsCollapsed flag is wrong.")
						, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
					UE_SLATE_LOG_ERROR_IF_FALSE(InvalidationWidget.Visibility.IsCollapseIndirectly() == bParentIsCollapsed
						, CVarSlateInvalidationRootVerifyWidgetVisibility
						, TEXT("Widget '%s' IsCollapseIndirectly flag is wrong.")
						, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
				}
				{
					if (!InvalidationWidget.Visibility.IsVisible())
					{
						UE_SLATE_LOG_ERROR_IF_FALSE(!Widget->GetPersistentState().CachedElementHandle.HasCachedElements()
							, CVarSlateInvalidationRootVerifyWidgetVisibility
							, TEXT("Widget '%s' has cached element and is not visible.")
							, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
					}
				}
				{
					// Cache last frame visibility
					InvalidationWidget.bDebug_LastFrameVisible = InvalidationWidget.Visibility.IsVisible();
					InvalidationWidget.bDebug_LastFrameVisibleSet = true;
				}
			}
		});
}

void VerifyWidgetVolatile(FSlateInvalidationWidgetList& WidgetList, TArray<FSlateInvalidationWidgetHeapElement>& FinalUpdateList)
{
	SWidget* Root = WidgetList.GetRoot().Pin().Get();
	check(Root);
	WidgetList.ForEachWidget([Root, &FinalUpdateList](SWidget& Widget)
		{
			if (&Widget != Root && GSlateInvalidationRootVerifyWidgetVolatile)
			{
				{
					const bool bWasVolatile = Widget.IsVolatile();
					Widget.CacheVolatility();
					const bool bIsVolatile = Widget.IsVolatile();
					UE_SLATE_LOG_ERROR_IF_FALSE(bWasVolatile == bIsVolatile
						, CVarSlateInvalidationRootVerifyWidgetVolatile
						, TEXT("Widget '%s' volatily changed without an invalidation.")
						, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
				}

				const TSharedPtr<const SWidget> ParentWidget = Widget.GetParentWidget();
				UE_SLATE_LOG_ERROR_IF_FALSE(ParentWidget
					, CVarSlateInvalidationRootVerifyWidgetVolatile
					, TEXT("Parent widget of widget '%s' is invalid.")
					, *FReflectionMetaData::GetWidgetDebugInfo(Widget));

				{
					const bool bShouldBeVolatileIndirectly = ParentWidget->IsVolatileIndirectly() || ParentWidget->IsVolatile();
					UE_SLATE_LOG_ERROR_IF_FALSE(Widget.IsVolatileIndirectly() == bShouldBeVolatileIndirectly
					, CVarSlateInvalidationRootVerifyWidgetVolatile
					, TEXT("Widget '%s' should be set as %s.")
					, *FReflectionMetaData::GetWidgetDebugInfo(Widget)
					, (bShouldBeVolatileIndirectly ? TEXT("volatile indirectly") : TEXT("not volatile indirectly")));
				}

				if (Widget.IsVolatile() && !Widget.IsVolatileIndirectly())
				{
					UE_SLATE_LOG_ERROR_IF_FALSE(Widget.HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePaint)
						, CVarSlateInvalidationRootVerifyWidgetVolatile
						, TEXT("Widget '%s' is volatile but doesn't have the update flag NeedsVolatilePaint.")
						, *FReflectionMetaData::GetWidgetDebugInfo(Widget));

					if (Widget.GetProxyHandle().IsValid(Widget))
					{
						const bool bIsVisible = Widget.GetProxyHandle().GetProxy().Visibility.IsVisible();
						const FSlateInvalidationWidgetIndex WidgetIndex = Widget.GetProxyHandle().GetWidgetIndex();
						const bool bIsContains = FinalUpdateList.ContainsByPredicate([WidgetIndex](const FSlateInvalidationWidgetHeapElement& Other)
							{
								return Other.GetWidgetIndex() == WidgetIndex;
							});
						UE_SLATE_LOG_ERROR_IF_FALSE(bIsContains || !bIsVisible
							, CVarSlateInvalidationRootVerifyWidgetVolatile
							, TEXT("Widget '%s' is volatile but is not in the update list.")
							, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
					}
				}
			}
		});
}

void VerifyWidgetsUpdateList_BeforeProcessPreUpdate(const TSharedRef<SWidget>& RootWidget,
	FSlateInvalidationWidgetList* FastWidgetPathList,
	FSlateInvalidationWidgetPreHeap* WidgetsNeedingPreUpdate,
	FSlateInvalidationWidgetPostHeap* WidgetsNeedingPostUpdate,
	TArray<FSlateInvalidationWidgetHeapElement>& FinalUpdateList)
{
	if (FastWidgetPathList->GetRoot().Pin() != RootWidget)
	{
		return;
	}

	for (const FSlateInvalidationWidgetHeapElement& WidgetElement : FinalUpdateList)
	{
		UE_SLATE_LOG_ERROR_IF_FALSE(FastWidgetPathList->IsValidIndex(WidgetElement.GetWidgetIndex())
			, CVarSlateInvalidationRootVerifyWidgetsUpdateList
			, TEXT("A WidgetIndex is invalid. The Widget can be invalid (because it's not been processed yet)."));
	}

	UE_SLATE_LOG_ERROR_IF_FALSE(WidgetsNeedingPreUpdate->IsValidHeap_Debug()
		, CVarSlateInvalidationRootVerifyWidgetsUpdateList
		, TEXT("The PreUpdate list need to stay a valid heap"));

	WidgetsNeedingPreUpdate->ForEachIndexes([FastWidgetPathList](const FSlateInvalidationWidgetPreHeap::FElement& Element)
		{
			UE_SLATE_LOG_ERROR_IF_FALSE(FastWidgetPathList->IsValidIndex(Element.GetWidgetIndex())
				, CVarSlateInvalidationRootVerifyWidgetsUpdateList
				, TEXT("An element is not valid."));
			const FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget = (*FastWidgetPathList)[Element.GetWidgetIndex()];
			if (SWidget* Widget = (*FastWidgetPathList)[Element.GetWidgetIndex()].GetWidget())
			{
				UE_SLATE_LOG_ERROR_IF_FALSE(Widget->GetProxyHandle().GetWidgetSortOrder() == Element.GetWidgetSortOrder()
					, CVarSlateInvalidationRootVerifyWidgetsUpdateList
					, TEXT("The sort order of the widget '%s' do not matches what is in the heap.")
					, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
				UE_SLATE_LOG_ERROR_IF_FALSE(Widget->GetProxyHandle().GetWidgetIndex() == Element.GetWidgetIndex()
					, CVarSlateInvalidationRootVerifyWidgetsUpdateList
					, TEXT("The widget index of the widget '%s' do not matches what is in the heap.")
					, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
			}
		});

	FastWidgetPathList->ForEachInvalidationWidget([WidgetsNeedingPreUpdate](FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
		{
			UE_SLATE_LOG_ERROR_IF_FALSE(WidgetsNeedingPreUpdate->Contains_Debug(InvalidationWidget.Index) == InvalidationWidget.bContainedByWidgetPreHeap
				, CVarSlateInvalidationRootVerifyWidgetsUpdateList
				, TEXT("Widget '%s' is or is not in the PreUpdate but the flag say otherwise.")
				, *FReflectionMetaData::GetWidgetDebugInfo(InvalidationWidget.GetWidget()));
		});
}

void VerifyWidgetsUpdateList_ProcessPrepassUpdate(FSlateInvalidationWidgetList* FastWidgetPathList, FSlateInvalidationWidgetPrepassHeap* WidgetsNeedingPrepassUpdate, FSlateInvalidationWidgetPostHeap* WidgetsNeedingPostUpdate)
{
	// Make sure every widget in the WidgetsNeedingPrepassUpdate is also in the WidgetsNeedingPostUpdate list.
	WidgetsNeedingPrepassUpdate->ForEachIndexes([FastWidgetPathList, WidgetsNeedingPostUpdate](const FSlateInvalidationWidgetPrepassHeap::FElement& Element)
		{
			const FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget = (*FastWidgetPathList)[Element.GetWidgetIndex()];
			const SWidget* Widget = InvalidationWidget.GetWidget();
			UE_SLATE_LOG_ERROR_IF_FALSE(WidgetsNeedingPostUpdate->Contains_Debug(Element.GetWidgetIndex())
				, CVarSlateInvalidationRootVerifyWidgetsUpdateList
				, TEXT("Widget '%s' is in the WidgetsNeedingPrepassUpdate list but not in the WidgetsNeedingPostUpdate.")
				, *FReflectionMetaData::GetWidgetDebugInfo(InvalidationWidget.GetWidget()));
		});
}

void VerifyWidgetsUpdateList_BeforeProcessPostUpdate(const TSharedRef<SWidget>& RootWidget,
	FSlateInvalidationWidgetList* FastWidgetPathList,
	FSlateInvalidationWidgetPreHeap* WidgetsNeedingPreUpdate,
	FSlateInvalidationWidgetPostHeap* WidgetsNeedingPostUpdate,
	TArray<FSlateInvalidationWidgetHeapElement>& FinalUpdateList)
{
	if (FastWidgetPathList->GetRoot().Pin() != RootWidget)
	{
		return;
	}

	UE_SLATE_LOG_ERROR_IF_FALSE(WidgetsNeedingPostUpdate->IsValidHeap_Debug()
		, CVarSlateInvalidationRootVerifyWidgetsUpdateList
		, TEXT("The PostUpdate list need to stay a valid heap"));

	// Widget in SWidget::Prepass may have added new widget.
	//This is not desired, it will introduce a frame delay compare to slow path.
	//UE_SLATE_LOG_ERROR_IF_FALSE(WidgetsNeedingPreUpdate->Num() == 0
	//	, CVarSlateInvalidationRootVerifyWidgetsUpdateList
	//	, TEXT("The PreUpdate list should be empty"));

	UE_SLATE_LOG_ERROR_IF_FALSE(FinalUpdateList.Num() == 0
		, CVarSlateInvalidationRootVerifyWidgetsUpdateList
		, TEXT("The Final Update list should be empty."));

	WidgetsNeedingPostUpdate->ForEachIndexes([FastWidgetPathList](const FSlateInvalidationWidgetPreHeap::FElement& Element)
		{
			UE_SLATE_LOG_ERROR_IF_FALSE(FastWidgetPathList->IsValidIndex(Element.GetWidgetIndex())
				, CVarSlateInvalidationRootVerifyWidgetsUpdateList
				, TEXT("An element is not valid."));

			const FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget = (*FastWidgetPathList)[Element.GetWidgetIndex()];
			const SWidget* Widget = InvalidationWidget.GetWidget();

			// The widget was valid but a Custom Prepass could have destroyed the widget.
			if (Widget)
			{
				UE_SLATE_LOG_ERROR_IF_FALSE(Widget->GetProxyHandle().GetWidgetSortOrder() == Element.GetWidgetSortOrder()
					, CVarSlateInvalidationRootVerifyWidgetsUpdateList
					, TEXT("The sort order of the widget '%s' do not matches what is in the heap.")
					, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
				UE_SLATE_LOG_ERROR_IF_FALSE(Widget->GetProxyHandle().GetWidgetIndex() == Element.GetWidgetIndex()
					, CVarSlateInvalidationRootVerifyWidgetsUpdateList
					, TEXT("The widget index of the widget '%s' do not matches what is in the heap.")
					, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
			}
		});

	FastWidgetPathList->ForEachInvalidationWidget([WidgetsNeedingPostUpdate](FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
		{
			UE_SLATE_LOG_ERROR_IF_FALSE(WidgetsNeedingPostUpdate->Contains_Debug(InvalidationWidget.Index) == InvalidationWidget.bContainedByWidgetPostHeap
				, CVarSlateInvalidationRootVerifyWidgetsUpdateList
				, TEXT("Widget '%s' is or is not in the PostUpdate but the flag say otherwise.")
				, *FReflectionMetaData::GetWidgetDebugInfo(InvalidationWidget.GetWidget()));
		});
}

void VerifyWidgetsUpdateList_AfterProcessPostUpdate(const TSharedRef<SWidget>& RootWidget,
	FSlateInvalidationWidgetList* FastWidgetPathList,
	FSlateInvalidationWidgetPreHeap* WidgetsNeedingPreUpdate,
	FSlateInvalidationWidgetPostHeap* WidgetsNeedingPostUpdate,
	TArray<FSlateInvalidationWidgetHeapElement>& FinalUpdateList)
{
	if (FastWidgetPathList->GetRoot().Pin() != RootWidget)
	{
		return;
	}

	UE_SLATE_LOG_ERROR_IF_FALSE(WidgetsNeedingPostUpdate->Num() == 0
		, CVarSlateInvalidationRootVerifyWidgetsUpdateList
		, TEXT("The list of Post Update should already been processed."));

	for(const FSlateInvalidationWidgetHeapElement& WidgetElement : FinalUpdateList)
	{
		const FSlateInvalidationWidgetIndex WidgetIndex = WidgetElement.GetWidgetIndex();
		const FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget = (*FastWidgetPathList)[WidgetIndex];
		const SWidget* Widget = InvalidationWidget.GetWidget();

		// The widget was valid but a Custom Prepass could have destroyed the widget.
		if (Widget)
		{
			UE_SLATE_LOG_ERROR_IF_FALSE(InvalidationWidget.CurrentInvalidateReason == EInvalidateWidgetReason::None
				, CVarSlateInvalidationRootVerifyWidgetsUpdateList
				, TEXT("The widget '%s' is in the update list and it still has a Invalidation Reason.")
				, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
			UE_SLATE_LOG_ERROR_IF_FALSE(Widget->HasAnyUpdateFlags(EWidgetUpdateFlags::AnyUpdate)
				, CVarSlateInvalidationRootVerifyWidgetsUpdateList
				, TEXT("The widget '%s' is in the update list but doesn't have an update flag set.")
				, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
		}
	}
}

void VerifySlateAttribute_BeforeUpdate(FSlateInvalidationWidgetList& FastWidgetPathList)
{
	FastWidgetPathList.ForEachInvalidationWidget([](FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
		{
			InvalidationWidget.bDebug_AttributeUpdated = false;
		});
}

void VerifySlateAttribute_AfterUpdate(const FSlateInvalidationWidgetList& FastWidgetPathList)
{
	const bool bElementIndexListValid = FastWidgetPathList.VerifyElementIndexList();
	UE_SLATE_LOG_ERROR_IF_FALSE(bElementIndexListValid
		, CVarSlateInvalidationRootVerifySlateAttribute
		, TEXT("The VerifySlateAttribute failed in post."));
}

void VerifyCachedElementDataList(const FSlateInvalidationRoot* InvalidationRoot)
{
	for (const TSharedPtr<FSlateCachedElementList>& CachedElementList : InvalidationRoot->GetCachedElements().GetCachedElementLists())
	{
		if (CachedElementList)
		{
			const SWidget* WidgetPtr = CachedElementList->OwningWidget;
			UE_SLATE_LOG_ERROR_IF_FALSE(WidgetPtr
				, CVarSlateInvalidationRootVerifyCachedElementDataList
				, TEXT("A widget is invalid and should not be cached."));
			if (WidgetPtr)
			{
				UE_SLATE_LOG_ERROR_IF_FALSE(WidgetPtr->GetProxyHandle().IsValid(WidgetPtr)
					, CVarSlateInvalidationRootVerifyCachedElementDataList
					, TEXT("The widget '%s' proxy is invalid and should not be cached.")
					, *FReflectionMetaData::GetWidgetDebugInfo(WidgetPtr));
				UE_SLATE_LOG_ERROR_IF_FALSE((WidgetPtr->GetProxyHandle().GetWidgetVisibility(WidgetPtr).IsVisible()
					|| WidgetPtr->GetProxyHandle().HasAllInvalidationReason(WidgetPtr, EInvalidateWidgetReason::Visibility)
					|| CachedElementList->IsEmpty())
					, CVarSlateInvalidationRootVerifyCachedElementDataList
					, TEXT("The widget '%s' proxy is not visible and should not be cached.")
					, *FReflectionMetaData::GetWidgetDebugInfo(WidgetPtr));
			}
		}
	}
}

#undef UE_SLATE_LOG_ERROR_IF_FALSE
#endif //UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING

/**
 * 
 */
FSlateInvalidationRootHandle::FSlateInvalidationRootHandle()
	: InvalidationRoot(nullptr)
	, UniqueId(INDEX_NONE)
{

}

FSlateInvalidationRootHandle::FSlateInvalidationRootHandle(int32 InUniqueId)
	: UniqueId(InUniqueId)
{
	InvalidationRoot = GSlateInvalidationRootListInstance.GetInvalidationRoot(UniqueId);
}

FSlateInvalidationRoot* FSlateInvalidationRootHandle::GetInvalidationRoot() const
{
	return GSlateInvalidationRootListInstance.GetInvalidationRoot(UniqueId);
}

