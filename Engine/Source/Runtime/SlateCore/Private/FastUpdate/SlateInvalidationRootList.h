// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FSlateInvalidationRoot;

/**
 * List of the invalidation root. Used to create unique id to retrieve them safely.
 */
class FSlateInvalidationRootList
{
public:
	int32 AddInvalidationRoot(FSlateInvalidationRoot* InvalidationRoot)
	{
		++GenerationNumber;
		check(GenerationNumber != INDEX_NONE);
		InvalidationRoots.Add(GenerationNumber, InvalidationRoot);
		return GenerationNumber;
	}

	void RemoveInvalidationRoot(int32 Id)
	{
		InvalidationRoots.Remove(Id);
	}

	const FSlateInvalidationRoot* GetInvalidationRoot(int32 UniqueId) const
	{
		if (UniqueId != INDEX_NONE)
		{
			const FSlateInvalidationRoot* const * FoundElement = InvalidationRoots.Find(UniqueId);
			return FoundElement ? *FoundElement : nullptr;
		}
		return nullptr;
	}

	FSlateInvalidationRoot* GetInvalidationRoot(int32 UniqueId)
	{
		return const_cast<FSlateInvalidationRoot*>(const_cast<const FSlateInvalidationRootList*>(this)->GetInvalidationRoot(UniqueId));
	}
	
	const TMap<int32, FSlateInvalidationRoot*>& GetInvalidationRoots() const { return InvalidationRoots; }

private:
	TMap<int32, FSlateInvalidationRoot*> InvalidationRoots;
	int32 GenerationNumber = INDEX_NONE;
};

extern FSlateInvalidationRootList GSlateInvalidationRootListInstance;
