// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastUpdate/SlateInvalidationWidgetHeap.h"

namespace UE
{
namespace Slate
{
namespace Private
{
#if UE_SLATE_WITH_INVALIDATIONWIDGETHEAP_DEBUGGING
	bool GSlateInvalidationWidgetHeapVerifyWidgetContains = false;
	static FAutoConsoleVariableRef CVarSlateInvalidationWidgetHeapVerifyWidgetContains(
		TEXT("Slate.InvalidationRoot.VerifyWidgetHeapContains"),
		GSlateInvalidationWidgetHeapVerifyWidgetContains,
		TEXT("Verify that the widget is not already in the list before adding it.")
	);
#endif

} // Private
} // Slate
} // UE
