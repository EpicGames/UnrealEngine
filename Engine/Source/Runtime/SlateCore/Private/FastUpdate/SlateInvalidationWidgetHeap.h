// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "FastUpdate/SlateInvalidationWidgetIndex.h"
#include "FastUpdate/SlateInvalidationWidgetList.h"
#include "FastUpdate/SlateInvalidationWidgetSortOrder.h"

#ifndef UE_SLATE_WITH_INVALIDATIONWIDGETHEAP_DEBUGGING
	#define UE_SLATE_WITH_INVALIDATIONWIDGETHEAP_DEBUGGING DO_CHECK
#endif

namespace UE::Slate::Private
{
#if UE_SLATE_WITH_INVALIDATIONWIDGETHEAP_DEBUGGING
	extern bool GSlateInvalidationWidgetHeapVerifyWidgetContains;
#endif
} // namespace

/**
 * Heap of widget that is ordered by increasing sort order. The list need to always stay ordered.
 */
class FSlateInvalidationWidgetPreHeap
{
public:
	using FElement = FSlateInvalidationWidgetHeapElement;
	struct FWidgetOrderLess
	{
		FORCEINLINE bool operator()(const FElement& A, const FElement& B) const
		{
			return A.GetWidgetSortOrder() < B.GetWidgetSortOrder();
		}
	};
	using SortPredicate = FWidgetOrderLess;
	
	static constexpr int32 NumberOfPreAllocatedElement = 32;
	using FElementContainer = TArray<FElement, TInlineAllocator<NumberOfPreAllocatedElement>>;

	FSlateInvalidationWidgetPreHeap(FSlateInvalidationWidgetList& InOwnerList)
		: OwnerList(InOwnerList)
	{ }

public:
	/** Insert into the list at the proper order (see binary heap) only if it's not already contains by the list. */
	void HeapPushUnique(FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
	{
		check(InvalidationWidget.Index != FSlateInvalidationWidgetIndex::Invalid);
		VerifyContainsFlag(InvalidationWidget);

		if (!InvalidationWidget.bContainedByWidgetPreHeap)
		{
			InvalidationWidget.bContainedByWidgetPreHeap = true;
			Heap.HeapPush(FElement{ InvalidationWidget.Index, FSlateInvalidationWidgetSortOrder{OwnerList, InvalidationWidget.Index} }, SortPredicate());
		}
	}

	/** Returns and removes the biggest WidgetIndex from the list. */
	[[nodiscard]] FSlateInvalidationWidgetIndex HeapPop()
	{
		check(Heap.Num() > 0);
		FSlateInvalidationWidgetIndex Result = Heap.HeapTop().GetWidgetIndex();
		Heap.HeapPopDiscard(SortPredicate(), EAllowShrinking::No);
		OwnerList[Result].bContainedByWidgetPreHeap = false;
		return Result;
	}

	/** Removes the biggest WidgetIndex from the list. */
	void HeapPopDiscard()
	{
		check(Heap.Num() > 0);
		FSlateInvalidationWidgetIndex Result = Heap.HeapTop().GetWidgetIndex();
		Heap.HeapPopDiscard(SortPredicate(), EAllowShrinking::No);
		OwnerList[Result].bContainedByWidgetPreHeap = false;
	}
	
	/** Returns the biggest WidgetIndex from the list. */
	[[nodiscard]] inline FSlateInvalidationWidgetIndex HeapPeek() const
	{
		check(Heap.Num() > 0);
		return Heap.HeapTop().GetWidgetIndex();
	}
	
	/** Returns the biggest WidgetIndex from the list. */
	[[nodiscard]] inline const FElement& HeapPeekElement() const
	{
		check(Heap.Num() > 0);
		return Heap.HeapTop();
	}

	/** Remove range */
	int32 RemoveRange(const FSlateInvalidationWidgetList::FIndexRange& Range)
	{
		int32 RemoveCount = 0;
		for (int32 Index = 0; Index < Heap.Num(); ++Index)
		{
			if (Range.Include(Heap[Index].GetWidgetSortOrder()))
			{
				const FSlateInvalidationWidgetIndex WidgetIndex = Heap[Index].GetWidgetIndex();
				OwnerList[WidgetIndex].bContainedByWidgetPreHeap = false;
				Heap.HeapRemoveAt(Index, SortPredicate());
				Index = INDEX_NONE; // start again
			}
		}
		return RemoveCount;
	}

	/** Empties the list, but doesn't change memory allocations. */
	void Reset(bool bResetContained)
	{
		if (bResetContained)
		{
			for (const FElement& Element : Heap)
			{
				OwnerList[Element.GetWidgetIndex()].bContainedByWidgetPreHeap = false;
			}
		}

		// Destroy the second allocator, if it exists.
		Heap.Empty(NumberOfPreAllocatedElement);
	}

	/** Returns the number of elements in the list. */
	[[nodiscard]] inline int32 Num() const
	{
		return Heap.Num();
	}

	/** Does it contains the widget index. */
	[[nodiscard]] bool Contains_Debug(const FSlateInvalidationWidgetIndex WidgetIndex) const
	{
		return Heap.ContainsByPredicate([WidgetIndex](const FElement& Element)
			{
				return Element.GetWidgetIndex() == WidgetIndex;
			});
	}

	/** @retuns true if the heap is heapified. */
	[[nodiscard]] inline bool IsValidHeap_Debug()
	{
		return Algo::IsHeap(Heap, SortPredicate());
	}

	/** Returns the raw list. */
	[[nodiscard]] inline const FElementContainer& GetRaw() const
	{
		return Heap;
	}

	/** Iterate over each element in the list in any order. */
	template<typename Predicate>
	void ForEachIndexes(Predicate Pred)
	{
		for (FElement& Element : Heap)
		{
			Pred(Element);
		}
	}

private:
	void VerifyContainsFlag(const FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
	{
#if UE_SLATE_WITH_INVALIDATIONWIDGETHEAP_DEBUGGING
		if (UE::Slate::Private::GSlateInvalidationWidgetHeapVerifyWidgetContains)
		{
			check(Contains_Debug(InvalidationWidget.Index) == InvalidationWidget.bContainedByWidgetPreHeap);
		}
#endif
	}

	FElementContainer Heap;
	FSlateInvalidationWidgetList& OwnerList;
};


/** */
class FSlateInvalidationWidgetPostHeap
{
public:
	using FElement = FSlateInvalidationWidgetHeapElement;
	struct FWidgetOrderGreater
	{
		FORCEINLINE bool operator()(const FElement& A, const FElement& B) const
		{
			return B.GetWidgetSortOrder() < A.GetWidgetSortOrder();
		}
	};
	using SortPredicate = FWidgetOrderGreater;
	
	static constexpr int32 NumberOfPreAllocatedElement = 100;
	using FElementContainer = TArray<FElement, TInlineAllocator<NumberOfPreAllocatedElement>>;

	FSlateInvalidationWidgetPostHeap(FSlateInvalidationWidgetList& InOwnerList)
		: OwnerList(InOwnerList)
		, WidgetCannotBeAdded(FSlateInvalidationWidgetIndex::Invalid)
		, bIsHeap(false)
	{ }

public:
	/** Insert into the list at the proper order (see binary heap) but only if it's not already contains by the list. */
	void HeapPushUnique(FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
	{
		check(bIsHeap);
		check(InvalidationWidget.Index != FSlateInvalidationWidgetIndex::Invalid);
		VerifyContainsFlag(InvalidationWidget);

		if (!InvalidationWidget.bContainedByWidgetPostHeap)
		{
			InvalidationWidget.bContainedByWidgetPostHeap = true;
			Heap.HeapPush(FElement{ InvalidationWidget.Index, FSlateInvalidationWidgetSortOrder{OwnerList, InvalidationWidget.Index} }, SortPredicate());
		}
	}

	/** Insert at the end of the list but only if it's not already contains by the list. */
	void PushBackUnique(FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
	{
		check(bIsHeap == false);
		check(InvalidationWidget.Index != FSlateInvalidationWidgetIndex::Invalid);
		VerifyContainsFlag(InvalidationWidget);

		if (!InvalidationWidget.bContainedByWidgetPostHeap)
		{
			InvalidationWidget.bContainedByWidgetPostHeap = true;
			Heap.Emplace(InvalidationWidget.Index, FSlateInvalidationWidgetSortOrder{OwnerList, InvalidationWidget.Index});
		}
	}

	/** PushBackUnique or PushHeapUnique depending if the list is Heapified. */
	void PushBackOrHeapUnique(FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
	{
		if (bIsHeap)
		{
			HeapPushUnique(InvalidationWidget);
		}
		else
		{
			PushBackUnique(InvalidationWidget);
		}
	}

	/** Returns and removes the biggest WidgetIndex from the list. */
	[[nodiscard]] FElement HeapPop()
	{
		check(bIsHeap == true);
		FElement Result = Heap.HeapTop();
		Heap.HeapPopDiscard(SortPredicate(), EAllowShrinking::No);
		OwnerList[Result.GetWidgetIndex()].bContainedByWidgetPostHeap = false;
		return Result;
	}

	/** Remove range */
	int32 RemoveRange(const FSlateInvalidationWidgetList::FIndexRange& Range)
	{
		check(bIsHeap == false);
		int32 RemoveCount = 0;
		for (int32 Index = Heap.Num() -1; Index >= 0 ; --Index)
		{
			if (Range.Include(Heap[Index].GetWidgetSortOrder()))
			{
				const FSlateInvalidationWidgetIndex WidgetIndex = Heap[Index].GetWidgetIndex();
				OwnerList[WidgetIndex].bContainedByWidgetPostHeap = false;
				Heap.RemoveAtSwap(Index);
				++RemoveCount;
			}
		}

		return RemoveCount;
	}

	/** Empties the list, but doesn't change memory allocations. */
	void Reset(bool bResetContained)
	{
		if (bResetContained)
		{
			for (const FElement& Element : Heap)
			{
				OwnerList[Element.GetWidgetIndex()].bContainedByWidgetPostHeap = false;
			}
		}

		// Destroy the second allocator, if it exists.
		Heap.Empty(NumberOfPreAllocatedElement);
		bIsHeap = false;
	}

	void Heapify()
	{
		check(!bIsHeap);
		Heap.Heapify(SortPredicate());
		bIsHeap = true;
	}

	/** Returns the number of elements in the list. */
	[[nodiscard]] inline int32 Num() const
	{
		return Heap.Num();
	}

	/** Returns the number of elements in the list. */
	[[nodiscard]] inline bool IsHeap() const
	{
		return bIsHeap;
	}

	/** Does it contains the widget index. */
	[[nodiscard]] bool Contains_Debug(const FSlateInvalidationWidgetIndex WidgetIndex) const
	{
		return Heap.ContainsByPredicate([WidgetIndex](const FElement& Element)
			{
				return Element.GetWidgetIndex() == WidgetIndex;
			});
	}

	/** @retuns true if the heap is heapified. */
	[[nodiscard]] inline bool IsValidHeap_Debug()
	{
		return Algo::IsHeap(Heap, SortPredicate());
	}

	/** Returns the raw list. */
	[[nodiscard]] inline const FElementContainer& GetRaw() const
	{
		return Heap;
	}

	template<typename Predicate>
	void ForEachIndexes(Predicate Pred)
	{
		for (FElement& Element : Heap)
		{
			Pred(Element);
		}
	}
	
public:
	struct FScopeWidgetCannotBeAdded
	{
		FScopeWidgetCannotBeAdded(FSlateInvalidationWidgetPostHeap& InHeap, FSlateInvalidationWidgetList::InvalidationWidgetType& InInvalidationWidget)
			: Heap(InHeap)
			, InvalidationWidget(InInvalidationWidget)
			, WidgetIndex(InvalidationWidget.Index)
		{
			check(!InvalidationWidget.bContainedByWidgetPostHeap
				&& Heap.WidgetCannotBeAdded == FSlateInvalidationWidgetIndex::Invalid);
			Heap.WidgetCannotBeAdded = InvalidationWidget.Index;
			InvalidationWidget.bContainedByWidgetPostHeap = true;
		}
		~FScopeWidgetCannotBeAdded()
		{
			Heap.WidgetCannotBeAdded = FSlateInvalidationWidgetIndex::Invalid;
			check(Heap.OwnerList.IsValidIndex(WidgetIndex));
			check(&Heap.OwnerList[WidgetIndex] == &InvalidationWidget);
			InvalidationWidget.bContainedByWidgetPostHeap = false;
		}
		FSlateInvalidationWidgetPostHeap& Heap;
		FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget;
		FSlateInvalidationWidgetIndex WidgetIndex;
	};

private:
	void VerifyContainsFlag(const FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
	{
#if UE_SLATE_WITH_INVALIDATIONWIDGETHEAP_DEBUGGING
		if (UE::Slate::Private::GSlateInvalidationWidgetHeapVerifyWidgetContains)
		{
			check(Contains_Debug(InvalidationWidget.Index) == InvalidationWidget.bContainedByWidgetPostHeap
				|| WidgetCannotBeAdded == InvalidationWidget.Index);
		}
#endif
	}

	FElementContainer Heap;
	FSlateInvalidationWidgetList& OwnerList;
	FSlateInvalidationWidgetIndex WidgetCannotBeAdded;
	bool bIsHeap;
};

/**
 * Heap of widget, ordered by increasing sort order, of the widgets that need a SlatePrepass.
 * Since a SlatePrepass of a parent widget will call the SlatePrepass of a child, only the parent will be in the list.
 */
class FSlateInvalidationWidgetPrepassHeap
{
public:
	using FElement = FSlateInvalidationWidgetHeapElement;
	struct FWidgetOrderGreater
	{
		FORCEINLINE bool operator()(const FElement& A, const FElement& B) const
		{
			return A.GetWidgetSortOrder() < B.GetWidgetSortOrder();
		}
	};
	using SortPredicate = FWidgetOrderGreater;

	static constexpr int32 NumberOfPreAllocatedElement = 32;
	using FElementContainer = TArray<FElement, TInlineAllocator<NumberOfPreAllocatedElement>>;

	FSlateInvalidationWidgetPrepassHeap(FSlateInvalidationWidgetList& InOwnerList)
		: OwnerList(InOwnerList)
		, bIsHeap(false)
	{ }

public:
	/** Insert into the list at the proper order (see binary heap) but only if it's not already contains by the list. */
	void HeapPushUnique(FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
	{
		check(bIsHeap);
		check(InvalidationWidget.Index != FSlateInvalidationWidgetIndex::Invalid);
		VerifyContainsFlag(InvalidationWidget);

		if (!InvalidationWidget.bContainedByWidgetPrepassList)
		{
			InvalidationWidget.bContainedByWidgetPrepassList = true;
			Heap.HeapPush(FElement{ InvalidationWidget.Index, FSlateInvalidationWidgetSortOrder{OwnerList, InvalidationWidget.Index} }, SortPredicate());
		}
	}

	/** Insert at the end of the list but only if it's not already contains by the list. */
	void PushBackUnique(FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
	{
		check(bIsHeap == false);
		check(InvalidationWidget.Index != FSlateInvalidationWidgetIndex::Invalid);
		VerifyContainsFlag(InvalidationWidget);

		if (!InvalidationWidget.bContainedByWidgetPrepassList)
		{
			InvalidationWidget.bContainedByWidgetPrepassList = true;
			Heap.Emplace(InvalidationWidget.Index, FSlateInvalidationWidgetSortOrder{ OwnerList, InvalidationWidget.Index });
		}
	}

	/** Returns and removes the biggest WidgetIndex from the list. */
	[[nodiscard]] FElement HeapPop()
	{
		check(bIsHeap == true);
		FElement Result = Heap.HeapTop();
		Heap.HeapPopDiscard(SortPredicate(), EAllowShrinking::No);
		OwnerList[Result.GetWidgetIndex()].bContainedByWidgetPrepassList = false;
		return Result;
	}

	/** Remove range */
	int32 RemoveRange(const FSlateInvalidationWidgetList::FIndexRange& Range)
	{
		check(bIsHeap == false);
		int32 RemoveCount = 0;
		for (int32 Index = Heap.Num() - 1; Index >= 0; --Index)
		{
			if (Range.Include(Heap[Index].GetWidgetSortOrder()))
			{
				const FSlateInvalidationWidgetIndex WidgetIndex = Heap[Index].GetWidgetIndex();
				OwnerList[WidgetIndex].bContainedByWidgetPrepassList = false;
				Heap.RemoveAtSwap(Index);
				++RemoveCount;
			}
		}

		return RemoveCount;
	}

	/** Empties the list, but doesn't change memory allocations. */
	void Reset(bool bResetContained)
	{
		if (bResetContained)
		{
			for (const FElement& Element : Heap)
			{
				OwnerList[Element.GetWidgetIndex()].bContainedByWidgetPrepassList = false;
			}
		}

		// Destroy the second allocator, if it exists.
		Heap.Empty(NumberOfPreAllocatedElement);
		bIsHeap = false;
	}

	void Heapify()
	{
		check(!bIsHeap);
		Heap.Heapify(SortPredicate());
		bIsHeap = true;
	}

	/** Returns the number of elements in the list. */
	[[nodiscard]] inline int32 Num() const
	{
		return Heap.Num();
	}

	/** Returns the number of elements in the list. */
	[[nodiscard]] inline bool IsHeap() const
	{
		return bIsHeap;
	}

	/** Does it contains the widget index. */
	[[nodiscard]] bool Contains_Debug(const FSlateInvalidationWidgetIndex WidgetIndex) const
	{
		return Heap.ContainsByPredicate([WidgetIndex](const FElement& Element)
			{
				return Element.GetWidgetIndex() == WidgetIndex;
			});
	}

	/** @retuns true if the heap is heapified. */
	[[nodiscard]] inline bool IsValidHeap_Debug()
	{
		return Algo::IsHeap(Heap, SortPredicate());
	}

	/** Returns the raw list. */
	[[nodiscard]] inline const FElementContainer& GetRaw() const
	{
		return Heap;
	}

	template<typename Predicate>
	void ForEachIndexes(Predicate Pred)
	{
		for (FElement& Element : Heap)
		{
			Pred(Element);
		}
	}

private:
	void VerifyContainsFlag(const FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget)
	{
#if UE_SLATE_WITH_INVALIDATIONWIDGETHEAP_DEBUGGING
		if (UE::Slate::Private::GSlateInvalidationWidgetHeapVerifyWidgetContains)
		{
			check(Contains_Debug(InvalidationWidget.Index) == InvalidationWidget.bContainedByWidgetPrepassList);
		}
#endif
	}

	FElementContainer Heap;
	FSlateInvalidationWidgetList& OwnerList;
	bool bIsHeap;
};
