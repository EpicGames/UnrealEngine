// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastUpdate/SlateInvalidationWidgetIndex.h"

#include <limits>

const FSlateInvalidationWidgetIndex FSlateInvalidationWidgetIndex::Invalid = { std::numeric_limits<FSlateInvalidationWidgetIndex::IndexType>::max(), std::numeric_limits<FSlateInvalidationWidgetIndex::IndexType>::max() };

