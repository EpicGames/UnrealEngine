// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastUpdate/SlateInvalidationWidgetList.h"

#include "FastUpdate/SlateInvalidationRoot.h"
#include "FastUpdate/SlateInvalidationRootHandle.h"
#include "FastUpdate/WidgetProxy.h"

#include "Algo/Unique.h"
#include "Layout/Children.h"
#include "Misc/StringBuilder.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Templates/Tuple.h"
#include "Types/ReflectionMetadata.h"
#include "Types/SlateAttributeMetaData.h"

#include <limits>


DECLARE_CYCLE_STAT(TEXT("WidgetList ChildOrder Invalidation"), STAT_WidgetList_ProcessChildOrderInvalidation, STATGROUP_Slate);

#define UE_SLATE_WITH_WIDGETLIST_UPDATEONLYWHATISNEEDED 0

// See FSlateInvalidationWidgetSortOrder::FSlateInvalidationWidgetSortOrder
int32 FSlateInvalidationWidgetList::FArguments::MaxPreferedElementsNum = (1 << 10) - 1;
int32 FSlateInvalidationWidgetList::FArguments::MaxSortOrderPaddingBetweenArray = (1 << 22) - 1;

/** */
FSlateInvalidationWidgetIndex FSlateInvalidationWidgetList::IProcessChildOrderInvalidationCallback::FReIndexOperation::ReIndex(FSlateInvalidationWidgetIndex Index) const
{
	if (Index.ArrayIndex == Range.GetInclusiveMaxWidgetIndex().ArrayIndex)
	{
		check(Range.GetInclusiveMinWidgetIndex().ArrayIndex == Range.GetInclusiveMaxWidgetIndex().ArrayIndex);
		const IndexType NewElementIndex = Index.ElementIndex - Range.GetInclusiveMinWidgetIndex().ElementIndex + ReIndexTarget.ElementIndex;
		return { ReIndexTarget.ArrayIndex, NewElementIndex };
	}
	return Index;
}


/** */
FSlateInvalidationWidgetList::FWidgetAttributeIterator::FWidgetAttributeIterator(const FSlateInvalidationWidgetList& InWidgetList)
	: WidgetList(InWidgetList)
	, CurrentWidgetIndex(FSlateInvalidationWidgetIndex::Invalid)
	, CurrentWidgetSortOrder(FSlateInvalidationWidgetSortOrder::LimitMax())
	, AttributeIndex(0)
	, MoveToWidgetIndexOnNextAdvance(FSlateInvalidationWidgetIndex::Invalid)
	, bNeedsWidgetFixUp(false)
{
	++WidgetList.NumberOfLock;

	int32 ArrayIndex = WidgetList.FirstArrayIndex;
	while(ArrayIndex != INDEX_NONE)
	{
		check(WidgetList.Data.IsValidIndex(ArrayIndex));

		const FArrayNode& ArrayNode = WidgetList.Data[ArrayIndex];
		if (ArrayNode.ElementIndexList_WidgetWithRegisteredSlateAttribute.Num() > 0)
		{
			CurrentWidgetIndex = FSlateInvalidationWidgetIndex{(IndexType)ArrayIndex, ArrayNode.ElementIndexList_WidgetWithRegisteredSlateAttribute[0]};
			CurrentWidgetSortOrder = FSlateInvalidationWidgetSortOrder{InWidgetList, CurrentWidgetIndex};
			break;
		}
		ArrayIndex = ArrayNode.NextArrayIndex;
	}
}


FSlateInvalidationWidgetList::FWidgetAttributeIterator::~FWidgetAttributeIterator()
{
	--WidgetList.NumberOfLock;
	check(WidgetList.NumberOfLock >= 0);
}


void FSlateInvalidationWidgetList::FWidgetAttributeIterator::PreChildRemove(const FIndexRange& Range)
{
	// The current widget will be removed or rebuild. Attributes will be updated in the Building loop.
	//Move to the last item (that item, may be re-indexed soon)
	if (CurrentWidgetSortOrder <= Range.GetInclusiveMaxWidgetSortOrder())
	{
		MoveToWidgetIndexOnNextAdvance = WidgetList.IncrementIndex(Range.GetInclusiveMaxWidgetIndex());
		bNeedsWidgetFixUp = true;
	}
}


void FSlateInvalidationWidgetList::FWidgetAttributeIterator::ReIndexed(const IProcessChildOrderInvalidationCallback::FReIndexOperation& Operation)
{
	if (MoveToWidgetIndexOnNextAdvance != FSlateInvalidationWidgetIndex::Invalid)
	{
		if (Operation.GetRange().Include(FSlateInvalidationWidgetSortOrder{ WidgetList, MoveToWidgetIndexOnNextAdvance }))
		{
			MoveToWidgetIndexOnNextAdvance = Operation.ReIndex(MoveToWidgetIndexOnNextAdvance);
		}
	}
	else if (CurrentWidgetIndex != FSlateInvalidationWidgetIndex::Invalid)
	{
		CurrentWidgetIndex = Operation.ReIndex(CurrentWidgetIndex);
		CurrentWidgetSortOrder = FSlateInvalidationWidgetSortOrder{ WidgetList, CurrentWidgetIndex };
	}
}


void FSlateInvalidationWidgetList::FWidgetAttributeIterator::PostResort()
{
	if (MoveToWidgetIndexOnNextAdvance != FSlateInvalidationWidgetIndex::Invalid && CurrentWidgetIndex != FSlateInvalidationWidgetIndex::Invalid)
	{
		CurrentWidgetSortOrder = FSlateInvalidationWidgetSortOrder{ WidgetList, CurrentWidgetIndex };
	}
}


void FSlateInvalidationWidgetList::FWidgetAttributeIterator::ProxiesBuilt(const FIndexRange& Range)
{
	// The last built item is already updated. We want to update the following item next.
	MoveToWidgetIndexOnNextAdvance = WidgetList.IncrementIndex(Range.GetInclusiveMaxWidgetIndex());
	bNeedsWidgetFixUp = true;
}


void FSlateInvalidationWidgetList::FWidgetAttributeIterator::FixCurrentWidgetIndex()
{
	//MoveToWidgetIndexOnNextAdvance is to be consider as a valid Attribute
	if (MoveToWidgetIndexOnNextAdvance != FSlateInvalidationWidgetIndex::Invalid)
	{
		const int32 ArrayIndex = MoveToWidgetIndexOnNextAdvance.ArrayIndex;
		check(WidgetList.Data.IsValidIndex(ArrayIndex));

		const FArrayNode& ArrayNode = WidgetList.Data[ArrayIndex];
		AttributeIndex = ArrayNode.ElementIndexList_WidgetWithRegisteredSlateAttribute.FindLowerBound(MoveToWidgetIndexOnNextAdvance.ElementIndex);
		if (AttributeIndex == INDEX_NONE)
		{
			int32 NextArrayIndex = ArrayNode.NextArrayIndex;
			AdvanceArrayIndex(NextArrayIndex);
		}
		else
		{
			CurrentWidgetIndex = FSlateInvalidationWidgetIndex{ (IndexType)ArrayIndex, ArrayNode.ElementIndexList_WidgetWithRegisteredSlateAttribute[AttributeIndex] };
			CurrentWidgetSortOrder = FSlateInvalidationWidgetSortOrder{ WidgetList, CurrentWidgetIndex };
		}
	}
	else if (bNeedsWidgetFixUp)
	{
		Clear();
	}

	MoveToWidgetIndexOnNextAdvance = FSlateInvalidationWidgetIndex::Invalid;
	bNeedsWidgetFixUp = false;
}


void FSlateInvalidationWidgetList::FWidgetAttributeIterator::Seek(FSlateInvalidationWidgetIndex SeekTo)
{
	check(SeekTo != FSlateInvalidationWidgetIndex::Invalid);
	check(WidgetList.Data.IsValidIndex(SeekTo.ArrayIndex));

	const FArrayNode& ArrayNode = WidgetList.Data[SeekTo.ArrayIndex];
	AttributeIndex = ArrayNode.ElementIndexList_WidgetWithRegisteredSlateAttribute.FindLowerBound(SeekTo.ElementIndex);
	if (AttributeIndex == INDEX_NONE)
	{
		int32 NextArrayIndex = ArrayNode.NextArrayIndex;
		AdvanceArrayIndex(NextArrayIndex);
	}
	else
	{
		CurrentWidgetIndex = FSlateInvalidationWidgetIndex{ (IndexType)SeekTo.ArrayIndex, ArrayNode.ElementIndexList_WidgetWithRegisteredSlateAttribute[AttributeIndex] };
		CurrentWidgetSortOrder = FSlateInvalidationWidgetSortOrder{ WidgetList, CurrentWidgetIndex };
		check(WidgetList.IsValidIndex(CurrentWidgetIndex));
	}
}


void FSlateInvalidationWidgetList::FWidgetAttributeIterator::Advance()
{
	check(WidgetList.Data.IsValidIndex(CurrentWidgetIndex.ArrayIndex));

	++AttributeIndex;

	const FArrayNode& ArrayNode = WidgetList.Data[CurrentWidgetIndex.ArrayIndex];
	if (AttributeIndex >= ArrayNode.ElementIndexList_WidgetWithRegisteredSlateAttribute.Num())
	{
		int32 ArrayIndex = ArrayNode.NextArrayIndex;
		AdvanceArrayIndex(ArrayIndex);
	}
	else
	{
		int32 ArrayIndex = CurrentWidgetIndex.ArrayIndex;
		CurrentWidgetIndex = FSlateInvalidationWidgetIndex{ (IndexType)ArrayIndex, ArrayNode.ElementIndexList_WidgetWithRegisteredSlateAttribute[AttributeIndex] };
		CurrentWidgetSortOrder = (CurrentWidgetIndex != FSlateInvalidationWidgetIndex::Invalid)
			? FSlateInvalidationWidgetSortOrder{ WidgetList, CurrentWidgetIndex }
		: FSlateInvalidationWidgetSortOrder::LimitMax();
	}
}


void FSlateInvalidationWidgetList::FWidgetAttributeIterator::AdvanceToNextSibling()
{
	check(MoveToWidgetIndexOnNextAdvance == FSlateInvalidationWidgetIndex::Invalid);

	const FSlateInvalidationWidgetIndex SeekToIndex = WidgetList.IncrementIndex(WidgetList[CurrentWidgetIndex].LeafMostChildIndex);
	if (SeekToIndex != FSlateInvalidationWidgetIndex::Invalid)
	{
		Seek(SeekToIndex);
	}
	else
	{
		Clear();
	}
}


void FSlateInvalidationWidgetList::FWidgetAttributeIterator::AdvanceToNextParent()
{
	check(MoveToWidgetIndexOnNextAdvance == FSlateInvalidationWidgetIndex::Invalid);

	const FSlateInvalidationWidgetIndex ParentIndex = WidgetList[CurrentWidgetIndex].ParentIndex;
	if (ParentIndex != FSlateInvalidationWidgetIndex::Invalid)
	{
		const FSlateInvalidationWidgetIndex SeekToIndex = WidgetList.IncrementIndex(WidgetList[ParentIndex].LeafMostChildIndex);
		if (SeekToIndex != FSlateInvalidationWidgetIndex::Invalid)
		{
			Seek(SeekToIndex);
		}
		else
		{
			Clear();
		}
	}
	else
	{
		Clear();
	}
}


void FSlateInvalidationWidgetList::FWidgetAttributeIterator::AdvanceArrayIndex(int32 ArrayIndex)
{
	Clear();

	while (ArrayIndex != INDEX_NONE)
	{
		check(WidgetList.Data.IsValidIndex(ArrayIndex));

		const FArrayNode& NewArrayNode = WidgetList.Data[ArrayIndex];
		if (NewArrayNode.ElementIndexList_WidgetWithRegisteredSlateAttribute.Num() > 0)
		{
			CurrentWidgetIndex = FSlateInvalidationWidgetIndex{ (IndexType)ArrayIndex, NewArrayNode.ElementIndexList_WidgetWithRegisteredSlateAttribute[0] };
			CurrentWidgetSortOrder = FSlateInvalidationWidgetSortOrder{ WidgetList, CurrentWidgetIndex };
			break;
		}
		ArrayIndex = NewArrayNode.NextArrayIndex;
	}
}


void FSlateInvalidationWidgetList::FWidgetAttributeIterator::Clear()
{
	AttributeIndex = 0;
	CurrentWidgetIndex = FSlateInvalidationWidgetIndex::Invalid;
	CurrentWidgetSortOrder = FSlateInvalidationWidgetSortOrder::LimitMax();
}


/** */
FSlateInvalidationWidgetList::FWidgetVolatileUpdateIterator::FWidgetVolatileUpdateIterator(const FSlateInvalidationWidgetList& InWidgetList, bool bInSkipCollapsed)
	: WidgetList(InWidgetList)
	, CurrentWidgetIndex(FSlateInvalidationWidgetIndex::Invalid)
	, AttributeIndex(0)
	, bSkipCollapsed(bInSkipCollapsed)
{
	AdvanceArray(WidgetList.FirstArrayIndex);
	if (bSkipCollapsed)
	{
		SkipToNextExpend();
	}
}

void FSlateInvalidationWidgetList::FWidgetVolatileUpdateIterator::Advance()
{
	Internal_Advance();
	if (bSkipCollapsed)
	{
		SkipToNextExpend();
	}
}


void FSlateInvalidationWidgetList::FWidgetVolatileUpdateIterator::Internal_Advance()
{
	check(WidgetList.Data.IsValidIndex(CurrentWidgetIndex.ArrayIndex));

	++AttributeIndex;

	const FArrayNode& ArrayNode = WidgetList.Data[CurrentWidgetIndex.ArrayIndex];
	if (AttributeIndex >= ArrayNode.ElementIndexList_VolatileUpdateWidget.Num())
	{
		AttributeIndex = 0;
		CurrentWidgetIndex = FSlateInvalidationWidgetIndex::Invalid;
		AdvanceArray(ArrayNode.NextArrayIndex);
	}
	else
	{
		int32 ArrayIndex = CurrentWidgetIndex.ArrayIndex;
		CurrentWidgetIndex = FSlateInvalidationWidgetIndex{ (IndexType)ArrayIndex, ArrayNode.ElementIndexList_VolatileUpdateWidget[AttributeIndex] };;
	}
}


void FSlateInvalidationWidgetList::FWidgetVolatileUpdateIterator::SkipToNextExpend()
{
	while (CurrentWidgetIndex != FSlateInvalidationWidgetIndex::Invalid && WidgetList[CurrentWidgetIndex].Visibility.IsCollapsed())
	{
		const FSlateInvalidationWidgetIndex SeekToIndex = WidgetList.IncrementIndex(WidgetList[CurrentWidgetIndex].LeafMostChildIndex);
		if (SeekToIndex != FSlateInvalidationWidgetIndex::Invalid)
		{
			Seek(SeekToIndex);
		}
		else
		{
			AttributeIndex = 0;
			CurrentWidgetIndex = FSlateInvalidationWidgetIndex::Invalid;
			break;
		}
	}
}


void FSlateInvalidationWidgetList::FWidgetVolatileUpdateIterator::Seek(FSlateInvalidationWidgetIndex SeekTo)
{
	check(SeekTo != FSlateInvalidationWidgetIndex::Invalid);
	check(WidgetList.Data.IsValidIndex(SeekTo.ArrayIndex));

	const FArrayNode& ArrayNode = WidgetList.Data[SeekTo.ArrayIndex];
	AttributeIndex = ArrayNode.ElementIndexList_VolatileUpdateWidget.FindLowerBound(SeekTo.ElementIndex);
	if (AttributeIndex == INDEX_NONE)
	{
		AttributeIndex = 0;
		CurrentWidgetIndex = FSlateInvalidationWidgetIndex::Invalid;
		AdvanceArray(ArrayNode.NextArrayIndex);
	}
	else
	{
		CurrentWidgetIndex = FSlateInvalidationWidgetIndex{ (IndexType)SeekTo.ArrayIndex, ArrayNode.ElementIndexList_VolatileUpdateWidget[AttributeIndex] };
	}
}


void FSlateInvalidationWidgetList::FWidgetVolatileUpdateIterator::AdvanceArray(int32 ArrayIndex)
{
	while (ArrayIndex != INDEX_NONE)
	{
		check(WidgetList.Data.IsValidIndex(ArrayIndex));

		const FArrayNode& NewArrayNode = WidgetList.Data[ArrayIndex];
		if (NewArrayNode.ElementIndexList_VolatileUpdateWidget.Num() > 0)
		{
			CurrentWidgetIndex = FSlateInvalidationWidgetIndex{ (IndexType)ArrayIndex, NewArrayNode.ElementIndexList_VolatileUpdateWidget[0] };
			break;
		}
		ArrayIndex = NewArrayNode.NextArrayIndex;
	}
}


/** */
void FSlateInvalidationWidgetList::FArrayNode::RemoveElementIndexBiggerOrEqualThan(IndexType ElementIndex)
{
	auto RemoveLowerBound = [ElementIndex](WidgetListType& ListType)
	{
		const int32 FoundIndex = ListType.FindLowerBound(ElementIndex);
		if (ListType.IsValidIndex(FoundIndex))
		{
			ListType.RemoveAt(FoundIndex, ListType.Num() - FoundIndex);
		}
	};
	RemoveLowerBound(ElementIndexList_WidgetWithRegisteredSlateAttribute);
	RemoveLowerBound(ElementIndexList_VolatileUpdateWidget);
}

void FSlateInvalidationWidgetList::FArrayNode::RemoveElementIndexBetweenOrEqualThan(IndexType StartElementIndex, IndexType EndElementIndex)
{
	auto RemoveBound = [StartElementIndex, EndElementIndex](WidgetListType& ListType)
	{
		const int32 StartFoundIndex = ListType.FindLowerBound(StartElementIndex);
		if (ListType.IsValidIndex(StartFoundIndex))
		{
			const int32 EndFoundIndex = ListType.FindUpperBound(EndElementIndex);
			if (ListType.IsValidIndex(EndFoundIndex))
			{
				ListType.RemoveAt(StartFoundIndex, EndFoundIndex - StartFoundIndex);
			}
			else
			{
				ListType.RemoveAt(StartFoundIndex, ListType.Num() - StartFoundIndex);
			}
		}
	};
	RemoveBound(ElementIndexList_WidgetWithRegisteredSlateAttribute);
	RemoveBound(ElementIndexList_VolatileUpdateWidget);
}


/** */
FSlateInvalidationWidgetList::FSlateInvalidationWidgetList(FSlateInvalidationRootHandle InOwner, const FArguments& Args)
	: Owner(InOwner)
	, WidgetListConfig(Args)
{
	if (WidgetListConfig.PreferedElementsNum <= 1
		|| WidgetListConfig.PreferedElementsNum > FArguments::MaxPreferedElementsNum
		 || WidgetListConfig.SortOrderPaddingBetweenArray <= WidgetListConfig.PreferedElementsNum
		 || WidgetListConfig.SortOrderPaddingBetweenArray > FArguments::MaxSortOrderPaddingBetweenArray)
	{
		ensureMsgf(false, TEXT("The PreferedElementsNum or SortOrderPaddingBetweenArray have incorrect values. '%d,%d'. Reset to default value.")
			, WidgetListConfig.PreferedElementsNum
			, WidgetListConfig.SortOrderPaddingBetweenArray);
		const_cast<FArguments&>(WidgetListConfig).PreferedElementsNum = FArguments().PreferedElementsNum;
		const_cast<FArguments&>(WidgetListConfig).SortOrderPaddingBetweenArray = FArguments().SortOrderPaddingBetweenArray;
	}
}


FSlateInvalidationWidgetIndex FSlateInvalidationWidgetList::Internal_BuildWidgetList_Recursive(SWidget& Widget, FSlateInvalidationWidgetIndex ParentIndex, IndexType& LastestIndex, FSlateInvalidationWidgetVisibility ParentVisibility, bool bParentVolatile)
{
	const bool bIsEmpty = IsEmpty();
	const FSlateInvalidationWidgetIndex NewIndex = EmplaceInsertAfter(LastestIndex, Widget);
	LastestIndex = NewIndex.ArrayIndex;

	FSlateInvalidationWidgetIndex LeafMostChildIndex = NewIndex;
	FSlateInvalidationWidgetVisibility NewVisibility {ParentVisibility, Widget.GetVisibility()};

	if (ShouldBeAddedToAttributeList(Widget))
	{
		// The list is already sorted at this point. Add to the end.
		Data[LastestIndex].ElementIndexList_WidgetWithRegisteredSlateAttribute.AddUnsorted(NewIndex.ElementIndex);

		// Update attributes now because the visibility flag may change
		{
			if (!NewVisibility.IsCollapseIndirectly())
			{
				FSlateAttributeMetaData::UpdateOnlyVisibilityAttributes(Widget, FSlateAttributeMetaData::EInvalidationPermission::DenyAndClearDelayedInvalidation);
				NewVisibility.SetVisibility(ParentVisibility, Widget.GetVisibility());
			}
		}
	}

	if (ShouldBeAddedToVolatileUpdateList(Widget))
	{
		// The list is already sorted at this point. Add to the end.
		Data[LastestIndex].ElementIndexList_VolatileUpdateWidget.AddUnsorted(NewIndex.ElementIndex);
	}

	const bool bIsInvalidationRoot = Widget.Advanced_IsInvalidationRoot();

	{
		InvalidationWidgetType& WidgetProxy = (*this)[NewIndex];
		WidgetProxy.Index = NewIndex;
		WidgetProxy.ParentIndex = ParentIndex;
		WidgetProxy.LeafMostChildIndex = LeafMostChildIndex;
		WidgetProxy.Visibility = NewVisibility;
		WidgetProxy.bIsInvalidationRoot = bIsInvalidationRoot;
		WidgetProxy.bIsVolatilePrepass = Widget.HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePrepass);
	}

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	if (WidgetListConfig.bAssignedWidgetIndex)
#endif
	{
		const FSlateInvalidationWidgetSortOrder SortIndex = { *this, NewIndex };
		Widget.SetFastPathProxyHandle(FWidgetProxyHandle{ Owner, NewIndex, SortIndex }, NewVisibility, bParentVolatile);
	}

	const bool bParentOrSelfVolatile = bParentVolatile || Widget.IsVolatile();

	// N.B. The SInvalidationBox needs a valid Proxy to decide if it's a root or not.
	//const bool bDoRecursion = ShouldDoRecursion(Widget);
	const bool bDoRecursion = !bIsInvalidationRoot || bIsEmpty;
	if (bDoRecursion)
	{
		Widget.GetAllChildren()->ForEachWidget([&, this](SWidget& NewWidget)
			{
				const bool bShouldAdd = ShouldBeAdded(NewWidget);
				if (bShouldAdd)
				{
					if (NewWidget.GetParentWidget().Get() != &Widget)
					{
						TStringBuilder<512> Builder;
						Builder << TEXT("The widget '") << FReflectionMetaData::GetWidgetDebugInfo(NewWidget) << TEXT("' has the widget '")
						<< FReflectionMetaData::GetWidgetPath(NewWidget.GetParentWidget().Get()) << TEXT("' as parent, but is also a child of widget '")
						<< FReflectionMetaData::GetWidgetPath(Widget) << ("' This is not supported and may eventually crash. Please fix.");
#if UE_BUILD_SHIPPING
						UE_LOGF(LogSlate, Error, "%ls", Builder.ToString());
#else
						UE_LOGF(LogSlate, Fatal, "%ls", Builder.ToString());
#endif
					}
					LeafMostChildIndex = Internal_BuildWidgetList_Recursive(NewWidget, NewIndex, LastestIndex, NewVisibility, bParentOrSelfVolatile);
				}
			});

		InvalidationWidgetType& WidgetProxy = (*this)[NewIndex];
		WidgetProxy.LeafMostChildIndex = LeafMostChildIndex;
	}

	return LeafMostChildIndex;
}


void FSlateInvalidationWidgetList::BuildWidgetList(const TSharedRef<SWidget>& InRoot)
{
	ensureMsgf(NumberOfLock == 0, TEXT("You are not allowed to modify the list while iterating on it."));
	SCOPED_NAMED_EVENT(Slate_InvalidationList_ProcessBuild, FColorList::Blue);

	Reset();
	Root = InRoot;
	const bool bShouldAdd = ShouldBeAdded(InRoot);
	if (bShouldAdd)
	{
		FSlateInvalidationWidgetVisibility ParentVisibility = EVisibility::Visible;
		if (const TSharedPtr<SWidget> Parent = InRoot->GetParentWidget())
		{
			ParentVisibility = Parent->GetVisibility();
		}
		const bool bParentVolatile = false;
		IndexType LatestArrayIndex = FSlateInvalidationWidgetIndex::Invalid.ArrayIndex;
		Internal_BuildWidgetList_Recursive(InRoot.Get(), FSlateInvalidationWidgetIndex::Invalid, LatestArrayIndex, ParentVisibility, bParentVolatile);
	}
}


void FSlateInvalidationWidgetList::Internal_RebuildWidgetListTree(SWidget& Widget, int32 ChildAtIndex)
{
	const bool bShouldAddWidget = ShouldBeAdded(Widget);
	FChildren* ParentChildren = Widget.GetAllChildren();
	const FSlateInvalidationWidgetIndex WidgetIndex = Widget.GetProxyHandle().GetWidgetIndex();
	if (bShouldAddWidget && ChildAtIndex < ParentChildren->Num() && WidgetIndex != FSlateInvalidationWidgetIndex::Invalid)
	{
		// Since we are going to add item, the array may get invalidated. Do not use after Internal_BuildWidgetList_Recursive
		const InvalidationWidgetType& CurrentInvalidationWidget = (*this)[WidgetIndex];
		ensure(CurrentInvalidationWidget.GetWidget() == &Widget);
		FSlateInvalidationWidgetIndex PreviousLeafIndex = CurrentInvalidationWidget.LeafMostChildIndex;
		FSlateInvalidationWidgetIndex NewLeafIndex = PreviousLeafIndex;
		IndexType LastestArrayIndex = PreviousLeafIndex.ArrayIndex;
		const bool bDoRecursion = ShouldDoRecursion(Widget);
		if (bDoRecursion)
		{
			const FSlateInvalidationWidgetVisibility CurrentVisibility = CurrentInvalidationWidget.Visibility;
			const bool bParentVolatile = Widget.IsVolatileIndirectly() || Widget.IsVolatile();
			ParentChildren->ForEachWidget([&, this](SWidget& ChildWidget)
				{
					const bool bShouldAddChild = ShouldBeAdded(ChildWidget);
					if (bShouldAddChild)
					{
						check(ChildWidget.GetParentWidget().Get() == &Widget);
						NewLeafIndex = Internal_BuildWidgetList_Recursive(ChildWidget, WidgetIndex, LastestArrayIndex, CurrentVisibility, bParentVolatile);
					}
				});
		}

		if (NewLeafIndex != PreviousLeafIndex)
		{
			InvalidationWidgetType& InvalidationWidget = (*this)[WidgetIndex];
			InvalidationWidget.LeafMostChildIndex = NewLeafIndex;
			UpdateParentLeafIndex(InvalidationWidget, PreviousLeafIndex, NewLeafIndex);
		}
	}
}


bool FSlateInvalidationWidgetList::ProcessChildOrderInvalidation(FSlateInvalidationWidgetIndex WidgetIndex, IProcessChildOrderInvalidationCallback& Callback)
{
	ensureMsgf(NumberOfLock == 0, TEXT("You are not allowed to modify the list while iterating on it."));
	SCOPE_CYCLE_COUNTER(STAT_WidgetList_ProcessChildOrderInvalidation);

	bool bIsInvalidationWidgetStillValid = true;
	TGuardValue<IProcessChildOrderInvalidationCallback*> TmpGuard(CurrentInvalidationCallback, &Callback);

	SWidget* WidgetPtr = nullptr;
	int32 StartChildIndex = INDEX_NONE;
	enum class EOperation : uint8 { Remove, Cut, None } CurrentOperation = EOperation::None;

	// Find all the InvalidationWidget children
	{
		const InvalidationWidgetType& InvalidationWidget = (*this)[WidgetIndex];
		WidgetPtr = InvalidationWidget.GetWidget();
		check(WidgetPtr);
		{
			const bool bWidgetIndexMatches = WidgetPtr->GetProxyHandle().GetWidgetIndex() == InvalidationWidget.Index;
			if (!bWidgetIndexMatches)
			{
				ensureMsgf(false, TEXT("The widget index doesn't match the index in the InvalidationWidgetList"));
				return false;
			}
		}

		FIndexRange ChildToRemoveRange;

		// Was added, but it should not be there anymore
		if (!ShouldBeAdded(*WidgetPtr))
		{
			// This proxy will not be valid after this call
			bIsInvalidationWidgetStillValid = false;
			// Remove it and all it's child
			ChildToRemoveRange = { *this, InvalidationWidget.Index, InvalidationWidget.LeafMostChildIndex };
			CurrentOperation = EOperation::Remove;
			// Do not add new child
			//Result.ChildBuilt
		}
		// If it is not supposed to had child, but had some, them remove them
		else if (!ShouldDoRecursion(*WidgetPtr))
		{
			if (InvalidationWidget.Index != InvalidationWidget.LeafMostChildIndex)
			{
				// All its children (but not itself)
				ChildToRemoveRange = { *this, IncrementIndex(InvalidationWidget.Index), InvalidationWidget.LeafMostChildIndex };
				CurrentOperation = EOperation::Remove;
				// Do not add new child
				//Result.ChildBuilt
			}
		}
		// Find the difference between list and the reality (if it used to have at least 1 child)
		else if (InvalidationWidget.Index != InvalidationWidget.LeafMostChildIndex)
		{
#if UE_SLATE_WITH_WIDGETLIST_UPDATEONLYWHATISNEEDED
			// Find all it's previous children
			TArray<FFindChildrenElement, FConcurrentLinearArrayAllocator> PreviousChildrenWidget;
			Internal_FindChildren(InvalidationWidget.Index, PreviousChildrenWidget);

			FChildren* InvalidatedChildren = WidgetPtr->GetAllChildren();
			check(InvalidatedChildren);

			// Find where it starts to get different
			int32 InvalidatedChildrenIndex = 0;

			const int32 InvalidatedChildrenNum = InvalidatedChildren->Num();
			const int32 PreviousChildrenNum = PreviousChildrenWidget.Num();
			int32 PreviousIndex = 0;
			for (; InvalidatedChildrenIndex < InvalidatedChildrenNum && PreviousIndex < PreviousChildrenNum; ++InvalidatedChildrenIndex)
			{
				TSharedRef<SWidget> NewWidget = InvalidatedChildren->GetChildAt(InvalidatedChildrenIndex);
				if (ShouldBeAdded(NewWidget))
				{
					if (&NewWidget.Get() != PreviousChildrenWidget[PreviousIndex].Get<0>())
					{
						break;
					}

					++PreviousIndex;
				}
			}

			if (InvalidatedChildrenIndex >= InvalidatedChildrenNum && PreviousIndex >= PreviousChildrenNum)
			{
				// The widget was invalidated but nothing changed. This could be normal if a widget was removed, then re-added.
				//InvalidationWidgetReason::Layout still need to be process
			}
			else if (PreviousIndex >= PreviousChildrenNum)
			{
				// Nothing to remove.
				//Result.ChildRemoved
				// Need to add widgets. We want to break the array first.
				StartChildIndex = InvalidatedChildrenIndex;
				CurrentOperation = EOperation::Cut;
			}
			else
			{
				check(PreviousIndex < PreviousChildrenNum && PreviousChildrenNum != 0);
				const FIndexRange Range = { *this, PreviousChildrenWidget[PreviousIndex].Get<1>(), PreviousChildrenWidget.Last().Get<1>() };
				ChildToRemoveRange = Range;
				StartChildIndex = InvalidatedChildrenIndex;
				CurrentOperation = EOperation::Remove;
			}
#else
			// All its children (but not itself)
			ChildToRemoveRange = { *this, IncrementIndex(InvalidationWidget.Index), InvalidationWidget.LeafMostChildIndex };
			StartChildIndex = 0;
			CurrentOperation = EOperation::Remove;
#endif
		}
		// There was not child, but maybe it has some now
		else
		{
			// Nothing to remove.
			//Result.ChildRemoved
			// Add new child after this InvalidationWidget, so we want to break/cut the array.
			StartChildIndex = 0;
			CurrentOperation = EOperation::Cut;
		}

		// Remove child
		{
			SCOPED_NAMED_EVENT(Slate_InvalidationList_ProcessRemove, FColorList::Blue);
			if (CurrentOperation == EOperation::Remove)
			{
				ensureMsgf(ChildToRemoveRange.IsValid(), TEXT("The ChildToRemoveRange should be valid at this point."));
				Callback.PreChildRemove(ChildToRemoveRange);
				Internal_RemoveRangeFromSameParent(ChildToRemoveRange);
			}
			else if (CurrentOperation == EOperation::Cut)
			{
				CutArray(InvalidationWidget.LeafMostChildIndex);
			}
		}
	}

	// Rebuild
	if (StartChildIndex != INDEX_NONE)
	{
		check(WidgetPtr);
		FChildren* InvalidatedChildren = WidgetPtr->GetAllChildren();
		check(InvalidatedChildren);
		if (InvalidatedChildren->Num() > 0)
		{
			SCOPED_NAMED_EVENT(Slate_InvalidationList_ProcessRebuild, FColorList::Blue);
			Internal_RebuildWidgetListTree(*WidgetPtr, StartChildIndex);

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_CHILDORDERCHECK
			FSlateInvalidationWidgetIndex NewWidgetIndex = WidgetPtr->GetProxyHandle().GetWidgetIndex();
			const InvalidationWidgetType& NewInvalidationWidget = (*this)[NewWidgetIndex];
			const FIndexRange BuiltRange = (NewInvalidationWidget.LeafMostChildIndex == NewWidgetIndex)
				? FIndexRange{ *this, NewWidgetIndex, NewInvalidationWidget.LeafMostChildIndex }
				: FIndexRange{ *this, IncrementIndex(NewWidgetIndex), NewInvalidationWidget.LeafMostChildIndex };
			if (BuiltRange.IsValid())
			{
				Callback.ProxiesBuilt(BuiltRange);
			}
#endif
		}
	}

	return bIsInvalidationWidgetStillValid;
}


void FSlateInvalidationWidgetList::ProcessAttributeRegistrationInvalidation(const InvalidationWidgetType& InvalidationWidget)
{
	ensureMsgf(NumberOfLock == 0, TEXT("You are not allowed to modify the list while iterating on it."));

	SWidget* WidgetPtr = InvalidationWidget.GetWidget();
	check(WidgetPtr);

	if (ShouldBeAddedToAttributeList(*WidgetPtr))
	{
		Data[InvalidationWidget.Index.ArrayIndex].ElementIndexList_WidgetWithRegisteredSlateAttribute.InsertUnique(InvalidationWidget.Index.ElementIndex);
	}
	else
	{
		Data[InvalidationWidget.Index.ArrayIndex].ElementIndexList_WidgetWithRegisteredSlateAttribute.RemoveSingle(InvalidationWidget.Index.ElementIndex);
	}
}


void FSlateInvalidationWidgetList::ProcessVolatileUpdateInvalidation(InvalidationWidgetType& InvalidationWidget)
{
	ensureMsgf(NumberOfLock == 0, TEXT("You are not allowed to modify the list while iterating on it."));

	SWidget* WidgetPtr = InvalidationWidget.GetWidget();
	check(WidgetPtr);

	if (ShouldBeAddedToVolatileUpdateList(*WidgetPtr))
	{
		Data[InvalidationWidget.Index.ArrayIndex].ElementIndexList_VolatileUpdateWidget.InsertUnique(InvalidationWidget.Index.ElementIndex);
	}
	else
	{
		Data[InvalidationWidget.Index.ArrayIndex].ElementIndexList_VolatileUpdateWidget.RemoveSingle(InvalidationWidget.Index.ElementIndex);
	}
	InvalidationWidget.bIsVolatilePrepass = WidgetPtr->HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePrepass);
}


namespace InvalidationList
{
	template<typename TSlateInvalidationWidgetList, typename Predicate>
	void ForEachChildren(TSlateInvalidationWidgetList& Self, const typename TSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget, FSlateInvalidationWidgetIndex WidgetIndex, Predicate InPredicate)
	{
		using SlateInvalidationWidgetType = std::conditional_t<std::is_const_v<TSlateInvalidationWidgetList>, const typename TSlateInvalidationWidgetList::InvalidationWidgetType, typename TSlateInvalidationWidgetList::InvalidationWidgetType>;
		if (InvalidationWidget.LeafMostChildIndex != WidgetIndex)
		{
			FSlateInvalidationWidgetIndex CurrentWidgetIndex = Self.IncrementIndex(WidgetIndex);
			while (true)
			{
				SlateInvalidationWidgetType& CurrentInvalidationWidget = Self[CurrentWidgetIndex];
				InPredicate(CurrentInvalidationWidget);
				CurrentWidgetIndex = CurrentInvalidationWidget.LeafMostChildIndex;
				if (InvalidationWidget.LeafMostChildIndex == CurrentWidgetIndex)
				{
					break;
				}
				CurrentWidgetIndex = Self.IncrementIndex(CurrentWidgetIndex);
				if (InvalidationWidget.LeafMostChildIndex == CurrentWidgetIndex)
				{
					InPredicate(Self[CurrentWidgetIndex]);
					break;
				}
			}
		}
	}
}


void FSlateInvalidationWidgetList::Internal_FindChildren(FSlateInvalidationWidgetIndex WidgetIndex, TArray<FFindChildrenElement, FConcurrentLinearArrayAllocator>& Widgets) const
{
	Widgets.Reserve(16);
	const InvalidationWidgetType& InvalidationWidget = (*this)[WidgetIndex];
	InvalidationList::ForEachChildren(*this, InvalidationWidget, WidgetIndex, [&Widgets](const InvalidationWidgetType& ChildWidget)
		{
			Widgets.Emplace(ChildWidget.GetWidget(), ChildWidget.Index);
		});
}


FSlateInvalidationWidgetIndex FSlateInvalidationWidgetList::IncrementIndex(FSlateInvalidationWidgetIndex Index) const
{
	check(Data.IsValidIndex(Index.ArrayIndex));
	++Index.ElementIndex;
	if (Index.ElementIndex >= Data[Index.ArrayIndex].ElementList.Num())
	{
		if (Data[Index.ArrayIndex].NextArrayIndex == INDEX_NONE)
		{
			return FSlateInvalidationWidgetIndex::Invalid;
		}
		check(Data[Index.ArrayIndex].NextArrayIndex < FSlateInvalidationWidgetIndex::Invalid.ArrayIndex);
		Index.ArrayIndex = (IndexType)Data[Index.ArrayIndex].NextArrayIndex;
		Index.ElementIndex = Data[Index.ArrayIndex].StartIndex;
	}
	return Index;
}


FSlateInvalidationWidgetIndex FSlateInvalidationWidgetList::DecrementIndex(FSlateInvalidationWidgetIndex Index) const
{
	check(Data.IsValidIndex(Index.ArrayIndex));
	if (Index.ElementIndex == Data[Index.ArrayIndex].StartIndex)
	{
		if (Data[Index.ArrayIndex].PreviousArrayIndex == INDEX_NONE)
		{
			return FSlateInvalidationWidgetIndex::Invalid;
		}
		check(Data[Index.ArrayIndex].PreviousArrayIndex < FSlateInvalidationWidgetIndex::Invalid.ArrayIndex);
		Index.ArrayIndex = (IndexType)Data[Index.ArrayIndex].PreviousArrayIndex;
		check(Data[Index.ArrayIndex].ElementList.Num() > 0);
		Index.ElementIndex = (IndexType)(Data[Index.ArrayIndex].ElementList.Num() - 1);
	}
	else
	{
		--Index.ElementIndex;
	}
	return Index;
}



FSlateInvalidationWidgetIndex FSlateInvalidationWidgetList::FindNextSibling(FSlateInvalidationWidgetIndex WidgetIndex) const
{
	FSlateInvalidationWidgetIndex Result = FSlateInvalidationWidgetIndex::Invalid;
	const InvalidationWidgetType& InvalidationWidget = (*this)[WidgetIndex];
	const FSlateInvalidationWidgetIndex NextSiblingIndex = IncrementIndex(InvalidationWidget.LeafMostChildIndex);
	if (NextSiblingIndex != FSlateInvalidationWidgetIndex::Invalid)
	{
		const InvalidationWidgetType& NextSiblingInvalidationWidget = (*this)[NextSiblingIndex];
		if (NextSiblingInvalidationWidget.ParentIndex == InvalidationWidget.ParentIndex)
		{
			Result = NextSiblingIndex;
		}
	}
	return Result;
}


void FSlateInvalidationWidgetList::Empty()
{
	ensureMsgf(NumberOfLock == 0, TEXT("You are not allowed to modify the list while iterating on it."));

	Data.Empty();
	Root.Reset();
	FirstArrayIndex = INDEX_NONE;
	LastArrayIndex = INDEX_NONE;
}


void FSlateInvalidationWidgetList::Reset()
{
	ensureMsgf(NumberOfLock == 0, TEXT("You are not allowed to modify the list while iterating on it."));

	Data.Reset();
	Root.Reset();
	FirstArrayIndex = INDEX_NONE;
	LastArrayIndex = INDEX_NONE;
}


FSlateInvalidationWidgetList::IndexType FSlateInvalidationWidgetList::AddArrayNodeIfNeeded(bool bReserveElementList)
{
	if (LastArrayIndex == INDEX_NONE || Data[LastArrayIndex].ElementList.Num() + 1 > WidgetListConfig.PreferedElementsNum)
	{
		if (Data.Num() + 1 == FSlateInvalidationWidgetIndex::Invalid.ArrayIndex)
		{
			ensureAlwaysMsgf(false, TEXT("The widget array is split more time that we support. Widget will not be updated properly. Try to increase Slate.InvalidationList.MaxArrayElements"));
			return (IndexType)LastArrayIndex;
		}

		check(NumberOfLock == 0);
		const int32 Index = Data.Add(FArrayNode());
		check(Index < std::numeric_limits<IndexType>::max());
		if (bReserveElementList)
		{
			Data[Index].ElementList.Reserve(WidgetListConfig.PreferedElementsNum);
		}

		if (LastArrayIndex != INDEX_NONE)
		{
			Data[LastArrayIndex].NextArrayIndex = Index;
			Data[Index].SortOrder = Data[LastArrayIndex].SortOrder + WidgetListConfig.SortOrderPaddingBetweenArray;
		}
		Data[Index].PreviousArrayIndex = LastArrayIndex;

		LastArrayIndex = Index;
		if (FirstArrayIndex == INDEX_NONE)
		{
			FirstArrayIndex = LastArrayIndex;
		}
	}
	return (IndexType)LastArrayIndex;
}


FSlateInvalidationWidgetList::IndexType FSlateInvalidationWidgetList::InsertArrayNodeIfNeeded(IndexType AfterArrayIndex, bool bReserveElementList)
{
	if (AfterArrayIndex == FSlateInvalidationWidgetIndex::Invalid.ArrayIndex)
	{
		return AddArrayNodeIfNeeded(bReserveElementList);
	}
	else if (Data[AfterArrayIndex].ElementList.Num() + 1 > WidgetListConfig.PreferedElementsNum)
	{
		return InsertDataNodeAfter(AfterArrayIndex, bReserveElementList);
	}
	return AfterArrayIndex;
}


void FSlateInvalidationWidgetList::RebuildOrderIndex(IndexType StartFrom)
{
	check(StartFrom != INDEX_NONE);
	check(WidgetListConfig.PreferedElementsNum < WidgetListConfig.MaxPreferedElementsNum);

	FSlateInvalidationWidgetList* Self = this;
	auto SetValue = [Self](IndexType ArrayIndex, int32 NewSortOrder)
		{
			check(Self->NumberOfLock == 0);
			ElementListType& ElementList = Self->Data[ArrayIndex].ElementList;
			int32 ElementIndex = Self->Data[ArrayIndex].StartIndex;
			const int32 PreviousSortOrder = Self->Data[ArrayIndex].SortOrder;
			if (ElementIndex < ElementList.Num())
			{
				const FSlateInvalidationWidgetIndex FirstWidgetIndex = { ArrayIndex, (IndexType)ElementIndex };
				const FSlateInvalidationWidgetIndex LastWidgetIndex = { ArrayIndex, (IndexType)(ElementIndex + ElementList.Num() - 1) };
				const FIndexRange ResortRange = { *Self, FirstWidgetIndex, LastWidgetIndex };

				Self->Data[ArrayIndex].SortOrder = NewSortOrder;

				if (Self->CurrentInvalidationCallback)
				{
					const IProcessChildOrderInvalidationCallback::FReSortOperation SortOperation { ResortRange };
					Self->CurrentInvalidationCallback->ProxiesPreResort(SortOperation);
				}


				for (; ElementIndex < ElementList.Num(); ++ElementIndex)
				{
					FSlateInvalidationWidgetIndex WidgetIndex = { ArrayIndex, (IndexType)ElementIndex };
					FSlateInvalidationWidgetSortOrder SortOrder = { *Self, WidgetIndex };
					if (ElementList[ElementIndex].GetWidget())
					{
						ElementList[ElementIndex].GetWidget()->SetFastPathSortOrder(SortOrder);
					}
				}

				if (Self->CurrentInvalidationCallback)
				{
					Self->CurrentInvalidationCallback->ProxiesPostResort();
				}
			}
			else
			{
				Self->Data[ArrayIndex].SortOrder = NewSortOrder;
			}
		};

	checkf(WidgetListConfig.PreferedElementsNum <= WidgetListConfig.SortOrderPaddingBetweenArray, TEXT("It should have been confirmed at initialization."));
	for (int32 CurrentIndex = StartFrom; CurrentIndex != INDEX_NONE; CurrentIndex = Data[CurrentIndex].NextArrayIndex)
	{
		const int32 PreviousIndex = Data[CurrentIndex].PreviousArrayIndex;
		const int32 NextIndex = Data[CurrentIndex].NextArrayIndex;

		if (PreviousIndex == INDEX_NONE)
		{
			SetValue((IndexType)CurrentIndex, 0);
		}
		else if (NextIndex == INDEX_NONE)
		{
			const int32 PreviousMaxSortOrder = Data[PreviousIndex].SortOrder + WidgetListConfig.PreferedElementsNum;
			const int32 CurrentSortOrder = Data[CurrentIndex].SortOrder;
			if (CurrentSortOrder < PreviousMaxSortOrder)
			{
				const int32 NewSortOrder = Data[PreviousIndex].SortOrder + WidgetListConfig.SortOrderPaddingBetweenArray;
				SetValue((IndexType)CurrentIndex, NewSortOrder);
			}
			break;
		}
		else
		{
			const int32 PreviousMinSortOrder = Data[PreviousIndex].SortOrder;
			const int32 PreviousMaxSortOrder = Data[PreviousIndex].SortOrder + WidgetListConfig.PreferedElementsNum;
			const int32 NextMinSortOrder = Data[NextIndex].SortOrder;
			const int32 CurrentSortOrder = Data[CurrentIndex].SortOrder;

			// Is everything already good
			if (PreviousMaxSortOrder < CurrentSortOrder && CurrentSortOrder + WidgetListConfig.PreferedElementsNum < NextMinSortOrder)
			{
				break;
			}
			// Would the normal padding be valid
			else if (PreviousMinSortOrder + WidgetListConfig.SortOrderPaddingBetweenArray < NextMinSortOrder)
			{
				const int32 NewSortOrder = PreviousMinSortOrder + WidgetListConfig.SortOrderPaddingBetweenArray;
				SetValue((IndexType)CurrentIndex, NewSortOrder);
			}
			// Would padding by half would be valid
			else if (NextMinSortOrder > PreviousMaxSortOrder && NextMinSortOrder - PreviousMaxSortOrder >= WidgetListConfig.PreferedElementsNum)
			{
				// We prefer to keep space of the exact amount in PreferedElementsNum in front Previous sort order and in the back Next sort order.
				//That way we potently need less reorder in the future.
				const int32 NumSpacesAvailable = (NextMinSortOrder - PreviousMaxSortOrder) / WidgetListConfig.PreferedElementsNum;
				const int32 NewCurrentOrderIndex = PreviousMaxSortOrder + (WidgetListConfig.PreferedElementsNum * (NumSpacesAvailable/2));
				check(PreviousMaxSortOrder <= NewCurrentOrderIndex && NewCurrentOrderIndex + WidgetListConfig.PreferedElementsNum <= NextMinSortOrder);
				const int32 NewSortOrder = NewCurrentOrderIndex;
				SetValue((IndexType)CurrentIndex, NewSortOrder);
			}
			// Worst case, need to also rebuild the next array
			else
			{
				const int32 NewSortOrder = PreviousMaxSortOrder + FMath::Min(WidgetListConfig.PreferedElementsNum * 2, WidgetListConfig.SortOrderPaddingBetweenArray);
				SetValue((IndexType)CurrentIndex, NewSortOrder);
			}
		}
		ensureMsgf(Data[CurrentIndex].SortOrder <= WidgetListConfig.MaxSortOrderPaddingBetweenArray
			, TEXT("The order index '%d' is too big to be contained inside the WidgetSortIndex. The Widget order will not be valid.")
			, Data[CurrentIndex].SortOrder); // See FSlateInvalidationWidgetSortOrder
	}
}


FSlateInvalidationWidgetList::IndexType FSlateInvalidationWidgetList::InsertDataNodeAfter(IndexType AfterIndex, bool bReserveElementList)
{
	if (FirstArrayIndex == INDEX_NONE)
	{
		check(AfterIndex == INDEX_NONE);
		check(LastArrayIndex == INDEX_NONE);
		const IndexType NewLastArrayIndex = AddArrayNodeIfNeeded(bReserveElementList);
		check(NewLastArrayIndex == LastArrayIndex);
		return NewLastArrayIndex;
	}
	else
	{
		check(NumberOfLock == 0);
		check(AfterIndex != INDEX_NONE);

		if (Data.Num() + 1 == FSlateInvalidationWidgetIndex::Invalid.ArrayIndex)
		{
			ensureAlwaysMsgf(false, TEXT("The widget array is split more time that we support. Widget will not be updated properly. Try to increase Slate.InvalidationList.MaxArrayElements"));
			return (IndexType)LastArrayIndex;
		}

		const int32 NewIndex = Data.Add(FArrayNode());
		check(NewIndex < std::numeric_limits<IndexType>::max());
		if (bReserveElementList)
		{
			Data[NewIndex].ElementList.Reserve(WidgetListConfig.PreferedElementsNum);
		}

		FArrayNode& AfterArrayNode = Data[AfterIndex];
		if (AfterArrayNode.NextArrayIndex != INDEX_NONE)
		{
			Data[AfterArrayNode.NextArrayIndex].PreviousArrayIndex = NewIndex;
			Data[NewIndex].NextArrayIndex = AfterArrayNode.NextArrayIndex;

			AfterArrayNode.NextArrayIndex = NewIndex;
			Data[NewIndex].PreviousArrayIndex = AfterIndex;

			if (LastArrayIndex == AfterIndex)
			{
				LastArrayIndex = NewIndex;
			}

			RebuildOrderIndex((IndexType)NewIndex);
		}
		else
		{
			check(LastArrayIndex == AfterIndex);
			LastArrayIndex = NewIndex;
			Data[NewIndex].PreviousArrayIndex = AfterIndex;
			Data[AfterIndex].NextArrayIndex = NewIndex;
			Data[NewIndex].SortOrder = Data[AfterIndex].SortOrder + WidgetListConfig.SortOrderPaddingBetweenArray;
		}

		return (IndexType)NewIndex;
	}
}


void FSlateInvalidationWidgetList::RemoveDataNode(IndexType Index)
{
	check(NumberOfLock == 0);
	check(Index != INDEX_NONE && Index != std::numeric_limits<IndexType>::max());
	FArrayNode& ArrayNode = Data[Index];
	if (ArrayNode.PreviousArrayIndex != INDEX_NONE)
	{
		Data[ArrayNode.PreviousArrayIndex].NextArrayIndex = ArrayNode.NextArrayIndex;
	}
	else
	{
		FirstArrayIndex = ArrayNode.NextArrayIndex;
	}

	if (ArrayNode.NextArrayIndex != INDEX_NONE)
	{
		Data[ArrayNode.NextArrayIndex].PreviousArrayIndex = ArrayNode.PreviousArrayIndex;
	}
	else
	{
		LastArrayIndex = ArrayNode.PreviousArrayIndex;
	}
	ArrayNode.ElementList.Empty();
	ArrayNode.ElementIndexList_WidgetWithRegisteredSlateAttribute.Empty();
	ArrayNode.ElementIndexList_VolatileUpdateWidget.Empty();
	Data.RemoveAt(Index);

	// No need to rebuild the order when we remove.
	//OrderIndex is incremental and only use to sort.
	//RebuildOrderIndex();

	check(FirstArrayIndex != Index);
	check(LastArrayIndex != Index);
	if (Data.Num() == 0)
	{
		check(LastArrayIndex == INDEX_NONE && FirstArrayIndex == INDEX_NONE);
	}
	else
	{
		check(FirstArrayIndex != INDEX_NONE && LastArrayIndex != INDEX_NONE);
	}
}


void FSlateInvalidationWidgetList::UpdateParentLeafIndex(const InvalidationWidgetType& NewInvalidationWidget, FSlateInvalidationWidgetIndex OldWidgetIndex, FSlateInvalidationWidgetIndex NewWidgetIndex)
{
	if (NewInvalidationWidget.ParentIndex != FSlateInvalidationWidgetIndex::Invalid)
	{
		InvalidationWidgetType* ParentInvalidationWidget = &(*this)[NewInvalidationWidget.ParentIndex];
		while (ParentInvalidationWidget->LeafMostChildIndex == OldWidgetIndex)
		{
			ParentInvalidationWidget->LeafMostChildIndex = NewWidgetIndex;
			if (ParentInvalidationWidget->ParentIndex != FSlateInvalidationWidgetIndex::Invalid)
			{
				ParentInvalidationWidget = &(*this)[ParentInvalidationWidget->ParentIndex];
			}
		}
	}
}


/**
 * To remove child from the same parent or all it's children.
 * A ( B (C,D), E (F,G) )
 * Can use to remove range (B,D) or (C,C) or (C,D) or (E, G) or (F,G) or (B,G).
 * Cannot use (B,E) or (B,C) or (B,F)
 */
void FSlateInvalidationWidgetList::Internal_RemoveRangeFromSameParent(const FIndexRange Range)
{
	// Fix up Parent's LeafIndex if they are in Range
	{
		//N.B. The algo doesn't support cross family removal. We do not need to worry about fixing the ParentIndex.
		//(i)	There is no other child. Set Parent's LeafIndex to itself (recursive) [remove (F,G), leaf of E=E and leaf of A=E]
		//(ii)	The Parent's LeafIndex is already set properly [remove (B,C), leaf of A is already G]
		//(iii)	Parent's LeafIndex should be set to the previous sibling' leaf (recursive) [remove (G,G), leaf of E=F, A=F]
		const InvalidationWidgetType& InvalidationWidgetStart = (*this)[Range.GetInclusiveMinWidgetIndex()];
		const InvalidationWidgetType& InvalidationWidgetEnd = (*this)[Range.GetInclusiveMaxWidgetIndex()];

		// Parent index could be invalid if there is only one widget and we are removing it
		if (InvalidationWidgetStart.ParentIndex != FSlateInvalidationWidgetIndex::Invalid)
		{
			// Is the parent's leaf being removed
			const InvalidationWidgetType& InvalidationWidgetParentStart = (*this)[InvalidationWidgetStart.ParentIndex];
			if (Range.Include(FSlateInvalidationWidgetSortOrder{ *this, InvalidationWidgetParentStart.LeafMostChildIndex }))
			{
				// _RemoveRangeFromSameParent doesn't support cross family removal
				check(Range.Include(FSlateInvalidationWidgetSortOrder{ *this, (*this)[InvalidationWidgetEnd.ParentIndex].LeafMostChildIndex }));
				const FSlateInvalidationWidgetIndex PreviousWidget = DecrementIndex(Range.GetInclusiveMinWidgetIndex());
				UpdateParentLeafIndex(InvalidationWidgetStart, InvalidationWidgetParentStart.LeafMostChildIndex, PreviousWidget);
			}
		}
	}

	//N.B. Theres is no parent/left relation in the array.
	//ie.		1234 5678 90ab	(they have a size of 4, we cut at 2)
	//(i)		123x xxxx x0ab => 123 x0ab			=> no cut, remove 4, remove 5-8, set StartIndex to 0
	//(ii)		123x xxxx xxxb => 123 b				=> cut, remove 4, remove 5-8, remove 9-b (b was moved)
	//(iii)		12xx 5478 90ab => 12 5478 90ab		=> cut, remove 3-4
	//(iv)		1234 x678 90ab => 1234 x678 90ab	=> no cut, set StartIndex to 6
	//(v)		1234 xxx8 90ab => 1234 8 90ab		=> cut, remove 5-8, (8 was moved)
	//(vi)		1234 5xx8 90ab => 1234 5 8 90ab		=> cut, remove 6-8, (8 was moved)

	const int32 NumberElementLeft = Data[Range.GetInclusiveMaxWidgetIndex().ArrayIndex].ElementList.Num()
		//- Data[Range.GetInclusiveMax().ArrayIndex].StartIndex
		- Range.GetInclusiveMaxWidgetIndex().ElementIndex
		- 1;
	const bool bRangeIsInSameElementArray = Range.GetInclusiveMinWidgetIndex().ArrayIndex == Range.GetInclusiveMaxWidgetIndex().ArrayIndex;
	const bool bShouldCutArray = NumberElementLeft < WidgetListConfig.NumberElementsLeftBeforeSplitting
		|| (bRangeIsInSameElementArray && Data[Range.GetInclusiveMinWidgetIndex().ArrayIndex].StartIndex != Range.GetInclusiveMinWidgetIndex().ElementIndex);
	if (bShouldCutArray)
	{
		Internal_CutArray(Range.GetInclusiveMaxWidgetIndex());
	}

	// Destroy/Remove the data that is not needed anymore
	{
		FSlateInvalidationWidgetList* Self = this;
		auto ResetInvalidationWidgetHandle = [Self](IndexType ArrayIndex, int32 StartIndex, int32 Num)
		{
			for (int32 ElementIndex = StartIndex; ElementIndex < Num; ++ElementIndex)
			{
				InvalidationWidgetType& InvalidationWidget = Self->Data[ArrayIndex].ElementList[ElementIndex];
				if (SWidget* Widget = InvalidationWidget.GetWidget())
				{
					Widget->SetFastPathProxyHandle(FWidgetProxyHandle());
				}
			}
		};
		auto ResetInvalidationWidget = [Self](IndexType ArrayIndex, IndexType StartIndex, int32 Num)
		{
			if (Num > 0)
			{
				FArrayNode& ArrayNode = Self->Data[ArrayIndex];
				ElementListType& ResetElementList = ArrayNode.ElementList;
				for (int32 ElementIndex = StartIndex; ElementIndex < Num; ++ElementIndex)
				{
					InvalidationWidgetType& InvalidationWidget = ResetElementList[ElementIndex];
					if (SWidget* Widget = InvalidationWidget.GetWidget())
					{
						Widget->SetFastPathProxyHandle(FWidgetProxyHandle());
					}
					InvalidationWidget.ResetWidget();
				}
				ArrayNode.RemoveElementIndexBetweenOrEqualThan(StartIndex, (IndexType)(StartIndex + Num - 1));
			}
		};

		auto RemoveDataNodeIfNeeded = [Self](IndexType ArrayIndex) -> bool
		{
			if (Self->Data[ArrayIndex].StartIndex >= Self->Data[ArrayIndex].ElementList.Num())
			{
				Self->RemoveDataNode(ArrayIndex);
				return true;
			}
			return false;
		};

		// Remove the other arrays that are between min and max	(ie. i, ii)
		if (!bRangeIsInSameElementArray)
		{
			const IndexType BeginArrayIndex = (IndexType)Data[Range.GetInclusiveMinWidgetIndex().ArrayIndex].NextArrayIndex;
			const IndexType EndArrayIndex = Range.GetInclusiveMaxWidgetIndex().ArrayIndex;

			IndexType CurrentArrayIndex = BeginArrayIndex;
			while (CurrentArrayIndex != EndArrayIndex)
			{
				ResetInvalidationWidgetHandle(CurrentArrayIndex, Data[CurrentArrayIndex].StartIndex, Data[CurrentArrayIndex].ElementList.Num());
				IndexType NextToRemove = (IndexType)Data[CurrentArrayIndex].NextArrayIndex;
				RemoveDataNode(CurrentArrayIndex);
				CurrentArrayIndex = NextToRemove;
			}
		}

		// Remove the start of the Max array
		if (bShouldCutArray && !bRangeIsInSameElementArray)
		{
			// The valid data in the array was moved (ie. ii)
			ResetInvalidationWidgetHandle(Range.GetInclusiveMaxWidgetIndex().ArrayIndex, Data[Range.GetInclusiveMaxWidgetIndex().ArrayIndex].StartIndex, Range.GetInclusiveMaxWidgetIndex().ElementIndex + 1);
			RemoveDataNode(Range.GetInclusiveMaxWidgetIndex().ArrayIndex);
		}
		else if (!bRangeIsInSameElementArray)
		{
			// Set StartIndex (ie. i)
			check(Range.GetInclusiveMinWidgetIndex().ArrayIndex != Range.GetInclusiveMaxWidgetIndex().ArrayIndex);

			ResetInvalidationWidget(Range.GetInclusiveMaxWidgetIndex().ArrayIndex, Data[Range.GetInclusiveMaxWidgetIndex().ArrayIndex].StartIndex, Range.GetInclusiveMaxWidgetIndex().ElementIndex + 1);
			Data[Range.GetInclusiveMaxWidgetIndex().ArrayIndex].StartIndex = Range.GetInclusiveMaxWidgetIndex().ElementIndex + 1;
			RemoveDataNodeIfNeeded(Range.GetInclusiveMaxWidgetIndex().ArrayIndex);
		}

		// Remove what is left of the Min array
		if (bShouldCutArray || !bRangeIsInSameElementArray)
		{
			// RemoveAt Min to Num of the array. (ie. i, ii, iii, v, vi)
			FArrayNode& ArrayNode = Data[Range.GetInclusiveMinWidgetIndex().ArrayIndex];
			ElementListType& RemoveElementList = ArrayNode.ElementList;
			if (bRangeIsInSameElementArray)
			{
				ResetInvalidationWidgetHandle(Range.GetInclusiveMinWidgetIndex().ArrayIndex, Range.GetInclusiveMinWidgetIndex().ElementIndex, Range.GetInclusiveMaxWidgetIndex().ElementIndex);
			}
			else
			{
				ResetInvalidationWidgetHandle(Range.GetInclusiveMinWidgetIndex().ArrayIndex, Range.GetInclusiveMinWidgetIndex().ElementIndex, RemoveElementList.Num());
			}

			const IndexType RemoveArrayAt = Range.GetInclusiveMinWidgetIndex().ElementIndex;
			RemoveElementList.RemoveAt(RemoveArrayAt, RemoveElementList.Num() - RemoveArrayAt, EAllowShrinking::Yes);
			if (!RemoveDataNodeIfNeeded(Range.GetInclusiveMinWidgetIndex().ArrayIndex))
			{
				ArrayNode.RemoveElementIndexBiggerOrEqualThan(RemoveArrayAt);
			}
		}
		else
		{
			// Set StartIndex (ie. iv)
			check(Range.GetInclusiveMinWidgetIndex().ArrayIndex == Range.GetInclusiveMaxWidgetIndex().ArrayIndex);
			check(Range.GetInclusiveMinWidgetIndex().ElementIndex == Data[Range.GetInclusiveMinWidgetIndex().ArrayIndex].StartIndex);

			ResetInvalidationWidget(Range.GetInclusiveMinWidgetIndex().ArrayIndex, Range.GetInclusiveMinWidgetIndex().ElementIndex, Range.GetInclusiveMaxWidgetIndex().ElementIndex + 1);
			Data[Range.GetInclusiveMinWidgetIndex().ArrayIndex].StartIndex = Range.GetInclusiveMaxWidgetIndex().ElementIndex + 1;
			RemoveDataNodeIfNeeded(Range.GetInclusiveMinWidgetIndex().ArrayIndex);
		}
	}
}


FSlateInvalidationWidgetList::FCutResult FSlateInvalidationWidgetList::CutArray(const FSlateInvalidationWidgetIndex WhereToCut)
{
	FCutResult CutResult = Internal_CutArray(WhereToCut);

	// Remove the old data that is now moved to the new array
	if (CutResult.OldElementIndexStart != INDEX_NONE)
	{
		FArrayNode& ArrayNode = Data[WhereToCut.ArrayIndex];
		ElementListType& RemoveElementList = Data[WhereToCut.ArrayIndex].ElementList;
		RemoveElementList.RemoveAt(CutResult.OldElementIndexStart, RemoveElementList.Num() - CutResult.OldElementIndexStart, EAllowShrinking::Yes);
		if (RemoveElementList.Num() == 0)
		{
			RemoveDataNode(WhereToCut.ArrayIndex);
		}
		else
		{
			ArrayNode.RemoveElementIndexBiggerOrEqualThan((IndexType)CutResult.OldElementIndexStart);
		}
	}

	return CutResult;
}


FSlateInvalidationWidgetList::FCutResult FSlateInvalidationWidgetList::Internal_CutArray(const FSlateInvalidationWidgetIndex WhereToCut)
{
	FCutResult Result;
	// Widget proxies that got moved/re-indexed by the operation.
	FIndexRange ProxiesReIndexed;
	//The new index that now correspond to the previous ProxiesReIndexed.GetInclusiveMin()
	FSlateInvalidationWidgetIndex ReIndexTarget = FSlateInvalidationWidgetIndex::Invalid;

	// Should we cut the array and move the data to another array
	// N.B. We can cut/move anywhere. Cross family may occur. Fix up everything.
	if (WhereToCut.ElementIndex < Data[WhereToCut.ArrayIndex].ElementList.Num() - 1)
	{
		check(NumberOfLock == 0);

		//From where to where we are moving the item
		const IndexType OldArrayIndex = WhereToCut.ArrayIndex;
		const IndexType OldElementIndexStart = WhereToCut.ElementIndex + 1;
		const IndexType OldElementIndexEnd = (IndexType)Data[WhereToCut.ArrayIndex].ElementList.Num();
		const IndexType NewArrayIndex = InsertDataNodeAfter(OldArrayIndex, false);
		const int32 NewExpectedElementArraySize = OldElementIndexEnd - OldElementIndexStart;
		Data[NewArrayIndex].ElementList.Reserve(NewExpectedElementArraySize);

		const FIndexRange OldRange = { *this, FSlateInvalidationWidgetIndex{ OldArrayIndex, OldElementIndexStart },  FSlateInvalidationWidgetIndex{ OldArrayIndex, (IndexType)(OldElementIndexEnd - 1) } };

		auto OldToNewIndex = [NewArrayIndex, OldElementIndexStart](FSlateInvalidationWidgetIndex OldIndex) -> FSlateInvalidationWidgetIndex
		{
			const IndexType NewElementIndex = OldIndex.ElementIndex - OldElementIndexStart;
			const FSlateInvalidationWidgetIndex NewWidgetIndex = { NewArrayIndex, NewElementIndex };
			return NewWidgetIndex;
		};

		if (OldElementIndexStart < OldElementIndexEnd)
		{
			FSlateInvalidationWidgetIndex ResultStart = { OldArrayIndex, OldElementIndexStart };
			FSlateInvalidationWidgetIndex ResultEnd = { OldArrayIndex, (IndexType)(OldElementIndexEnd - 1) };
			ProxiesReIndexed = FIndexRange{ *this, ResultStart , ResultEnd };
		}

		Result.OldElementIndexStart = OldElementIndexStart;

		// Copy and assign the new index to the widget
		for (IndexType OldElementIndex = OldElementIndexStart; OldElementIndex < OldElementIndexEnd; ++OldElementIndex)
		{
			const FSlateInvalidationWidgetIndex MoveWidgetIndex = { OldArrayIndex, OldElementIndex };
			const IndexType NewElementIndex = (IndexType)Data[NewArrayIndex].ElementList.Add(MoveTemp((*this)[MoveWidgetIndex]));
			(*this)[MoveWidgetIndex].ResetWidget();
			const FSlateInvalidationWidgetIndex NewWidgetIndex = { NewArrayIndex, NewElementIndex };
			check(OldToNewIndex(MoveWidgetIndex) == NewWidgetIndex);
			InvalidationWidgetType& NewInvalidationWidget = (*this)[NewWidgetIndex];

			if (ReIndexTarget == FSlateInvalidationWidgetIndex::Invalid)
			{
				ReIndexTarget = NewWidgetIndex;
			}

			// Fix up Index
			NewInvalidationWidget.Index = NewWidgetIndex;

			// Fix up the parent index
			{
				if (OldRange.Include(FSlateInvalidationWidgetSortOrder{ *this, NewInvalidationWidget.ParentIndex }))
				{
					NewInvalidationWidget.ParentIndex = OldToNewIndex(NewInvalidationWidget.ParentIndex);
				}
			}

			// Fix up the leaf index
			{
				if (OldRange.Include(FSlateInvalidationWidgetSortOrder{ *this, NewInvalidationWidget.LeafMostChildIndex }))
				{
					NewInvalidationWidget.LeafMostChildIndex = OldToNewIndex(NewInvalidationWidget.LeafMostChildIndex);
				}
				check(NewInvalidationWidget.LeafMostChildIndex != FSlateInvalidationWidgetIndex::Invalid);
			}

			// Anyone in the hierarchy can point to a invalid LeafIndex.
			{
				InvalidationWidgetType* ParentInvalidationWidget = &(*this)[NewInvalidationWidget.ParentIndex];
				// If the Leaf is in the range, then recursive up.
				while (OldRange.Include(FSlateInvalidationWidgetSortOrder{ *this, ParentInvalidationWidget->LeafMostChildIndex }))
				{
					ParentInvalidationWidget->LeafMostChildIndex = OldToNewIndex(ParentInvalidationWidget->LeafMostChildIndex);
					if (ParentInvalidationWidget->ParentIndex == FSlateInvalidationWidgetIndex::Invalid)
					{
						break;
					}
					ParentInvalidationWidget = &(*this)[ParentInvalidationWidget->ParentIndex];
				}
			}

			// Set new index
			if (SWidget* Widget = NewInvalidationWidget.GetWidget())
			{
				FSlateInvalidationWidgetSortOrder SortIndex = { *this, NewWidgetIndex };
				Widget->SetFastPathProxyHandle(FWidgetProxyHandle{ Owner, NewWidgetIndex, SortIndex });

				// Check for ElementIndexList
				if (ShouldBeAddedToAttributeList(*Widget))
				{
					Data[NewArrayIndex].ElementIndexList_WidgetWithRegisteredSlateAttribute.AddUnsorted(NewElementIndex);
					EnumRemoveFlags(NewInvalidationWidget.CurrentInvalidateReason, EInvalidateWidgetReason::AttributeRegistration);
				}
				if (ShouldBeAddedToVolatileUpdateList(*Widget))
				{
					Data[NewArrayIndex].ElementIndexList_VolatileUpdateWidget.AddUnsorted(NewElementIndex);
				}
			}
		}

		// Random children may still point to the old ParentIndex
		check(Data[NewArrayIndex].ElementList.Num() == NewExpectedElementArraySize);
		check(Data[NewArrayIndex].StartIndex == 0);
		for (IndexType NewElementIndex = 0; NewElementIndex < NewExpectedElementArraySize; ++NewElementIndex)
		{
			const FSlateInvalidationWidgetIndex NewWidgetIndex = { NewArrayIndex, NewElementIndex };
			InvalidationWidgetType& NewInvalidationWidget = (*this)[NewWidgetIndex];

			// The parent is only invalid when it's the root. We can't remove it, but we shouldn't move it.
			check(NewInvalidationWidget.ParentIndex != FSlateInvalidationWidgetIndex::Invalid);
			InvalidationList::ForEachChildren(*this, NewInvalidationWidget, NewWidgetIndex, [NewWidgetIndex](InvalidationWidgetType& NewChildInvalidationWidget)
				{
					NewChildInvalidationWidget.ParentIndex = NewWidgetIndex;
				});
		}
	}

	if (CurrentInvalidationCallback && ProxiesReIndexed.IsValid())
	{
		IProcessChildOrderInvalidationCallback::FReIndexOperation Operation{ ProxiesReIndexed, ReIndexTarget };
		CurrentInvalidationCallback->ProxiesReIndexed(Operation);
	}

	return Result;
}

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
FSlateInvalidationWidgetIndex FSlateInvalidationWidgetList::FindWidget(const SWidget& WidgetToFind) const
{
	const SWidget* WidgetToFindPtr = &WidgetToFind;
	for (FSlateInvalidationWidgetIndex Index = FirstIndex(); Index != FSlateInvalidationWidgetIndex::Invalid; Index = IncrementIndex(Index))
	{
		if (WidgetToFindPtr == (*this)[Index].GetWidget())
		{
			return Index;
		}
	}
	return FSlateInvalidationWidgetIndex::Invalid;
}


void FSlateInvalidationWidgetList::RemoveWidget(const FSlateInvalidationWidgetIndex WidgetIndex)
{
	if (WidgetIndex != FSlateInvalidationWidgetIndex::Invalid && IsValidIndex(WidgetIndex))
	{
		const InvalidationWidgetType& InvalidationWidget = (*this)[WidgetIndex];

		const FIndexRange Range = { *this, WidgetIndex, InvalidationWidget.LeafMostChildIndex };
		Internal_RemoveRangeFromSameParent(Range);
	}
}


void FSlateInvalidationWidgetList::RemoveWidget(const SWidget& WidgetToRemove)
{
	if (ensure(WidgetToRemove.GetProxyHandle().GetInvalidationRootHandle().GetUniqueId() == Owner.GetUniqueId()))
	{
		FSlateInvalidationWidgetIndex WidgetIndex = WidgetToRemove.GetProxyHandle().GetWidgetIndex();
		if (WidgetIndex != FSlateInvalidationWidgetIndex::Invalid)
		{
			const InvalidationWidgetType& InvalidationWidget = (*this)[WidgetIndex];
			const FIndexRange Range = { *this, WidgetIndex, InvalidationWidget.LeafMostChildIndex };
			Internal_RemoveRangeFromSameParent(Range);
		}
	}

}


TArray<TSharedPtr<SWidget>> FSlateInvalidationWidgetList::FindChildren(const SWidget& Widget) const
{
	TArray<TSharedPtr<SWidget>> Result;
	if (ensure(Widget.GetProxyHandle().GetInvalidationRootHandle().GetUniqueId() == Owner.GetUniqueId()))
	{
		FSlateInvalidationWidgetIndex WidgetIndex = Widget.GetProxyHandle().GetWidgetIndex();
		if (WidgetIndex == FSlateInvalidationWidgetIndex::Invalid)
		{
			return Result;
		}

		TArray<FFindChildrenElement, FConcurrentLinearArrayAllocator> PreviousChildrenWidget;
		Internal_FindChildren(WidgetIndex, PreviousChildrenWidget);

		Result.Reserve(PreviousChildrenWidget.Num());
		for (const FFindChildrenElement& ChildrenElement : PreviousChildrenWidget)
		{
			Result.Add(ChildrenElement.Get<0>() ? ChildrenElement.Get<0>()->AsShared() : TSharedPtr<SWidget>());
		}
	}
	return Result;
}


bool FSlateInvalidationWidgetList::DeepCompare(const FSlateInvalidationWidgetList& Other) const
{
	if (Root.Pin() != Other.Root.Pin())
	{
		return false;
	}

	FSlateInvalidationWidgetIndex IndexA = FirstIndex();
	FSlateInvalidationWidgetIndex IndexB = Other.FirstIndex();
	for (; IndexA != FSlateInvalidationWidgetIndex::Invalid && IndexB != FSlateInvalidationWidgetIndex::Invalid; IndexA = IncrementIndex(IndexA), IndexB = Other.IncrementIndex(IndexB))
	{
		const InvalidationWidgetType& InvalidationWidgetA = (*this)[IndexA];
		const InvalidationWidgetType& InvalidationWidgetB = Other[IndexB];
		if (InvalidationWidgetA.GetWidget() != InvalidationWidgetB.GetWidget())
		{
			return false;
		}
		if (InvalidationWidgetA.ParentIndex == FSlateInvalidationWidgetIndex::Invalid)
		{
			if (InvalidationWidgetA.ParentIndex != InvalidationWidgetB.ParentIndex)
			{
				return false;
			}
		}
		else
		{
			if ((*this)[InvalidationWidgetA.ParentIndex].GetWidget() != Other[InvalidationWidgetB.ParentIndex].GetWidget())
			{
				return false;
			}
		}
		check(InvalidationWidgetA.LeafMostChildIndex != FSlateInvalidationWidgetIndex::Invalid);
		check(InvalidationWidgetB.LeafMostChildIndex != FSlateInvalidationWidgetIndex::Invalid);
		if ((*this)[InvalidationWidgetA.LeafMostChildIndex].GetWidget() != Other[InvalidationWidgetB.LeafMostChildIndex].GetWidget())
		{
			return false;
		}
	}

	if (IndexA != FSlateInvalidationWidgetIndex::Invalid || IndexB != FSlateInvalidationWidgetIndex::Invalid)
	{
		return false;
	}

	return true;
}


void FSlateInvalidationWidgetList::LogWidgetsList(bool bOnlyVisible) const
{
	auto GetDepth = [this](FSlateInvalidationWidgetIndex Index)
	{
		int32 Depth = 0;
		while(Index != FSlateInvalidationWidgetIndex::Invalid)
		{
			++Depth;
			Index = (*this)[Index].ParentIndex;
		}
		return Depth-1;
	};

	TStringBuilder<512> Builder;
	for (FSlateInvalidationWidgetIndex Index = FirstIndex(); Index != FSlateInvalidationWidgetIndex::Invalid; Index = IncrementIndex(Index))
	{
		Builder.Reset();

		const InvalidationWidgetType& InvalidateWidget = (*this)[Index];
		if (!bOnlyVisible || InvalidateWidget.Visibility.IsVisible())
		{
			for (int32 Depth = GetDepth(Index); Depth > 0; --Depth)
			{
				Builder << TEXT(' ');
			}

			if (SWidget* Widget = InvalidateWidget.GetWidget())
			{
				Builder << FReflectionMetaData::GetWidgetDebugInfo(Widget)
					<< TEXT('(') << Widget->GetTag() << TEXT(')')
					<< TEXT('[') << Widget->GetPersistentState().LayerId << TEXT(',') << Widget->GetPersistentState().OutgoingLayerId << TEXT(']');
			}
			else
			{
				Builder << TEXT("[None]");
			}
			Builder << TEXT("\t");

			if (InvalidateWidget.ParentIndex != FSlateInvalidationWidgetIndex::Invalid)
			{
				if (SWidget* Widget = (*this)[InvalidateWidget.ParentIndex].GetWidget())
				{
					Builder << FReflectionMetaData::GetWidgetDebugInfo(Widget) << TEXT("(") << Widget->GetTag() << TEXT(")");
				}
				else
				{
					Builder << TEXT("[None]");
				}
			}
			else
			{
				Builder << TEXT("[---]");
			}
			Builder << TEXT("\t");

			if (InvalidateWidget.LeafMostChildIndex != FSlateInvalidationWidgetIndex::Invalid)
			{
				if (SWidget* Widget = (*this)[InvalidateWidget.LeafMostChildIndex].GetWidget())
				{
					Builder << FReflectionMetaData::GetWidgetDebugInfo(Widget) << TEXT("(") << Widget->GetTag() << TEXT(")");
				}
				else
				{
					Builder << TEXT("[None]");
				}
			}
			else
			{
				Builder << TEXT("[---]");
			}
			Builder << TEXT("\t");

			UE_LOGF(LogSlate, Log, "%ls", Builder.ToString());
		}
	}
}


bool FSlateInvalidationWidgetList::VerifyWidgetsIndex() const
{
	bool bResult = true;
	for (FSlateInvalidationWidgetIndex Index = FirstIndex(); Index != FSlateInvalidationWidgetIndex::Invalid; Index = IncrementIndex(Index))
	{
		const InvalidationWidgetType& InvalidateWidget = (*this)[Index];
		if (SWidget* Widget = InvalidateWidget.GetWidget())
		{
			const FSlateInvalidationWidgetIndex WidgetIndex = Widget->GetProxyHandle().GetWidgetIndex();
			if (Index != WidgetIndex)
			{
				UE_LOGF(LogSlate, Warning, "Widget '%ls' at index [%d,%d] is set to [%d,%d]."
					, *FReflectionMetaData::GetWidgetDebugInfo(Widget)
					, Index.ArrayIndex, Index.ElementIndex
					, WidgetIndex.ArrayIndex, WidgetIndex.ElementIndex);
				bResult = false;
			}
			else if (InvalidateWidget.Index != Index)
			{
				UE_LOGF(LogSlate, Warning, "Widget '%ls' at index [%d,%d] is set to the correct proxy index [%d,%d]."
					, *FReflectionMetaData::GetWidgetDebugInfo(Widget)
					, Index.ArrayIndex, Index.ElementIndex
					, WidgetIndex.ArrayIndex, WidgetIndex.ElementIndex);
				bResult = false;
			}
		}
		else
		{
			UE_LOGF(LogSlate, Warning, "Widget at index [%d,%d] is [null]", Index.ArrayIndex, Index.ElementIndex);
			bResult = false;
		}
	}
	return bResult;
}


bool FSlateInvalidationWidgetList::VerifyProxiesWidget() const
{
	bool bResult = true;
	for (const FArrayNode& Node : Data)
	{
		// Before StartIndex, pointer need to be empty.
		for (int32 ElementIndex = 0; ElementIndex < Node.StartIndex; ++ElementIndex)
		{
			if (SWidget* Widget = Node.ElementList[ElementIndex].GetWidget())
			{
				UE_LOGF(LogSlate, Warning, "Element '%d' in the array of sort value '%d' as a valid widget '%ls' when it should be set to none."
					, ElementIndex, Node.SortOrder
					, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
				bResult = false;
			}
		}

		// Every other element need to point to a valid widget.
		for (int32 ElementIndex = Node.StartIndex; ElementIndex < Node.ElementList.Num(); ++ElementIndex)
		{
			SWidget* Widget = Node.ElementList[ElementIndex].GetWidget();
			if (!Widget)
			{
				UE_LOGF(LogSlate, Warning, "Element '%d' in the array of sort value '%d' does not have a valid widget."
					, ElementIndex, Node.SortOrder);
				bResult = false;
			}
		}
	}
	return bResult;
}


bool FSlateInvalidationWidgetList::VerifySortOrder() const
{
	bool bResult = true;
	if (FirstArrayIndex != INDEX_NONE)
	{
		int32 PreviousSortOrder = Data[FirstArrayIndex].SortOrder;
 		for (int32 ArrayIndex = Data[FirstArrayIndex].NextArrayIndex; ArrayIndex != INDEX_NONE; ArrayIndex = Data[ArrayIndex].NextArrayIndex)
		{
			if (PreviousSortOrder >= Data[ArrayIndex].SortOrder)
			{
				UE_LOGF(LogSlate, Warning, "Array '%d' has a bigger sort order than previous array node '%d'."
					, ArrayIndex, Data[ArrayIndex].PreviousArrayIndex);
				bResult = false;
				break;
			}
		}
	}
	return bResult;
}


bool FSlateInvalidationWidgetList::VerifyElementIndexList() const
{
	bool bResult = true;

	// Test that the iterator return the same result as a new calculated list.
	if (bResult)
	{
		TArray<const SWidget*> WidgetListWithIterator;
		FSlateInvalidationWidgetList::FWidgetAttributeIterator AttributeItt = CreateWidgetAttributeIterator();
		while (AttributeItt.IsValid())
		{
			const SWidget* WidgetPtr = (*this)[AttributeItt.GetCurrentIndex()].GetWidget();
			if (WidgetPtr)
			{
				WidgetListWithIterator.Add(WidgetPtr);
			}
			AttributeItt.Advance();
		}


		TArray<const SWidget*> WidgetListWithForEachWidget;
		WidgetListWithForEachWidget.Reset(WidgetListWithIterator.Num());
		int32 Index = 0;
		const_cast<FSlateInvalidationWidgetList*>(this)->ForEachWidget([&](const SWidget& Widget)
			{
				if (ShouldBeAddedToAttributeList(Widget))
				{
					WidgetListWithForEachWidget.Add(&Widget);
				}
			});

		bResult = WidgetListWithIterator == WidgetListWithForEachWidget;
		if (!bResult)
		{
			UE_LOGF(LogSlate, Warning, "The 2 lists are not identical. With Attribute: %d. With ForEach: %d", WidgetListWithIterator.Num(), WidgetListWithForEachWidget.Num());
		}
	}

	// Test if bDebug_AttributeUpdated was updated.
	if (bResult)
	{
		for (const FArrayNode& ArrayNode : Data)
		{
			for (IndexType ElementIndex : ArrayNode.ElementIndexList_WidgetWithRegisteredSlateAttribute)
			{
				if (!ArrayNode.ElementList.IsValidIndex(ElementIndex))
				{
					UE_LOGF(LogSlate, Warning, "ElementIndex '%d' in the array of sort value '%d' is invalid."
						, ElementIndex, ArrayNode.SortOrder);
					bResult = false;
					break;
				}

				const FSlateInvalidationWidgetList::InvalidationWidgetType& InvalidationWidget = ArrayNode.ElementList[ElementIndex];
				if (const SWidget* Widget = InvalidationWidget.GetWidget())
				{
					if (!Widget->HasRegisteredSlateAttribute())
					{
						UE_LOGF(LogSlate, Warning, "ElementIndex '%d' in the array of sort value '%d' should not be there since widget '%ls' doesn't have registered SlateAttribute."
							, ElementIndex, ArrayNode.SortOrder
							, *FReflectionMetaData::GetWidgetDebugInfo(Widget));
						bResult = false;
					}
					else if (!InvalidationWidget.Visibility.IsCollapsed() && !InvalidationWidget.bDebug_AttributeUpdated)
					{
						UE_LOGF(LogSlate, Warning, "The widget '%ls' was not updated.", *FReflectionMetaData::GetWidgetDebugInfo(Widget));
						bResult = false;
					}
				}
			}
		}
	}

	return bResult;
}
#endif //UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING

#undef UE_SLATE_WITH_WIDGETLIST_UPDATEONLYWHATISNEEDED
