// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "FastUpdate/SlateElementSortedArray.h"
#include "FastUpdate/SlateInvalidationRootHandle.h"
#include "FastUpdate/SlateInvalidationWidgetIndex.h"
#include "FastUpdate/SlateInvalidationWidgetSortOrder.h"
#include "FastUpdate/WidgetUpdateFlags.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

#define UE_SLATE_WITH_INVALIDATIONWIDGETLIST_RANGECHECK UE_BUILD_DEBUG
#define UE_SLATE_WITH_INVALIDATIONWIDGETLIST_CHILDORDERCHECK UE_BUILD_DEBUG || UE_BUILD_DEVELOPMENT


class FSlateInvalidationWidgetList
{
	friend struct FSlateInvalidationWidgetSortOrder;
public:
	using InvalidationWidgetType = FWidgetProxy;
private:
	using IndexType = FSlateInvalidationWidgetIndex::IndexType;
	using ElementListType = TArray<InvalidationWidgetType>;
	using WidgetListType = TSlateElementSortedArray<IndexType>;


public:
	/** */
	struct FIndexRange
	{
	private:
		FSlateInvalidationWidgetIndex InclusiveMin = FSlateInvalidationWidgetIndex::Invalid;
		FSlateInvalidationWidgetIndex InclusiveMax = FSlateInvalidationWidgetIndex::Invalid;
		FSlateInvalidationWidgetSortOrder OrderMin;
		FSlateInvalidationWidgetSortOrder OrderMax;

	public:
		FIndexRange() = default;
		FIndexRange(const FSlateInvalidationWidgetList& Self, FSlateInvalidationWidgetIndex InFrom, FSlateInvalidationWidgetIndex InEnd)
			: InclusiveMin(InFrom), InclusiveMax(InEnd)
			, OrderMin(Self, InFrom), OrderMax(Self, InEnd)
		{
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_RANGECHECK
			check(OrderMin <= OrderMax);
#endif
		}
		[[nodiscard]] bool Include(FSlateInvalidationWidgetSortOrder Other) const
		{
			return OrderMin <= Other && Other <= OrderMax;
		}
		[[nodiscard]] bool IsValid() const { return InclusiveMin != FSlateInvalidationWidgetIndex::Invalid; }

		FSlateInvalidationWidgetIndex GetInclusiveMinWidgetIndex() const { return InclusiveMin; }
		FSlateInvalidationWidgetIndex GetInclusiveMaxWidgetIndex() const { return InclusiveMax; }
		FSlateInvalidationWidgetSortOrder GetInclusiveMinWidgetSortOrder() const { return OrderMin; }
		FSlateInvalidationWidgetSortOrder GetInclusiveMaxWidgetSortOrder() const { return OrderMax; }

		[[nodiscard]] bool operator==(const FIndexRange& Other) const { return Other.InclusiveMin == InclusiveMin && Other.InclusiveMax == InclusiveMax; }
	};

public:
	/** */
	struct FArguments
	{
		static int32 MaxPreferedElementsNum;
		static int32 MaxSortOrderPaddingBetweenArray;
		/**
		 * Prefered size of the elements array.
		 * The value should be between 2 and MaxPreferedElementsNum.
		 */
		int32 PreferedElementsNum = 64;

		/**
		 * When splitting, the elements will be copied to another array when the number of element is bellow this.
		 * The value should be between 1 and PreferedElementsNum.
		 */
		int32 NumberElementsLeftBeforeSplitting = 40;

		/**
		 * The sort order is used by the HittestGrid and the LayerId.
		 * The value should be between PreferedElementsNum and MaxSortOrderPaddingBetweenArray.
		 */
		int32 SortOrderPaddingBetweenArray = 1000;
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
		/** Change the Widget Index when building the array. Use when building temporary list. */
		bool bAssignedWidgetIndex = true;
#endif
	};
	FSlateInvalidationWidgetList(FSlateInvalidationRootHandle Owner, const FArguments& Args);

	/** Build the widget list from the root widget. */
	void BuildWidgetList(const TSharedRef<SWidget>& Root);

	/** Get the root the widget list was built with. */
	TWeakPtr<SWidget> GetRoot() { return Root; };
	/** Get the root the widget list was built with. */
	const TWeakPtr<SWidget> GetRoot() const { return Root; };

	/** */
	struct IProcessChildOrderInvalidationCallback
	{
		struct FReIndexOperation
		{
			FReIndexOperation(const FIndexRange& InRange, FSlateInvalidationWidgetIndex InReIndexTarget) : Range(InRange), ReIndexTarget(InReIndexTarget) {}
			[[nodiscard]] const FIndexRange& GetRange() const { return Range; }
			[[nodiscard]] FSlateInvalidationWidgetIndex ReIndex(FSlateInvalidationWidgetIndex Index) const;
		private:
			const FIndexRange& Range;
			FSlateInvalidationWidgetIndex ReIndexTarget = FSlateInvalidationWidgetIndex::Invalid;
		};
		struct FReSortOperation
		{
			FReSortOperation(const FIndexRange& InRange) : Range(InRange) {}
			[[nodiscard]] const FIndexRange& GetRange() const { return Range; }
		private:
			const FIndexRange& Range;
		};

		/** Widget proxies that will be removed and will not be valid anymore. */
		virtual void PreChildRemove(const FIndexRange& Range) {}
		/** Widget proxies that got moved/re-indexed by the operation. */
		virtual void ProxiesReIndexed(const FReIndexOperation& Operation) {}
		/** Widget proxies that got resorted by the operation. */
		virtual void ProxiesPreResort(const FReSortOperation& Operation) {}
		/** Widget proxies that got resorted by the operation. */
		virtual void ProxiesPostResort() {}
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_CHILDORDERCHECK
		/** Widget proxies built by the operation. */
		virtual void ProxiesBuilt(const FIndexRange& Range) {}
#endif
	};

	/**
	 * Process widget that have a ChildOrder invalidation.
	 * @note the Invalidation may break the reference. You shouldn't use the InvalidationWidget after this point.
	 * @returns true if the WidgetIndex is still valid.
	 */
	bool ProcessChildOrderInvalidation(FSlateInvalidationWidgetIndex WidgetIndex, IProcessChildOrderInvalidationCallback& Callback);


	/** Test, then adds or removes from the registered attribute list. */
	void ProcessAttributeRegistrationInvalidation(const InvalidationWidgetType& InvalidationWidget);

	/** Test, then adds or removes from the volatile update list. */
	void ProcessVolatileUpdateInvalidation(InvalidationWidgetType& InvalidationWidget);

	/** Performs an operation on all SWidget in the list. */
	template<typename Predicate>
	void ForEachWidget(Predicate Pred)
	{
		int32 ArrayIndex = FirstArrayIndex;
		while (ArrayIndex != INDEX_NONE)
		{
			ElementListType& ElementList = Data[ArrayIndex].ElementList;
			const int32 ElementNum = ElementList.Num();
			for (int32 ElementIndex = Data[ArrayIndex].StartIndex; ElementIndex < ElementNum; ++ElementIndex)
			{
				if (SWidget* Widget = ElementList[ElementIndex].GetWidget())
				{
					Pred(*Widget);
				}
			}

			ArrayIndex = Data[ArrayIndex].NextArrayIndex;
		}
	}

	/** Performs an operation on all SWidget in the list. */
	template<typename Predicate>
	void ForEachWidget(Predicate Pred) const
	{
		int32 ArrayIndex = FirstArrayIndex;
		while (ArrayIndex != INDEX_NONE)
		{
			const ElementListType& ElementList = Data[ArrayIndex].ElementList;
			const int32 ElementNum = ElementList.Num();
			for (int32 ElementIndex = Data[ArrayIndex].StartIndex; ElementIndex < ElementNum; ++ElementIndex)
			{
				if (const SWidget* Widget = ElementList[ElementIndex].GetWidget())
				{
					Pred(*Widget);
				}
			}

			ArrayIndex = Data[ArrayIndex].NextArrayIndex;
		}
	}

	/** Performs an operation on all InvalidationWidget in the list. */
	template<typename Predicate>
	void ForEachInvalidationWidget(Predicate Pred)
	{
		int32 ArrayIndex = FirstArrayIndex;
		while (ArrayIndex != INDEX_NONE)
		{
			ElementListType& ElementList = Data[ArrayIndex].ElementList;
			const int32 ElementNum = ElementList.Num();
			for (int32 ElementIndex = Data[ArrayIndex].StartIndex; ElementIndex < ElementNum; ++ElementIndex)
			{
				Pred(ElementList[ElementIndex]);
			}

			ArrayIndex = Data[ArrayIndex].NextArrayIndex;
		}
	}

	/** Performs an operation on all InvalidationWidget in the list. */
	template<typename Predicate>
	void ForEachInvalidationWidget(Predicate Pred) const
	{
		int32 ArrayIndex = FirstArrayIndex;
		while (ArrayIndex != INDEX_NONE)
		{
			const ElementListType& ElementList = Data[ArrayIndex].ElementList;
			const int32 ElementNum = ElementList.Num();
			for (int32 ElementIndex = Data[ArrayIndex].StartIndex; ElementIndex < ElementNum; ++ElementIndex)
			{
				Pred(ElementList[ElementIndex]);
			}

			ArrayIndex = Data[ArrayIndex].NextArrayIndex;
		}
	}

	/** Performs an operation on all InvalidationWidget in the list bellow the provided (WidgetIndex not including it). */
	template<typename Predicate>
	void ForEachInvalidationWidget(FSlateInvalidationWidgetIndex BellowWidgetIndex, Predicate Pred)
	{
		check(IsValidIndex(BellowWidgetIndex));
		const InvalidationWidgetType& BeginInvalidationWidget = (*this)[BellowWidgetIndex];
		const FSlateInvalidationWidgetIndex EndWidgetIndex = BeginInvalidationWidget.LeafMostChildIndex;
		if (BellowWidgetIndex == EndWidgetIndex)
		{
			return;
		}

		const FSlateInvalidationWidgetIndex BeginWidgetIndex = IncrementIndex(BellowWidgetIndex);
		if (BeginWidgetIndex != FSlateInvalidationWidgetIndex::Invalid)
		{
			const bool bSameWidgetArrayIndex = BeginWidgetIndex.ArrayIndex == EndWidgetIndex.ArrayIndex;
			int32 ArrayIndex = BeginWidgetIndex.ArrayIndex;
			{
				ElementListType& ElementList = Data[ArrayIndex].ElementList;
				const int32 ElementNum = bSameWidgetArrayIndex ? EndWidgetIndex.ElementIndex + 1 : ElementList.Num();
				for (int32 ElementIndex = BeginWidgetIndex.ElementIndex; ElementIndex < ElementNum; ++ElementIndex)
				{
					Pred(ElementList[ElementIndex]);
				}

				ArrayIndex = Data[ArrayIndex].NextArrayIndex;
			}

			if (!bSameWidgetArrayIndex)
			{
				while (ArrayIndex != EndWidgetIndex.ArrayIndex)
				{
					ElementListType& ElementList = Data[ArrayIndex].ElementList;
					const int32 ElementNum = ElementList.Num();
					for (int32 ElementIndex = Data[ArrayIndex].StartIndex; ElementIndex < ElementNum; ++ElementIndex)
					{
						Pred(ElementList[ElementIndex]);
					}

					ArrayIndex = Data[ArrayIndex].NextArrayIndex;
				}

				{
					check(ArrayIndex == EndWidgetIndex.ArrayIndex);
					ElementListType& ElementList = Data[ArrayIndex].ElementList;
					const int32 ElementNum = EndWidgetIndex.ElementIndex + 1;
					for (int32 ElementIndex = Data[ArrayIndex].StartIndex; ElementIndex < ElementNum; ++ElementIndex)
					{
						Pred(ElementList[ElementIndex]);
					}
				}
			}
		}
	}

	/** Iterator that goes over all the widgets with registered attribute. */
	struct FWidgetAttributeIterator
	{
	private:
		const FSlateInvalidationWidgetList& WidgetList;
		FSlateInvalidationWidgetIndex CurrentWidgetIndex;
		FSlateInvalidationWidgetSortOrder CurrentWidgetSortOrder;
		int32 AttributeIndex;

		FSlateInvalidationWidgetIndex MoveToWidgetIndexOnNextAdvance;
		/** A fix-up is required. MoveToWidgetIndexOnNextAdvance cannot be used since it may point to the last element (invalid) */
		bool bNeedsWidgetFixUp;

	public:
		~FWidgetAttributeIterator();
		FWidgetAttributeIterator(const FSlateInvalidationWidgetList& InWidgetList);
		FWidgetAttributeIterator(const FWidgetAttributeIterator&) = delete;
		FWidgetAttributeIterator& operator= (const FWidgetAttributeIterator&) = delete;

		//~ Handle operation
		UE_DEPRECATED(5.4, "PreChildRemove is deprecated. It was unused.")
		void PreChildRemove(const FIndexRange& Range);
		UE_DEPRECATED(5.4, "ReIndexed is deprecated. It was unused.")
		void ReIndexed(const IProcessChildOrderInvalidationCallback::FReIndexOperation& Operation);
		UE_DEPRECATED(5.4, "PostResort is deprecated. It was unused.")
		void PostResort();
		UE_DEPRECATED(5.4, "ProxiesBuilt is deprecated. It was unused.")
		void ProxiesBuilt(const FIndexRange& Range);
		UE_DEPRECATED(5.4, "FixCurrentWidgetIndex is deprecated. It was unused.")
		void FixCurrentWidgetIndex();
		void Seek(FSlateInvalidationWidgetIndex SeekTo);

		/** Get the current widget index the iterator is pointing to. */
		FSlateInvalidationWidgetIndex GetCurrentIndex() const { return CurrentWidgetIndex; }
		
		/** Get the current widget sort order the iterator is pointing to. */
		FSlateInvalidationWidgetSortOrder GetCurrentSortOrder() const { return CurrentWidgetSortOrder; }

		/** Advance the iterator to the next valid widget index. */
		void Advance();

		/** Advance the iterator to the next valid widget index that is a child of this widget. */
		void AdvanceToNextSibling();

		/** Advance the iterator to the next valid widget index that is a sibling of this widget's parent. */
		void AdvanceToNextParent();

		/** Is the iterator pointing to a valid widget index. */
		[[nodiscard]] bool IsValid() const { return CurrentWidgetIndex != FSlateInvalidationWidgetIndex::Invalid; }

	private:
		void AdvanceArrayIndex(int32 ArrayIndex);
		void Clear();
	};

	FWidgetAttributeIterator CreateWidgetAttributeIterator() const
	{
		return FWidgetAttributeIterator(*this);
	}

public:
	/** Iterator that goes over all the widgets that needs to be updated every frame. */
	struct FWidgetVolatileUpdateIterator
	{
	private:
		const FSlateInvalidationWidgetList& WidgetList;
		FSlateInvalidationWidgetIndex CurrentWidgetIndex;
		int32 AttributeIndex;
		bool bSkipCollapsed;

	public:
		FWidgetVolatileUpdateIterator(const FSlateInvalidationWidgetList& InWidgetList, bool bInSkipCollapsed);

		/** Get the current widget index the iterator is pointing to. */
		FSlateInvalidationWidgetIndex GetCurrentIndex() const { return CurrentWidgetIndex; }

		/** Advance the iterator to the next valid widget index. */
		void Advance();

		/** Is the iterator pointing to a valid widget index. */
		[[nodiscard]] bool IsValid() const { return CurrentWidgetIndex != FSlateInvalidationWidgetIndex::Invalid; }

	private:
		void Internal_Advance();
		void SkipToNextExpend();
		void Seek(FSlateInvalidationWidgetIndex SeekTo);
		void AdvanceArray(int32 ArrayIndex);
	};

	FWidgetVolatileUpdateIterator CreateWidgetVolatileUpdateIterator(bool bSkipCollapsed) const
	{
		return FWidgetVolatileUpdateIterator(*this, bSkipCollapsed);
	}


public:
	/** Returns reference to element at give index. */
	InvalidationWidgetType& operator[](const FSlateInvalidationWidgetIndex Index)
	{
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_RANGECHECK
		check(IsValidIndex(Index));
#endif
		return Data[Index.ArrayIndex].ElementList[Index.ElementIndex];
	}

	/** Returns reference to element at give index. */
	const InvalidationWidgetType& operator[](const FSlateInvalidationWidgetIndex Index) const
	{
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_RANGECHECK
		check(IsValidIndex(Index));
#endif
		return Data[Index.ArrayIndex].ElementList[Index.ElementIndex];
	}

	/** Tests if index is in the WidgetList range. */
	bool IsValidIndex(const FSlateInvalidationWidgetIndex Index) const
	{
		if (Data.IsValidIndex(Index.ArrayIndex))
		{
			return Index.ElementIndex >= Data[Index.ArrayIndex].StartIndex && Index.ElementIndex < Data[Index.ArrayIndex].ElementList.Num();
		}
		return false;
	}

	/** Returns true if there is not element in the WidgetList. */
	[[nodiscard]] bool IsEmpty() const
	{
		return FirstArrayIndex == INDEX_NONE || Data[FirstArrayIndex].ElementList.Num() == 0;
	}

	/** Returns the first index from the WidgetList. */
	[[nodiscard]] FSlateInvalidationWidgetIndex FirstIndex() const
	{
		return FirstArrayIndex == INDEX_NONE
			? FSlateInvalidationWidgetIndex::Invalid
			: FSlateInvalidationWidgetIndex{ (IndexType)FirstArrayIndex, Data[FirstArrayIndex].StartIndex };
	}

	/** Returns the last index from the WidgetList. */
	[[nodiscard]] FSlateInvalidationWidgetIndex LastIndex() const
	{
		return LastArrayIndex == INDEX_NONE ? FSlateInvalidationWidgetIndex::Invalid : FSlateInvalidationWidgetIndex{ (IndexType)LastArrayIndex, (IndexType)(Data[LastArrayIndex].ElementList.Num() - 1) };
	}

	/** Increment a widget index to the next entry in the WidgetList. */
	[[nodiscard]] FSlateInvalidationWidgetIndex IncrementIndex(FSlateInvalidationWidgetIndex Index) const;

	/** Decrement a widget index to the next entry in the WidgetList. */
	[[nodiscard]] FSlateInvalidationWidgetIndex DecrementIndex(FSlateInvalidationWidgetIndex Index) const;

	/** Find the next widget index that share the same parent. Return Invalid if there is no sibling. */
	[[nodiscard]] FSlateInvalidationWidgetIndex FindNextSibling(FSlateInvalidationWidgetIndex WidgetIndex) const;

	/** Empties the WidgetList. */
	void Empty();

	/** Empties the WidgetList, but doesn't change the memory allocations. */
	void Reset();

public:
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	/** For testing purposes. Return the InvalidationWidgetIndex of the Widget within the InvalidationWidgetList. */
	FSlateInvalidationWidgetIndex FindWidget(const SWidget& Widget) const;

	/** For testing purposes. Use ProcessChildOrderInvalidation. */
	void RemoveWidget(const FSlateInvalidationWidgetIndex ToRemove);
	/** For testing purposes. Use ProcessChildOrderInvalidation. */
	void RemoveWidget(const SWidget& WidgetToRemove);

	/** For testing purposes. Use to test ProcessChildOrderInvalidation */
	[[nodiscard]] TArray<TSharedPtr<SWidget>> FindChildren(const SWidget& Widget) const;

	/**
	 * For testing purposes.
	 * The list may not be the same, but the InvalidationWidgetType must:
	 * (1) be in the same order
	 * (2) point the same SWidget (itself, parent, leaf)
	 */
	bool DeepCompare(const FSlateInvalidationWidgetList& Other) const;

	/** For testing purposes. Log the tree. */
	void LogWidgetsList(bool bOnlyVisible) const;

	/** For testing purposes. Verify that every widgets has the correct index. */
	bool VerifyWidgetsIndex() const;

	/** For testing purposes. Verify that every WidgetProxy has a valid SWidget. */
	bool VerifyProxiesWidget() const;

	/** For testing purposes. Verify that every the sorting order is increasing between widgets. */
	bool VerifySortOrder() const;

	/** For testing purposes. Verify that the ElementIndexList_ have valid indexes and are sorted. */
	bool VerifyElementIndexList() const;
#endif

public:
	bool ShouldDoRecursion(const SWidget& Widget) const
	{
		return !Widget.Advanced_IsInvalidationRoot() || IsEmpty();
	}
	bool ShouldDoRecursion(const TSharedRef<const SWidget>& Widget) const
	{
		return ShouldDoRecursion(Widget.Get());
	}
	static bool ShouldBeAdded(const SWidget& Widget)
	{
		return &Widget != &(SNullWidget::NullWidget.Get());
	}
	static bool ShouldBeAdded(const TSharedRef<const SWidget>& Widget)
	{
		return Widget != SNullWidget::NullWidget;
	}
	static bool ShouldBeAddedToAttributeList(const SWidget& Widget)
	{
		return Widget.HasRegisteredSlateAttribute() && Widget.IsAttributesUpdatesEnabled();
	}
	static bool ShouldBeAddedToAttributeList(const TSharedRef<const SWidget>& Widget)
	{
		return ShouldBeAddedToAttributeList(Widget.Get());
	}
	static bool HasVolatileUpdateFlags(EWidgetUpdateFlags UpdateFlags)
	{
		return EnumHasAnyFlags(UpdateFlags, EWidgetUpdateFlags::NeedsTick | EWidgetUpdateFlags::NeedsActiveTimerUpdate | EWidgetUpdateFlags::NeedsVolatilePaint | EWidgetUpdateFlags::NeedsVolatilePrepass);
	}
	static bool ShouldBeAddedToVolatileUpdateList(const SWidget& Widget)
	{
		return Widget.HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsTick | EWidgetUpdateFlags::NeedsActiveTimerUpdate | EWidgetUpdateFlags::NeedsVolatilePaint | EWidgetUpdateFlags::NeedsVolatilePrepass);
	}
	static bool ShouldBeAddedToVolatileUpdateList(const TSharedRef<const SWidget>& Widget)
	{
		return ShouldBeAddedToVolatileUpdateList(Widget.Get());
	}

private:
	template <typename... ArgsType>
	FSlateInvalidationWidgetIndex Emplace(ArgsType&&... Args)
	{
		const IndexType ArrayIndex = AddArrayNodeIfNeeded(true);
		const IndexType ElementIndex = (IndexType)Data[ArrayIndex].ElementList.Emplace(Forward<ArgsType>(Args)...);
		return FSlateInvalidationWidgetIndex{ ArrayIndex, ElementIndex };
	}
	template <typename... ArgsType>
	FSlateInvalidationWidgetIndex EmplaceInsertAfter(IndexType AfterArrayIndex, ArgsType&&... Args)
	{
		const IndexType ArrayIndex = InsertArrayNodeIfNeeded(AfterArrayIndex, true);
		const IndexType ElementIndex = (IndexType)Data[ArrayIndex].ElementList.Emplace(Forward<ArgsType>(Args)...);
		return FSlateInvalidationWidgetIndex{ ArrayIndex, ElementIndex };
	}

private:
	struct FCutResult
	{
		FCutResult() = default;

		/** Where in the previous array the reindexed element starts. */
		int32 OldElementIndexStart = INDEX_NONE;
	};
private:
	[[nodiscard]] IndexType AddArrayNodeIfNeeded(bool bReserveElementList);
	[[nodiscard]] IndexType InsertArrayNodeIfNeeded(IndexType AfterArrayIndex, bool bReserveElementList);
	[[nodiscard]] IndexType InsertDataNodeAfter(IndexType AfterIndex, bool bReserveElementList);
	void RemoveDataNode(IndexType Index);
	void RebuildOrderIndex(IndexType StartFrom);
	void UpdateParentLeafIndex(const InvalidationWidgetType& InvalidationWidget, FSlateInvalidationWidgetIndex OldIndex, FSlateInvalidationWidgetIndex NewIndex);
	FCutResult CutArray(const FSlateInvalidationWidgetIndex WhereToCut);

private:
	FSlateInvalidationWidgetIndex Internal_BuildWidgetList_Recursive(SWidget& Widget, FSlateInvalidationWidgetIndex ParentIndex, IndexType& LastestIndex, FSlateInvalidationWidgetVisibility ParentVisibility, bool bParentVolatile);
	void Internal_RebuildWidgetListTree(SWidget& Widget, int32 ChildAtIndex);
	using FFindChildrenElement = TPair<SWidget*, FSlateInvalidationWidgetIndex>;
	void Internal_FindChildren(FSlateInvalidationWidgetIndex WidgetIndex, TArray<FFindChildrenElement, FConcurrentLinearArrayAllocator>& Widgets) const;
	void Internal_RemoveRangeFromSameParent(const FIndexRange Range);
	FCutResult Internal_CutArray(const FSlateInvalidationWidgetIndex WhereToCut);

private:
	struct FArrayNode
	{
		FArrayNode() = default;
		int32 PreviousArrayIndex = INDEX_NONE;
		int32 NextArrayIndex = INDEX_NONE;
		int32 SortOrder = 0;
		IndexType StartIndex = 0; // The array may start further in the ElementList as a result of a split.
		ElementListType ElementList;
		WidgetListType ElementIndexList_WidgetWithRegisteredSlateAttribute;
		WidgetListType ElementIndexList_VolatileUpdateWidget;

		void RemoveElementIndexBiggerOrEqualThan(IndexType ElementIndex);
		void RemoveElementIndexBetweenOrEqualThan(IndexType StartElementIndex, IndexType EndElementIndex);
	};
	using ArrayNodeType = TSparseArray<FArrayNode>;

	FSlateInvalidationRootHandle Owner;
	ArrayNodeType Data;
	TWeakPtr<SWidget> Root;
	int32 FirstArrayIndex = INDEX_NONE;
	int32 LastArrayIndex = INDEX_NONE;
	mutable int32 NumberOfLock = 0;
	IProcessChildOrderInvalidationCallback* CurrentInvalidationCallback = nullptr;
	const FArguments WidgetListConfig;
};