// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastUpdate/SlateInvalidationWidgetSortOrder.h"

#include "FastUpdate/SlateInvalidationWidgetList.h"

#include <limits>

FSlateInvalidationWidgetSortOrder::FSlateInvalidationWidgetSortOrder(const FSlateInvalidationWidgetList& List, FSlateInvalidationWidgetIndex Index)
	: Order()
{
	// 22 bits for the order (array index)
	// 10 bits for the element index
	Order = (List.Data[Index.ArrayIndex].SortOrder << 10) + Index.ElementIndex;
}


FSlateInvalidationWidgetSortOrder FSlateInvalidationWidgetSortOrder::LimitMax()
{
	return FSlateInvalidationWidgetSortOrder{ std::numeric_limits<uint32>::max() };
}


FSlateInvalidationWidgetSortOrder FSlateInvalidationWidgetSortOrder::LimitMin()
{
	return FSlateInvalidationWidgetSortOrder{0};
}
