// Copyright Epic Games, Inc. All Rights Reserved.

#include "FastUpdate/WidgetProxy.h"
#include "Widgets/SWidget.h"
#include "Application/SlateApplicationBase.h"
#include "Rendering/DrawElements.h"
#include "Rendering/SlateRenderer.h"
#include "Widgets/SWindow.h"
#include "FastUpdate/SlateInvalidationWidgetHeap.h"
#include "FastUpdate/SlateInvalidationWidgetList.h"
#include "HAL/LowLevelMemStats.h"
#include "UObject/Package.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Types/ReflectionMetadata.h"
#include "Input/HittestGrid.h"
#include "Trace/SlateTrace.h"
#include "Widgets/SWidgetUtils.h"

const FSlateWidgetPersistentState FSlateWidgetPersistentState::NoState;

FWidgetProxy::FWidgetProxy(SWidget& InWidget)
#if UE_SLATE_WITH_WIDGETPROXY_WEAKPTR
	: Widget(InWidget.AsShared())
#else
	: Widget(&InWidget)
#endif
	, Index(FSlateInvalidationWidgetIndex::Invalid)
	, ParentIndex(FSlateInvalidationWidgetIndex::Invalid)
	, LeafMostChildIndex(FSlateInvalidationWidgetIndex::Invalid)
	, CurrentInvalidateReason(EInvalidateWidgetReason::None)
	, Visibility()
	, PrivateFlags(0)
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	, PrivateDebugFlags(0)
#endif
{
#if UE_SLATE_WITH_WIDGETPROXY_WIDGETTYPE
	WidgetName = GetWidget()->GetType();
#endif
}

TSharedPtr<SWidget> FWidgetProxy::GetWidgetAsShared() const
{
#if UE_SLATE_WITH_WIDGETPROXY_WEAKPTR
	return Widget.Pin();
#else
	return Widget ? Widget->AsShared() : TSharedPtr<SWidget>();
#endif
}

FWidgetProxy::FUpdateResult FWidgetProxy::Update(const FPaintArgs& PaintArgs, FSlateWindowElementList& OutDrawElements)
{
	TSharedPtr<SWidget> CurrentWidget = GetWidgetAsShared();
	check(Visibility.IsVisible());

	// If Outgoing layer id remains index none, there was no change
	FUpdateResult Result;

	if (!CurrentWidget.IsValid())
		return Result;
		
	if (CurrentWidget->HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsRepaint|EWidgetUpdateFlags::NeedsVolatilePaint))
	{
		Result = Repaint(PaintArgs, OutDrawElements);
	}
	else
	{
		UE_TRACE_SCOPED_SLATE_WIDGET_UPDATE(CurrentWidget.Get());
		EWidgetUpdateFlags PreviousUpdateFlag = CurrentWidget->UpdateFlags;
		if (CurrentWidget->HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsActiveTimerUpdate))
		{
			SCOPE_CYCLE_COUNTER(STAT_SlateExecuteActiveTimers);
			CurrentWidget->ExecuteActiveTimers(PaintArgs.GetCurrentTime(), PaintArgs.GetDeltaTime());

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
			bDebug_Updated = true;
#endif
		}

		if (CurrentWidget->HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsTick))
		{
			const FSlateWidgetPersistentState& MyState = CurrentWidget->GetPersistentState();

			INC_DWORD_STAT(STAT_SlateNumTickedWidgets);
			SCOPE_CYCLE_COUNTER(STAT_SlateTickWidgets);

			CurrentWidget->Tick(MyState.DesktopGeometry, PaintArgs.GetCurrentTime(), PaintArgs.GetDeltaTime());

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
			bDebug_Updated = true;
#endif
		}

		// If the widet was invalidated while ticking. In slow mode the widget would be painted right away.
		//For now postpone for the next frame. Enabling this code would cause more issues with the list management.
		//if (EnumHasAnyFlags(CurrentInvalidateReason, EInvalidateWidgetReason::Paint))
		//{
		//	EnumRemoveFlags(CurrentInvalidateReason, EInvalidateWidgetReason::Paint);
		//	const EWidgetUpdateFlags FlagToAddAfterRepaint = CurrentWidget->UpdateFlags & (EWidgetUpdateFlags::NeedsActiveTimerUpdate | EWidgetUpdateFlags::NeedsTick);
		//	EnumRemoveFlags(CurrentWidget->UpdateFlags, EWidgetUpdateFlags::NeedsActiveTimerUpdate | EWidgetUpdateFlags::NeedsTick);
		//	Result = Repaint(PaintArgs, OutDrawElements);
		//	EnumAddFlags(CurrentWidget->UpdateFlags, FlagToAddAfterRepaint);
		//}

#if WITH_SLATE_DEBUGGING
		FSlateDebugging::BroadcastWidgetUpdated(CurrentWidget.Get(), PreviousUpdateFlag);
#endif
	}

	return Result;
}

void FWidgetProxy::ProcessLayoutInvalidation(FSlateInvalidationWidgetPostHeap& UpdateList, FSlateInvalidationWidgetList& FastWidgetPathList, FSlateInvalidationRoot& Root)
{
	SWidget* WidgetPtr = GetWidget();
	SCOPE_CYCLE_SWIDGET(WidgetPtr);
#if UE_TRACE_ASSET_METADATA_ENABLED
	FName AssetName = NAME_None;
	FName ClassName = NAME_None;
	FName PackageName = NAME_None;
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(AssetMetadataChannel))
	{
		TSharedPtr<FReflectionMetaData> AssetMetaData = FReflectionMetaData::GetWidgetOrParentMetaData(WidgetPtr);
		if (AssetMetaData.IsValid())
		{
			if (const UObject* AssetPtr = AssetMetaData->Asset.Get())
			{
				AssetName = AssetMetaData->Name;
				ClassName = AssetMetaData->Class.Get()->GetFName();
				PackageName = AssetPtr->GetPackage()->GetFName();
			}
		}
	}
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(PackageName, ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(ClassName, ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(AssetName, ClassName, PackageName);
#endif
	// When layout changes compute a new desired size for this widget
	const FVector2f CurrentDesiredSize = WidgetPtr->GetDesiredSize();
	FVector2f NewDesiredSize = FVector2f::ZeroVector;
	if (!Visibility.IsCollapseDirectly())
	{
		if (WidgetPtr->NeedsPrepass())
		{
			WidgetPtr->SlatePrepass(WidgetPtr->GetPrepassLayoutScaleMultiplier());
		}
		else
		{
			WidgetPtr->CacheDesiredSize(WidgetPtr->GetPrepassLayoutScaleMultiplier());
		}

		NewDesiredSize = WidgetPtr->GetDesiredSize();
	}

	// Note even if volatile we need to recompute desired size. We do not need to invalidate parents though if they are volatile since they will naturally redraw this widget
	if (!WidgetPtr->IsVolatileIndirectly() && Visibility.IsVisible())
	{
		// Set the value directly instead of calling AddUpdateFlags as an optimization
		WidgetPtr->UpdateFlags |= EWidgetUpdateFlags::NeedsRepaint;
	}

	// If the desired size changed, invalidate the parent if it is visible
	if (NewDesiredSize != CurrentDesiredSize || EnumHasAnyFlags(CurrentInvalidateReason, EInvalidateWidgetReason::RenderTransform)) // NB Also, EInvalidateWidgetReason::Visibility. It use the same RenderTransform flag
	{
		if (ParentIndex != FSlateInvalidationWidgetIndex::Invalid)
		{
			FWidgetProxy& ParentProxy = FastWidgetPathList[ParentIndex];
			if (ParentIndex == FastWidgetPathList.FirstIndex())
			{
				// root of the invalidation panel just invalidate the whole thing
				Root.InvalidateRootLayout(WidgetPtr);
			}
			else if (!ParentProxy.Visibility.IsCollapseDirectly())
			{
				ParentProxy.CurrentInvalidateReason |= EInvalidateWidgetReason::Layout;
#if WITH_SLATE_DEBUGGING
				FSlateDebugging::BroadcastWidgetInvalidate(ParentProxy.GetWidget(), WidgetPtr, EInvalidateWidgetReason::Layout);
#endif
				UE_TRACE_SLATE_WIDGET_INVALIDATED(ParentProxy.GetWidget(), WidgetPtr, EInvalidateWidgetReason::Layout);
				UpdateList.PushBackOrHeapUnique(ParentProxy);
			}
		}
		else if (TSharedPtr<SWidget> ParentWidget = WidgetPtr->GetParentWidget())
		{
			ParentWidget->Invalidate(EInvalidateWidgetReason::Layout);
		}
	}
}

bool FWidgetProxy::ProcessPostInvalidation(FSlateInvalidationWidgetPostHeap& UpdateList, FSlateInvalidationWidgetList& FastWidgetPathList, FSlateInvalidationRoot& Root)
{
	bool bWidgetNeedsRepaint = false;
	SWidget* WidgetPtr = GetWidget();

	if (Visibility.IsVisible() && ParentIndex != FSlateInvalidationWidgetIndex::Invalid && !WidgetPtr->bPrepassLayoutScaleMultiplierSet)
	{
		SCOPE_CYCLE_SWIDGET(WidgetPtr);
		// If this widget has never been prepassed make sure the parent prepasses it to set the correct multiplier
		FWidgetProxy& ParentProxy = FastWidgetPathList[ParentIndex];
		if (SWidget* ParentWidgetPtr = ParentProxy.GetWidget())
		{
			ParentWidgetPtr->MarkPrepassAsDirty();
			ParentProxy.CurrentInvalidateReason |= EInvalidateWidgetReason::Layout;
#if WITH_SLATE_DEBUGGING
			FSlateDebugging::BroadcastWidgetInvalidate(ParentWidgetPtr, WidgetPtr, EInvalidateWidgetReason::Layout);
#endif
			UE_TRACE_SLATE_WIDGET_INVALIDATED(ParentWidgetPtr, WidgetPtr, EInvalidateWidgetReason::Layout);
			UpdateList.HeapPushUnique(ParentProxy);
		}
		bWidgetNeedsRepaint = true;
	}
	else if (EnumHasAnyFlags(CurrentInvalidateReason, EInvalidateWidgetReason::RenderTransform | EInvalidateWidgetReason::Layout)) // NB Also, EInvalidateWidgetReason::Visibility. It use the same RenderTransform flag
	{
		ProcessLayoutInvalidation(UpdateList, FastWidgetPathList, Root);
		
		bWidgetNeedsRepaint = true;
	}
	else if (EnumHasAnyFlags(CurrentInvalidateReason, EInvalidateWidgetReason::Paint) && !WidgetPtr->IsVolatileIndirectly())
	{
		SCOPE_CYCLE_SWIDGET(WidgetPtr);
		// Set the value directly instead of calling AddUpdateFlags as an optimization
		WidgetPtr->UpdateFlags |= EWidgetUpdateFlags::NeedsRepaint;

		bWidgetNeedsRepaint = true;
	}

	return bWidgetNeedsRepaint;
}

void FWidgetProxy::MarkProxyUpdatedThisFrame(FSlateInvalidationWidgetPostHeap& PostUpdateList)
{
	SWidget* WidgetPtr = GetWidget();
	if (WidgetPtr && Visibility.IsVisible() && WidgetPtr->HasAnyUpdateFlags(EWidgetUpdateFlags::AnyUpdate))
	{
		// If there are any updates still needed add them to the next update list
		PostUpdateList.PushBackUnique(*this);
	}
}

FWidgetProxy::FUpdateResult FWidgetProxy::Repaint(const FPaintArgs& PaintArgs, FSlateWindowElementList& OutDrawElements) const
{
	SWidget* WidgetPtr = GetWidget();
	check(WidgetPtr);

	const FSlateWidgetPersistentState& MyState = WidgetPtr->GetPersistentState();

	const int32 StartingClipIndex = OutDrawElements.GetClippingIndex();

	// Get the clipping manager into the correct state
	const bool bNeedsNewClipState = MyState.InitialClipState.IsSet();
	if (bNeedsNewClipState)
	{
		OutDrawElements.GetClippingManager().PushClippingState(MyState.InitialClipState.GetValue());
	}
	
	const int32 PrevUserIndex = PaintArgs.GetHittestGrid().GetUserIndex();

	PaintArgs.GetHittestGrid().SetUserIndex(MyState.IncomingUserIndex);

	FSlateRenderer* Renderer = FSlateApplicationBase::Get().GetRenderer();
	const int32 PrevSceneIndex = Renderer->GetCurrentSceneIndex();
	Renderer->SetCurrentSceneIndex(MyState.IncomingSceneIndex);

	TGuardValue<EFlowDirection> FlowGuard(GSlateFlowDirection, MyState.IncomingFlowDirection);
	
	FPaintArgs UpdatedArgs = PaintArgs.WithNewParent(MyState.PaintParent.Pin().Get());
	UpdatedArgs.SetInheritedHittestability(MyState.bInheritedHittestability);
	UpdatedArgs.SetDeferredPaint(MyState.bDeferredPainting);

	int32 PrevLayerId = MyState.OutgoingLayerId;

	if (GSlateEnableGlobalInvalidation)
	{
		if (WidgetPtr->HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePaint))
		{
			// Todo: this should be deprecated in favor of NeedsVolatilePrepass
			if (WidgetPtr->ShouldInvalidatePrepassDueToVolatility())
			{
				WidgetPtr->MarkPrepassAsDirty();
				WidgetPtr->SlatePrepass(WidgetPtr->GetPrepassLayoutScaleMultiplier());
			}
		}
	}

	if (MyState.InitialPixelSnappingMethod != EWidgetPixelSnapping::Inherit)
	{
		OutDrawElements.PushPixelSnappingMethod(MyState.InitialPixelSnappingMethod);
	}
	
	const int32 NewLayerId = WidgetPtr->Paint(UpdatedArgs, MyState.AllottedGeometry, MyState.CullingBounds, OutDrawElements, MyState.LayerId, MyState.WidgetStyle, MyState.bParentEnabled);

	PaintArgs.GetHittestGrid().SetUserIndex(PrevUserIndex);
	Renderer->SetCurrentSceneIndex(PrevSceneIndex);

	if (bNeedsNewClipState)
	{
		OutDrawElements.PopClip();
		// clip index should be what it was before.  if this assert fails something internal inside the above paint call did not pop clip properly
		check(StartingClipIndex == OutDrawElements.GetClippingIndex());
	}
	
	if (MyState.InitialPixelSnappingMethod != EWidgetPixelSnapping::Inherit)
	{
		OutDrawElements.PopPixelSnappingMethod();
	}

	OutDrawElements.SetIsInProjectContent(MyState.IsProjectContent());
	return FUpdateResult{ PrevLayerId, NewLayerId };
}

FWidgetProxyHandle::FWidgetProxyHandle(const FSlateInvalidationRootHandle& InInvalidationRoot, FSlateInvalidationWidgetIndex InIndex, FSlateInvalidationWidgetSortOrder InSortIndex)
	: InvalidationRootHandle(InInvalidationRoot)
	, WidgetIndex(InIndex)
	, WidgetSortOrder(InSortIndex)
{
}

FWidgetProxyHandle::FWidgetProxyHandle(FSlateInvalidationWidgetIndex InIndex)
	: InvalidationRootHandle()
	, WidgetIndex(InIndex)
	, WidgetSortOrder()
{
}

bool FWidgetProxyHandle::IsValid(const SWidget& Widget) const
{
	FSlateInvalidationRoot* InvalidationRoot = InvalidationRootHandle.GetInvalidationRoot();
	return InvalidationRoot
		&& InvalidationRoot->GetFastPathWidgetList().IsValidIndex(WidgetIndex)
		&& InvalidationRoot->GetFastPathWidgetList()[WidgetIndex].GetWidget() == &Widget;
}

bool FWidgetProxyHandle::IsValid(const SWidget* Widget) const
{
	check(Widget);
	return IsValid(*Widget);
}

FSlateInvalidationWidgetVisibility FWidgetProxyHandle::GetWidgetVisibility(const SWidget* Widget) const
{
	if (IsValid(Widget))
	{
		return GetProxy().Visibility;
	}
	return FSlateInvalidationWidgetVisibility();
}

bool FWidgetProxyHandle::HasAllInvalidationReason(const SWidget* Widget, EInvalidateWidgetReason Reason) const
{
	if (IsValid(Widget))
	{
		return EnumHasAllFlags(GetProxy().CurrentInvalidateReason, Reason);
	}
	return false;
}

bool FWidgetProxyHandle::HasAnyInvalidationReason(const SWidget* Widget, EInvalidateWidgetReason Reason) const
{
	if (IsValid(Widget))
	{
		return EnumHasAnyFlags(GetProxy().CurrentInvalidateReason, Reason);
	}
	return false;
}

FWidgetProxy& FWidgetProxyHandle::GetProxy()
{
	return GetInvalidationRoot_NoCheck()->GetFastPathWidgetList()[WidgetIndex];
}

const FWidgetProxy& FWidgetProxyHandle::GetProxy() const
{
	return GetInvalidationRoot_NoCheck()->GetFastPathWidgetList()[WidgetIndex];
}

void FWidgetProxyHandle::MarkWidgetDirty_NoCheck(EInvalidateWidgetReason InvalidateReason)
{
	FWidgetProxy& Proxy = GetInvalidationRoot_NoCheck()->GetFastPathWidgetList()[WidgetIndex];
	GetInvalidationRoot_NoCheck()->InvalidateWidget(Proxy, InvalidateReason);
}

void FWidgetProxyHandle::MarkWidgetDirty_NoCheck(FWidgetProxy& Proxy)
{
	GetInvalidationRoot_NoCheck()->InvalidateWidget(Proxy, Proxy.CurrentInvalidateReason);
}

void FWidgetProxyHandle::UpdateWidgetFlags(const SWidget* Widget, EWidgetUpdateFlags PreviousFlags, EWidgetUpdateFlags NewFlags)
{
	if (PreviousFlags != NewFlags)
	{
		if ( FSlateInvalidationWidgetList::HasVolatileUpdateFlags(PreviousFlags) != FSlateInvalidationWidgetList::HasVolatileUpdateFlags(NewFlags)
			|| EnumHasAnyFlags(PreviousFlags, EWidgetUpdateFlags::NeedsVolatilePrepass) != EnumHasAnyFlags(NewFlags, EWidgetUpdateFlags::NeedsVolatilePrepass))
		{
			if (IsValid(Widget))
			{
				FWidgetProxy& Proxy = GetProxy();
				GetInvalidationRoot_NoCheck()->GetFastPathWidgetList().ProcessVolatileUpdateInvalidation(Proxy);
			}
		}
	}
}
