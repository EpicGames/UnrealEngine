// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/CompositeFont.h"
#include "Fonts/FontCache.h"
#include "SlateGlobals.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Fonts/FontFaceInterface.h"
#include "Fonts/PreprocessedFontGeometry.h"
#include "Templates/Casts.h"
#include "Fonts/FontBulkData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CompositeFont)

#include <limits>

// The total true type memory we are using for resident font faces
DECLARE_MEMORY_STAT(TEXT("Resident Font Memory (TTF/OTF)"), STAT_SlateRawFontDataMemory, STATGROUP_SlateMemory);

FFontFaceData::FFontFaceData()
{
}

FFontFaceData::FFontFaceData(TArray<uint8>&& InData)
	: Data(MoveTemp(InData))
{
	TrackMemoryUsage();
}

FFontFaceData::~FFontFaceData()
{
	UntrackMemoryUsage();
}

void FFontFaceData::SetData(TArray<uint8>&& InData)
{
	UntrackMemoryUsage();
	Data = MoveTemp(InData);
	PreprocessedFontGeometries.Reset();
	TrackMemoryUsage();
}

void FFontFaceData::SetPreprocessedFontGeometries(TArray<UE::Slate::FPreprocessedFontGeometry>&& InPreprocessedFontGeometries)
{
	UntrackMemoryUsage();
	PreprocessedFontGeometries = MoveTemp(InPreprocessedFontGeometries);
	TrackMemoryUsage();
}

SIZE_T FFontFaceData::GetAllocatedSize() const
{
	SIZE_T TotalSize = Data.GetAllocatedSize()+PreprocessedFontGeometries.GetAllocatedSize();
	for (const UE::Slate::FPreprocessedFontGeometry &SubFaceGeometry : PreprocessedFontGeometries)
	{
		TotalSize += SubFaceGeometry.GetAllocatedSize();
	}
	return TotalSize;
}

void FFontFaceData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		UntrackMemoryUsage();
	}
	Ar << Data;
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AddedPreprocessedFontGeometry)
	{
		Ar << PreprocessedFontGeometries;
	}
	else if (Ar.IsLoading())
	{
		PreprocessedFontGeometries.Reset();
	}
	if (Ar.IsLoading())
	{
		TrackMemoryUsage();
	}
}

void FFontFaceData::TrackMemoryUsage() const
{
	INC_DWORD_STAT_BY(STAT_SlateRawFontDataMemory, GetAllocatedSize());
}

void FFontFaceData::UntrackMemoryUsage() const
{
	DEC_DWORD_STAT_BY(STAT_SlateRawFontDataMemory, GetAllocatedSize());
}

FFontData::FFontData()
	: FontFilename()
	, Hinting(EFontHinting::Default)
	, LoadingPolicy(EFontLoadingPolicy::LazyLoad)
	, SubFaceIndex(0)
	, FontFaceAsset(nullptr)
#if WITH_EDITORONLY_DATA
	, BulkDataPtr_DEPRECATED(nullptr)
	, FontData_DEPRECATED()
#endif // WITH_EDITORONLY_DATA
{
	FontFilenameHash = GetTypeHash(FontFilename);
}

FFontData::FFontData(const UObject* const InFontFaceAsset, const int32 InSubFaceIndex)
	: FontFilename()
	, Hinting(EFontHinting::Default)
	, LoadingPolicy(EFontLoadingPolicy::LazyLoad)
	, SubFaceIndex(InSubFaceIndex)
	, FontFaceAsset(InFontFaceAsset)
#if WITH_EDITORONLY_DATA
	, BulkDataPtr_DEPRECATED(nullptr)
	, FontData_DEPRECATED()
#endif // WITH_EDITORONLY_DATA
{
	FontFilenameHash = GetTypeHash(FontFilename);
	if (FontFaceAsset)
	{
		CastChecked<const IFontFaceInterface>(FontFaceAsset);
	}
}

FFontData::FFontData(FString InFontFilename, const EFontHinting InHinting, const EFontLoadingPolicy InLoadingPolicy, const int32 InSubFaceIndex)
	: FontFilename(MoveTemp(InFontFilename))
	, Hinting(InHinting)
	, LoadingPolicy(InLoadingPolicy)
	, SubFaceIndex(InSubFaceIndex)
	, FontFaceAsset(nullptr)
#if WITH_EDITORONLY_DATA
	, BulkDataPtr_DEPRECATED(nullptr)
	, FontData_DEPRECATED()
#endif // WITH_EDITORONLY_DATA
{
	check(InLoadingPolicy != EFontLoadingPolicy::Inline);
	FontFilenameHash = GetTypeHash(FontFilename);
}

bool FFontData::HasFont() const
{
	FFontFaceDataConstPtr FontFaceData = GetFontFaceData();
	return (FontFaceData.IsValid() && FontFaceData->HasData()) || (!GetFontFilename().IsEmpty());
}

const FString& FFontData::GetFontFilename() const
{
	if (FontFaceAsset)
	{
		const IFontFaceInterface* FontFace = CastChecked<const IFontFaceInterface>(FontFaceAsset);
		return FontFace->GetFontFilename();
	}
	return FontFilename;
}

EFontHinting FFontData::GetHinting() const
{
	if (FontFaceAsset)
	{
		const IFontFaceInterface* FontFace = CastChecked<const IFontFaceInterface>(FontFaceAsset);
		return FontFace->GetHinting();
	}
	return Hinting;
}

EFontLoadingPolicy FFontData::GetLoadingPolicy() const
{
	if (FontFaceAsset)
	{
		const IFontFaceInterface* FontFace = CastChecked<const IFontFaceInterface>(FontFaceAsset);
		return FontFace->GetLoadingPolicy();
	}
	return LoadingPolicy;
}

int32 FFontData::GetSubFaceIndex() const
{
	return SubFaceIndex;
}

void FFontData::SetSubFaceIndex(const int32 InSubFaceIndex)
{
	SubFaceIndex = InSubFaceIndex;
}

EFontLayoutMethod FFontData::GetLayoutMethod() const
{
	if (FontFaceAsset)
	{
		const IFontFaceInterface* FontFace = CastChecked<const IFontFaceInterface>(FontFaceAsset);
		return FontFace->GetLayoutMethod();
	}
	return EFontLayoutMethod::Metrics;
}

bool FFontData::IsAscendOverridden() const
{
	if (FontFaceAsset)
	{
		const IFontFaceInterface* FontFace = CastChecked<const IFontFaceInterface>(FontFaceAsset);
		return FontFace->IsAscendOverridden();
	}
	return false;
}

int32 FFontData::GetAscendOverriddenValue() const
{
	if (FontFaceAsset)
	{
		const IFontFaceInterface* FontFace = CastChecked<const IFontFaceInterface>(FontFaceAsset);
		return FontFace->GetAscendOverriddenValue();
	}
	return 0;
}

bool FFontData::IsDescendOverridden() const
{
	if (FontFaceAsset)
	{
		const IFontFaceInterface* FontFace = CastChecked<const IFontFaceInterface>(FontFaceAsset);
		return FontFace->IsDescendOverridden();
	}
	return false;
}

int32 FFontData::GetDescendOverriddenValue() const
{
	if (FontFaceAsset)
	{
		const IFontFaceInterface* FontFace = CastChecked<const IFontFaceInterface>(FontFaceAsset);
		return FontFace->GetDescendOverriddenValue();
	}
	return 0;
}

int32 FFontData::GetStrikeBrushHeightPercentage() const
{
	if (FontFaceAsset)
	{
		const IFontFaceInterface* FontFace = CastChecked<const IFontFaceInterface>(FontFaceAsset);
		return FontFace->GetStrikeBrushHeightPercentage();
	}
	return 60;
}

FFontFaceDataConstPtr FFontData::GetFontFaceData() const
{
	if (FontFaceAsset)
	{
		const IFontFaceInterface* FontFace = CastChecked<const IFontFaceInterface>(FontFaceAsset);
		return FontFace->GetFontFaceData();
	}
	return nullptr;
}

const UObject* FFontData::GetFontFaceAsset() const
{
	return FontFaceAsset;
}

FFontRasterizationSettings FFontData::GetFontRasterizationSettings() const
{
	if (FontFaceAsset && IsSlateSdfTextFeatureEnabled())
	{
		const IFontFaceInterface* FontFace = CastChecked<const IFontFaceInterface>(FontFaceAsset);
		return FontFace->GetRasterizationSettings();
	}
	return FFontRasterizationSettings();
}

#if WITH_EDITORONLY_DATA
bool FFontData::HasLegacyData() const
{
	return FontData_DEPRECATED.Num() > 0 || BulkDataPtr_DEPRECATED;
}

void FFontData::ConditionalUpgradeFontDataToBulkData(UObject* InOuter)
{
	if (FontData_DEPRECATED.Num() > 0)
	{
		UFontBulkData* NewBulkData = NewObject<UFontBulkData>(InOuter);
		BulkDataPtr_DEPRECATED = NewBulkData;

		NewBulkData->Initialize(FontData_DEPRECATED.GetData(), FontData_DEPRECATED.Num());
		FontData_DEPRECATED.Empty();
	}
}

void FFontData::ConditionalUpgradeBulkDataToFontFace(UObject* InOuter, UClass* InFontFaceClass, const FName InFontFaceName)
{
	if (BulkDataPtr_DEPRECATED)
	{
		int64 RawBulkDataSizeBytes = 0;
		const void* RawBulkData = BulkDataPtr_DEPRECATED->Lock(RawBulkDataSizeBytes);
		if (RawBulkDataSizeBytes > 0)
		{
			UObject* NewFontFaceAsset = NewObject<UObject>(InOuter, InFontFaceClass, InFontFaceName);
			FontFaceAsset = NewFontFaceAsset;

			IFontFaceInterface* NewFontFace = CastChecked<IFontFaceInterface>(NewFontFaceAsset);
			ensure(RawBulkDataSizeBytes >= 0 && RawBulkDataSizeBytes <= std::numeric_limits<int32>::max());
			NewFontFace->InitializeFromBulkData(FontFilename, Hinting, RawBulkData, (int32)RawBulkDataSizeBytes);
		}
		BulkDataPtr_DEPRECATED->Unlock();
		BulkDataPtr_DEPRECATED = nullptr;
	}
}
#endif // WITH_EDITORONLY_DATA

bool FFontData::operator==(const FFontData& Other) const
{
	// Skip the entire comparison if we are the same object
	// This avoids costly stricmp on FontFilename when FontFaceAsset is null.
	if (this == &Other)
	{
		return true;
	}

	if (FontFaceAsset != Other.FontFaceAsset)
	{
		// Using different assets
		return false;
	}

	if (FontFaceAsset && FontFaceAsset == Other.FontFaceAsset)
	{
		// Using the same asset
		return true;
	}

	// Compare inline properties
	return FontFilenameHash == Other.FontFilenameHash 
		&& Hinting == Other.Hinting
		&& LoadingPolicy == Other.LoadingPolicy
		&& SubFaceIndex == Other.SubFaceIndex
		&& FontFilename == Other.FontFilename;
}

bool FFontData::operator!=(const FFontData& Other) const
{
	return !(*this == Other);
}

bool FFontData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);

	if (Ar.CustomVer(FEditorObjectVersion::GUID) < FEditorObjectVersion::AddedFontFaceAssets)
	{
		// Too old, so use the default serialization
		return false;
	}

	bool bIsCooked = Ar.IsCooking();
	Ar << bIsCooked;

	if (bIsCooked)
	{
		// Cooked data uses a more compact format
		UObject* LocalFontFaceAsset = const_cast<UObject*>(ToRawPtr(FontFaceAsset));
		Ar << LocalFontFaceAsset;
		FontFaceAsset = LocalFontFaceAsset;
		
		if (!FontFaceAsset)
		{
			// Only need to serialize the other properties when we lack a font face asset
			Ar << FontFilename;
			Ar << Hinting;
			Ar << LoadingPolicy;
		}

		Ar << SubFaceIndex;
	}
	else
	{
		// Uncooked data uses the standard struct serialization to avoid versioning on editor-only data
		UScriptStruct* FontDataStruct = StaticStruct();
		if (FontDataStruct->UseBinarySerialization(Ar))
		{
			FontDataStruct->SerializeBin(Ar, (uint8*)this);
		}
		else
		{
			FontDataStruct->SerializeTaggedProperties(Ar, (uint8*)this, FontDataStruct, nullptr);
		}
	}

	FontFilenameHash = GetTypeHash(FontFilename);
	return true;
}

void FFontData::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(FontFaceAsset);
#if WITH_EDITORONLY_DATA
	Collector.AddReferencedObject(BulkDataPtr_DEPRECATED);
#endif // WITH_EDITORONLY_DATA
}

void FStandaloneCompositeFont::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (FTypefaceEntry& TypefaceEntry : DefaultTypeface.Fonts)
	{
		TypefaceEntry.Font.AddReferencedObjects(Collector);
	}

	for (FCompositeSubFont& SubFont : SubTypefaces)
	{
		for (FTypefaceEntry& TypefaceEntry : SubFont.Typeface.Fonts)
		{
			TypefaceEntry.Font.AddReferencedObjects(Collector);
		}
	}
}

FString FStandaloneCompositeFont::GetReferencerName() const
{
	return TEXT("FStandaloneCompositeFont");
}

