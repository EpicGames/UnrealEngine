// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/FontBulkData.h"
#include "HAL/FileManager.h"
#include "Misc/ScopeLock.h"
#include "SlateGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FontBulkData)

#include <limits>

// The total amount of memory we are using to store raw font bytes in bulk data
DECLARE_MEMORY_STAT(TEXT("Font BulkData Memory"), STAT_SlateBulkFontDataMemory, STATGROUP_SlateMemory);

UFontBulkData::UFontBulkData()
{
	
}

void UFontBulkData::Initialize(const FString& InFontFilename)
{
	// The bulk data cannot be removed if we are loading from source file
	BulkData.ClearBulkDataFlags( BULKDATA_SingleUse );
	
	TUniquePtr<FArchive> Reader(IFileManager::Get().CreateFileReader(*InFontFilename, 0));
	if(Reader)
	{
		const int64 FontDataSizeBytes = Reader->TotalSize();

		BulkData.Lock(LOCK_READ_WRITE);
		void* const LockedFontData = BulkData.Realloc(FontDataSizeBytes);
		Reader->Serialize(LockedFontData, FontDataSizeBytes);
		BulkData.Unlock();
		INC_DWORD_STAT_BY( STAT_SlateBulkFontDataMemory, BulkData.GetBulkDataSize() );
	}
	else
	{
		UE_LOGF(LogSlate, Warning, "Failed to load font data from '%ls'", *InFontFilename);
	}
}

void UFontBulkData::Initialize(const void* const InFontData, const int64 InFontDataSizeBytes)
{
	// The bulk data cannot be removed if we are loading a memory location since we 
	// have no knowledge of this memory later
	BulkData.ClearBulkDataFlags( BULKDATA_SingleUse );

	BulkData.Lock(LOCK_READ_WRITE);
	void* const LockedFontData = BulkData.Realloc(InFontDataSizeBytes);
	FMemory::Memcpy(LockedFontData, InFontData, InFontDataSizeBytes);
	BulkData.Unlock();
	
	INC_DWORD_STAT_BY( STAT_SlateBulkFontDataMemory, BulkData.GetBulkDataSize() );
}

const void* UFontBulkData::Lock(int64& OutFontDataSizeBytes) const
{
	CriticalSection.Lock();

#if STATS
	bool bWasLoaded = BulkData.IsBulkDataLoaded();
#endif

	OutFontDataSizeBytes = BulkData.GetBulkDataSize();

	const void* Data = BulkData.LockReadOnly();

#if STATS
	if( !bWasLoaded && BulkData.IsBulkDataLoaded() )
	{
		INC_DWORD_STAT_BY( STAT_SlateBulkFontDataMemory, BulkData.GetBulkDataSize() );
	}
#endif

	return Data;
}

void UFontBulkData::Unlock()
{
	bool bWasLoaded = BulkData.IsBulkDataLoaded();
	int64 BulkDataSize = BulkData.GetBulkDataSize(); 
	BulkData.Unlock();

#if STATS
	if( bWasLoaded && !BulkData.IsBulkDataLoaded() )
	{
		DEC_DWORD_STAT_BY( STAT_SlateBulkFontDataMemory, BulkDataSize );
	}
#endif
	CriticalSection.Unlock();
}

int64 UFontBulkData::GetBulkDataSize() const
{
	return BulkData.GetBulkDataSize();
}

void UFontBulkData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsSaving())
	{
		BulkData.SetBulkDataFlags(BULKDATA_SerializeCompressed);
	}
	
	BulkData.Serialize(Ar, this, INDEX_NONE, false);

	if( !GIsEditor && Ar.IsLoading() )
	{
		BulkData.SetBulkDataFlags( BULKDATA_SingleUse );
	}

#if STATS
	if( Ar.IsLoading() && BulkData.IsBulkDataLoaded() )
	{
		INC_DWORD_STAT_BY( STAT_SlateBulkFontDataMemory, BulkData.GetBulkDataSize() );
	}
#endif
}

