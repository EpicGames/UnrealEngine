// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/FontCache.h"
#include "Misc/ScopeLock.h"
#include "HAL/IConsoleManager.h"
#include "Algo/Find.h"
#include "Application/SlateApplicationBase.h"
#include "Internationalization/TextChar.h"
#include "Fonts/FontCacheFreeType.h"
#include "Fonts/FontCacheHarfBuzz.h"
#include "Fonts/FontCacheCompositeFont.h"
#include "Fonts/SlateFontRenderer.h"
#include "Fonts/SlateTextShaper.h"
#include "Fonts/LegacySlateFontInfoCache.h"
#include "Fonts/FontProviderInterface.h"
#include "Fonts/FontCacheUtils.h"
#include "Fonts/SlateSdfGenerator.h"
#include "Fonts/FontGeometryPreprocessing.h"
#include "Trace/SlateMemoryTags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FontCache)

#include <limits>

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num Font Atlases"), STAT_SlateNumFontAtlases, STATGROUP_SlateMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num Font Non-Atlased Textures"), STAT_SlateNumFontNonAtlasedTextures, STATGROUP_SlateMemory);
DECLARE_MEMORY_STAT(TEXT("Shaped Glyph Sequence Memory"), STAT_SlateShapedGlyphSequenceMemory, STATGROUP_SlateMemory);
DEFINE_STAT(STAT_SlateFontMeasureCacheMemory);

static FAtlasFlushParams FontAtlasFlushParams;
FAutoConsoleVariableRef CVarMaxAtlasPagesBeforeFlush(
	TEXT("Slate.MaxFontAtlasPagesBeforeFlush"),
	FontAtlasFlushParams.InitialMaxAtlasPagesBeforeFlushRequest,
	TEXT("The number of font atlas textures created and used before we flush the font cache if a texture atlas is full"));

FAutoConsoleVariableRef CVarMaxFontNonAtlasPagesBeforeFlush(
	TEXT("Slate.MaxFontNonAtlasTexturesBeforeFlush"),
	FontAtlasFlushParams.InitialMaxNonAtlasPagesBeforeFlushRequest,
	TEXT("The number of large glyph font textures initially."));

FAutoConsoleVariableRef CVarGrowFontAtlasFrameWindow(
	TEXT("Slate.GrowFontAtlasFrameWindow"),
	FontAtlasFlushParams.GrowAtlasFrameWindow,
	TEXT("The number of frames within the font atlas will resize rather than flush."));

FAutoConsoleVariableRef CVarGrowFontNonAtlasFrameWindow(
	TEXT("Slate.GrowFontNonAtlasFrameWindow"),
	FontAtlasFlushParams.GrowNonAtlasFrameWindow,
	TEXT("The number of frames within the large font glyph pool will resize rather than flush."));

static int32 UnloadFreeTypeDataOnFlush = 1;
FAutoConsoleVariableRef CVarUnloadFreeTypeDataOnFlush(
	TEXT("Slate.UnloadFreeTypeDataOnFlush"),
	UnloadFreeTypeDataOnFlush,
	TEXT("Releases the free type data when the font cache is flushed"));

static TAutoConsoleVariable<int32> CVarDefaultTextShapingMethod(
	TEXT("Slate.DefaultTextShapingMethod"),
	static_cast<int32>(ETextShapingMethod::Auto),
	TEXT("0: Auto (default), 1: KerningOnly, 2: FullShaping."),
	ECVF_Default
	);

#if !UE_BUILD_SHIPPING
static bool bDumpFontCacheStats = false;
FAutoConsoleVariableRef CVarDumpFontCacheStats(
	TEXT("Slate.DumpFontCacheStats"),
	bDumpFontCacheStats,
	TEXT("Dump statistics about font cache usage."));

static bool bFlushFontCache = false;
FAutoConsoleVariableRef CVarFlushFontCache(
	TEXT("Slate.FlushFontCache"),
	bFlushFontCache,
	TEXT("Flush the font cache."));
#endif

static bool bSlateSdfTextEnable = true;

void OnSlateSdfTextEnablementChanged(IConsoleVariable* Var);

FAutoConsoleVariableRef CVarSlateSdfTextEnable(
	TEXT("UI.SlateSDFText.Enable"),
	bSlateSdfTextEnable,
	TEXT("Enables signed distance field-based text rendering in Slate"),
	FConsoleVariableDelegate::CreateStatic(OnSlateSdfTextEnablementChanged));


FAutoConsoleVariableShadow ShadowCVarSlateSdfTextEnable(TEXT("SlateSdfText.Enable"), TEXT("UI.SlateSDFText.Enable"));

bool IsSlateSdfTextFeatureEnabled()
{
	return bSlateSdfTextEnable;
}

void OnSlateSdfTextEnablementChanged(IConsoleVariable*)
{
	if (FSlateApplicationBase::IsInitialized())
	{
		if (const FSlateRenderer* Renderer = FSlateApplicationBase::Get().GetRenderer())
		{
			Renderer->GetFontCache()->RequestFlushCache(TEXT("Value of UI.SlateSDFText.Enable changed."));
		}
	}
}

ETextShapingMethod GetDefaultTextShapingMethod()
{
	const int32 DefaultTextShapingMethodAsInt = CVarDefaultTextShapingMethod.AsVariable()->GetInt();
	if (DefaultTextShapingMethodAsInt >= static_cast<int32>(ETextShapingMethod::Auto) && DefaultTextShapingMethodAsInt <= static_cast<int32>(ETextShapingMethod::FullShaping))
	{
		return static_cast<ETextShapingMethod>(DefaultTextShapingMethodAsInt);
	}
	return ETextShapingMethod::Auto;
}


bool FShapedGlyphEntry::HasValidGlyph() const
{
#if WITH_FREETYPE
	TSharedPtr<FFreeTypeFace> FontFacePin = FontFaceData ? FontFaceData->FontFace.Pin() : nullptr;
	if (FontFacePin && GlyphIndex != 0)
	{
		const uint32 InvalidSubCharGlyphIndex = FT_Get_Char_Index(FontFacePin->GetFace(), SlateFontRendererUtils::InvalidSubChar);
		return GlyphIndex != InvalidSubCharGlyphIndex;
	}
#endif	// WITH_FREETYPE

	return false;
}

float FShapedGlyphEntry::GetBitmapRenderScale() const
{
	return FontFaceData ? FontFaceData->BitmapRenderScale : 1.0f;
}


FShapedGlyphEntryKey::FShapedGlyphEntryKey(const FShapedGlyphFaceData& InFontFaceData, uint32 InGlyphIndex, const FFontOutlineSettings& InOutlineSettings)
	: FontFace(InFontFaceData.FontFace)
	, OutlineMiteredCorners(InOutlineSettings.bMiteredCorners)
	, OutlineSeparateFillAlpha(InOutlineSettings.bSeparateFillAlpha)
	, GlyphIndex(InGlyphIndex)
	, KeyHash(0)
	, FontSkew(InFontFaceData.FontSkew)
{
#if WITH_FREETYPE
	FontRenderSize = FreeTypeUtils::ComputeFontPixelSize(InFontFaceData.FontSize, InFontFaceData.FontScale);
#else
	FontRenderSize = 0;
#endif
	OutlineRenderSize = FMath::TruncToInt(FMath::RoundToFloat(InOutlineSettings.OutlineSize * InFontFaceData.FontScale));

	KeyHash = HashCombine(KeyHash, GetTypeHash(FontFace));
	KeyHash = HashCombine(KeyHash, GetTypeHash(FontRenderSize));
	KeyHash = HashCombine(KeyHash, GetTypeHash(OutlineRenderSize));
	KeyHash = HashCombine(KeyHash, GetTypeHash(OutlineMiteredCorners));
	KeyHash = HashCombine(KeyHash, GetTypeHash(OutlineSeparateFillAlpha));
	KeyHash = HashCombine(KeyHash, GetTypeHash(GlyphIndex));
	KeyHash = HashCombine(KeyHash, GetTypeHash(FontSkew));
}

/*
*
*/

FSdfGlyphEntryKey::FSdfGlyphEntryKey(const TWeakPtr<FFreeTypeFace> InFontFace, uint32 InGlyphIndex, ESlateFontAtlasContentType InAtlasContentType, int32 InPpem, float InEmOuterSpread, float InEmInnerSpread)
	: FontFace(InFontFace)
	, GlyphIndex(InGlyphIndex)
	, AtlasContentType(InAtlasContentType)
	, Ppem(InPpem)
	, SpreadCategory(GetSpreadCategory(InEmOuterSpread, InEmInnerSpread))
	, KeyHash(0)
{
	KeyHash = HashCombine(KeyHash, GetTypeHash(FontFace));
	KeyHash = HashCombine(KeyHash, GetTypeHash(GlyphIndex));
	KeyHash = HashCombine(KeyHash, GetTypeHash(AtlasContentType));
	KeyHash = HashCombine(KeyHash, GetTypeHash(Ppem));
	KeyHash = HashCombine(KeyHash, GetTypeHash(SpreadCategory));
}

int32 FSdfGlyphEntryKey::GetSpreadCategory(float InEmOuterSpread, float InEmInnerSpread)
{
	(void) InEmInnerSpread;
	return FMath::FloorToInt(FMath::Loge(InEmOuterSpread));
}

FSdfGlyphTaskKey::FSdfGlyphTaskKey(const FSdfGlyphEntryKey& InSdfGlyphEntryKey, float InEmOuterSpread, float InEmInnerSpread)
	: SdfGlyphEntryKey(InSdfGlyphEntryKey)
	, EmOuterSpread(InEmOuterSpread)
	, EmInnerSpread(InEmInnerSpread)
	, KeyHash(0)
{
	KeyHash = HashCombine(KeyHash, GetTypeHash(SdfGlyphEntryKey));
	KeyHash = HashCombine(KeyHash, GetTypeHash(EmOuterSpread));
	KeyHash = HashCombine(KeyHash, GetTypeHash(EmInnerSpread));
}

/*
* 
*/

FShapedGlyphSequence::FShapedGlyphSequence(TArray<FShapedGlyphEntry> InGlyphsToRender, const int16 InTextBaseline, const uint16 InMaxTextHeight, const UObject* InFontMaterial, const bool InMaterialIsStencil, const FFontOutlineSettings& InOutlineSettings, const FSourceTextRange& InSourceTextRange)
	: GlyphsToRender(MoveTemp(InGlyphsToRender))
	, TextBaseline(InTextBaseline)
	, MaxTextHeight(InMaxTextHeight)
	, bMaterialIsStencil(InMaterialIsStencil)
	, FontMaterial(InFontMaterial)
	, OutlineSettings(InOutlineSettings)
	, SequenceWidth(0)
	, GlyphFontFaces()
	, SourceIndicesToGlyphData(InSourceTextRange)
	, CachedFontSkew(GlyphsToRender.Num() > 0 ? GlyphsToRender[0].FontFaceData->FontSkew : 0.f)
{
	const int32 NumGlyphsToRender = GlyphsToRender.Num();
	for (int32 CurrentGlyphIndex = 0; CurrentGlyphIndex < NumGlyphsToRender; ++CurrentGlyphIndex)
	{
		const FShapedGlyphEntry& CurrentGlyph = GlyphsToRender[CurrentGlyphIndex];
		check(FMath::IsNearlyEqual(CachedFontSkew, CurrentGlyph.FontFaceData->FontSkew));
		// Track unique font faces
		if (CurrentGlyph.FontFaceData->FontFace.IsValid())
		{
			GlyphFontFaces.AddUnique(CurrentGlyph.FontFaceData->FontFace);
		}

		// Update the measured width
		SequenceWidth += CurrentGlyph.XAdvance;

		FSourceIndexToGlyphData* SourceIndexToGlyphData = SourceIndicesToGlyphData.GetGlyphData(CurrentGlyph.SourceIndex);
		
		// Skip if index is invalid or hidden
		if (!SourceIndexToGlyphData)
		{
			// Track reverse look-up data
			UE_LOGF(LogSlate, Warning, "No Glyph! Index %i. Valid %i, %i, Valid Glyph %i, Visible %i, Num %i, Grapheme %i, Dir %u",
				CurrentGlyph.SourceIndex,
				SourceIndicesToGlyphData.GetSourceTextStartIndex(),
				SourceIndicesToGlyphData.GetSourceTextEndIndex(),
				CurrentGlyph.HasValidGlyph(),
				CurrentGlyph.bIsVisible,
				CurrentGlyph.NumCharactersInGlyph,
				CurrentGlyph.NumGraphemeClustersInGlyph,
				int(CurrentGlyph.TextDirection));

#if WITH_FREETYPE
			// Track font info if possible
			if (CurrentGlyph.FontFaceData.IsValid())
			{
				if (TSharedPtr<FFreeTypeFace> FontFacePtr = CurrentGlyph.FontFaceData->FontFace.Pin())
				{
					FT_Face FontFace = FontFacePtr->GetFace();
					UE_LOGF(LogSlate, Warning, "Font missing Gylph. Valid %i, Loading %i",
						FontFacePtr->IsFaceValid(),
						FontFacePtr->IsFaceLoading());
				}
			}
#endif // WITH_FREETYPE
			
			if (!CurrentGlyph.HasValidGlyph() || !CurrentGlyph.bIsVisible)
			{
				continue;
			}
		}

		checkSlow(SourceIndexToGlyphData);
		if (SourceIndexToGlyphData->IsValid())
		{
			// If this data already exists then it means a single character produced multiple glyphs and we need to track it as an additional glyph (these are always within the same cluster block)
			SourceIndexToGlyphData->AdditionalGlyphIndices.Add(CurrentGlyphIndex);
		}
		else
		{
			*SourceIndexToGlyphData = FSourceIndexToGlyphData(CurrentGlyphIndex);
		}
	}

#if SLATE_CHECK_UOBJECT_SHAPED_GLYPH_SEQUENCE
	if (FontMaterial)
	{
		FontMaterialWeakPtr = FontMaterial;
		DebugFontMaterialName = FontMaterial->GetFName();
	}
	if (OutlineSettings.OutlineMaterial)
	{
		OutlineMaterialWeakPtr = OutlineSettings.OutlineMaterial;
		DebugOutlineMaterialName = OutlineSettings.OutlineMaterial->GetFName();
	}
#endif

	// Track memory usage
	INC_MEMORY_STAT_BY(STAT_SlateShapedGlyphSequenceMemory, GetAllocatedSize());
}

FShapedGlyphSequence::~FShapedGlyphSequence()
{
	// Untrack memory usage
	DEC_MEMORY_STAT_BY(STAT_SlateShapedGlyphSequenceMemory, GetAllocatedSize());
}

SIZE_T FShapedGlyphSequence::GetAllocatedSize() const
{
	return GlyphsToRender.GetAllocatedSize() + GlyphFontFaces.GetAllocatedSize() + SourceIndicesToGlyphData.GetAllocatedSize();
}

bool FShapedGlyphSequence::IsDirty() const
{
	for (const auto& GlyphFontFace : GlyphFontFaces)
	{
		if (!GlyphFontFace.IsValid())
		{
			return true;
		}
	}

	return false;
}

float FShapedGlyphSequence::GetFontSkew() const
{
	return CachedFontSkew;
}

int32 FShapedGlyphSequence::GetMeasuredWidth() const
{
	return SequenceWidth;
}

TOptional<int32> FShapedGlyphSequence::GetMeasuredWidth(const int32 InStartIndex, const int32 InEndIndex, const bool InIncludeKerningWithPrecedingGlyph) const
{
	int32 MeasuredWidth = 0;

	if (InIncludeKerningWithPrecedingGlyph && InStartIndex > 0)
	{
		const TOptional<int8> Kerning = GetKerning(InStartIndex - 1);
		MeasuredWidth += Kerning.Get(0);
	}

	auto GlyphCallback = [&](const FShapedGlyphEntry& CurrentGlyph, int32 CurrentGlyphIndex) -> bool
	{
		MeasuredWidth += CurrentGlyph.XAdvance;
		return true;
	};

	if (EnumerateLogicalGlyphsInSourceRange(InStartIndex, InEndIndex, GlyphCallback) == EEnumerateGlyphsResult::EnumerationComplete)
	{
		return MeasuredWidth;
	}

	return TOptional<int32>();
}

FShapedGlyphSequence::FGlyphOffsetResult FShapedGlyphSequence::GetGlyphAtOffset(FSlateFontCache& InFontCache, const int32 InHorizontalOffset, const int32 InStartOffset) const
{
	if (GlyphsToRender.Num() == 0)
	{
		return FGlyphOffsetResult();
	}

	int32 CurrentOffset = InStartOffset;
	const FShapedGlyphEntry* MatchedGlyph = nullptr;

	const int32 NumGlyphsToRender = GlyphsToRender.Num();
	for (int32 CurrentGlyphIndex = 0; CurrentGlyphIndex < NumGlyphsToRender; ++CurrentGlyphIndex)
	{
		const FShapedGlyphEntry& CurrentGlyph = GlyphsToRender[CurrentGlyphIndex];

		if (HasFoundGlyphAtOffset(InFontCache, InHorizontalOffset, CurrentGlyph, CurrentGlyphIndex, /*InOut*/CurrentOffset, /*Out*/MatchedGlyph))
		{
			break;
		}
	}

	// Found a valid glyph?
	if (MatchedGlyph)
	{
		return FGlyphOffsetResult(MatchedGlyph, CurrentOffset);
	}

	// Hit was outside of our measure boundary, so return the start or end source index, depending on the reading direction of the right-most glyph
	if (GlyphsToRender.Last().TextDirection == TextBiDi::ETextDirection::LeftToRight)
	{
		return FGlyphOffsetResult(SourceIndicesToGlyphData.GetSourceTextEndIndex());
	}
	else
	{
		return FGlyphOffsetResult(SourceIndicesToGlyphData.GetSourceTextStartIndex());
	}
}

TOptional<FShapedGlyphSequence::FGlyphOffsetResult> FShapedGlyphSequence::GetGlyphAtOffset(FSlateFontCache& InFontCache, const int32 InStartIndex, const int32 InEndIndex, const int32 InHorizontalOffset, const int32 InStartOffset, const bool InIncludeKerningWithPrecedingGlyph) const
{
	int32 CurrentOffset = InStartOffset;
	const FShapedGlyphEntry* MatchedGlyph = nullptr;
	const FShapedGlyphEntry* RightmostGlyph = nullptr;

	if (InIncludeKerningWithPrecedingGlyph && InStartIndex > 0)
	{
		const TOptional<int8> Kerning = GetKerning(InStartIndex - 1);
		CurrentOffset += Kerning.Get(0);
	}

	auto GlyphCallback = [&](const FShapedGlyphEntry& CurrentGlyph, int32 CurrentGlyphIndex) -> bool
	{
		if (HasFoundGlyphAtOffset(InFontCache, InHorizontalOffset, CurrentGlyph, CurrentGlyphIndex, /*InOut*/CurrentOffset, /*Out*/MatchedGlyph))
		{
			return false; // triggers the enumeration to abort
		}

		RightmostGlyph = &CurrentGlyph;
		return true;
	};

	if (EnumerateVisualGlyphsInSourceRange(InStartIndex, InEndIndex, GlyphCallback) != EEnumerateGlyphsResult::EnumerationFailed)
	{
		// Found a valid glyph?
		if (MatchedGlyph)
		{
			return FGlyphOffsetResult(MatchedGlyph, CurrentOffset);
		}

		// Hit was outside of our measure boundary, so return the start or end index (if valid), depending on the reading direction of the right-most glyph we tested
		if (!RightmostGlyph || RightmostGlyph->TextDirection == TextBiDi::ETextDirection::LeftToRight)
		{
			if (InEndIndex >= SourceIndicesToGlyphData.GetSourceTextStartIndex() && InEndIndex <= SourceIndicesToGlyphData.GetSourceTextEndIndex())
			{
				return FGlyphOffsetResult(InEndIndex);
			}
		}
		else
		{
			if (InStartIndex >= SourceIndicesToGlyphData.GetSourceTextStartIndex() && InStartIndex <= SourceIndicesToGlyphData.GetSourceTextEndIndex())
			{
				return FGlyphOffsetResult(InStartIndex);
			}
		}
	}

	return TOptional<FGlyphOffsetResult>();
}

bool FShapedGlyphSequence::HasFoundGlyphAtOffset(FSlateFontCache& InFontCache, const int32 InHorizontalOffset, const FShapedGlyphEntry& InCurrentGlyph, const int32 InCurrentGlyphIndex, int32& InOutCurrentOffset, const FShapedGlyphEntry*& OutMatchedGlyph) const
{
	// Skip any glyphs that don't represent any characters (these are additional glyphs when a character produces multiple glyphs, and we process them below when we find their primary glyph, so can ignore them now)
	if (InCurrentGlyph.NumCharactersInGlyph == 0)
	{
		return false;
	}

	// A single character may produce multiple glyphs which must be treated as a single logic unit
	int32 TotalGlyphSpacing = 0;
	int32 TotalGlyphAdvance = 0;
	for (int32 SubGlyphIndex = InCurrentGlyphIndex;; ++SubGlyphIndex)
	{
		const FShapedGlyphEntry& SubGlyph = GlyphsToRender[SubGlyphIndex];
		int16 HorizontalOffset = 0;
		
		if (SubGlyph.bIsVisible)
		{
			const FShapedGlyphFontAtlasData SubGlyphAtlasData = InFontCache.GetShapedGlyphFontAtlasData(SubGlyph, FFontOutlineSettings::NoOutline);
			HorizontalOffset = SubGlyphAtlasData.HorizontalOffset;
		}

		TotalGlyphSpacing += HorizontalOffset + SubGlyph.XAdvance;
		TotalGlyphAdvance += SubGlyph.XAdvance;

		const bool bIsWithinGlyphCluster = GlyphsToRender.IsValidIndex(SubGlyphIndex + 1) && SubGlyph.SourceIndex == GlyphsToRender[SubGlyphIndex + 1].SourceIndex;
		if (!bIsWithinGlyphCluster)
		{
			break;
		}
	}

	// Round our test toward the glyphs center position, but don't do this for ligatures as they're handled outside of this function
	const int32 GlyphWidthToTest = (InCurrentGlyph.NumGraphemeClustersInGlyph > 1) ? TotalGlyphSpacing : TotalGlyphSpacing / 2;

	// Did we reach our desired hit-point?
	if (InHorizontalOffset < (InOutCurrentOffset + GlyphWidthToTest))
	{
		if (InCurrentGlyph.TextDirection == TextBiDi::ETextDirection::LeftToRight)
		{
			OutMatchedGlyph = &InCurrentGlyph;
		}
		else
		{
			// Right-to-left text needs to return the previous glyph index, since that is the logical "next" glyph
			const int32 PreviousGlyphIndex = InCurrentGlyphIndex - 1;
			if (GlyphsToRender.IsValidIndex(PreviousGlyphIndex))
			{
				OutMatchedGlyph = &GlyphsToRender[PreviousGlyphIndex];
			}
			else
			{
				OutMatchedGlyph = &InCurrentGlyph;
			}
		}

		return true;
	}

	InOutCurrentOffset += TotalGlyphAdvance;
	return false;
}

TOptional<int8> FShapedGlyphSequence::GetKerning(const int32 InIndex) const
{
	const FSourceIndexToGlyphData* SourceIndexToGlyphData = SourceIndicesToGlyphData.GetGlyphData(InIndex);
	if (SourceIndexToGlyphData && SourceIndexToGlyphData->IsValid())
	{
		const FShapedGlyphEntry& CurrentGlyph = GlyphsToRender[SourceIndexToGlyphData->GlyphIndex];
		checkSlow(CurrentGlyph.SourceIndex == InIndex);
		return CurrentGlyph.Kerning;
	}

	// If we got here it means we couldn't find the glyph
	return TOptional<int8>();
}

FShapedGlyphSequencePtr FShapedGlyphSequence::GetSubSequence(const int32 InStartIndex, const int32 InEndIndex) const
{
	TArray<FShapedGlyphEntry> SubGlyphsToRender;
	SubGlyphsToRender.Reserve(InEndIndex - InStartIndex);

	FSourceTextRange SubSequenceRange(InStartIndex, InEndIndex - InStartIndex);

	auto GlyphCallback = [&SubGlyphsToRender, &SubSequenceRange](const FShapedGlyphEntry& CurrentGlyph, int32 CurrentGlyphIndex) -> bool
	{
		SubGlyphsToRender.Add(CurrentGlyph);

		SubSequenceRange.TextStart = FMath::Min(SubSequenceRange.TextStart, CurrentGlyph.SourceIndex);
		SubSequenceRange.TextLen = FMath::Max(SubSequenceRange.TextLen, static_cast<int32>(CurrentGlyph.NumCharactersInGlyph));

		return true;
	};

	if (EnumerateVisualGlyphsInSourceRange(InStartIndex, InEndIndex, GlyphCallback) == EEnumerateGlyphsResult::EnumerationComplete)
	{
		return MakeShared<FShapedGlyphSequence>(MoveTemp(SubGlyphsToRender), TextBaseline, MaxTextHeight, FontMaterial, bMaterialIsStencil, OutlineSettings, SubSequenceRange);
	}

	return nullptr;
}

void FShapedGlyphSequence::AddReferencedObjects(FReferenceCollector& Collector)
{
#if SLATE_CHECK_UOBJECT_SHAPED_GLYPH_SEQUENCE
	if (GSlateCheckUObjectShapedGlyphSequence)
	{
		// pending kill objects may still be rendered for a frame so it is valid for the check to pass
		const bool bEvenIfPendingKill = true;
		// This test needs to be thread safe. It doesn't give us as many chances to trap bugs here but it is still useful
		const bool bThreadSafe = true;
		if (!DebugFontMaterialName.IsNone())
		{
			const UObject* FontMaterialPin = FontMaterialWeakPtr.GetEvenIfUnreachable();
			if (FontMaterial != FontMaterialPin)
			{
				UE_LOGF(LogSlate, Fatal, "Material % ls has become invalid. This means the FShapedGlyphSequence::FontMaterial was garbage collected while slate was using it", *DebugFontMaterialName.ToString());
			}
		}
		if (!DebugOutlineMaterialName.IsNone())
		{
			const UObject* OutlineMaterialPin = OutlineMaterialWeakPtr.GetEvenIfUnreachable();
			if (OutlineSettings.OutlineMaterial != OutlineMaterialPin)
			{
				UE_LOGF(LogSlate, Fatal, "Material %ls has become invalid. This means the FShapedGlyphSequence::OutlineSettings::OutlineMaterial was garbage collected while slate was using it", *DebugOutlineMaterialName.ToString());
			}
		}
	}
#endif

	Collector.AddReferencedObject(FontMaterial);
	Collector.AddReferencedObject(OutlineSettings.OutlineMaterial);
}

FShapedGlyphSequence::EEnumerateGlyphsResult FShapedGlyphSequence::EnumerateLogicalGlyphsInSourceRange(const int32 InStartIndex, const int32 InEndIndex, const FForEachShapedGlyphEntryCallback& InGlyphCallback) const
{
	if (InStartIndex == InEndIndex)
	{
		// Nothing to enumerate, but don't say we failed
		return EEnumerateGlyphsResult::EnumerationComplete;
	}

	// Enumerate the corresponding glyph for each source index in the given range
	int32 SourceIndex = InStartIndex;
	while (SourceIndex < InEndIndex)
	{
		// Get the glyph(s) that correspond to this source index
		const FSourceIndexToGlyphData* SourceIndexToGlyphData = SourceIndicesToGlyphData.GetGlyphData(SourceIndex);
		if (!(SourceIndexToGlyphData && SourceIndexToGlyphData->IsValid()))
		{
			return EEnumerateGlyphsResult::EnumerationFailed;
		}

		// Enumerate each glyph generated by the given source index
		const int32 StartGlyphIndex = SourceIndexToGlyphData->GetLowestGlyphIndex();
		const int32 EndGlyphIndex = SourceIndexToGlyphData->GetHighestGlyphIndex();
		for (int32 CurrentGlyphIndex = StartGlyphIndex; CurrentGlyphIndex <= EndGlyphIndex; ++CurrentGlyphIndex)
		{
			const FShapedGlyphEntry& CurrentGlyph = GlyphsToRender[CurrentGlyphIndex];

			if (!InGlyphCallback(CurrentGlyph, CurrentGlyphIndex))
			{
				return EEnumerateGlyphsResult::EnumerationAborted;
			}

			// Advance the source index by the number of characters within this glyph
			SourceIndex += CurrentGlyph.NumCharactersInGlyph;
		}
	}

	return (SourceIndex == InEndIndex) ? EEnumerateGlyphsResult::EnumerationComplete : EEnumerateGlyphsResult::EnumerationFailed;
}

FShapedGlyphSequence::EEnumerateGlyphsResult FShapedGlyphSequence::EnumerateVisualGlyphsInSourceRange(const int32 InStartIndex, const int32 InEndIndex, const FForEachShapedGlyphEntryCallback& InGlyphCallback) const
{
	if (InStartIndex == InEndIndex)
	{
		// Nothing to enumerate, but don't say we failed
		return EEnumerateGlyphsResult::EnumerationComplete;
	}

	// The given range is exclusive, but we use an inclusive range when performing all the bounds testing below (as it makes things simpler)
	const FSourceIndexToGlyphData* StartSourceIndexToGlyphData = SourceIndicesToGlyphData.GetGlyphData(InStartIndex);
	const FSourceIndexToGlyphData* EndSourceIndexToGlyphData = SourceIndicesToGlyphData.GetGlyphData(InEndIndex - 1);

	// If we found a start glyph but no end glyph, test to see whether the start glyph spans to the end glyph (as may happen with a ligature)
	if ((StartSourceIndexToGlyphData && StartSourceIndexToGlyphData->IsValid()) && !(EndSourceIndexToGlyphData && EndSourceIndexToGlyphData->IsValid()))
	{
		const FShapedGlyphEntry& StartGlyph = GlyphsToRender[StartSourceIndexToGlyphData->GlyphIndex];

		const int32 GlyphEndSourceIndex = StartGlyph.SourceIndex + StartGlyph.NumCharactersInGlyph;
		if (GlyphEndSourceIndex == InEndIndex)
		{
			EndSourceIndexToGlyphData = StartSourceIndexToGlyphData;
		}
	}

	// Found valid glyphs to enumerate between?
	if (!(StartSourceIndexToGlyphData && StartSourceIndexToGlyphData->IsValid() && EndSourceIndexToGlyphData && EndSourceIndexToGlyphData->IsValid()))
	{
		return EEnumerateGlyphsResult::EnumerationFailed;
	}

	// Find the real start and end glyph indices - taking into account characters that may have produced multiple glyphs when shaped
	int32 StartGlyphIndex = INDEX_NONE;
	int32 EndGlyphIndex = INDEX_NONE;
	if (StartSourceIndexToGlyphData->GlyphIndex <= EndSourceIndexToGlyphData->GlyphIndex)
	{
		StartGlyphIndex = StartSourceIndexToGlyphData->GetLowestGlyphIndex();
		EndGlyphIndex = EndSourceIndexToGlyphData->GetHighestGlyphIndex();
	}
	else
	{
		StartGlyphIndex = EndSourceIndexToGlyphData->GetLowestGlyphIndex();
		EndGlyphIndex = StartSourceIndexToGlyphData->GetHighestGlyphIndex();
	}
	check(StartGlyphIndex <= EndGlyphIndex);

	bool bStartIndexInRange = SourceIndicesToGlyphData.GetSourceTextStartIndex() == InStartIndex;
	bool bEndIndexInRange = SourceIndicesToGlyphData.GetSourceTextEndIndex() == InEndIndex;

	// Enumerate everything in the found range
	for (int32 CurrentGlyphIndex = StartGlyphIndex; CurrentGlyphIndex <= EndGlyphIndex; ++CurrentGlyphIndex)
	{
		const FShapedGlyphEntry& CurrentGlyph = GlyphsToRender[CurrentGlyphIndex];

		if (!bStartIndexInRange || !bEndIndexInRange)
		{
			const int32 GlyphStartSourceIndex = CurrentGlyph.SourceIndex;
			const int32 GlyphEndSourceIndex = CurrentGlyph.SourceIndex + CurrentGlyph.NumCharactersInGlyph;

			if (!bStartIndexInRange && GlyphStartSourceIndex == InStartIndex)
			{
				bStartIndexInRange = true;
			}

			if (!bEndIndexInRange && GlyphEndSourceIndex == InEndIndex)
			{
				bEndIndexInRange = true;
			}
		}

		if (!InGlyphCallback(CurrentGlyph, CurrentGlyphIndex))
		{
			return EEnumerateGlyphsResult::EnumerationAborted;
		}
	}

	return (bStartIndexInRange && bEndIndexInRange) ? EEnumerateGlyphsResult::EnumerationComplete : EEnumerateGlyphsResult::EnumerationFailed;
}

FCharacterList::FCharacterList( const FSlateFontKey& InFontKey, FSlateFontCache& InFontCache )
	: FontKey( InFontKey )
	, FontCache( InFontCache )
#if WITH_EDITORONLY_DATA
	, CompositeFontHistoryRevision( INDEX_NONE )
#endif	// WITH_EDITORONLY_DATA
	, MaxHeight( 0 )
	, Baseline( 0 )
{
#if WITH_EDITORONLY_DATA
	const FCompositeFont* const CompositeFont = InFontKey.GetFontInfo().GetCompositeFont();
	if( CompositeFont )
	{
		CompositeFontHistoryRevision = CompositeFont->HistoryRevision;
	}
#endif	// WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
bool FCharacterList::IsStale() const
{
	const FCompositeFont* const CompositeFont = FontKey.GetFontInfo().GetCompositeFont();
	return CompositeFontHistoryRevision != (CompositeFont ? CompositeFont->HistoryRevision : INDEX_NONE);
}
#endif	// WITH_EDITORONLY_DATA

int8 FCharacterList::GetKerning(TCHAR FirstChar, TCHAR SecondChar, const EFontFallback MaxFontFallback)
{
	const FCharacterEntry& First = GetCharacter(FirstChar, MaxFontFallback);
	const FCharacterEntry& Second = GetCharacter(SecondChar, MaxFontFallback);
	return GetKerning(First, Second);
}

int8 FCharacterList::GetKerning( const FCharacterEntry& FirstCharacterEntry, const FCharacterEntry& SecondCharacterEntry )
{
#if WITH_FREETYPE
	// We can only get kerning if both characters are using the same font
	if (FirstCharacterEntry.Valid &&
		SecondCharacterEntry.Valid &&
		FirstCharacterEntry.FontData && 
		FirstCharacterEntry.HasKerning && 
		*FirstCharacterEntry.FontData == *SecondCharacterEntry.FontData
		)
	{
		const TSharedPtr<FFreeTypeKerningCache>& KerningCache = FirstCharacterEntry.KerningCache;
		if (KerningCache)
		{
			FT_Vector KerningVector;
			if (KerningCache->FindOrCache(FirstCharacterEntry.GlyphIndex, SecondCharacterEntry.GlyphIndex, KerningVector))
			{
				return FreeTypeUtils::Convert26Dot6ToRoundedPixel<int8>(KerningVector.x);
			}
		}
	}
#endif // WITH_FREETYPE
	return 0;
}

uint16 FCharacterList::GetMaxHeight() const
{
	if( MaxHeight == 0 )
	{
		MaxHeight = FontCache.GetMaxCharacterHeight( FontKey.GetFontInfo(), FontKey.GetScale() );
	}

	return MaxHeight;
}

int16 FCharacterList::GetBaseline() const
{
	if (Baseline == 0)
	{
		Baseline = FontCache.GetBaseline( FontKey.GetFontInfo(), FontKey.GetScale() );
	}

	return Baseline;
}

bool FCharacterList::CanCacheCharacter(TCHAR Character, const EFontFallback MaxFontFallback) const
{
	bool bReturnVal = false;

	if (Character == SlateFontRendererUtils::InvalidSubChar)
	{
		bReturnVal = true;
	}
	else
	{
		float SubFontScalingFactor = 1.0f;
		const FFontData& FontData = FontCache.CompositeFontCache->GetFontDataForCodepoint(FontKey.GetFontInfo(), Character, SubFontScalingFactor);

		bReturnVal = FontCache.FontRenderer->CanLoadCodepoint(FontData, Character, MaxFontFallback);
	}

	return bReturnVal;
}

const FCharacterEntry& FCharacterList::GetCharacter(TCHAR Character, const EFontFallback MaxFontFallback)
{
#if WITH_FREETYPE
	if (const FCharacterEntry* const FoundEntry = MappedEntries.Find(Character))
	{
		// For already-cached characters, reject characters that don't fall within maximum font fallback level requirements
		if (Character == SlateFontRendererUtils::InvalidSubChar || MaxFontFallback >= FoundEntry->FallbackLevel)
		{
			return *FoundEntry;
		}
	}
	else if (CanCacheCharacter(Character, MaxFontFallback))
	{
		if (const FCharacterEntry* NewEntry = CacheCharacter(Character))
		{
			return *NewEntry;
		}
	}
	
	// If we weren't able to cache the character, try fetching an invalid character to display instead.
	// If we're already invalid, just return an invalid entry so we don't loop.
	if (Character != SlateFontRendererUtils::InvalidSubChar)
	{
		return GetCharacter(SlateFontRendererUtils::InvalidSubChar, MaxFontFallback);
	}
#endif

	// CacheCharacter always returns invalid characters when FreeType is not available
	// so just shortcut to this instead.
	static const FCharacterEntry Invalid{};
	return Invalid;
}

const FCharacterEntry* FCharacterList::CacheCharacter(TCHAR Character)
{
#if WITH_FREETYPE
	// Fake shape the character
	{
		const FSlateFontInfo& FontInfo = FontKey.GetFontInfo();

		// Get the data needed to render this character
		float SubFontScalingFactor = 1.0f;
		const FFontData* FontDataPtr = &FontCache.CompositeFontCache->GetFontDataForCodepoint(FontInfo, Character, SubFontScalingFactor);
		FFreeTypeFaceGlyphData FaceGlyphData = FontCache.FontRenderer->GetFontFaceForCodepoint(*FontDataPtr, Character, FontInfo.FontFallback);

		// Found a valid font face?
		if (FaceGlyphData.FaceAndMemory.IsValid() && !FaceGlyphData.FaceAndMemory->IsFaceLoading() && FaceGlyphData.FaceAndMemory->IsFaceValid())
		{
			// Only scalable font types can use sub-font scaling
			if (!FT_IS_SCALABLE(FaceGlyphData.FaceAndMemory->GetFace()))
			{
				SubFontScalingFactor = 1.0f;
			}

			const float FinalFontScale = FontKey.GetScale() * SubFontScalingFactor;

			uint32 GlyphFlags = 0;
			SlateFontRendererUtils::AppendGlyphFlags(*FaceGlyphData.FaceAndMemory, *FontDataPtr, GlyphFlags);

			const bool bHasKerning = FT_HAS_KERNING(FaceGlyphData.FaceAndMemory->GetFace()) != 0;

			const bool bIsWhitespace = FTextChar::IsWhitespace(Character);
			const uint32 GlyphIndex = FT_Get_Char_Index(FaceGlyphData.FaceAndMemory->GetFace(), Character);
			const uint32 FontRenderSize = FreeTypeUtils::ComputeFontPixelSize(FontInfo.Size, FinalFontScale);
			int16 XAdvance = 0;
			{
				FT_Fixed CachedAdvanceData = 0;
				TSharedRef<FFreeTypeAdvanceCache> AdvanceCache = FontCache.FTCacheDirectory->GetAdvanceCache(FaceGlyphData.FaceAndMemory->GetFace(), GlyphFlags, FontRenderSize);
				if (AdvanceCache->FindOrCache(GlyphIndex, CachedAdvanceData))
				{
					XAdvance = FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>((CachedAdvanceData + (1 << 9)) >> 10);
				}
			}

			FShapedGlyphEntry ShapedGlyphEntry;
			const FFontRasterizationSettings FontRasterizationSettings = FontDataPtr->GetFontRasterizationSettings();
			ShapedGlyphEntry.FontFaceData = MakeShared<FShapedGlyphFaceData>(FaceGlyphData.FaceAndMemory, 
																			 GlyphFlags, 
																			 FontInfo.Size, 
																			 FinalFontScale, 
																			 0.f,
																			 FontRasterizationSettings.Mode,
																			 FontRasterizationSettings.DistanceFieldPpem);
			ShapedGlyphEntry.GlyphIndex = GlyphIndex;
			ShapedGlyphEntry.XAdvance = XAdvance;
			ShapedGlyphEntry.bIsVisible = !bIsWhitespace;

			FCharacterEntry CharEntry;
			CharEntry.Valid = Character == 0 || GlyphIndex != 0;

			// Cache the shaped entry in the font cache
			if (CharEntry.Valid)
			{
				const FShapedGlyphFontAtlasData ShapedGlyphFontAtlasData = FontCache.GetShapedGlyphFontAtlasData(ShapedGlyphEntry, FontKey.GetFontOutlineSettings());
				if (ShapedGlyphFontAtlasData.Valid)
				{
					CharEntry.Character = Character;
					CharEntry.GlyphIndex = GlyphIndex;
					CharEntry.FontData = FontDataPtr;
					CharEntry.KerningCache = FontCache.FontRenderer->GetKerningCache(*FontDataPtr, FontRenderSize);
					CharEntry.FontScale = ShapedGlyphEntry.FontFaceData->FontScale;
					CharEntry.BitmapRenderScale = ShapedGlyphEntry.FontFaceData->BitmapRenderScale;
					CharEntry.StartU = ShapedGlyphFontAtlasData.StartU;
					CharEntry.StartV = ShapedGlyphFontAtlasData.StartV;
					CharEntry.USize = ShapedGlyphFontAtlasData.USize;
					CharEntry.VSize = ShapedGlyphFontAtlasData.VSize;
					CharEntry.VerticalOffset = ShapedGlyphFontAtlasData.VerticalOffset;
					CharEntry.HorizontalOffset = ShapedGlyphFontAtlasData.HorizontalOffset;
					CharEntry.GlobalDescender = GetBaseline(); // All fonts within a composite font need to use the baseline of the default font
					CharEntry.XAdvance = ShapedGlyphEntry.XAdvance;
					CharEntry.TextureIndex = ShapedGlyphFontAtlasData.TextureIndex;
					CharEntry.FallbackLevel = FaceGlyphData.CharFallbackLevel;
					CharEntry.HasKerning = bHasKerning;
					CharEntry.SupportsOutline = ShapedGlyphFontAtlasData.SupportsOutline;
				}
				else
				{
					CharEntry.Valid = false;
				}

				return &MappedEntries.Add(Character, MoveTemp(CharEntry));
			}
		}
	}
	
#endif // WITH_FREETYPE

	return nullptr;
}

FSlateFontCache::FSlateFontCache( TSharedRef<ISlateFontAtlasFactory> InFontAtlasFactory, ESlateTextureAtlasThreadId InOwningThread)
	: FSlateFlushableAtlasCache(&FontAtlasFlushParams)
	, FTLibrary( new FFreeTypeLibrary() )
	, FTCacheDirectory( new FFreeTypeCacheDirectory() )
	, CompositeFontCache( new FCompositeFontCache( FTLibrary.Get() ) )
	, FontRenderer( new FSlateFontRenderer( FTLibrary.Get(), FTCacheDirectory.Get(), CompositeFontCache.Get() ) )
	, TextShaper( new FSlateTextShaper( FTCacheDirectory.Get(), CompositeFontCache.Get(), FontRenderer.Get(), this ) )
	, SdfGenerator(FSlateSdfGenerator::create())
	, FontAtlasFactory( InFontAtlasFactory )
	, bFlushRequested( false )
	, OwningThread(InOwningThread)
	, EllipsisText(NSLOCTEXT("FontCache", "TextOverflowIndicator", "\u2026"))
{
	FInternationalization::Get().OnCultureChanged().AddRaw(this, &FSlateFontCache::HandleCultureChanged);
}

FSlateFontCache::~FSlateFontCache()
{
	FInternationalization::Get().OnCultureChanged().RemoveAll(this);

	// Make sure things get destroyed in the correct order
	FTCacheDirectory.Reset();
	FontToCharacterListCache.Empty();
	ShapedGlyphToAtlasData.Empty();
	TextShaper.Reset();
	FontRenderer.Reset();
	CompositeFontCache.Reset();
	FTLibrary.Reset();
}

int32 FSlateFontCache::GetNumAtlasPages() const
{
	return GrayscaleFontAtlasIndices.Num() + ColorFontAtlasIndices.Num() + MsdfFontAtlasIndices.Num();
}

FSlateShaderResource* FSlateFontCache::GetAtlasPageResource(const int32 InIndex) const
{
	return AllFontTextures[GetAllFontTexturesIndex(InIndex)]->GetSlateTexture();
}

bool FSlateFontCache::IsAtlasPageResourceAlphaOnly(const int32 InIndex) const
{
	return AllFontTextures[GetAllFontTexturesIndex(InIndex)]->GetContentType() == ESlateFontAtlasContentType::Alpha;
}

#if WITH_ATLAS_DEBUGGING
FAtlasSlotInfo FSlateFontCache::GetAtlasSlotInfoAtPosition(FIntPoint InPosition, int32 AtlasIndex) const
{
	if (!AllFontTextures.IsValidIndex(AtlasIndex)) //The currently selected texture page can become empty for a short amount of time when flushing cache, so prevent crash when flushing with Font Atlas Visualizer open!
	{
		return FAtlasSlotInfo();
	}

	const int32 TextureIndex = GetAllFontTexturesIndex(AtlasIndex);
	const FSlateTextureAtlas& Atlas = static_cast<FSlateFontAtlas&>(AllFontTextures[TextureIndex].Get());

	const FAtlasedTextureSlot* Slot = Atlas.GetSlotAtPosition(InPosition);
	if (Slot)
	{
		TStringBuilder<512> Builder;
		auto FeedFaceInfo = [&Builder](const TWeakPtr<FFreeTypeFace>& FontFace)
			{
#if WITH_FREETYPE
				FFreeTypeFace* FreeTypeFace = FontFace.Pin().Get();
				if (FreeTypeFace)
				{
					FT_Face FT_face = FreeTypeFace->GetFace();
					Builder += TEXT("Font: ");
					Builder += FT_face->family_name;
					Builder += TEXT(" ");
					Builder += FT_face->style_name;
				}
#endif
			};

		//Try to get info from bitmap map.
		const TPair<FShapedGlyphEntryKey, TSharedRef<FShapedGlyphFontAtlasData>>* Glyph = Algo::FindByPredicate(ShapedGlyphToAtlasData, [&Slot, &TextureIndex](const TPair<FShapedGlyphEntryKey, TSharedRef<FShapedGlyphFontAtlasData>>& Entry)
			{
				const FShapedGlyphFontAtlasData& AtlasData = Entry.Value.Get();
				return	AtlasData.StartU == Slot->X && 
						AtlasData.StartV == Slot->Y && 
						TextureIndex == AtlasData.TextureIndex;
			});

		//Display info from bitmap and return if found.
		if (Glyph != nullptr)
		{
			const FShapedGlyphEntryKey& EntryKey = Glyph->Key;

			FeedFaceInfo(EntryKey.FontFace);
			Builder += TEXT("\nSize: ");
			Builder += FString::FromInt(EntryKey.FontRenderSize);
			Builder += TEXT("\nOutlineSize: ");
			Builder += LexToString(EntryKey.OutlineRenderSize);
			Builder += TEXT("\nGlyphIndex: ");
			Builder += FString::FromInt(EntryKey.GlyphIndex);
			Builder += TEXT("\nFontSkew: ");
			Builder += LexToString(EntryKey.FontSkew);

			FAtlasSlotInfo NewInfo;
			NewInfo.AtlasSlotRect = FSlateRect(FVector2f(static_cast<float>(Slot->X), static_cast<float>(Slot->Y)), FVector2f(static_cast<float>(Slot->X + Slot->Width), static_cast<float>(Slot->Y + Slot->Height)));
			NewInfo.TextureName = Builder.ToString();
			return NewInfo;
		}

		//Try to get info from SDF map
		const TPair<FSdfGlyphEntryKey, TSharedRef<FSdfGlyphFontAtlasData>>* SdfGlyph = Algo::FindByPredicate(SdfGlyphToAtlasData, [&Slot, &TextureIndex](const TPair<FSdfGlyphEntryKey, TSharedRef<FSdfGlyphFontAtlasData>>& Entry)
			{
				const FSdfGlyphFontAtlasData& AtlasData = Entry.Value.Get();
				return	AtlasData.StartU == Slot->X + Slot->Padding && 
						AtlasData.StartV == Slot->Y + Slot->Padding && 
						AtlasData.TextureIndex == TextureIndex;
			});


		const FSdfGlyphEntryKey* EntryKey = nullptr;
		if (SdfGlyph != nullptr)
		{
			EntryKey = &(SdfGlyph->Key);
		}
		else
		{
			//Last chance, try to get info from SDF tasks (rasterization in progress)
			const TPair<FSdfGlyphTaskKey, TSharedRef<FSdfGlyphFontAtlasData>>* SdfTaskGlyph = Algo::FindByPredicate(SdfTaskToAtlasData, [&Slot, &TextureIndex](const TPair<FSdfGlyphTaskKey, TSharedRef<FSdfGlyphFontAtlasData>>& Entry)
				{
					const FSdfGlyphFontAtlasData& AtlasData = Entry.Value.Get();
					return	AtlasData.StartU == Slot->X + Slot->Padding &&
							AtlasData.StartV == Slot->Y + Slot->Padding &&
							AtlasData.TextureIndex == TextureIndex;
				});

			if (SdfTaskGlyph != nullptr)
			{
				EntryKey = &(SdfTaskGlyph->Key.SdfGlyphEntryKey);
				Builder += TEXT("***Temporary glyph***\n");
			}
		}

		//Display info from any SDF if found.
		if (EntryKey != nullptr)
		{
			FeedFaceInfo(EntryKey->FontFace);
			Builder += TEXT("\nSize: ");
			Builder += FString::FromInt(EntryKey->Ppem);
			Builder += TEXT("\nGlyphIndex: ");
			Builder += FString::FromInt(EntryKey->GlyphIndex);
			Builder += TEXT("\nSpreadCategory: ");
			Builder += LexToString(EntryKey->SpreadCategory);

			FAtlasSlotInfo NewInfo;
			NewInfo.AtlasSlotRect = FSlateRect(FVector2f(static_cast<float>(Slot->X), static_cast<float>(Slot->Y)), FVector2f(static_cast<float>(Slot->X + Slot->Width), static_cast<float>(Slot->Y + Slot->Height)));
			NewInfo.TextureName = Builder.ToString();
			return NewInfo;
		}
	}

	return FAtlasSlotInfo();
}
#endif

bool FSlateFontCache::AddNewEntry(const FShapedGlyphEntry& InShapedGlyph, const FFontOutlineSettings& InOutlineSettings, FShapedGlyphFontAtlasData& OutAtlasData)
{
	// Render the glyph
	FCharacterRenderData RenderData;
	const bool bDidRender = FontRenderer->GetRenderData(InShapedGlyph, InOutlineSettings, RenderData);

	uint8 PaddingOffset = 0;
	OutAtlasData.Valid = bDidRender && AddNewEntry(RenderData, OutAtlasData.TextureIndex, OutAtlasData.StartU, OutAtlasData.StartV, OutAtlasData.USize, OutAtlasData.VSize, PaddingOffset);
	if (OutAtlasData.Valid)
	{
		int32 VerticalOffset = RenderData.VerticalOffset + (int32)PaddingOffset;
		int32 HorizontalOffset = RenderData.HorizontalOffset - (int32)PaddingOffset;
		ensureMsgf(VerticalOffset >= std::numeric_limits<int16>::min() && VerticalOffset <= std::numeric_limits<int16>::max(), TEXT("The Glyph offset is too big"));
		ensureMsgf(HorizontalOffset >= std::numeric_limits<int16>::min() && HorizontalOffset <= std::numeric_limits<int16>::max(), TEXT("The Glyph offset is too big"));
		OutAtlasData.VerticalOffset = (int16)VerticalOffset;
		OutAtlasData.HorizontalOffset = (int16)HorizontalOffset;
		OutAtlasData.SupportsOutline = RenderData.bSupportsOutline;
	}

	return OutAtlasData.Valid;
}

bool FSlateFontCache::AddNewEntry( const FCharacterRenderData InRenderData, uint8& OutTextureIndex, uint16& OutGlyphX, uint16& OutGlyphY, uint16& OutGlyphWidth, uint16& OutGlyphHeight, uint8& OutPaddingOffset )
{
	// Will this entry fit within any atlas texture?
	const FIntPoint FontAtlasSize = FontAtlasFactory->GetAtlasSize(InRenderData.ContentType);
	if (InRenderData.SizeX > FontAtlasSize.X || InRenderData.SizeY > FontAtlasSize.Y)
	{
		TSharedPtr<ISlateFontTexture> NonAtlasedTexture = FontAtlasFactory->CreateNonAtlasedTexture(InRenderData.SizeX, InRenderData.SizeY, InRenderData.ContentType, InRenderData.RawPixels);
		if (NonAtlasedTexture.IsValid())
		{
			INC_DWORD_STAT_BY(STAT_SlateNumFontNonAtlasedTextures, 1);

			UE_LOGF(LogSlate, Warning, "SlateFontCache - Glyph texture is too large to store in the font atlas, so we're falling back to a non-atlased texture for this glyph. This may have SERIOUS performance implications. Atlas page size: { %d, %d }. Glyph render size: { %d, %d }",
				FontAtlasSize.X, FontAtlasSize.Y,
				InRenderData.SizeX, InRenderData.SizeY
				);

			if (AllFontTextures.Num() >= std::numeric_limits<uint8>::max())
			{
				UE_LOGF(LogSlate, Warning, "SlateFontCache - Glyph texture has more than 256 textures.");
				return false;
			}
			OutTextureIndex = (uint8)AllFontTextures.Add(NonAtlasedTexture.ToSharedRef());
			NonAtlasedTextureIndices.Add(OutTextureIndex);

			OutGlyphX = 0;
			OutGlyphY = 0;
			OutGlyphWidth = InRenderData.SizeX;
			OutGlyphHeight = InRenderData.SizeY;
			OutPaddingOffset = 0;

			if (!bFlushRequested)
			{
				UpdateFlushCounters(GrayscaleFontAtlasIndices.Num(), ColorFontAtlasIndices.Num(), MsdfFontAtlasIndices.Num(), NonAtlasedTextureIndices.Num());
			}

			return true;
		}

		UE_LOGF(LogSlate, Warning, "SlateFontCache - Glyph texture is too large to store in the font atlas, but we cannot support rendering such a large texture. Atlas page size: { %d, %d }. Glyph render size: { %d, %d }",
			FontAtlasSize.X, FontAtlasSize.Y,
			InRenderData.SizeX, InRenderData.SizeY);

		return false;
	}

	auto FillOutputParamsFromAtlasedTextureSlot = [&](const FAtlasedTextureSlot& AtlasedTextureSlot)
	{
		uint32 GlyphX = AtlasedTextureSlot.X;
		uint32 GlyphY = AtlasedTextureSlot.Y;
		uint32 GlyphWidth = AtlasedTextureSlot.Width;
		uint32 GlyphHeight = AtlasedTextureSlot.Height;
		ensureMsgf(GlyphX <= std::numeric_limits<uint16>::max(), TEXT("The Glyph size is too big"));
		ensureMsgf(GlyphY <= std::numeric_limits<uint16>::max(), TEXT("The Glyph size is too big"));
		ensureMsgf(GlyphWidth <= std::numeric_limits<uint16>::max(), TEXT("The Glyph size is too big"));
		ensureMsgf(GlyphHeight <= std::numeric_limits<uint16>::max(), TEXT("The Glyph size is too big"));
		OutGlyphX = (uint16)GlyphX;
		OutGlyphY = (uint16)GlyphY;
		OutGlyphWidth = (uint16)GlyphWidth;
		OutGlyphHeight = (uint16)GlyphHeight;
		OutPaddingOffset = AtlasedTextureSlot.Padding;
	};

	TArray<uint8>* FontAtlasIndicesPtr = NULL;
	switch (InRenderData.ContentType)
	{
		case ESlateFontAtlasContentType::Alpha:
			FontAtlasIndicesPtr = &GrayscaleFontAtlasIndices;
			break;
		case ESlateFontAtlasContentType::Color:
			FontAtlasIndicesPtr = &ColorFontAtlasIndices;
			break;
		case ESlateFontAtlasContentType::Msdf:
			FontAtlasIndicesPtr = &MsdfFontAtlasIndices;
			break;
		default:
			checkNoEntry();
			// Default to Color
			FontAtlasIndicesPtr = &ColorFontAtlasIndices;
			break;
	}
	check(FontAtlasIndicesPtr != NULL);

	for (const uint8 FontAtlasIndex : *FontAtlasIndicesPtr)
	{
		FSlateFontAtlas& FontAtlas = static_cast<FSlateFontAtlas&>(AllFontTextures[FontAtlasIndex].Get());
		checkSlow(FontAtlas.GetContentType() == InRenderData.ContentType);

		// Add the character to the texture
		const FAtlasedTextureSlot* NewSlot = FontAtlas.AddCharacter(InRenderData);
		if( NewSlot )
		{
			OutTextureIndex = FontAtlasIndex;
			FillOutputParamsFromAtlasedTextureSlot(*NewSlot);
			return true;
		}
	}

	if (AllFontTextures.Num() >= std::numeric_limits<uint8>::max())
	{
		UE_LOGF(LogSlate, Warning, "SlateFontCache - Atlas has more than 256 textures.");
		return false;
	}

	TSharedRef<FSlateFontAtlas> FontAtlas = FontAtlasFactory->CreateFontAtlas(InRenderData.ContentType);
	OutTextureIndex = (uint8)AllFontTextures.Add(FontAtlas);
	FontAtlasIndicesPtr->Add(OutTextureIndex);

	INC_DWORD_STAT_BY(STAT_SlateNumFontAtlases, 1);

	// Add the character to the texture
	const FAtlasedTextureSlot* NewSlot = FontAtlas->AddCharacter(InRenderData);
	if( NewSlot )
	{
		FillOutputParamsFromAtlasedTextureSlot(*NewSlot);
	}

	if (!bFlushRequested)
	{
		UpdateFlushCounters(GrayscaleFontAtlasIndices.Num(), ColorFontAtlasIndices.Num(), MsdfFontAtlasIndices.Num(), NonAtlasedTextureIndices.Num());
	}

	return NewSlot != nullptr;
}

FShapedGlyphSequenceRef FSlateFontCache::ShapeBidirectionalText( const FString& InText, const FSlateFontInfo &InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InBaseDirection, const ETextShapingMethod InTextShapingMethod , TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces) const
{
	return ShapeBidirectionalText(*InText, 0, InText.Len(), InFontInfo, InFontScale, InBaseDirection, InTextShapingMethod, OutLoadingFontFaces);
}

FShapedGlyphSequenceRef FSlateFontCache::ShapeBidirectionalText( const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo &InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InBaseDirection, const ETextShapingMethod InTextShapingMethod , TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces) const
{
	return TextShaper->ShapeBidirectionalText(InText, InTextStart, InTextLen, InFontInfo, InFontScale, InBaseDirection, InTextShapingMethod, OutLoadingFontFaces);
}

FShapedGlyphSequenceRef FSlateFontCache::ShapeUnidirectionalText( const FString& InText, const FSlateFontInfo &InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InTextDirection, const ETextShapingMethod InTextShapingMethod , TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces) const
{
	return ShapeUnidirectionalText(*InText, 0, InText.Len(), InFontInfo, InFontScale, InTextDirection, InTextShapingMethod, OutLoadingFontFaces);
}

FShapedGlyphSequenceRef FSlateFontCache::ShapeUnidirectionalText( const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo &InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InTextDirection, const ETextShapingMethod InTextShapingMethod , TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces) const
{
	return TextShaper->ShapeUnidirectionalText(InText, InTextStart, InTextLen, InFontInfo, InFontScale, InTextDirection, InTextShapingMethod, OutLoadingFontFaces);
}

FCharacterList& FSlateFontCache::GetCharacterList( const FSlateFontInfo &InFontInfo, float FontScale, const FFontOutlineSettings& InOutlineSettings )
{
	// Create a key for looking up each character
	const FSlateFontKey FontKey( InFontInfo, InOutlineSettings, FontScale );

	TUniquePtr<FCharacterList>* CachedCharacterList = FontToCharacterListCache.Find( FontKey );

	if(CachedCharacterList)
	{
#if WITH_EDITORONLY_DATA
		// Clear out this entry if it's stale so that we make a new one
		if((*CachedCharacterList)->IsStale())
		{
			FontToCharacterListCache.Remove(FontKey);
			FlushData();
		}
		else
#endif	// WITH_EDITORONLY_DATA
		{
			return **CachedCharacterList;
		}
	}

	return *FontToCharacterListCache.Add(FontKey, MakeUnique<FCharacterList>(FontKey, *this));
}

FShapedGlyphFontAtlasData FSlateFontCache::GetShapedGlyphFontAtlasData( const FShapedGlyphEntry& InShapedGlyph, const FFontOutlineSettings& InOutlineSettings )
{
	uint8 CachedTypeIndex = (uint8)(InOutlineSettings.OutlineSize <= 0 ? EFontCacheAtlasDataType::Regular : EFontCacheAtlasDataType::Outline);

	const int32 CachedAtlasDataThreadIndex = static_cast<int32>(OwningThread);

	// Has the atlas data already been cached on the glyph?
	{
		TSharedPtr<FShapedGlyphFontAtlasData> CachedAtlasDataPin = InShapedGlyph.CachedAtlasData[CachedTypeIndex][CachedAtlasDataThreadIndex].Pin();
		if (CachedAtlasDataPin.IsValid())
		{
			return *CachedAtlasDataPin;
		}
	}

	// Not cached on the glyph, so create a key for to look up this glyph, as it may
	// have already been cached by another shaped text sequence
	const FShapedGlyphEntryKey GlyphKey(*InShapedGlyph.FontFaceData, InShapedGlyph.GlyphIndex, InOutlineSettings);

	// Has the atlas data already been cached by another shaped text sequence?
	const TSharedRef<FShapedGlyphFontAtlasData>* FoundAtlasData = ShapedGlyphToAtlasData.Find(GlyphKey);
	if (FoundAtlasData)
	{
		InShapedGlyph.CachedAtlasData[CachedTypeIndex][CachedAtlasDataThreadIndex] = *FoundAtlasData;
		return **FoundAtlasData;
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_SlateFontCacheAddNewShapedEntry)

		// Not cached at all... create a new entry
		TSharedRef<FShapedGlyphFontAtlasData> NewAtlasData = MakeShareable(new FShapedGlyphFontAtlasData());
		AddNewEntry(InShapedGlyph, InOutlineSettings, *NewAtlasData);

		if (NewAtlasData->Valid)
		{
			InShapedGlyph.CachedAtlasData[CachedTypeIndex][CachedAtlasDataThreadIndex] = NewAtlasData;
			ShapedGlyphToAtlasData.Add(GlyphKey, NewAtlasData);
		}

		return *NewAtlasData;
	}
}

static float CeilSdfHalfSpread(float InEmSpread, float Ppem)
{
	const float InPixelSpread = Ppem*InEmSpread;
	float PixelSpread = 2.f;
	while (PixelSpread < InPixelSpread)
	{
		PixelSpread *= 2.f;
	}
	return PixelSpread/Ppem;
}

FSdfGlyphFontAtlasData FSlateFontCache::GetSdfGlyphFontAtlasData(const FShapedGlyphEntry& InShapedGlyph, const FFontOutlineSettings& InOutlineSettings)
{
	LLM_SCOPE_BYTAG(UI_Text);

	if (!InShapedGlyph.bIsVisible
		|| InShapedGlyph.FontFaceData->RasterizationMode == EFontRasterizationMode::Bitmap
		|| !IsSlateSdfTextFeatureEnabled())
	{
		return { /* EarlyCheckIfCannotUseSdf */ };
	}
	const int32 CachedAtlasDataThreadIndex = static_cast<int32>(OwningThread);
	check(CachedAtlasDataThreadIndex >= 0 && CachedAtlasDataThreadIndex < 2);

	const bool bPlaceholderOnly = InShapedGlyph.FontFaceData->RasterizationMode == EFontRasterizationMode::SdfApproximation;
#if WITH_FREETYPE
	const float TargetPpem = FMath::Max(1.f, static_cast<float>(FreeTypeUtils::ComputeFontPixelSize(InShapedGlyph.FontFaceData->FontSize, InShapedGlyph.FontFaceData->FontScale)));
#else
	const float TargetPpem = FMath::Max(1.f, 1.f/72.f*static_cast<float>(FontConstants::RenderDPI)*InShapedGlyph.FontFaceData->FontSize*InShapedGlyph.FontFaceData->FontScale);
#endif
	const int32 SdfPpem = InShapedGlyph.FontFaceData->SdfPpem;
	const float Scale = TargetPpem/float(SdfPpem);
	const float EmOutlineSize = FMath::RoundToFloat(InOutlineSettings.OutlineSize * InShapedGlyph.FontFaceData->FontScale)/TargetPpem;

	float EmInnerSpread = FMath::Clamp(2.f/TargetPpem, 0.05f, 0.25f);
	float EmOuterSpread = FMath::Min(EmInnerSpread + EmOutlineSize, 8.f);

	FSlateSdfGenerator::ESdfType SdfType = FSlateSdfGenerator::ESdfType::MultichannelAndSimple;
	ESlateFontAtlasContentType AtlasType = ESlateFontAtlasContentType::Msdf;
	TArray<uint8>* FontAtlasIndicesPtr = &MsdfFontAtlasIndices;
	switch (InShapedGlyph.FontFaceData->RasterizationMode)
	{
		case EFontRasterizationMode::Bitmap:
		default:
			checkNoEntry();
			break;
		case EFontRasterizationMode::Msdf:
			SdfType = FSlateSdfGenerator::ESdfType::MultichannelAndSimple;
			AtlasType = ESlateFontAtlasContentType::Msdf;
			FontAtlasIndicesPtr = &MsdfFontAtlasIndices;
			break;
		case EFontRasterizationMode::Sdf:
		case EFontRasterizationMode::SdfApproximation:
			SdfType = FSlateSdfGenerator::ESdfType::Simple;
			AtlasType = ESlateFontAtlasContentType::Alpha;
			FontAtlasIndicesPtr = &GrayscaleFontAtlasIndices;
			break;
	}

	auto TryRespawn = [&](const FSdfGlyphFontAtlasData& GlyphAtlasData) -> bool
	{
		const FSdfGlyphEntryKey GlyphKey(InShapedGlyph.FontFaceData->FontFace, InShapedGlyph.GlyphIndex, AtlasType, SdfPpem, GlyphAtlasData.EmOuterSpread, GlyphAtlasData.EmInnerSpread);
		FSlateSdfGenerator::FRequestDescriptor SdfRequestDescriptor = { 
			GlyphKey.FontFace,
			GlyphKey.GlyphIndex,
			SdfType,
			GlyphAtlasData.EmOuterSpread,
			GlyphAtlasData.EmInnerSpread,
			GlyphKey.Ppem
		};
		FSlateSdfGenerator::FRequestOutputInfo SdfRequestInfo = {};
		SdfRequestInfo.ImageWidth = GlyphAtlasData.USize;
		SdfRequestInfo.ImageHeight = GlyphAtlasData.VSize;
		SdfRequestInfo.BearingX = GlyphAtlasData.HorizontalOffset;
		SdfRequestInfo.BearingY = GlyphAtlasData.VerticalOffset;
		const FSlateSdfGenerator::ERequestResponse RespawnStatus = SdfGenerator->Respawn(SdfRequestDescriptor, SdfRequestInfo);
		check(RespawnStatus != FSlateSdfGenerator::ERequestResponse::BAD_REQUEST);
		return RespawnStatus == FSlateSdfGenerator::ERequestResponse::SUCCESS;
	};

	// Has the atlas data already been cached on the glyph?
	{
		TSharedPtr<FSdfGlyphFontAtlasData> CachedAtlasDataPin = InShapedGlyph.CachedSdfFontAtlasData[CachedAtlasDataThreadIndex].Pin();
		if (CachedAtlasDataPin.IsValid())
		{
			FSdfGlyphFontAtlasData Output = *CachedAtlasDataPin;
			if (!Output.bSupportsSdf)
			{
				return Output;
			}

			// Does the cached glyph have the minimum required outer and inner spread?
			if (CachedAtlasDataPin->EmOuterSpread >= EmOuterSpread && CachedAtlasDataPin->EmInnerSpread >= EmInnerSpread)
			{
				if (CachedAtlasDataPin->bPendingRespawn)
				{
					CachedAtlasDataPin->bPendingRespawn = !TryRespawn(*CachedAtlasDataPin);
				}
				Output.Metrics = InShapedGlyph.CachedSdfMetrics[CachedAtlasDataThreadIndex];
				return Output;
			}

			EmOuterSpread = FMath::Max(EmOuterSpread, CachedAtlasDataPin->EmOuterSpread);
			EmInnerSpread = FMath::Max(EmInnerSpread, CachedAtlasDataPin->EmInnerSpread);
		}
	}

	EmOuterSpread = CeilSdfHalfSpread(EmOuterSpread, float(SdfPpem));
	EmInnerSpread = CeilSdfHalfSpread(EmInnerSpread, float(SdfPpem));

	auto FinalizeEntry = [&](const TSharedRef<FSdfGlyphFontAtlasData>& InFontData) -> FSdfGlyphFontAtlasData
	{
		InShapedGlyph.CachedSdfFontAtlasData[CachedAtlasDataThreadIndex] = InFontData;
		FSdfGlyphFontAtlasData Output = *InFontData;
#if WITH_FREETYPE
		if (Output.bSupportsSdf)
		{
			// One half of a pixel's width is cut off from each side (so that we have texel centers at edges)
			Output.Metrics.BearingX = Scale*(static_cast<float>(Output.HorizontalOffset)+0.5f);
			Output.Metrics.BearingY = Scale*(static_cast<float>(Output.VerticalOffset)-0.5f);
			Output.Metrics.Width = Scale*static_cast<float>(Output.USize-1);
			Output.Metrics.Height = Scale*static_cast<float>(Output.VSize-1);
			InShapedGlyph.CachedSdfMetrics[CachedAtlasDataThreadIndex] = Output.Metrics;
		}
#endif
		return Output;
	};
	// Not cached on the glyph, so create a key for to look up this glyph, as it may
	// have already been cached by another shaped text sequence
	const FSdfGlyphEntryKey GlyphKey(InShapedGlyph.FontFaceData->FontFace, InShapedGlyph.GlyphIndex, AtlasType, SdfPpem, EmOuterSpread, EmInnerSpread);

	{
		// Has the atlas data already been cached by another shaped text sequence?
		const TSharedRef<FSdfGlyphFontAtlasData>* FoundAtlasData = SdfGlyphToAtlasData.Find(GlyphKey);
		if (FoundAtlasData && (*FoundAtlasData)->EmOuterSpread >= EmOuterSpread && (*FoundAtlasData)->EmInnerSpread >= EmInnerSpread)
		{
			if ((*FoundAtlasData)->bPendingRespawn)
			{
				(*FoundAtlasData)->bPendingRespawn = !TryRespawn(**FoundAtlasData);
			}
			return FinalizeEntry(*FoundAtlasData);
		}
	}

	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FSlateFontCache::GetSdfGlyphFontAtlasData_AddEntry");

	// Not cached at all... create a new entry
	FSlateSdfGenerator::FRequestDescriptor SdfRequestDescriptor = { 
		GlyphKey.FontFace,
		GlyphKey.GlyphIndex,
		SdfType,
		EmOuterSpread,
		EmInnerSpread,
		GlyphKey.Ppem
	};
	FSlateSdfGenerator::FRequestOutputInfo SdfRequestOutputInfo = {};
	FCharacterRenderData PlaceholderRenderData;
	const FSlateSdfGenerator::ERequestResponse SpawnStatus = bPlaceholderOnly ?
		SdfGenerator->MakePlaceholder(SdfRequestDescriptor, SdfRequestOutputInfo, PlaceholderRenderData.RawPixels) :
		SdfGenerator->SpawnWithPlaceholder(SdfRequestDescriptor, SdfRequestOutputInfo, PlaceholderRenderData.RawPixels);
	TSharedRef<FSdfGlyphFontAtlasData> NewAtlasData = MakeShareable(new FSdfGlyphFontAtlasData());

	switch (SpawnStatus)
	{
		case FSlateSdfGenerator::ERequestResponse::SUCCESS:
			break;
		case FSlateSdfGenerator::ERequestResponse::SDF_UNAVAILABLE:
			NewAtlasData->bSupportsSdf = false;
			NewAtlasData->Valid = true;
			SdfGlyphToAtlasData.Add(GlyphKey, NewAtlasData);
			return FinalizeEntry(NewAtlasData);
		case FSlateSdfGenerator::ERequestResponse::BUSY:
			// This can happen if font is still loading
			return { };
		case FSlateSdfGenerator::ERequestResponse::PLACEHOLDER_ONLY:
			NewAtlasData->bPendingRespawn = !bPlaceholderOnly;
			break;
		case FSlateSdfGenerator::ERequestResponse::BAD_REQUEST:
		default:
			checkNoEntry();
			return { };
	}

	NewAtlasData->bSupportsSdf = true;
	NewAtlasData->HorizontalOffset = SdfRequestOutputInfo.BearingX;
	NewAtlasData->VerticalOffset = SdfRequestOutputInfo.BearingY;
	NewAtlasData->EmOuterSpread = EmOuterSpread;
	NewAtlasData->EmInnerSpread = EmInnerSpread;
	NewAtlasData->Metrics.BearingX = SdfRequestOutputInfo.BearingX;
	NewAtlasData->Metrics.BearingY = SdfRequestOutputInfo.BearingY;
	NewAtlasData->Metrics.Width = SdfRequestOutputInfo.ImageWidth;
	NewAtlasData->Metrics.Height = SdfRequestOutputInfo.ImageHeight;
	NewAtlasData->Valid = true;

	check(SdfRequestOutputInfo.ImageWidth*SdfRequestOutputInfo.ImageHeight > 0);
	const bool bPlaceholderAvailable = PlaceholderRenderData.RawPixels.GetAllocatedSize() >= SdfRequestOutputInfo.ImageWidth*SdfRequestOutputInfo.ImageHeight;
	check(bPlaceholderAvailable);
	if (bPlaceholderAvailable)
	{
		PlaceholderRenderData.SizeX = SdfRequestOutputInfo.ImageWidth;
		PlaceholderRenderData.SizeY = SdfRequestOutputInfo.ImageHeight;
		PlaceholderRenderData.HorizontalOffset = SdfRequestOutputInfo.BearingX;
		PlaceholderRenderData.VerticalOffset = SdfRequestOutputInfo.BearingY;
		PlaceholderRenderData.ContentType = AtlasType;
		PlaceholderRenderData.bSupportsOutline = true;
	}
	else if (bPlaceholderOnly)
	{
		return { };
	}

	auto TryAddToAtlas = [&](FSlateFontAtlas& FontAtlas, uint8 FontAtlasIndex) -> bool
	{
		if (bPlaceholderAvailable)
		{
			if (const FAtlasedTextureSlot* NewSlot = FontAtlas.AddCharacter(PlaceholderRenderData))
			{
				NewAtlasData->TextureIndex = FontAtlasIndex;
				NewAtlasData->StartU = NewSlot->X + NewSlot->Padding;
				NewAtlasData->StartV = NewSlot->Y + NewSlot->Padding;
				NewAtlasData->USize = NewSlot->Width - 2 * NewSlot->Padding;
				NewAtlasData->VSize = NewSlot->Height - 2 * NewSlot->Padding;
				return true;
			}
		}
		else if (!bPlaceholderOnly)
		{
			// Deferred add character begins by reserving the character box/slot in the texture
			FDeferredCharacterRenderData RenderData;
			if (FontAtlas.BeginDeferredAddCharacter(SdfRequestOutputInfo.ImageWidth, SdfRequestOutputInfo.ImageHeight, RenderData))
			{
				NewAtlasData->TextureIndex = FontAtlasIndex;
				NewAtlasData->StartU = RenderData.StartU;
				NewAtlasData->StartV = RenderData.StartV;
				NewAtlasData->USize = RenderData.USize;
				NewAtlasData->VSize = RenderData.VSize;
				check(RenderData.USize == SdfRequestOutputInfo.ImageWidth && RenderData.VSize == SdfRequestOutputInfo.ImageHeight);
				return true;
			}
		}
		return false;
	};

	check(FontAtlasIndicesPtr);
	for (const uint8 FontAtlasIndex : *FontAtlasIndicesPtr)
	{
		FSlateFontAtlas& FontAtlas = static_cast<FSlateFontAtlas&>(AllFontTextures[FontAtlasIndex].Get());
		if (TryAddToAtlas(FontAtlas, FontAtlasIndex))
		{
			SdfTaskToAtlasData.Add(FSdfGlyphTaskKey(GlyphKey, EmOuterSpread, EmInnerSpread), NewAtlasData);
			SdfGlyphToAtlasData.Add(GlyphKey, NewAtlasData);
			return FinalizeEntry(NewAtlasData);
		}
	}

	if (AllFontTextures.Num() >= std::numeric_limits<uint8>::max())
	{
		UE_LOGF(LogSlate, Warning, "SlateFontCache - Atlas has more than 256 textures.");
		return { /*AtlasHasTooManyTexturesTryAgainLater*/ };
	}

	TSharedRef<FSlateFontAtlas> FontAtlas = FontAtlasFactory->CreateFontAtlas(AtlasType);
	const uint8 TextureIndex = (uint8)AllFontTextures.Add(FontAtlas);
	FontAtlasIndicesPtr->Add(TextureIndex);

	INC_DWORD_STAT_BY(STAT_SlateNumFontAtlases, 1);

	if (TryAddToAtlas(*FontAtlas, TextureIndex))
	{
		SdfTaskToAtlasData.Add(FSdfGlyphTaskKey(GlyphKey, EmOuterSpread, EmInnerSpread), NewAtlasData);
	}
	else
	{
		// if the data wont fit in a fresh texture then it wont fit in any texture, so just consider the glyph as unavailable
		NewAtlasData->bSupportsSdf = false;
		UE_LOGF(LogSlate, Warning, "SlateFontCache - Requested SDF cannot fit font atlas.");
	}
	SdfGlyphToAtlasData.Add(GlyphKey, NewAtlasData);

	if (!bFlushRequested)
	{
		UpdateFlushCounters(GrayscaleFontAtlasIndices.Num(), ColorFontAtlasIndices.Num(), MsdfFontAtlasIndices.Num(), NonAtlasedTextureIndices.Num());
	}

	return FinalizeEntry(NewAtlasData);
}


FShapedGlyphSequenceRef FSlateFontCache::GetOverflowEllipsisText(const FSlateFontInfo& InFontInfo, const float InFontScale)
{
	return ShapeOverflowEllipsisText(InFontInfo, InFontScale);
}

FShapedGlyphSequenceRef FSlateFontCache::ShapeOverflowEllipsisText(const FSlateFontInfo& InFontInfo, const float InFontScale)
{
	return ShapeBidirectionalText(EllipsisText.ToString(), InFontInfo, InFontScale, TextBiDi::ETextDirection::LeftToRight, GetDefaultTextShapingMethod());
}

const FFontData& FSlateFontCache::GetDefaultFontData( const FSlateFontInfo& InFontInfo ) const
{
	return CompositeFontCache->GetDefaultFontData(InFontInfo);
}

const FFontData& FSlateFontCache::GetFontDataForCodepoint( const FSlateFontInfo& InFontInfo, const UTF32CHAR InCodepoint, float& OutScalingFactor ) const
{
	return CompositeFontCache->GetFontDataForCodepoint(InFontInfo, InCodepoint, OutScalingFactor);
}

bool FSlateFontCache::HasLoadedFontForCodepoint(const FSlateFontInfo& InFontInfo, const UTF32CHAR InCodepoint) const
{
	return CompositeFontCache->HasLoadedFontForCodepoint(InFontInfo, InCodepoint);
}

uint16 FSlateFontCache::GetMaxCharacterHeight( const FSlateFontInfo& InFontInfo, float FontScale ) const
{
	return FontRenderer->GetMaxHeight(InFontInfo, FontScale);
}

int16 FSlateFontCache::GetBaseline( const FSlateFontInfo& InFontInfo, float FontScale ) const
{
	return FontRenderer->GetBaseline(InFontInfo, FontScale);
}

void FSlateFontCache::GetUnderlineMetrics( const FSlateFontInfo& InFontInfo, const float FontScale, int16& OutUnderlinePos, int16& OutUnderlineThickness ) const
{
	FontRenderer->GetUnderlineMetrics(InFontInfo, FontScale, OutUnderlinePos, OutUnderlineThickness);
}

void FSlateFontCache::GetStrikeMetrics( const FSlateFontInfo& InFontInfo, const float FontScale, int16& OutStrikeLinePos, int16& OutStrikeLineThickness ) const
{
	FontRenderer->GetStrikeMetrics(InFontInfo, FontScale, OutStrikeLinePos, OutStrikeLineThickness);
}

int8 FSlateFontCache::GetKerning( const FFontData& InFontData, const float InSize, TCHAR First, TCHAR Second, float Scale ) const
{
	return FontRenderer->GetKerning(InFontData, InSize, First, Second, Scale);
}

bool FSlateFontCache::HasKerning( const FFontData& InFontData ) const
{
	return FontRenderer->HasKerning(InFontData);
}

bool FSlateFontCache::CanLoadCodepoint(const FFontData& InFontData, const UTF32CHAR InCodepoint, EFontFallback MaxFallbackLevel) const
{
	return FontRenderer->CanLoadCodepoint(InFontData, InCodepoint, MaxFallbackLevel);
}

const TSet<FName>& FSlateFontCache::GetFontAttributes( const FFontData& InFontData ) const
{
	return CompositeFontCache->GetFontAttributes(InFontData);
}

TArray<FString> FSlateFontCache::GetAvailableFontSubFaces(FFontFaceDataConstRef InMemory) const
{
	return FFreeTypeFace::GetAvailableSubFaces(FTLibrary.Get(), InMemory);
}

TArray<FString> FSlateFontCache::GetAvailableFontSubFaces(const FString& InFilename) const
{
	return FFreeTypeFace::GetAvailableSubFaces(FTLibrary.Get(), InFilename);
}

void FSlateFontCache::RequestFlushCache(const FString& FlushReason)
{
	if (!bFlushRequested)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE_LOGF(LogSlate, Log, "FontCache flush requested. Reason: %ls", *FlushReason);
#else
		UE_LOGF(LogSlate, Warning, "FontCache flush requested. Reason: %ls", *FlushReason);
#endif

		bFlushRequested = true;
	}
}

void FSlateFontCache::FlushObject( const UObject* const InObject )
{
	if (InObject)
	{
		// Add it to the list of pending objects to flush
		FScopeLock ScopeLock(&FontObjectsToFlushCS);
		FontObjectsToFlush.AddUnique(InObject);
	}
}

void FSlateFontCache::FlushCompositeFont(const FCompositeFont& InCompositeFont)
{
	CompositeFontCache->FlushCompositeFont(InCompositeFont);
}

#if (!UE_BUILD_SHIPPING && WITH_FREETYPE)
struct FFontCacheStatsKey
{
	friend inline uint32 GetTypeHash(const FFontCacheStatsKey& Key)
	{
		return Key.KeyHash;
	}

	FFontCacheStatsKey(const FShapedGlyphEntryKey& ShapedGlyphKey)
		: FontFace(ShapedGlyphKey.FontFace)
		, FontRenderSize(ShapedGlyphKey.FontRenderSize)
		, OutlineRenderSize(ShapedGlyphKey.OutlineRenderSize)
		, OutlineMiteredCorners(ShapedGlyphKey.OutlineMiteredCorners)
		, OutlineSeparateFillAlpha(ShapedGlyphKey.OutlineSeparateFillAlpha)
		, KeyHash(0)
		, FontSkew(ShapedGlyphKey.FontSkew)
	{

		KeyHash = HashCombine(KeyHash, GetTypeHash(FontFace));
		KeyHash = HashCombine(KeyHash, GetTypeHash(FontRenderSize));
		KeyHash = HashCombine(KeyHash, GetTypeHash(OutlineRenderSize));
		KeyHash = HashCombine(KeyHash, GetTypeHash(OutlineMiteredCorners));
		KeyHash = HashCombine(KeyHash, GetTypeHash(OutlineSeparateFillAlpha));
		KeyHash = HashCombine(KeyHash, GetTypeHash(FontSkew));
	}

	bool operator==(const FFontCacheStatsKey& Other) const
	{
		return FontFace == Other.FontFace
			&& FontRenderSize == Other.FontRenderSize
			&& OutlineRenderSize == Other.OutlineRenderSize
			&& OutlineMiteredCorners == Other.OutlineMiteredCorners
			&& OutlineSeparateFillAlpha == Other.OutlineSeparateFillAlpha
			&& FontSkew == Other.FontSkew;
	}

	bool operator!=(const FFontCacheStatsKey& Other) const
	{
		return !(*this == Other);
	}

	/** Weak pointer to the FreeType face to render with */
	TWeakPtr<FFreeTypeFace> FontFace;
	/** The size in pixel that Freetype will actually use to render the font, scale included */
	uint32 FontRenderSize;
	/** The size in pixels of the outline to render, scale included */
	uint32 OutlineRenderSize;
	/** If checked, the outline will have mitered corners, otherwise they will be rounded. @see FFontOutlineSettings */
	bool OutlineMiteredCorners;
	/** If checked, the outline will be completely translucent where the filled area will be. @see FFontOutlineSettings */
	bool OutlineSeparateFillAlpha;
	/** Cached hash value used for map lookups */
	uint32 KeyHash;
	/** The skew transform amount for the rendered font */
	float FontSkew;
};

void FSlateFontCache::ConditionalDumpFontCacheStats() const
{
	if (bDumpFontCacheStats)
	{
		TMap<FFontCacheStatsKey, int> Stats;

		// Prepare statistics.
		int GlyphCount = 0;
		int PixelsUsedInAtlasCount = 0;
		for (const TPair<FShapedGlyphEntryKey, TSharedRef<FShapedGlyphFontAtlasData>>& Kvp : ShapedGlyphToAtlasData)
		{
			const FShapedGlyphFontAtlasData& FontAtlasData = Kvp.Value.Get();
			PixelsUsedInAtlasCount += (FontAtlasData.USize * FontAtlasData.VSize);
			GlyphCount++;

			FFontCacheStatsKey Key(Kvp.Key);
			FFreeTypeFace* Face = Key.FontFace.Pin().Get();
			const bool bIsEntryValid = Face != nullptr && Face->GetFace() != nullptr;
			if (bIsEntryValid) //Only add entries that we'll be able to sort and display.
			{
				int& GlyphByFontCount = Stats.FindOrAdd(Key);
				GlyphByFontCount++;
			}
		}

		int FontAtlasPixelTotalCount = 0;
		for (const TSharedRef<ISlateFontTexture>& FontTexture : AllFontTextures)
		{
			ISlateFontTexture& SlateFontTexture = FontTexture.Get();
			const FSlateShaderResource* SlateShaderResource = SlateFontTexture.GetSlateTexture();
			check(SlateShaderResource);
 
			FontAtlasPixelTotalCount += SlateShaderResource->GetWidth() * SlateShaderResource->GetHeight();
		}

		//Display statistics
		UE_LOGF(LogSlate, Log, "%d glyphs in cache, using %d pixels on %d total. Using %f%% of all font textures.", GlyphCount, PixelsUsedInAtlasCount, FontAtlasPixelTotalCount, static_cast<float>(PixelsUsedInAtlasCount) * 100.0f / static_cast<float>(FontAtlasPixelTotalCount));
		UE_LOGF(LogSlate, Log, "FontFace & Style | Size (72DPI) | OutlineSize | Mitered | Sep. Fill a | Skew || Glyphs Count |");

		Stats.KeySort([](const FFontCacheStatsKey& A, const FFontCacheStatsKey& B)
			{
				if (A.FontFace.Pin().Get() == B.FontFace.Pin().Get())
				{
					if (A.FontRenderSize == B.FontRenderSize)
					{
						if (A.OutlineRenderSize == B.OutlineRenderSize)
						{
							if (A.OutlineMiteredCorners == B.OutlineMiteredCorners)
							{
								if (A.OutlineSeparateFillAlpha == B.OutlineSeparateFillAlpha)
								{
									return A.FontSkew < B.FontSkew;
								}
								return A.OutlineSeparateFillAlpha < B.OutlineSeparateFillAlpha;
							}
							return A.OutlineMiteredCorners < B.OutlineMiteredCorners;
						}
						return A.OutlineRenderSize < B.OutlineRenderSize;
					}
					return A.FontRenderSize < B.FontRenderSize;
				}
				return A.FontFace.Pin().Get()->GetFace()->family_name < B.FontFace.Pin().Get()->GetFace()->family_name;
			});

		for (const TPair<FFontCacheStatsKey, int>& Kvp : Stats)
		{
			FT_Face FT_face = Kvp.Key.FontFace.Pin().Get()->GetFace();
			UE_LOGF(LogSlate, Log, "%ls %ls | %d | %d | %ls | %ls | %.2f || %d |",
				*FString(FT_face->family_name),
				*FString(FT_face->style_name),
				Kvp.Key.FontRenderSize,
				Kvp.Key.OutlineRenderSize,
				Kvp.Key.OutlineMiteredCorners ? *FString("true") : *FString("false"),
				Kvp.Key.OutlineSeparateFillAlpha ? *FString("true") : *FString("false"),
				Kvp.Key.FontSkew,
				Kvp.Value);
		}

		bDumpFontCacheStats = false;
	}
}
#endif

bool FSlateFontCache::ConditionalFlushCache()
{
	bool bFlushed = false;
	if (bFlushRequested
#if !UE_BUILD_SHIPPING
		|| bFlushFontCache
#endif
		)
	{
		if (FlushCache())
		{
			bFlushRequested = false;
#if !UE_BUILD_SHIPPING
			bFlushFontCache = false;
#endif
			bFlushed = true;
		}
	}

	if (!bFlushed && IsInGameThread())
	{
		// Only bother calling this if we didn't do a full flush
		FlushFontObjects();
	}

	return bFlushed;
}

void FSlateFontCache::UpdateCache()
{
	const bool bSdfEnabled = IsSlateSdfTextFeatureEnabled();

	if (bSdfEnabled)
	{
		SdfGenerator->Update(
			[this](const FSlateSdfGenerator::FRequestDescriptor InRequestDescriptor, TArray<uint8> InRawPixels)
			{
				ESlateFontAtlasContentType AtlasType = ESlateFontAtlasContentType::Msdf;
				switch (InRequestDescriptor.SdfType)
				{
					case FSlateSdfGenerator::ESdfType::Simple:
					case FSlateSdfGenerator::ESdfType::Perpendicular:
						AtlasType = ESlateFontAtlasContentType::Alpha;
						break;
					case FSlateSdfGenerator::ESdfType::MultichannelAndSimple:
						AtlasType = ESlateFontAtlasContentType::Msdf;
						break;
					default:
						checkNoEntry();
				}
				const FSdfGlyphEntryKey GlyphKey(InRequestDescriptor.FontFace,
												 InRequestDescriptor.GlyphIndex,
												 AtlasType,
												 InRequestDescriptor.Ppem,
												 InRequestDescriptor.EmOuterSpread,
												 InRequestDescriptor.EmInnerSpread);
				const FSdfGlyphTaskKey TaskKey(GlyphKey, InRequestDescriptor.EmOuterSpread, InRequestDescriptor.EmInnerSpread);
				const TSharedRef<FSdfGlyphFontAtlasData>* FoundAtlasData = SdfTaskToAtlasData.Find(TaskKey);
				if (FoundAtlasData)
				{
					if (FoundAtlasData && (*FoundAtlasData)->bSupportsSdf)
					{
						check(AllFontTextures.IsValidIndex((*FoundAtlasData)->TextureIndex) &&
							  (InRequestDescriptor.SdfType == FSlateSdfGenerator::ESdfType::MultichannelAndSimple ? MsdfFontAtlasIndices : GrayscaleFontAtlasIndices).Contains((*FoundAtlasData)->TextureIndex));
						FSlateFontAtlas& FontAtlas = static_cast<FSlateFontAtlas&>(AllFontTextures[(*FoundAtlasData)->TextureIndex].Get());
						FDeferredCharacterRenderData RenderData;
						RenderData.StartU = (*FoundAtlasData)->StartU;
						RenderData.StartV = (*FoundAtlasData)->StartV;
						RenderData.USize = (*FoundAtlasData)->USize;
						RenderData.VSize = (*FoundAtlasData)->VSize;
						RenderData.RawPixels = MoveTemp(InRawPixels);
						FontAtlas.EndDeferredAddCharacter(RenderData);
					}
					SdfTaskToAtlasData.Remove(TaskKey);
				}
			}
		);
	}
	auto UpdateFontAtlasTextures = [this](const TArray<uint8>& FontAtlasIndices)
	{
		for (const uint8 FontAtlasIndex : FontAtlasIndices)
		{
			FSlateFontAtlas& FontAtlas = static_cast<FSlateFontAtlas&>(AllFontTextures[FontAtlasIndex].Get());
			FontAtlas.ConditionalUpdateTexture();
		}
	};

	UpdateFontAtlasTextures(GrayscaleFontAtlasIndices);
	UpdateFontAtlasTextures(ColorFontAtlasIndices);
	if (bSdfEnabled)
	{
		UpdateFontAtlasTextures(MsdfFontAtlasIndices);
	}

	CompositeFontCache->Update();

#if (!UE_BUILD_SHIPPING && WITH_FREETYPE)
	ConditionalDumpFontCacheStats();
#endif
}

void FSlateFontCache::ReleaseResources()
{
	for (const TSharedRef<ISlateFontTexture>& FontTexture : AllFontTextures)
	{
		FontTexture->ReleaseRenderingResources();
	}

	OnReleaseResourcesDelegate.Broadcast(*this);
}

bool FSlateFontCache::FlushCache()
{
	if ( IsInGameThread() )
	{
		SCOPED_NAMED_EVENT(Slate_FlushFontCache, FColor::Red);

		FlushData();
		ReleaseResources();

		SET_DWORD_STAT(STAT_SlateNumFontAtlases, 0);
		SET_DWORD_STAT(STAT_SlateNumFontNonAtlasedTextures, 0);

		GrayscaleFontAtlasIndices.Empty();
		ColorFontAtlasIndices.Empty();
		NonAtlasedTextureIndices.Empty();
		MsdfFontAtlasIndices.Empty();
		AllFontTextures.Empty();

		{
			FScopeLock ScopeLock(&FontObjectsToFlushCS);
			FontObjectsToFlush.Empty();
		}

#if !WITH_EDITOR
		UE_LOGF(LogSlate, Log, "Slate font cache was flushed");
#endif

		return true;
	}

	return false;
}

void FSlateFontCache::FlushData()
{
	// Ensure all invalidation panels are cleared of cached widgets
	FSlateApplicationBase::Get().InvalidateAllWidgets(false);

	if (IsSlateSdfTextFeatureEnabled())
	{
		SdfGenerator->Flush();
	}

	TextShaper->FlushCache();

	if (GIsEditor || UnloadFreeTypeDataOnFlush)
	{
		FTCacheDirectory->FlushCache();
		CompositeFontCache->FlushCache();
	}

	FontToCharacterListCache.Empty();

	ShapedGlyphToAtlasData.Empty();
	SdfGlyphToAtlasData.Empty();
	SdfTaskToAtlasData.Empty();
}

SIZE_T FSlateFontCache::GetFontDataAssetResidentMemory(const UObject* FontDataAsset) const
{
	return CompositeFontCache->GetFontDataAssetResidentMemory(FontDataAsset);
}

#if WITH_EDITOR && WITH_FREETYPE
bool FSlateFontCache::EnsurePreprocessedFontGeometry(FFontFaceDataRef FontFaceData)
{
	if (!IsSlateSdfTextFeatureEnabled() || !FontFaceData->GetPreprocessedFontGeometries().IsEmpty() || !FontFaceData->HasData())
	{
		return false;
	}
	const TArray<FString> SubFaces = FFreeTypeFace::GetAvailableSubFaces(FTLibrary.Get(), FontFaceData);
	TArray<UE::Slate::FPreprocessedFontGeometry> PreprocessedFontGeometries;
	PreprocessedFontGeometries.SetNum(SubFaces.Num());
	for (int32 SubFaceIndex = 0; SubFaceIndex < SubFaces.Num(); ++SubFaceIndex)
	{
		const FFreeTypeFace FreeTypeFace(FTLibrary.Get(), FontFaceData, SubFaceIndex, EFontLayoutMethod::Metrics);
		UE::Slate::PreprocessFontGeometry(PreprocessedFontGeometries[SubFaceIndex], FreeTypeFace.GetFace());
	}
	FontFaceData->SetPreprocessedFontGeometries((TArray<UE::Slate::FPreprocessedFontGeometry> &&) PreprocessedFontGeometries);
	return true;
}
#endif

int32 FSlateFontCache::GetAllFontTexturesIndex(const int32 InIndex) const
{
	int32 Index = InIndex;
	if (Index < GrayscaleFontAtlasIndices.Num())
	{
		return GrayscaleFontAtlasIndices[Index];
	}
	Index -= GrayscaleFontAtlasIndices.Num();
	if (Index < ColorFontAtlasIndices.Num())
	{
		return ColorFontAtlasIndices[Index];
	}
	Index -= ColorFontAtlasIndices.Num();
	if (Index < MsdfFontAtlasIndices.Num())
	{
		return MsdfFontAtlasIndices[Index];
	}
	checkNoEntry();
	return 0;
}

void FSlateFontCache::FlushFontObjects()
{
	check(IsInGameThread());

	bool bHasRemovedEntries = false;
	{
		FScopeLock ScopeLock(&FontObjectsToFlushCS);

		if (FontObjectsToFlush.Num() > 0)
		{
			for (auto It = FontToCharacterListCache.CreateIterator(); It; ++It)
			{
				if (FontObjectsToFlush.Contains(It.Key().GetFontInfo().FontObject))
				{
					bHasRemovedEntries = true;
					It.RemoveCurrent();
				}
			}

			FontObjectsToFlush.Empty();
		}
	}

	if (bHasRemovedEntries)
	{
		FlushData();
	}
}

void FSlateFontCache::HandleCultureChanged()
{
	// The culture has changed, so request the font cache be flushed once it is safe to do so
	// We don't flush immediately as the request may come in from a different thread than the one that owns the font cache
	RequestFlushCache(TEXT("Culture for localization was changed"));
}

