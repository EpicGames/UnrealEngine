// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/FontCacheCompositeFont.h"
#include "SlateGlobals.h"
#include "Algo/BinarySearch.h"
#include "Application/SlateApplicationBase.h"
#include "Async/Async.h"
#include "Fonts/FontBulkData.h"
#include "Fonts/FontCacheFreeType.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "Misc/FileHelper.h"
#include "Serialization/MemoryReader.h"
#include "Textures/TextureAtlas.h"
#include "Trace/SlateMemoryTags.h"

static bool GAsyncFontLazyLoad = false;
FAutoConsoleVariableRef CVarAsyncLazyLoad(
	TEXT("Slate.Font.AsyncLazyLoad"),
	GAsyncFontLazyLoad,
	TEXT("Causes unloaded font faces that are lazily loaded, to be loaded asynchronously, until then the font won't measure correctly.  Once complete the UI will invalidate. Note that async font loading is disallowed in Editor/PIE, standalone and cooked builds are fine. "));

static bool GNormalizeCachedFontRanges = true;
FAutoConsoleVariableRef CVarNormalizeCachedFontRanges(
	TEXT("Slate.Font.NormalizeCachedFontRanges"),
	GNormalizeCachedFontRanges,
	TEXT("Normalize the font ranges for subtypefaces in composite fonts when building out the cached font ranges. Given a list of font ranges F, normalization infolves removing all intersecting font ranges in F but ensuring that the union of all the font ranges are still the same."));

namespace UE::Private::FontCacheCompositeFont
{
	bool IsAsyncFontLoadingAllowed()
	{
		// We only allow async font loading in standalone or cooked builds, not in Editor/PIE. We want everything to be synchronously loaded for the actual Editor
#if WITH_EDITOR
		return GAsyncFontLazyLoad && !GIsEditor;
#else
		return GAsyncFontLazyLoad;
#endif
	}
}

DECLARE_CYCLE_STAT(TEXT("Load Font"), STAT_SlateLoadFont, STATGROUP_Slate);

class FAsyncLoadFontFaceData : public FNonAbandonableTask
{
public:
	FAsyncLoadFontFaceData(const FFontData& InFontData)
		: FontData(InFontData)
	{
	}

	const FFontData& GetFontData() { return FontData; }
	TArray<uint8>& GetFontFaceData() { return FontFaceData; }

	void DoWork()
	{
		if (!FFileHelper::LoadFileToArray(FontFaceData, *FontData.GetFontFilename()))
		{
			UE_LOGF(LogSlate, Warning, "FAsyncLoadFontFaceData failed to load or process '%ls' with face index %d", *FontData.GetFontFilename(), FontData.GetSubFaceIndex());
		}
		else
		{
			UE_LOGF(LogSlate, Display, "FAsyncLoadFontFaceData successfully loaded '%ls' with face index %d", *FontData.GetFontFilename(), FontData.GetSubFaceIndex());
		}
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncLoadFontFaceData, STATGROUP_ThreadPoolAsyncTasks);
	}

private:
	FFontData FontData;
	TArray<uint8> FontFaceData;
};

FCachedTypefaceData::FCachedTypefaceData()
	: Typeface(nullptr)
	, CachedFontData()
	, ScalingFactor(1.0f)
{
}

FCachedTypefaceData::FCachedTypefaceData(const FTypeface& InTypeface, const float InScalingFactor)
	: Typeface(&InTypeface)
	, CachedFontData()
	, ScalingFactor(InScalingFactor)
{
	// Add all the entries from the typeface
	for (const FTypefaceEntry& TypefaceEntry : Typeface->Fonts)
	{
		CachedFontData.Emplace(TypefaceEntry.Name, &TypefaceEntry.Font);
	}

	// Sort the entries now so we can binary search them later
	CachedFontData.Sort(&FCachedFontData::SortPredicate);
}

const FFontData* FCachedTypefaceData::GetPrimaryFontData() const
{
	check(Typeface);
	return Typeface->Fonts.Num() > 0 ? &Typeface->Fonts[0].Font : nullptr;
}

const FFontData* FCachedTypefaceData::GetFontData(const FName& InName) const
{
	const int32 FoundDataIndex = Algo::BinarySearchBy(CachedFontData, InName, &FCachedFontData::BinarySearchKey, &FCachedFontData::KeySortPredicate);
	return FoundDataIndex != INDEX_NONE ? CachedFontData[FoundDataIndex].FontData : nullptr;
}

void FCachedTypefaceData::GetCachedFontData(TArray<const FFontData*>& OutFontData) const
{
	for (const FCachedFontData& CachedFontDataEntry : CachedFontData)
	{
		OutFontData.Add(CachedFontDataEntry.FontData);
	}
}

namespace Private
{
	bool IsFontFileName(const FString& Filename)
	{	
		return Filename.EndsWith(".ttf") || Filename.EndsWith(".ttc") || Filename.EndsWith(".otf") || Filename.EndsWith(".otc") || Filename.EndsWith(".woff") || Filename.EndsWith(".woff2");
	}

	bool IsFontFileData(const TArray<uint8>& FontFaceDataArray)
	{	//Detect ttf, otf, ttc/otc, woff1 and woff2
		return FontFaceDataArray.Num() >= 4 &&	((FontFaceDataArray[0] == 0x00 && FontFaceDataArray[1] == 0x01 && FontFaceDataArray[2] == 0x00 && FontFaceDataArray[3] == 0x00) ||
												(FontFaceDataArray[0] == 0x4f && FontFaceDataArray[1] == 0x54 && FontFaceDataArray[2] == 0x54 && FontFaceDataArray[3] == 0x4f) ||
												(FontFaceDataArray[0] == 0x74 && FontFaceDataArray[1] == 0x74 && FontFaceDataArray[2] == 0x63 && FontFaceDataArray[3] == 0x66) ||	//TTC
												(FontFaceDataArray[0] == 0x77 && FontFaceDataArray[1] == 0x4f && FontFaceDataArray[2] == 0x46 && FontFaceDataArray[3] == 0x46) ||	//WOFF1
												(FontFaceDataArray[0] == 0x77 && FontFaceDataArray[1] == 0x4f && FontFaceDataArray[2] == 0x46 && FontFaceDataArray[3] == 0x32));	//WOFF2
	}
}

FCachedCompositeFontData::FCachedCompositeFontData()
	: CompositeFont(nullptr)
	, CachedTypefaces()
	, CachedPriorityFontRanges()
	, CachedFontRanges()
{
}

FCachedCompositeFontData::FCachedCompositeFontData(const FCompositeFont& InCompositeFont)
	: CompositeFont(&InCompositeFont)
	, CachedTypefaces()
	, CachedPriorityFontRanges()
	, CachedFontRanges()
{
	// Add all the entries from the composite font
	CachedTypefaces.Add(MakeShared<FCachedTypefaceData>(CompositeFont->DefaultTypeface));
	CachedTypefaces.Add(MakeShared<FCachedTypefaceData>(CompositeFont->FallbackTypeface.Typeface, CompositeFont->FallbackTypeface.ScalingFactor));
	for (const FCompositeSubFont& SubTypeface : CompositeFont->SubTypefaces)
	{
		TSharedPtr<FCachedTypefaceData> CachedTypeface = MakeShared<FCachedTypefaceData>(SubTypeface.Typeface, SubTypeface.ScalingFactor);
		CachedTypefaces.Add(CachedTypeface);	
	}

	RefreshFontRanges();
}

const FCachedTypefaceData* FCachedCompositeFontData::GetTypefaceForCodepoint(const UTF32CHAR InCodepoint) const
{
	const int32 CharIndex = static_cast<int32>(InCodepoint);

	auto GetTypefaceFromRange = [CharIndex](const TArray<FCachedFontRange>& InFontRanges) -> const FCachedTypefaceData*
	{
		auto GetTypefaceFromRangeIndex = [CharIndex, &InFontRanges](const int32 InRangeIndex)
		{
			return (InFontRanges.IsValidIndex(InRangeIndex) && InFontRanges[InRangeIndex].Range.Contains(CharIndex))
				? InFontRanges[InRangeIndex].CachedTypeface.Get()
				: nullptr;
		};

		// Early out if this character is outside the bounds of any range
		if (InFontRanges.Num() == 0 || CharIndex < InFontRanges[0].Range.GetLowerBoundValue() || CharIndex > InFontRanges.Last().Range.GetUpperBoundValue())
		{
			return nullptr;
		}

		// This is only searching the lower-bound of the range, so it will return either the index of the correct range (if the character searched for
		// *is* the lower-bound of the range), otherwise it will return the next range - we need to test both ranges to see which one we should use
		const int32 FoundRangeIndex = Algo::LowerBoundBy(InFontRanges, CharIndex, &FCachedFontRange::BinarySearchKey);
		
		if (const FCachedTypefaceData* RangeTypeface = GetTypefaceFromRangeIndex(FoundRangeIndex))
		{
			return RangeTypeface;
		}
		
		if (const FCachedTypefaceData* RangeTypeface = GetTypefaceFromRangeIndex(FoundRangeIndex - 1))
		{
			return RangeTypeface;
		}

		return nullptr;
	};

	if (const FCachedTypefaceData* RangeTypeface = GetTypefaceFromRange(CachedPriorityFontRanges))
	{
		return RangeTypeface;
	}

	if (const FCachedTypefaceData* RangeTypeface = GetTypefaceFromRange(CachedFontRanges))
	{
		return RangeTypeface;
	}

	return CachedTypefaces[CachedTypeface_DefaultIndex].Get();
}

void FCachedCompositeFontData::GetCachedFontData(TArray<const FFontData*>& OutFontData) const
{
	for (const auto& CachedTypeface : CachedTypefaces)
	{
		CachedTypeface->GetCachedFontData(OutFontData);
	}
}

void FCachedCompositeFontData::RefreshFontRanges()
{
	// Get the current list of prioritized cultures so we can work out whether a sub-font should be prioritized or not
	const TArray<FString> PrioritizedCultureNames = FInternationalization::Get().GetPrioritizedCultureNames(FInternationalization::Get().GetCurrentLanguage()->GetName());
	auto IsPriorityCulture = [&PrioritizedCultureNames](const FString& InCultures) -> bool
	{
		TArray<FString> CultureNames;
		InCultures.ParseIntoArray(CultureNames, TEXT(";"));

		for (const FString& CultureName : CultureNames)
		{
			if (PrioritizedCultureNames.Contains(CultureName))
			{
				return true;
			}
		}

		return false;
	};

	// Clear the lists of ranges
	CachedPriorityFontRanges.Reset();
	CachedFontRanges.Reset();

	// Build the lists of ranges
	for (int32 SubTypefaceIndex = 0; SubTypefaceIndex < CompositeFont->SubTypefaces.Num(); ++SubTypefaceIndex)
	{
		const int32 CachedTypefaceIndex = CachedTypeface_FirstSubTypefaceIndex + SubTypefaceIndex;
		if (!CachedTypefaces.IsValidIndex(CachedTypefaceIndex))
		{
			continue;
		}

		const FCompositeSubFont& SubTypeface = CompositeFont->SubTypefaces[SubTypefaceIndex];
		TSharedPtr<FCachedTypefaceData> CachedTypeface = CachedTypefaces[CachedTypefaceIndex];

		TArray<FCachedFontRange>* CachedFontRangesPtr = SubTypeface.Cultures.IsEmpty() ? &CachedFontRanges : IsPriorityCulture(SubTypeface.Cultures) ? &CachedPriorityFontRanges : nullptr;
		if (CachedFontRangesPtr)
		{
			for (const FInt32Range& Range : SubTypeface.CharacterRanges)
			{
				if (!Range.IsEmpty())
				{
					// We cap the upper bound at INT32_MAX because of the following:
					// 1. Unicode only has about 1.1 million values. A whole chunk of the range leading up to INT32_MAX will all be invalid so there really is no difference between INT32_MAX and INT32_MAX - 1
					// 2. Capping the upper bound to INT32_MAX -1 greatly simplifies the line sweep algorithm we use for normalizing font ranges. See FCachedCompositeFontData::NormalizeFontRanges
					FInt32Range ClampedRange = Range;
					ClampedRange.SetUpperBoundValue(FMath::Min(Range.GetUpperBoundValue(), TNumericLimits<int32>::Max() - 1));
					CachedFontRangesPtr->Emplace(MoveTemp(ClampedRange), CachedTypeface);
				}
			}
		}
	}

	if (GNormalizeCachedFontRanges)
	{
		// normalization would already leave the array sorted and the contiguous font ranges using the same font face merged 
		TArray<FCachedFontRange> NormalizedCachedPriorityFontRanges = FCachedCompositeFontData::NormalizeFontRanges(CachedPriorityFontRanges);
		CachedPriorityFontRanges = MoveTemp(NormalizedCachedPriorityFontRanges);
		TArray<FCachedFontRange> NormalizedCachedFontRanges = FCachedCompositeFontData::NormalizeFontRanges(CachedFontRanges);
		CachedFontRanges = MoveTemp(NormalizedCachedFontRanges);
		return;
	}

	// Sort the font ranges into ascending order so we can binary search them later
	CachedPriorityFontRanges.Sort(&FCachedFontRange::SortPredicate);
	CachedFontRanges.Sort(&FCachedFontRange::SortPredicate);

	// Merge any contiguous ranges to minimize our search count
	auto MergeContiguousRanges = [](TArray<FCachedFontRange>& InFontRanges)
	{
		for (int32 RangeIndex = 0; RangeIndex < InFontRanges.Num() - 1; ++RangeIndex)
		{
			FCachedFontRange& ThisRange = InFontRanges[RangeIndex];
			const FCachedFontRange& NextRange = InFontRanges[RangeIndex + 1];

			check(!ThisRange.Range.IsEmpty() && !NextRange.Range.IsEmpty());

			// Can only merge ranges with the same typeface
			if (ThisRange.CachedTypeface == NextRange.CachedTypeface && (ThisRange.Range.Overlaps(NextRange.Range) || ThisRange.Range.GetUpperBoundValue() + 1 == NextRange.Range.GetLowerBoundValue()))
			{
				ThisRange.Range = FInt32Range(
					FInt32Range::BoundsType::Inclusive(ThisRange.Range.GetLowerBoundValue()), 
					FInt32Range::BoundsType::Inclusive(FMath::Max(ThisRange.Range.GetUpperBoundValue(), NextRange.Range.GetUpperBoundValue()))
					);
				InFontRanges.RemoveAt(RangeIndex + 1, EAllowShrinking::No);
			}
		}
	};
	MergeContiguousRanges(CachedPriorityFontRanges);
	MergeContiguousRanges(CachedFontRanges);
}

TArray<FCachedCompositeFontData::FCachedFontRange> FCachedCompositeFontData::NormalizeFontRanges(const TArray<FCachedCompositeFontData::FCachedFontRange>& InFontRanges)
{
	TArray< FCachedFontRange> OutFontRanges;
	if (InFontRanges.IsEmpty())
	{
		return OutFontRanges;
	}
	// We will at minimum have as many unicode ranges as our input cached font ranges 
	OutFontRanges.Reserve(InFontRanges.Num());

	// We will be using a line sweep algorithm to normalize the font ranges and get rid of any overlapped ranges 
	// We split each unicode font range into 2 events: a start event and end event
	// Say we have the font range (1, 1000), then we will have a start event at position 1 and an end event at position 1001 (the offset will be explained later)
	// We will then sort all of the events based on its position so we basically have a bunch of points on a number line
	// We sweep through all of the events, process all the events at the same position, create a new font range based on the event positions and decide which font face will be associated with the new font range
	// The end result is an array of font ranges that do not overlap, but their union is the same as the union of the input font ranges

	struct FSweepEvent
	{
		FSweepEvent() = default;
		FSweepEvent(int32 InPosition, int32 InFontRangeIndex, bool bInStart)
			: Position(InPosition)
			, FontRangeIndex(InFontRangeIndex)
			, bStart(bInStart)
		{

		}
		static bool SortPredicate(const FSweepEvent& A, const FSweepEvent& B)
		{
			if (A.Position != B.Position)
			{
				return A.Position < B.Position;
			}
			else if (A.bStart != B.bStart)
			{
				// We will prioritize end events over start events
				return static_cast<int32>(A.bStart) < static_cast<int32>(B.bStart);
			}
			// We use the order we processed the font ranges be the tie breaker if everything else is equal 
			return A.FontRangeIndex < B.FontRangeIndex;
		}

		bool IsValid() const
		{
			return FontRangeIndex != INDEX_NONE && Position >= 0;
		}
		/** The position of this event on the number line*/
		int32 Position = INDEX_NONE;
		/** Index of the font range this event was decomposed from*/
		int32 FontRangeIndex = INDEX_NONE;
		/** If this event at the position of the number line is the start of a range. True if it is the start and false if it is the end of the range*/
		bool bStart = false;
	};
	TArray<FSweepEvent> SweepEvents;
	// We decompose each unicode interval into a start and end event, thus we double the size of the cached font ranges
	SweepEvents.Reserve(InFontRanges.Num() * 2);
	for (int32 Index = 0; Index < InFontRanges.Num(); ++Index)
	{
		const FCachedFontRange& FontRange = InFontRanges[Index];
		// All empty font ranges should have already been filtered out prior to calling this function 
		check(!FontRange.Range.IsEmpty());
		SweepEvents.Emplace(FontRange.Range.GetLowerBoundValue(), Index, true);
		// Unicode does not go that high and having this requirement simplifies the algorithm
		// Anything with an upper bound of TNumericLimits<int32>::Max() should have been filtered out prior to this function 
		check(FontRange.Range.GetUpperBoundValue() != TNumericLimits<int32>::Max());
		// We always off set the upper bound by +1 because there would be an ambiguity otherwise whether the upper bound is inclusive or exclusive
		// Say we have the font ranges {(1, 1000), FontA}, {(200, 299), FontB}
		// We decompose into events without the offset: 1S, 200S, 299E, 1000E(S and E for start and end)
		// At 299E, we have the ambiguity of whether 299 should be covered by font B and we'll have to order processing of events and querying carefully
		// If we add the offset, we don't need to worry about any of that
		SweepEvents.Emplace(FontRange.Range.GetUpperBoundValue() + 1, Index, false);
	}
	// We should have exactly double the number of entries as the input font range, otherwise somehow some input font ranges were not decomposed and added properly 
	check(SweepEvents.Num() == InFontRanges.Num() * 2);
	SweepEvents.Sort(&FSweepEvent::SortPredicate);

	// As we sweep through the events and come across overlaps, we need to be able to figure out which font face applies to a particular range
	// This priority queue keeps track of the indices of all active font ranges at a particular event and sorts them by the sizes of the font ranges
	// We want to give precedence to smaller font ranges and use the font face associated with that range instead of a larger range if there is overlap. Example in the comparison lambda below 
	TArray<int32> ActiveFontRangeIndexHeap;
	ActiveFontRangeIndexHeap.Reserve(InFontRanges.Num());

	auto FontRangeComparator = [&InFontRanges](const int32 A, const int32 B)
	{
		const int32 SizeA = InFontRanges[A].Range.Size<int32>();
		const int32 SizeB = InFontRanges[B].Range.Size<int32>();
		// We want to prioritize font ranges that are smaller and more specific
		// E.g We have a range (1, 1000) associated with Font A and (50, 100) associated with Font B. B has the smaller range
		// so we assume that it takes precedence over FontA 
		if (SizeA != SizeB)
		{
			return SizeA < SizeB;
		}
		// If the size of the unicode ranges are identical, then we return the range that came first in the user declaration 
		return A < B;
	};
	// Allows us to keep track of which font ranges are active at any point during the sweep
	TBitArray<> FontRangeActiveStatus;
	FontRangeActiveStatus.Init(false, InFontRanges.Num());

	for (int32 Index = 0; Index < SweepEvents.Num(); )
	{
		const int32 CurrentPosition = SweepEvents[Index].Position;
		// We apply all of the events that occur at the same position first 
		while (Index < SweepEvents.Num() && SweepEvents[Index].Position == CurrentPosition)
		{
			const FSweepEvent& CurrentEvent = SweepEvents[Index];
			check(CurrentEvent.IsValid());
			if (CurrentEvent.bStart)
			{
				FontRangeActiveStatus[CurrentEvent.FontRangeIndex] = true;
				ActiveFontRangeIndexHeap.HeapPush(CurrentEvent.FontRangeIndex, FontRangeComparator);
			}
			else
			{
				FontRangeActiveStatus[CurrentEvent.FontRangeIndex] = false;
				// We do not remove the element from the heap here as that can be expensive
				// Instead we keep popping elements off the heap afterwards and ignore the inactive elements
			}
			++Index;
		}

		// Now where we are, we get the top most priority range index that is active
		// This represents the index of the font range (and consequently font face) that will be used for this new font range we are constructing 
		int32 ActiveFontRangeIndex = INDEX_NONE;
		while (!ActiveFontRangeIndexHeap.IsEmpty())
		{
			int32 TopIndex = ActiveFontRangeIndexHeap.HeapTop();
			if (FontRangeActiveStatus[TopIndex])
			{
				ActiveFontRangeIndex = TopIndex;
				break;
			}
			// The top index is associated with a range that is already inactive and thus should be removed 
			ActiveFontRangeIndexHeap.HeapPopDiscard(FontRangeComparator, EAllowShrinking::No);
		}

		// There are no active font ranges/font faces at this event, we do not create a new font range 
		if (ActiveFontRangeIndex == INDEX_NONE)
		{
			continue;
		}
		// The only time we would be able to get here with Index == SweepEvents.Num() is if there is a unicode range that has no end. I.e it is somehow unbounded 
		check(Index != SweepEvents.Num());

		// Now we create a new range
		const int32 LowerBound = CurrentPosition;
		// We need to remove the +1 bias from when we created the sweep events 
		const int32 UpperBound = SweepEvents[Index].Position - 1;
		// If this is violated, the events were somehow not sorted properly 
		check(UpperBound >= LowerBound);
		const TSharedPtr<FCachedTypefaceData>& ActiveFontFace = InFontRanges[ActiveFontRangeIndex].CachedTypeface;

		FInt32Range NewRange(FInt32Range::BoundsType::Inclusive(LowerBound), FInt32Range::BoundsType::Inclusive(UpperBound));
		// if this new range is using the same font face as the previous range, we can merge the ranges
		if (!OutFontRanges.IsEmpty())
		{
			FCachedFontRange& PreviousFontRange = OutFontRanges.Last();
			// if the previous font range is using the same font face and is adjoined to the new range, the 2 ranges are contiguous and we can merge them 
			if (PreviousFontRange.CachedTypeface == ActiveFontFace && PreviousFontRange.Range.Adjoins(NewRange))
			{
				PreviousFontRange.Range.SetUpperBoundValue(UpperBound);
				continue;
			}
		}

		OutFontRanges.Emplace(MoveTemp(NewRange), ActiveFontFace);
	}
	return OutFontRanges;
}

FCompositeFontCache::FCompositeFontCache(const FFreeTypeLibrary* InFTLibrary)
	: FTLibrary(InFTLibrary)
{
	check(FTLibrary);

	FInternationalization::Get().OnCultureChanged().AddRaw(this, &FCompositeFontCache::HandleCultureChanged);
}

FCompositeFontCache::~FCompositeFontCache()
{
	FInternationalization::Get().OnCultureChanged().RemoveAll(this);

	for (FAsyncTask<FAsyncLoadFontFaceData>* LoadFontFaceTask : LoadFontFaceTasks)
	{
		LoadFontFaceTask->EnsureCompletion();
		delete LoadFontFaceTask;
	}
}


const FFontData& FCompositeFontCache::GetDefaultFontData(const FSlateFontInfo& InFontInfo)
{
	static const FFontData DummyFontData;

	const FCompositeFont* const ResolvedCompositeFont = InFontInfo.GetCompositeFont();
	const FCachedTypefaceData* const CachedTypefaceData = GetDefaultCachedTypeface(ResolvedCompositeFont);
	if (CachedTypefaceData)
	{
		// Try to find the correct font from the typeface
		const FFontData* FoundFontData = CachedTypefaceData->GetFontData(InFontInfo.TypefaceFontName);
		if (FoundFontData)
		{
			return *FoundFontData;
		}

		// Failing that, return the primary font
		FoundFontData = CachedTypefaceData->GetPrimaryFontData();
		if (FoundFontData)
		{
			return *FoundFontData;
		}
	}

	return DummyFontData;
}

bool FCompositeFontCache::HasLoadedFontForCodepoint (const FSlateFontInfo& InFontInfo, const UTF32CHAR InCodepoint)
{
	const FCompositeFont* const ResolvedCompositeFont = InFontInfo.GetCompositeFont();
	const FCachedTypefaceData* const CachedDefaultTypefaceData = GetDefaultCachedTypeface(ResolvedCompositeFont);
	const FCachedTypefaceData* const CachedTypefaceData = GetCachedTypefaceForCodepoint(ResolvedCompositeFont, InCodepoint);
	if (!CachedTypefaceData)
	{
		return false;
	}

	auto IsFontValidAndNotLoading = [this](const FFontData* InFoundFontData)
	{
		if (!InFoundFontData)
		{
			return false;
		}

		// if this font supports the codepoint and it's not in loading state, we are good, no need for further tests
		TSharedPtr<FFreeTypeFace> FaceAndMemory = FontFaceMap.FindRef(*InFoundFontData);
		if (FaceAndMemory.IsValid() && FaceAndMemory->IsFaceValid() && !FaceAndMemory->IsFaceLoading())
		{
			return true;
		}

		return false;
	};

	auto FontDataMightSupportCodepoint = [this, InCodepoint, &IsFontValidAndNotLoading](const FFontData* InFoundFontData)
	{
		if (!InFoundFontData)
		{
			return false;
		}

		if (!IsFontValidAndNotLoading(InFoundFontData))
		{
			// Font not loaded; assume it can render the codepoint
			return true;
		}

		return DoesFontDataSupportCodepoint(*InFoundFontData, InCodepoint);
	};

	// Check the preferred typeface for the font
	const FFontData* FoundFontData = CachedTypefaceData->GetFontData(InFontInfo.TypefaceFontName);
	if (!FoundFontData)
	{
		// if we can't find a font, return immediately false
		return false;
	}

	if (FontDataMightSupportCodepoint(FoundFontData))
	{
		return true;
	}

	// Failing that, try and find a font by the attributes of the default font with the given name
	if (CachedTypefaceData && CachedTypefaceData != CachedDefaultTypefaceData)
	{
		if (const FFontData* const FoundDefaultFontData = CachedDefaultTypefaceData->GetFontData(InFontInfo.TypefaceFontName))
		{
			const TSet<FName>& DefaultFontAttributes = GetFontAttributes(*FoundDefaultFontData);
			FoundFontData = GetBestMatchFontForAttributes(CachedTypefaceData, DefaultFontAttributes);
			if (FontDataMightSupportCodepoint(FoundFontData))
			{
				return true;
			}
		}
	}

	// if we are still try, try the primary font
	FoundFontData = CachedTypefaceData->GetPrimaryFontData();
	if (FontDataMightSupportCodepoint(FoundFontData))
	{
		return true;
	}

	return false;
}

const FFontData& FCompositeFontCache::GetFontDataForCodepoint(const FSlateFontInfo& InFontInfo, const UTF32CHAR InCodepoint, float& OutScalingFactor)
{
	static const FFontData DummyFontData;

	auto GetFontDataForCharacterInTypeface = [this, InCodepoint, &InFontInfo](const FCachedTypefaceData* InCachedTypefaceData, const FCachedTypefaceData* InCachedDefaultTypefaceData, const bool InSkipCharacterCheck) -> const FFontData*
	{
		// Try to find the correct font from the typeface
		const FFontData* FoundFontData = InCachedTypefaceData->GetFontData(InFontInfo.TypefaceFontName);
		if (FoundFontData && (InSkipCharacterCheck || DoesFontDataSupportCodepoint(*FoundFontData, InCodepoint)))
		{
			return FoundFontData;
		}

		// Failing that, try and find a font by the attributes of the default font with the given name
		if (InCachedDefaultTypefaceData && InCachedTypefaceData != InCachedDefaultTypefaceData)
		{
			check(InCachedDefaultTypefaceData);

			if (const FFontData* const FoundDefaultFontData = InCachedDefaultTypefaceData->GetFontData(InFontInfo.TypefaceFontName))
			{
				const TSet<FName>& DefaultFontAttributes = GetFontAttributes(*FoundDefaultFontData);
				FoundFontData = GetBestMatchFontForAttributes(InCachedTypefaceData, DefaultFontAttributes);
				if (FoundFontData && (InSkipCharacterCheck || DoesFontDataSupportCodepoint(*FoundFontData, InCodepoint)))
				{
					return FoundFontData;
				}
			}
		}

		// Failing that, try the primary font
		FoundFontData = InCachedTypefaceData->GetPrimaryFontData();
		if (FoundFontData && (InSkipCharacterCheck || DoesFontDataSupportCodepoint(*FoundFontData, InCodepoint)))
		{
			return FoundFontData;
		}

		return nullptr;
	};

	const FCompositeFont* const ResolvedCompositeFont = InFontInfo.GetCompositeFont();
	const FCachedTypefaceData* const CachedTypefaceData = GetCachedTypefaceForCodepoint(ResolvedCompositeFont, InCodepoint);
	if (CachedTypefaceData)
	{
		const FCachedTypefaceData* const CachedDefaultTypefaceData = GetDefaultCachedTypeface(ResolvedCompositeFont);
		const FCachedTypefaceData* const CachedFallbackTypefaceData = GetFallbackCachedTypeface(ResolvedCompositeFont);
		check(CachedDefaultTypefaceData && CachedFallbackTypefaceData); // If we found a typeface from the font, we should always have found the default and fallback typeface too
		
		// Check the preferred typeface first
		if (const FFontData* FoundFontData = GetFontDataForCharacterInTypeface(CachedTypefaceData, CachedDefaultTypefaceData, /*InSkipCharacterCheck*/false))
		{
			OutScalingFactor = CachedTypefaceData->GetScalingFactor();
			return *FoundFontData;
		}

		// Failing that, try the default typeface (as the sub-font may not actually support the character we need)
		if (CachedTypefaceData != CachedDefaultTypefaceData)
		{
			if (const FFontData* FoundFontData = GetFontDataForCharacterInTypeface(CachedDefaultTypefaceData, nullptr, /*InSkipCharacterCheck*/false))
			{
				OutScalingFactor = CachedDefaultTypefaceData->GetScalingFactor();
				return *FoundFontData;
			}
		}

		// Failing that, try the fallback typeface (as both the sub-font or default font may not actually support the character we need)
		if (const FFontData* FoundFontData = GetFontDataForCharacterInTypeface(CachedFallbackTypefaceData, CachedDefaultTypefaceData, /*InSkipCharacterCheck*/true))
		{
			OutScalingFactor = CachedFallbackTypefaceData->GetScalingFactor();
			UE_LOGF(LogSlate, Verbose, "Using fallback font %ls for codepoint U+%x, as it was not found in default nor sub typefaces of SlateFontInfo %ls.", *FoundFontData->GetFontFilename(), InCodepoint, *InFontInfo.TypefaceFontName.ToString());
			return *FoundFontData;
		}

		// If we got this far then the fallback font is empty, just force it to use the default font
		if (const FFontData* FoundFontData = GetFontDataForCharacterInTypeface(CachedDefaultTypefaceData, nullptr, /*InSkipCharacterCheck*/true))
		{
			OutScalingFactor = CachedDefaultTypefaceData->GetScalingFactor();
			return *FoundFontData;
		}
	}

	OutScalingFactor = 1.0f;
	return DummyFontData;
}

TSharedPtr<FFreeTypeFace> FCompositeFontCache::GetFontFace(const FFontData& InFontData)
{
	LLM_SCOPE_BYTAG(UI_Text);

	TSharedPtr<FFreeTypeFace> FaceAndMemory = FontFaceMap.FindRef(InFontData);
	if (!FaceAndMemory.IsValid() && InFontData.HasFont())
	{
		SCOPE_CYCLE_COUNTER(STAT_SlateLoadFont);

		// IMPORTANT: Do not log from this function until the new font has been added to the FontFaceMap, as it may be the Output Log font being loaded, which would cause an infinite recursion!
		TArray<FString> LogMessages;
		const FString& FontFilename = InFontData.GetFontFilename();
		{
			// If this font data is referencing an asset, we just need to load it from memory
			FFontFaceDataConstPtr FontFaceData = InFontData.GetFontFaceData();
			if (FontFaceData.IsValid() && FontFaceData->HasData())
			{
				FaceAndMemory = MakeShared<FFreeTypeFace>(FTLibrary, FontFaceData.ToSharedRef(), InFontData.GetSubFaceIndex(), InFontData.GetLayoutMethod());
			}
		}

		// If no asset was loaded, then we go through the normal font loading process
		if (!FaceAndMemory.IsValid())
		{
			switch (InFontData.GetLoadingPolicy())
			{
			case EFontLoadingPolicy::LazyLoad:
				{
					if (UE::Private::FontCacheCompositeFont::IsAsyncFontLoadingAllowed())
					{
						LogMessages.Add(FString::Printf(TEXT("Requested to async lazy load '%s' font file with subface index %d in %s thread."), *FontFilename, InFontData.GetSubFaceIndex(), GetCurrentSlateTextureAtlasThreadIdAsString()));
						FAsyncTask<FAsyncLoadFontFaceData>* AsyncFontLoadTaskPtr = new FAsyncTask<FAsyncLoadFontFaceData>(InFontData);
						LoadFontFaceTasks.Add(AsyncFontLoadTaskPtr);
						AsyncFontLoadTaskPtr->StartBackgroundTask();

						FaceAndMemory = MakeShared<FFreeTypeFace>(InFontData.GetLayoutMethod());
					}
					else
					{
						LogMessages.Add(FString::Printf(TEXT("Synchronously lazy loading font file '%s' with subface index %d in %s thread."), *FontFilename, InFontData.GetSubFaceIndex(), GetCurrentSlateTextureAtlasThreadIdAsString()));
						const double FontDataLoadStartTime = FPlatformTime::Seconds();

						TArray<uint8> FontFaceDataArray;
						if (FFileHelper::LoadFileToArray(FontFaceDataArray, *FontFilename))
						{
							const int32 FontDataSizeKB = (FontFaceDataArray.Num() + 1023) / 1024;
							LogMessages.Add(FString::Printf(TEXT("Took %f seconds to synchronously load lazily loaded font '%s' (%dK) on %s thread."), float(FPlatformTime::Seconds() - FontDataLoadStartTime), *FontFilename, FontDataSizeKB, GetCurrentSlateTextureAtlasThreadIdAsString()));

							if (Private::IsFontFileName(FontFilename) || Private::IsFontFileData(FontFaceDataArray))
							{
								FaceAndMemory = MakeShared<FFreeTypeFace>(FTLibrary, FFontFaceData::MakeFontFaceData(MoveTemp(FontFaceDataArray)), InFontData.GetSubFaceIndex(), InFontData.GetLayoutMethod());
								LogMessages.Add(FString::Printf(TEXT("Freetype font face in memory successfully created with synchronously loaded raw font file '%s' subface %d in %s thread."), *FontFilename, InFontData.GetSubFaceIndex(), GetCurrentSlateTextureAtlasThreadIdAsString()));
							}
							else
							{
								FFontFaceDataRef FontFaceData = FFontFaceData::MakeFontFaceData();
								FMemoryReader Ar(FontFaceDataArray, true);
								FontFaceData->Serialize(Ar);

								FaceAndMemory = MakeShared<FFreeTypeFace>(FTLibrary, FontFaceData, InFontData.GetSubFaceIndex(), InFontData.GetLayoutMethod());
								LogMessages.Add(FString::Printf(TEXT("Freetype font face in memory successfully created with synchronously loaded cooked font file '%s' subface %d in %s thread."), *FontFilename, InFontData.GetSubFaceIndex(), GetCurrentSlateTextureAtlasThreadIdAsString()));
							}
						}
					}
				}
				break;
			case EFontLoadingPolicy::Stream:
				{
					LogMessages.Add(FString::Printf(TEXT("Requested to stream font file '%s' with subfontface index %d in %s thread."), *FontFilename, InFontData.GetSubFaceIndex(), GetCurrentSlateTextureAtlasThreadIdAsString()));
					FaceAndMemory = MakeShared<FFreeTypeFace>(FTLibrary, InFontData.GetFontFilename(), InFontData.GetSubFaceIndex(), InFontData.GetLayoutMethod());
				}
				break;
			default:
				break;
			}
		}

		// Got a valid font?
		if (FaceAndMemory.IsValid())
		{
			FaceAndMemory->OverrideAscent(InFontData.IsAscendOverridden(), InFontData.GetAscendOverriddenValue());
			FaceAndMemory->OverrideDescent(InFontData.IsDescendOverridden(), InFontData.GetDescendOverriddenValue());

			FontFaceMap.Add(InFontData, FaceAndMemory);
			LogMessages.Add(FString::Printf(TEXT("Font face '%s' subface index %d successfully added to font face map in %s thread."), *FontFilename, InFontData.GetSubFaceIndex(), GetCurrentSlateTextureAtlasThreadIdAsString()));
			for (const FString& LogMessage : LogMessages)
			{
				UE_LOGF(LogSlate, Display, "%ls", *LogMessage);
			}
#if WITH_EDITOR //Triggers texts layout regeneration only in editor, don't want to slow down things in engine.
			GSlateLayoutGeneration++;
#endif
		}
		else
		{
			FaceAndMemory.Reset();
			UE_LOGF(LogSlate, Warning, "GetFontFace failed to load or process '%ls' with subface index %d", *FontFilename, InFontData.GetSubFaceIndex());
		}
	}
	return FaceAndMemory;
}

const TSet<FName>& FCompositeFontCache::GetFontAttributes(const FFontData& InFontData)
{
	static const TSet<FName> DummyAttributes;

	TSharedPtr<FFreeTypeFace> FaceAndMemory = GetFontFace(InFontData);
	return (FaceAndMemory.IsValid()) ? FaceAndMemory->GetAttributes() : DummyAttributes;
}

void FCompositeFontCache::FlushCompositeFont(const FCompositeFont& InCompositeFont)
{
	CompositeFontToCachedDataMap.Remove(&InCompositeFont);
}

void FCompositeFontCache::FlushCache()
{
	UE_LOGF(LogSlate, Display, "Flushing composite font cache on %ls thread. Clearing %d entries in composite font map and %d entries in font face map.", GetCurrentSlateTextureAtlasThreadIdAsString(), CompositeFontToCachedDataMap.Num(), FontFaceMap.Num());
	CompositeFontToCachedDataMap.Empty();
	FontFaceMap.Empty();
}

void FCompositeFontCache::Update()
{
	if (LoadFontFaceTasks.Num() > 0)
	{
		bool bRefreshNeeded = false;
		const int32 NumLoadFontFaceTasks = LoadFontFaceTasks.Num();
		int32 NumCompletedFontFaceTasks = 0;
		const TCHAR* CurrentThreadString = GetCurrentSlateTextureAtlasThreadIdAsString();
		UE_LOGF(LogSlate, Verbose, "Updating composite font cache on %ls thread. %d font loading tasks need updating.", CurrentThreadString, NumLoadFontFaceTasks);
		for (int32 LoadTaskIndex = 0; LoadTaskIndex < LoadFontFaceTasks.Num(); LoadTaskIndex++)
		{
			FAsyncTask<FAsyncLoadFontFaceData>* LoadFontFaceTask = LoadFontFaceTasks[LoadTaskIndex];
			if (LoadFontFaceTask->IsDone())
			{
				TArray<uint8>& FontFaceDataArray = LoadFontFaceTask->GetTask().GetFontFaceData();
				const FString& FontFilename = LoadFontFaceTask->GetTask().GetFontData().GetFontFilename();

				const FFontData& FontData = LoadFontFaceTask->GetTask().GetFontData();
				TSharedPtr<FFreeTypeFace> FaceAndMemory = FontFaceMap.FindRef(FontData);
				if (FaceAndMemory.IsValid())
				{
					if (FontFaceDataArray.Num() > 0)
					{
						if (Private::IsFontFileName(FontFilename) || Private::IsFontFileData(FontFaceDataArray))
						{
							FaceAndMemory->CompleteAsyncLoad(FTLibrary, FFontFaceData::MakeFontFaceData(MoveTemp(FontFaceDataArray)), FontData.GetSubFaceIndex());
							UE_LOGF(LogSlate, Display, "Freetype font face in memory successfully updated with async loaded raw font file '%ls' subface %d in %ls thread.", *FontFilename, FontData.GetSubFaceIndex(), CurrentThreadString);
						}
						// Handles cooked .ufont files 
						else
						{
							FFontFaceDataRef FontFaceData = FFontFaceData::MakeFontFaceData();
							FMemoryReader Ar(FontFaceDataArray, true);
							FontFaceData->Serialize(Ar);

							FaceAndMemory->CompleteAsyncLoad(FTLibrary, FontFaceData, FontData.GetSubFaceIndex());
							UE_LOGF(LogSlate, Display, "Freetype font face in memory successfully updated with async loaded cooked font file '%ls' subface %d in %ls thread.", *FontFilename, FontData.GetSubFaceIndex(), CurrentThreadString);
						}
					}
					else
					{
						FaceAndMemory->FailAsyncLoad();
						UE_LOGF(LogSlate, Warning, "Async loaded font data for '%ls' font file with subface index %d on %ls thread is empty. The font face will not be available.", *FontFilename, FontData.GetSubFaceIndex(), CurrentThreadString);
					}

					bRefreshNeeded = true;
				}

				delete LoadFontFaceTask;
				LoadFontFaceTasks.RemoveAt(LoadTaskIndex);
				LoadTaskIndex--;
				++NumCompletedFontFaceTasks;
			}
		}

		if (bRefreshNeeded)
		{
			AsyncTask(ENamedThreads::GameThread, [NumLoadFontFaceTasks, NumCompletedFontFaceTasks, CurrentThreadString]() {
				if (FSlateApplicationBase::IsInitialized())
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FCompositFontCache_InvalidateAfterAsyncLoad);
					UE_LOGF(LogSlate, Display, "Invalidating all widgets from %d/%d async loading font faces completing loading on %ls thread.", NumCompletedFontFaceTasks, NumLoadFontFaceTasks, CurrentThreadString);
					FSlateApplicationBase::Get().InvalidateAllWidgets(false);
					GSlateLayoutGeneration++;
				}
			});
		}
	}
}

SIZE_T FCompositeFontCache::GetFontDataAssetResidentMemory(const UObject* FontDataAsset) const
{
	SIZE_T TotalAllocatedSize = 0;
	for (const TPair<FFontData, TSharedPtr<FFreeTypeFace>>& FaceAndMemoryData : FontFaceMap)
	{
		const FFontData& ExistingFontData = FaceAndMemoryData.Key;
		if (ExistingFontData.GetFontFaceAsset() == FontDataAsset)
		{
			TotalAllocatedSize += FaceAndMemoryData.Value->GetAllocatedMemorySize();
		}
	}

	return TotalAllocatedSize;
}

const FCachedCompositeFontData* FCompositeFontCache::GetCachedCompositeFont(const FCompositeFont* const InCompositeFont)
{
	if (!InCompositeFont)
	{
		return nullptr;
	}

	TSharedPtr<FCachedCompositeFontData>* const FoundCompositeFontData = CompositeFontToCachedDataMap.Find(InCompositeFont);
	if (FoundCompositeFontData)
	{
		return FoundCompositeFontData->Get();
	}

	return CompositeFontToCachedDataMap.Add(InCompositeFont, MakeShared<FCachedCompositeFontData>(*InCompositeFont)).Get();
}

const FFontData* FCompositeFontCache::GetBestMatchFontForAttributes(const FCachedTypefaceData* const InCachedTypefaceData, const TSet<FName>& InFontAttributes)
{
	const FFontData* BestMatchFont = nullptr;
	int32 BestMatchCount = 0;

	const FTypeface& Typeface = InCachedTypefaceData->GetTypeface();
	for (const FTypefaceEntry& TypefaceEntry : Typeface.Fonts)
	{
		const TSet<FName>& FontAttributes = GetFontAttributes(TypefaceEntry.Font);

		int32 MatchCount = 0;
		for (const FName& InAttribute : InFontAttributes)
		{
			if (FontAttributes.Contains(InAttribute))
			{
				++MatchCount;
			}
		}

		if (MatchCount > BestMatchCount || !BestMatchFont)
		{
			BestMatchFont = &TypefaceEntry.Font;
			BestMatchCount = MatchCount;
		}
	}

	return BestMatchFont;
}

bool FCompositeFontCache::DoesFontDataSupportCodepoint(const FFontData& InFontData, const UTF32CHAR InCodepoint)
{
#if WITH_FREETYPE
	TSharedPtr<FFreeTypeFace> FaceAndMemory = GetFontFace(InFontData);
	if (FaceAndMemory.IsValid())
	{
		if (!FaceAndMemory->IsFaceLoading())
		{
			return FT_Get_Char_Index(FaceAndMemory->GetFace(), InCodepoint) != 0;
		}

		// If the face is still loading, assume it has the the font face, until it loads and proves otherwise.
		return true;
	}
#endif // WITH_FREETYPE

	return false;
}

void FCompositeFontCache::HandleCultureChanged()
{
	// We need to refresh the font ranges immediately as the full font cache flush may not happen for one or more frames
	for (const auto& CompositeFontToCachedDataPair : CompositeFontToCachedDataMap)
	{
		CompositeFontToCachedDataPair.Value->RefreshFontRanges();
	}
}
