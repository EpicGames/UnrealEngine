// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "CoreMinimal.h"
#include "Fonts/CompositeFont.h"

class FFreeTypeFace;
class FFreeTypeLibrary;
struct FSlateFontInfo;

/**
 * Cached data for a given typeface
 */
class FCachedTypefaceData
{
public:
	/** Default constructor */
	FCachedTypefaceData();

	/** Construct the cache from the given typeface */
	FCachedTypefaceData(const FTypeface& InTypeface, const float InScalingFactor = 1.0f);

	/** Get the typeface we cached data from */
	FORCEINLINE const FTypeface& GetTypeface() const
	{
		check(Typeface);
		return *Typeface;
	}

	/** Get the font data associated with the primary (first) entry in this typeface */
	const FFontData* GetPrimaryFontData() const;

	/** Find the font associated with the given name */
	const FFontData* GetFontData(const FName& InName) const;

	/** Get the scaling factor for this typeface */
	FORCEINLINE float GetScalingFactor() const
	{
		return ScalingFactor;
	}

	/** Get all the font data cached by this entry */
	void GetCachedFontData(TArray<const FFontData*>& OutFontData) const;

private:
	/** Entry containing a name and the font data associated with that name */
	struct FCachedFontData
	{
		/** Default constructor */
		FCachedFontData()
			: Name()
			, FontData(nullptr)
		{
		}

		/** Construct from the given name and font data */
		FCachedFontData(const FName InName, const FFontData* InFontData)
			: Name(InName)
			, FontData(InFontData)
		{
		}

		/** Binary search key */
		static FName BinarySearchKey(const FCachedFontData& InCachedFontData)
		{
			return InCachedFontData.Name;
		}

		/** Sort predicate */
		static bool KeySortPredicate(const FName& InOne, const FName& InTwo)
		{
			// CompareIndexes is much faster than Compare since we don't care about these being sorted alphabetically
			return InOne.CompareIndexes(InTwo) < 0;
		}

		/** Sort predicate */
		static bool SortPredicate(const FCachedFontData& InOne, const FCachedFontData& InTwo)
		{
			return KeySortPredicate(InOne.Name, InTwo.Name);
		}

		/** Name of the font */
		FName Name;

		/** Data of the font */
		const FFontData* FontData;
	};

	/** Typeface we cached data from */
	const FTypeface* Typeface;

	/** Array of font data - this is sorted by name for a binary search */
	TArray<FCachedFontData, TInlineAllocator<1>> CachedFontData;

	/** Scaling factor to apply to this typeface */
	float ScalingFactor;
};


/**
 * Cached data for a given composite font
 */
class FCachedCompositeFontData
{
public:
	/** Default constructor */
	FCachedCompositeFontData();

	/** Construct the cache from the given composite font */
	FCachedCompositeFontData(const FCompositeFont& InCompositeFont);

	/** Get the composite font we cached data from */
	FORCEINLINE const FCompositeFont& GetCompositeFont() const
	{
		check(CompositeFont);
		return *CompositeFont;
	}

	/** Get the default typeface for this composite font */
	FORCEINLINE const FCachedTypefaceData* GetDefaultTypeface() const
	{
		return CachedTypefaces[CachedTypeface_DefaultIndex].Get();
	}

	/** Get the fallback typeface for this composite font */
	FORCEINLINE const FCachedTypefaceData* GetFallbackTypeface() const
	{
		return CachedTypefaces[CachedTypeface_FallbackIndex].Get();
	}

	/** Get the typeface that should be used for the given codepoint */
	const FCachedTypefaceData* GetTypefaceForCodepoint(const UTF32CHAR InCodepoint) const;

	/** Get all the font data cached by this entry */
	void GetCachedFontData(TArray<const FFontData*>& OutFontData) const;

	/** Refresh the font ranges used by sub-fonts (should be called when the culture is changed) */
	void RefreshFontRanges();

private:
	/** Entry containing a range and the typeface associated with that range */
	struct FCachedFontRange
	{
		/** Default constructor */
		FCachedFontRange()
			: Range(FInt32Range::Empty())
			, CachedTypeface()
		{
		}

		/** Construct from the given range and typeface */
		FCachedFontRange(const FInt32Range& InRange, TSharedPtr<FCachedTypefaceData> InCachedTypeface)
			: Range(InRange)
			, CachedTypeface(MoveTemp(InCachedTypeface))
		{
		}

		/** Binary search key */
		static int32 BinarySearchKey(const FCachedFontRange& InCachedFontRange)
		{
			return InCachedFontRange.Range.GetLowerBoundValue();
		}

		/** Sort predicate */
		static bool SortPredicate(const FCachedFontRange& InOne, const FCachedFontRange& InTwo)
		{
			check(!InOne.Range.IsEmpty() && !InTwo.Range.IsEmpty());
			return InOne.Range.GetLowerBoundValue() < InTwo.Range.GetLowerBoundValue();
		}

		/** Range to use for the typeface */
		FInt32Range Range;

		/** Typeface to which the range applies */
		TSharedPtr<FCachedTypefaceData> CachedTypeface;
	};

	static TArray<FCachedCompositeFontData::FCachedFontRange> NormalizeFontRanges(const TArray<FCachedCompositeFontData::FCachedFontRange>& InFontRanges);
	/** Constants for indexing into CachedTypefaces */
	static const int32 CachedTypeface_DefaultIndex = 0;
	static const int32 CachedTypeface_FallbackIndex = 1;
	static const int32 CachedTypeface_FirstSubTypefaceIndex = 2;

	/** Composite font we cached data from */
	const FCompositeFont* CompositeFont;

	/** Array of cached typefaces - 0 is the default typeface, 1 is the fallback typeface, and the remaining entries are sub-typefaces (see the CachedTypeface_X constants above) */
	TArray<TSharedPtr<FCachedTypefaceData>, TInlineAllocator<2>> CachedTypefaces;

	/** Array of font ranges paired with their associated typefaces - this is sorted in ascending order for a binary search */
	TArray<FCachedFontRange> CachedPriorityFontRanges;

	/** Array of font ranges paired with their associated typefaces - this is sorted in ascending order for a binary search */
	TArray<FCachedFontRange> CachedFontRanges;
};


/**
 * High-level caching of composite fonts and FreeType font faces
 */
class FCompositeFontCache
{
public:
	/** Constructor */
	FCompositeFontCache(const FFreeTypeLibrary* InFTLibrary);

	/** Destructor */
	~FCompositeFontCache();

	/** Get the default font data to use for the given font info */
	const FFontData& GetDefaultFontData(const FSlateFontInfo& InFontInfo);

	/** Get the font data to use for the given font info and codepoint */
	const FFontData& GetFontDataForCodepoint(const FSlateFontInfo& InFontInfo, const UTF32CHAR InCodepoint, float& OutScalingFactor);

	/** Return true if the font data to use for the given font info and codepoint is already loaded */
	bool HasLoadedFontForCodepoint(const FSlateFontInfo& InFontInfo, const UTF32CHAR InCodepoint);

	/** Gets or loads a FreeType font face */
	TSharedPtr<FFreeTypeFace> GetFontFace(const FFontData& InFontData);

	/** Get the attributes associated with the given font data */
	const TSet<FName>& GetFontAttributes(const FFontData& InFontData);

	/** Flush a single composite font entry from this cache */
	void FlushCompositeFont(const FCompositeFont& InCompositeFont);

	/** Flush this cache */
	void FlushCache();

	/** Give the composite font cache a chance to perform any tasks it has queued. */
	void Update();

	/**
	 * Gets the allocated font data for a font data asset
	 */
	SIZE_T GetFontDataAssetResidentMemory(const UObject* FontDataAsset) const;

private:
	/** Get the cached composite font data for the given composite font */
	const FCachedCompositeFontData* GetCachedCompositeFont(const FCompositeFont* const InCompositeFont);

	/** Get the default typeface for the given composite font */
	FORCEINLINE const FCachedTypefaceData* GetDefaultCachedTypeface(const FCompositeFont* const InCompositeFont)
	{
		const FCachedCompositeFontData* const CachedCompositeFont = GetCachedCompositeFont(InCompositeFont);
		return (CachedCompositeFont) ? CachedCompositeFont->GetDefaultTypeface() : nullptr;
	}

	/** Get the fallback typeface for the given composite font */
	FORCEINLINE const FCachedTypefaceData* GetFallbackCachedTypeface(const FCompositeFont* const InCompositeFont)
	{
		const FCachedCompositeFontData* const CachedCompositeFont = GetCachedCompositeFont(InCompositeFont);
		return (CachedCompositeFont) ? CachedCompositeFont->GetFallbackTypeface() : nullptr;
	}

	/** Get the typeface that should be used for the given codepoint */
	FORCEINLINE const FCachedTypefaceData* GetCachedTypefaceForCodepoint(const FCompositeFont* const InCompositeFont, const UTF32CHAR InCodepoint)
	{
		const FCachedCompositeFontData* const CachedCompositeFont = GetCachedCompositeFont(InCompositeFont);
		return (CachedCompositeFont) ? CachedCompositeFont->GetTypefaceForCodepoint(InCodepoint) : nullptr;
	}

	/** Try and find some font data that best matches the given attributes */
	const FFontData* GetBestMatchFontForAttributes(const FCachedTypefaceData* const InCachedTypefaceData, const TSet<FName>& InFontAttributes);

	/** Check to see whether the given font data supports rendering the given codepoint */
	bool DoesFontDataSupportCodepoint(const FFontData& InFontData, const UTF32CHAR InCodepoint);

	/** Called after the active culture has changed */
	void HandleCultureChanged();

	/** FreeType library instance to use */
	const FFreeTypeLibrary* FTLibrary;

	/** Mapping of composite fonts to their cached lookup data */
	TMap<const FCompositeFont*, TSharedPtr<FCachedCompositeFontData>> CompositeFontToCachedDataMap;

	/** Mapping of font data to FreeType faces */
	TMap<FFontData, TSharedPtr<FFreeTypeFace>> FontFaceMap;

	/** Update tasks from other threads we need to process. */
	TArray<class FAsyncTask<class FAsyncLoadFontFaceData>*> LoadFontFaceTasks;

	friend class FAsyncLoadFontFaceData;
};
