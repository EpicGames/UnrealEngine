// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/FontCacheFreeType.h"
#include "Fonts/SlateFontRenderer.h"
#include "SlateGlobals.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/FileHelper.h"
#include "Application/SlateApplicationBase.h"
#include "Async/Async.h"
#include "HAL/PlatformProcess.h"
#include "Trace/SlateMemoryTags.h"

#include "Fonts/PreprocessedFontGeometry.h"
#include "Fonts/UnicodeBlockRange.h"
#include <limits>

#if WITH_FREETYPE

static int32 GSlateEnableLegacyFontHinting = 0;
static FAutoConsoleVariableRef CVarSlateEnableLegacyFontHinting(TEXT("Slate.EnableLegacyFontHinting"), GSlateEnableLegacyFontHinting, TEXT("Enable the legacy font hinting? (0/1)."), ECVF_Default);

// The total amount of memory freetype allocates internally
DECLARE_MEMORY_STAT(TEXT("FreeType Total Allocated Memory"), STAT_SlateFreetypeAllocatedMemory, STATGROUP_SlateMemory);

// The active counts of resident and streaming fonts
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num Resident Fonts"), STAT_SlateResidentFontCount, STATGROUP_SlateMemory);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Num Streaming Fonts"), STAT_SlateStreamingFontCount, STATGROUP_SlateMemory);

namespace FreeTypeMemory
{

static void* Alloc(FT_Memory Memory, long Size)
{
	LLM_SCOPE_BYTAG(UI_Text);
	void* Result = FMemory::Malloc(Size);

#if STATS
	const SIZE_T ActualSize = FMemory::GetAllocSize( Result );
	INC_DWORD_STAT_BY(STAT_SlateFreetypeAllocatedMemory, ActualSize);
#endif

	return Result;
}

static void* Realloc(FT_Memory Memory, long CurSize, long NewSize, void* Block)
{
	LLM_SCOPE_BYTAG(UI_Text);

#if STATS
	long DeltaNewSize = NewSize - CurSize;
	INC_DWORD_STAT_BY(STAT_SlateFreetypeAllocatedMemory, DeltaNewSize);
#endif

	return FMemory::Realloc(Block, NewSize);
}

static void Free(FT_Memory Memory, void* Block)
{	
#if STATS
	const SIZE_T ActualSize = FMemory::GetAllocSize( Block );
	DEC_DWORD_STAT_BY(STAT_SlateFreetypeAllocatedMemory, ActualSize);
#endif
	return FMemory::Free(Block);
}

} // namespace FreeTypeMemory

#endif // WITH_FREETYPE


namespace FreeTypeUtils
{

#if WITH_FREETYPE

bool IsFaceEligibleForSdf(FT_Face InFace)
{
	return InFace && !FT_IS_TRICKY(InFace) && FT_IS_SCALABLE(InFace) && (!FT_HAS_FIXED_SIZES(InFace) || InFace->num_glyphs > 0);
}


bool IsGlyphEligibleForSdf(FT_GlyphSlot InGlyph)
{
	return InGlyph && InGlyph->format == FT_GLYPH_FORMAT_OUTLINE;
}

FT_F26Dot6 Frac26Dot6(const FT_F26Dot6 InValue)
{
	return InValue & 63;
}

FT_F26Dot6 Floor26Dot6(const FT_F26Dot6 InValue)
{
	return InValue & -64;
}

FT_F26Dot6 Ceil26Dot6(const FT_F26Dot6 InValue)
{
	return Floor26Dot6(InValue + 63);
}

FT_F26Dot6 Round26Dot6(const FT_F26Dot6 InValue)
{
	return Floor26Dot6(InValue + 32);
}

FT_Fixed DetermineEmScale(const uint16 InEmSize, const FT_F26Dot6 InPpem)
{
	return static_cast<FT_Fixed>(::FT_DivFix(InPpem, static_cast<FT_Long>(FMath::Max(uint16{1}, InEmSize))));
}

FT_Fixed DeterminePpemAndEmScale(const uint16 InEmSize, const uint32 InRenderSize)
{
	return DetermineEmScale(InEmSize, ConvertPixelTo26Dot6<FT_F26Dot6>(InRenderSize));
}

uint32 ComputeFontPixelSize(float InFontSize, float InFontScale)
{
	// Convert to fixed scale for maximum precision
	const FT_F26Dot6 FixedFontSize = ConvertPixelTo26Dot6<FT_F26Dot6>(FMath::Max<float>(0.0f, InFontSize));
	const FT_Long FixedFontScale = ConvertPixelTo16Dot16<FT_Long>(FMath::Max<float>(0.0f, InFontScale));

	// Convert the requested font size to a pixel size based on our render DPI and the requested scale

	// Convert the 26.6 character size to the unscaled 26.6 pixel size
	// Note: The logic here mirrors what FT_REQUEST_WIDTH and FT_REQUEST_HEIGHT do internally when using FT_Set_Char_Size
	FT_F26Dot6 RequiredFixedFontPixelSize = ((FixedFontSize * (FT_Pos)FontConstants::RenderDPI) + 36) / 72;

	// Scale the 26.6 pixel size by the desired 16.16 fractional scaling value
	RequiredFixedFontPixelSize =  FT_MulFix(RequiredFixedFontPixelSize, FixedFontScale);

	return Convert26Dot6ToRoundedPixel<uint32>(RequiredFixedFontPixelSize);
}

void ApplySizeAndScale(FT_Face InFace, const float InFontSize, const float InFontScale)
{
	ApplySizeAndScale(InFace, ComputeFontPixelSize(InFontSize, InFontScale));
}

void ApplySizeAndScale(FT_Face InFace, const uint32 RequiredFontPixelSize)
{
	if (FT_IS_SCALABLE(InFace))
	{
		// Set the pixel size
		FT_Error Error = FT_Set_Pixel_Sizes(InFace, RequiredFontPixelSize, RequiredFontPixelSize);
		check(Error == 0);
	}
	else if (FT_HAS_FIXED_SIZES(InFace))
	{
		FT_F26Dot6 RequiredFixedFontPixelSize = ConvertPixelTo26Dot6<uint32>(RequiredFontPixelSize);

		// Find the best bitmap strike for our required pixel size
		// Height is the most important metric, so that is the one we search for
		int32 BestStrikeIndex = INDEX_NONE;
		{
			FT_F26Dot6 RunningBestFixedStrikeHeight = 0;
			for (int32 PotentialStrikeIndex = 0; PotentialStrikeIndex < InFace->num_fixed_sizes; ++PotentialStrikeIndex)
			{
				const FT_F26Dot6 PotentialFixedStrikeHeight = InFace->available_sizes[PotentialStrikeIndex].y_ppem;

				// An exact match always wins
				if (PotentialFixedStrikeHeight == RequiredFixedFontPixelSize)
				{
					BestStrikeIndex = PotentialStrikeIndex;
					break;
				}

				// First loop?
				if (BestStrikeIndex == INDEX_NONE)
				{
					BestStrikeIndex = PotentialStrikeIndex;
					RunningBestFixedStrikeHeight = PotentialFixedStrikeHeight;
					continue;
				}

				// Our running strike is smaller than our required size, so choose this as our running strike... 
				if (RunningBestFixedStrikeHeight < RequiredFixedFontPixelSize)
				{
					// ... but only if it grows the size of our running strike
					if (PotentialFixedStrikeHeight > RunningBestFixedStrikeHeight)
					{
						BestStrikeIndex = PotentialStrikeIndex;
						RunningBestFixedStrikeHeight = PotentialFixedStrikeHeight;
					}
					continue;
				}
				
				// Our running strike is larger than our required size, so choose this as our running strike... 
				if (RunningBestFixedStrikeHeight > RequiredFixedFontPixelSize)
				{
					// ... but only if it shrinks the size of our running strike to no less than the required size
					if (PotentialFixedStrikeHeight < RunningBestFixedStrikeHeight && PotentialFixedStrikeHeight > RequiredFixedFontPixelSize)
					{
						BestStrikeIndex = PotentialStrikeIndex;
						RunningBestFixedStrikeHeight = PotentialFixedStrikeHeight;
					}
					continue;
				}
			}
		}

		// Set the strike index
		check(BestStrikeIndex != INDEX_NONE);
		FT_Error Error = FT_Select_Size(InFace, BestStrikeIndex);
		check(Error == 0);

		// Work out the scale required to convert from the chosen strike to the desired pixel size
		FT_Long FixedStrikeScale = 0;
		{
			// Convert the 26.6 values to 16.16 space so that we can use FT_DivFix
			const FT_Long RequiredFixedFontPixelSize16Dot16 = RequiredFixedFontPixelSize << 10;
			const FT_Long BestFixedStrikeHeight16Dot16 = InFace->available_sizes[BestStrikeIndex].y_ppem << 10;
			FixedStrikeScale = FT_DivFix(RequiredFixedFontPixelSize16Dot16, BestFixedStrikeHeight16Dot16);
		}

		// Fixed size fonts don't use the x_scale/y_scale values so we use them to store the scaling adjustment we 
		// need to apply to the bitmap (as a 16.16 fractional scaling value) to scale it to our desired pixel size
		// Note: Technically metrics are supposed to be read-only, so if this causes issues then we'll have to add
		// the information to FFreeTypeFace instead and pass it through to everything that calls ApplySizeAndScale
		InFace->size->metrics.x_scale = FixedStrikeScale;
		InFace->size->metrics.y_scale = FixedStrikeScale;
	}
}

FT_Error LoadGlyph(FT_Face InFace, const uint32 InGlyphIndex, const int32 InLoadFlags, const float InFontSize, const float InFontScale)
{
	return LoadGlyph(InFace, InGlyphIndex, InLoadFlags, ComputeFontPixelSize(InFontSize, InFontScale));
}

FT_Error LoadGlyph(FT_Face InFace, const uint32 InGlyphIndex, const int32 InLoadFlags, const uint32 RequiredFontPixelSize)
{
#if WITH_VERY_VERBOSE_SLATE_STATS
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FreetypeLoadGlyph);
#endif
	check(!(InLoadFlags & FT_LOAD_NO_SCALE));
	ApplySizeAndScale(InFace, RequiredFontPixelSize);
	return FT_Load_Glyph(InFace, InGlyphIndex, InLoadFlags);
}

FT_Pos GetHeight(FT_Face InFace, const EFontLayoutMethod InLayoutMethod)
{
	if (FT_IS_SCALABLE(InFace))
	{
		// Scalable fonts use the unscaled value, as the metrics have had scaling and rounding applied to them
		return (InLayoutMethod == EFontLayoutMethod::Metrics) ? (FT_Pos)InFace->height : (InFace->bbox.yMax - InFace->bbox.yMin);
	}
	else if (FT_HAS_FIXED_SIZES(InFace))
	{
		// Fixed size fonts don't support scaling, so the metrics value is already unscaled
		return InFace->size->metrics.height;
	}
	return 0;
}

FT_Pos GetScaledHeight(FT_Face InFace, const EFontLayoutMethod InLayoutMethod)
{
	if (FT_IS_SCALABLE(InFace))
	{
		// Scalable fonts use the unscaled value (and apply the scale manually), as the metrics have had rounding applied to them
		return FT_MulFix((InLayoutMethod == EFontLayoutMethod::Metrics) ? InFace->height : (InFace->bbox.yMax - InFace->bbox.yMin), InFace->size->metrics.y_scale);
	}
	else if (FT_HAS_FIXED_SIZES(InFace))
	{
		// Fixed size fonts don't support scaling, but we calculated the scale to use for the glyph in ApplySizeAndScale
		return FT_MulFix(InFace->size->metrics.height, InFace->size->metrics.y_scale);
	}
	return 0;
}

FT_Pos GetAscender(FT_Face InFace, const EFontLayoutMethod InLayoutMethod)
{
	if (FT_IS_SCALABLE(InFace))
	{
		// Scalable fonts use the unscaled value (and apply the scale manually), as the metrics have had rounding applied to them
		return FT_MulFix((InLayoutMethod == EFontLayoutMethod::Metrics) ? InFace->ascender : InFace->bbox.yMax, InFace->size->metrics.y_scale);
	}
	else if (FT_HAS_FIXED_SIZES(InFace))
	{
		// Fixed size fonts don't support scaling, but we calculated the scale to use for the glyph in ApplySizeAndScale
		return FT_MulFix(InFace->size->metrics.ascender, InFace->size->metrics.y_scale);
	}
	return 0;
}

FT_Pos GetDescender(FT_Face InFace, const EFontLayoutMethod InLayoutMethod)
{
	if (FT_IS_SCALABLE(InFace))
	{
		// Scalable fonts use the unscaled value (and apply the scale manually), as the metrics have had rounding applied to them
		return FT_MulFix((InLayoutMethod == EFontLayoutMethod::Metrics) ? InFace->descender : InFace->bbox.yMin, InFace->size->metrics.y_scale);
	}
	else if (FT_HAS_FIXED_SIZES(InFace))
	{
		// Fixed size fonts don't support scaling, but we calculated the scale to use for the glyph in ApplySizeAndScale
		return FT_MulFix(InFace->size->metrics.descender, InFace->size->metrics.y_scale);
	}
	return 0;
}

float GetBitmapAtlasScale(FT_Face InFace)
{
	if (!FT_IS_SCALABLE(InFace) && FT_HAS_FIXED_SIZES(InFace))
	{
		// We only scale images down to fit into the atlas
		// If they're smaller than our desired size we just let them be scaled on the GPU when rendering (see GetBitmapRenderScale)
		if (InFace->size->metrics.x_scale < ConvertPixelTo16Dot16<FT_Fixed>(1))
		{
			// Fixed size fonts don't support scaling, but we calculated the scale to use for the glyph in ApplySizeAndScale
			return InFace->size->metrics.x_scale / 65536.0f; // 16.16 to pixel scale
		}
	}
	return 1.0f;
}

float GetBitmapRenderScale(FT_Face InFace)
{
	if (!FT_IS_SCALABLE(InFace) && FT_HAS_FIXED_SIZES(InFace))
	{
		// We only scale images up when rendering
		// If they're larger than our desired size we scale them before they go into the atlas (see GetBitmapAtlasScale)
		if (InFace->size->metrics.x_scale > ConvertPixelTo16Dot16<FT_Fixed>(1))
		{
			// Fixed size fonts don't support scaling, but we calculated the scale to use for the glyph in ApplySizeAndScale
			return InFace->size->metrics.x_scale / 65536.0f; // 16.16 to pixel scale
		}
	}
	return 1.0f;
}

#endif // WITH_FREETYPE

}


FFreeTypeLibrary::FFreeTypeLibrary()
{
#if WITH_FREETYPE
	CustomMemory = static_cast<FT_Memory>(FMemory::Malloc(sizeof(*CustomMemory)));

	// Init FreeType
	CustomMemory->alloc = &FreeTypeMemory::Alloc;
	CustomMemory->realloc = &FreeTypeMemory::Realloc;
	CustomMemory->free = &FreeTypeMemory::Free;
	CustomMemory->user = nullptr;

	FT_Error Error = FT_New_Library(CustomMemory, &FTLibrary);
		
	if (Error)
	{
		checkf(0, TEXT("Could not init FreeType. Error code: %d"), Error);
	}
		
	FT_Add_Default_Modules(FTLibrary);

	// Set the interpreter version based on the hinting method we'd like
	{
		FT_UInt FTInterpreterVersion = GSlateEnableLegacyFontHinting ? TT_INTERPRETER_VERSION_35 : TT_INTERPRETER_VERSION_40;
		FT_Property_Set(FTLibrary, "truetype", "interpreter-version", &FTInterpreterVersion);
	}

	static bool bLoggedVersion = false;

	// Write out the version of freetype for debugging purposes
	if(!bLoggedVersion)
	{
		bLoggedVersion = true;
		FT_Int Major, Minor, Patch;

		FT_Library_Version(FTLibrary, &Major, &Minor, &Patch);
		UE_LOGF(LogSlate, Log, "Using FreeType %d.%d.%d", Major, Minor, Patch);
	}

#endif // WITH_FREETYPE
}

FFreeTypeLibrary::~FFreeTypeLibrary()
{
#if WITH_FREETYPE
	FT_Done_Library(FTLibrary);
	FMemory::Free(CustomMemory);
#endif // WITH_FREETYPE
}

FFreeTypeFace::FFreeTypeFace(const FFreeTypeLibrary* InFTLibrary, FFontFaceDataConstRef InMemory, const int32 InFaceIndex, const EFontLayoutMethod InLayoutMethod)
#if WITH_FREETYPE
	: FTFace(nullptr)
	, bPendingAsyncLoad(true)
#if WITH_ATLAS_DEBUGGING
	, OwnerThread(GetCurrentSlateTextureAtlasThreadId())
#endif
#endif // WITH_FREETYPE
{
	LayoutMethod = InLayoutMethod;
	CompleteAsyncLoad(InFTLibrary, InMemory, InFaceIndex);
}

FFreeTypeFace::FFreeTypeFace(const EFontLayoutMethod InLayoutMethod)
#if WITH_FREETYPE
	: FTFace(nullptr)
	, bPendingAsyncLoad(true)
#if WITH_ATLAS_DEBUGGING
	, OwnerThread(GetCurrentSlateTextureAtlasThreadId())
#endif
#endif // WITH_FREETYPE
{
	LayoutMethod = InLayoutMethod;
}

SIZE_T FFreeTypeFace::GetAllocatedMemorySize() const
{
#if WITH_FREETYPE
	if (Memory.IsValid())
	{
		return Memory->GetAllocatedSize();
	}
#endif
	return 0;
}

const UE::Slate::FPreprocessedFontGeometry* FFreeTypeFace::GetPreprocessedFontGeometry() const
{
#if WITH_FREETYPE
	if (Memory.IsValid())
	{
		const int32 FaceIndex = FTFace->face_index&0x7fff; // Bits 0-15 according to FreeType documentation
		if (Memory->GetPreprocessedFontGeometries().IsValidIndex(FaceIndex))
		{
			return &Memory->GetPreprocessedFontGeometries()[FaceIndex];
		}
	}
#endif
	return nullptr;
}

void FFreeTypeFace::FailAsyncLoad()
{
#if WITH_FREETYPE
	ensure(bPendingAsyncLoad);
	bPendingAsyncLoad = false;
#endif
}

void FFreeTypeFace::CompleteAsyncLoad(const FFreeTypeLibrary* InFTLibrary, FFontFaceDataConstRef InMemory, const int32 InFaceIndex)
{
#if WITH_FREETYPE
	ensure(bPendingAsyncLoad);
	bPendingAsyncLoad = false;

	FTFace = nullptr;
	Memory = MoveTemp(InMemory);

	FT_Error Error = FT_New_Memory_Face(InFTLibrary->GetLibrary(), Memory->GetData().GetData(), static_cast<FT_Long>(Memory->GetData().Num()), InFaceIndex, &FTFace);

	if (Error)
	{
		// You can look these error codes up in the FreeType docs: https://www.freetype.org/freetype2/docs/reference/ft2-error_code_values.html
		//ensureAlwaysMsgf(0, TEXT("FT_New_Memory_Face failed with error code %i"), Error);

		// We assume the face is null if the function errored
		ensureAlwaysMsgf(FTFace == nullptr, TEXT("FT_New_Memory_Face failed but also returned a non-null FT_Face. This is unexpected and may leak memory!"));
		FTFace = nullptr;
	}

	ensureAlwaysMsgf(FTFace == nullptr || (FTFace->face_index&0x7fff) == InFaceIndex, TEXT("FTFace->face_index value differs from requested face index! Avoid using the face_index value"));

	ParseAttributes();

	if (Memory->HasData())
	{
		INC_DWORD_STAT_BY(STAT_SlateResidentFontCount, 1);
	}
#endif // WITH_FREETYPE
}

FFreeTypeFace::FFreeTypeFace(const FFreeTypeLibrary* InFTLibrary, const FString& InFilename, const int32 InFaceIndex, const EFontLayoutMethod InLayoutMethod)
#if WITH_FREETYPE
	: FTFace(nullptr)
	, FTStreamHandler(InFilename)
#if WITH_ATLAS_DEBUGGING
	, OwnerThread(GetCurrentSlateTextureAtlasThreadId())
#endif
#endif // WITH_FREETYPE
{
	LayoutMethod = InLayoutMethod;

#if WITH_FREETYPE
	ensure(FTStreamHandler.FontSizeBytes >= 0 && FTStreamHandler.FontSizeBytes <= std::numeric_limits<uint32>::max());
	FMemory::Memzero(FTStream);
	FTStream.size = (uint32)FTStreamHandler.FontSizeBytes;
	FTStream.descriptor.pointer = &FTStreamHandler;
	FTStream.close = &FFTStreamHandler::CloseFile;
	FTStream.read = &FFTStreamHandler::ReadData;

	FMemory::Memzero(FTFaceOpenArgs);
	FTFaceOpenArgs.flags = FT_OPEN_STREAM;
	FTFaceOpenArgs.stream = &FTStream;

	FT_Error Error = FT_Open_Face(InFTLibrary->GetLibrary(), &FTFaceOpenArgs, InFaceIndex, &FTFace);

	if (Error)
	{
		// You can look these error codes up in the FreeType docs: https://www.freetype.org/freetype2/docs/reference/ft2-error_code_values.html
		ensureAlwaysMsgf(0, TEXT("FT_Open_Face failed with error code 0x%02X, filename '%s'"), Error, *InFilename);

		// We assume the face is null if the function errored
		ensureAlwaysMsgf(FTFace == nullptr, TEXT("FT_Open_Face failed but also returned a non-null FT_Face. This is unexpected and may leak memory!"));
		FTFace = nullptr;
	}

	ParseAttributes();

	INC_DWORD_STAT_BY(STAT_SlateStreamingFontCount, 1);
#endif // WITH_FREETYPE
}

FFreeTypeFace::~FFreeTypeFace()
{
#if WITH_FREETYPE
	if (FTFace)
	{
		if (Memory.IsValid() && Memory->HasData())
		{
			DEC_DWORD_STAT_BY(STAT_SlateResidentFontCount, 1);
		}
		else
		{
			DEC_DWORD_STAT_BY(STAT_SlateStreamingFontCount, 1);
		}

		FT_Done_Face(FTFace);

		FTStreamHandler = FFTStreamHandler();
	}
#endif // WITH_FREETYPE
}

TArray<FString> FFreeTypeFace::GetAvailableSubFaces(const FFreeTypeLibrary* InFTLibrary, FFontFaceDataConstRef InMemory)
{
	TArray<FString> Result;

#if WITH_FREETYPE
	FT_Face FTFace = nullptr;
	FT_New_Memory_Face(InFTLibrary->GetLibrary(), InMemory->GetData().GetData(), static_cast<FT_Long>(InMemory->GetData().Num()), -1, &FTFace);
	if (FTFace)
	{
		const int32 NumFaces = FTFace->num_faces;
		FT_Done_Face(FTFace);
		FTFace = nullptr;

		Result.Reserve(NumFaces);
		for (int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
		{
			FT_New_Memory_Face(InFTLibrary->GetLibrary(), InMemory->GetData().GetData(), static_cast<FT_Long>(InMemory->GetData().Num()), FaceIndex, &FTFace);
			if (FTFace)
			{
				Result.Add(FString::Printf(TEXT("%s (%s)"), UTF8_TO_TCHAR(FTFace->family_name), UTF8_TO_TCHAR(FTFace->style_name)));
				FT_Done_Face(FTFace);
				FTFace = nullptr;
			}
		}
	}
#endif // WITH_FREETYPE

	return Result;
}

TArray<FString> FFreeTypeFace::GetAvailableSubFaces(const FFreeTypeLibrary* InFTLibrary, const FString& InFilename)
{
	TArray<FString> Result;

#if WITH_FREETYPE
	FFTStreamHandler FTStreamHandler(InFilename);

	ensure(FTStreamHandler.FontSizeBytes >= 0 && FTStreamHandler.FontSizeBytes <= std::numeric_limits<uint32>::max());
	FT_StreamRec FTStream;
	FMemory::Memzero(FTStream);
	FTStream.size = (uint32)FTStreamHandler.FontSizeBytes;
	FTStream.descriptor.pointer = &FTStreamHandler;
	FTStream.close = &FFTStreamHandler::CloseFile;
	FTStream.read = &FFTStreamHandler::ReadData;

	FT_Open_Args FTFaceOpenArgs;
	FMemory::Memzero(FTFaceOpenArgs);
	FTFaceOpenArgs.flags = FT_OPEN_STREAM;
	FTFaceOpenArgs.stream = &FTStream;

	FT_Face FTFace = nullptr;
	FT_Open_Face(InFTLibrary->GetLibrary(), &FTFaceOpenArgs, -1, &FTFace);
	if (FTFace)
	{
		const int32 NumFaces = FTFace->num_faces;
		FT_Done_Face(FTFace);
		FTFace = nullptr;

		Result.Reserve(NumFaces);
		for (int32 FaceIndex = 0; FaceIndex < NumFaces; ++FaceIndex)
		{
			FT_Open_Face(InFTLibrary->GetLibrary(), &FTFaceOpenArgs, FaceIndex, &FTFace);
			if (FTFace)
			{
				Result.Add(FString::Printf(TEXT("%s (%s)"), UTF8_TO_TCHAR(FTFace->family_name), UTF8_TO_TCHAR(FTFace->style_name)));
				FT_Done_Face(FTFace);
				FTFace = nullptr;
			}
		}
	}
#endif // WITH_FREETYPE

	return Result;
}

#if WITH_FREETYPE
void FFreeTypeFace::ParseAttributes()
{
	if (FTFace)
	{
		// Parse out the font attributes
		TArray<FString> Styles;
		FString(UTF8_TO_TCHAR(FTFace->style_name)).ParseIntoArray(Styles, TEXT(" "), true);

		for (const FString& Style : Styles)
		{
			Attributes.Add(*Style);
		}
	}
}

FFreeTypeFace::FFTStreamHandler::FFTStreamHandler()
	: FileHandle(nullptr)
	, FontSizeBytes(0)
{
}

FFreeTypeFace::FFTStreamHandler::FFTStreamHandler(const FString& InFilename)
	: FileHandle(FPlatformFileManager::Get().GetPlatformFile().OpenRead(*InFilename))
	, FontSizeBytes(FileHandle ? FileHandle->Size() : 0)
{
}

FFreeTypeFace::FFTStreamHandler::~FFTStreamHandler()
{
	check(!FileHandle);
}

void FFreeTypeFace::FFTStreamHandler::CloseFile(FT_Stream InStream)
{
	FFTStreamHandler* MyStream = (FFTStreamHandler*)InStream->descriptor.pointer;

	if (MyStream->FileHandle)
	{
		delete MyStream->FileHandle;
		MyStream->FileHandle = nullptr;
	}
}

unsigned long FFreeTypeFace::FFTStreamHandler::ReadData(FT_Stream InStream, unsigned long InOffset, unsigned char* InBuffer, unsigned long InCount)
{
	FFTStreamHandler* MyStream = (FFTStreamHandler*)InStream->descriptor.pointer;

	if (MyStream->FileHandle)
	{
		if (!MyStream->FileHandle->Seek(InOffset))
		{
			return 0;
		}
	}

	if (InCount > 0)
	{
		if (MyStream->FileHandle)
		{
			if (!MyStream->FileHandle->Read(InBuffer, InCount))
			{
				return 0;
			}
		}
		else
		{
			return 0;
		}
	}

	return InCount;
}
#endif // WITH_FREETYPE


#if WITH_FREETYPE

class FFreeTypeSizeLock
{
public:
	// When we are using the face unscaled
	// for the sdf cache / pipeline we 
	// change the face size setting that the 
	// non-sdf pipeline could have scaled.
	// When we are done we reapply the 
	// previous size in case some logic
	// expects the size and scale to
	// not have changed.
	explicit FFreeTypeSizeLock(FT_Size InSize)
	{
		check(InSize && InSize->face && InSize->face->size);
		PreviousSize = InSize->face->size;
		::FT_Activate_Size(InSize);
	}

	~FFreeTypeSizeLock()
	{
		check(PreviousSize)
		::FT_Activate_Size(PreviousSize);
	}

private:
	FT_Size PreviousSize;
};

FFreeTypeGlyphCache::FFreeTypeGlyphCache(FT_Face InFace, const int32 InLoadFlags, const uint32 InFontRenderSize)
	: Face(InFace)
	, LoadFlags(InLoadFlags)
	, FontRenderSize(InFontRenderSize)
	, GlyphDataMap()
{
}

bool FFreeTypeGlyphCache::FindOrCache(const uint32 InGlyphIndex, FCachedGlyphData& OutCachedGlyphData)
{
	// Try and find the data from the cache...
	{
		const FCachedGlyphData* FoundCachedGlyphData = GlyphDataMap.Find(InGlyphIndex);
		if (FoundCachedGlyphData)
		{
			OutCachedGlyphData = *FoundCachedGlyphData;
			return true;
		}
	}

	// No cached data, go ahead and add an entry for it...
	FT_Error Error = FreeTypeUtils::LoadGlyph(Face, InGlyphIndex, LoadFlags, FontRenderSize);
	if (Error == 0)
	{
		OutCachedGlyphData = FCachedGlyphData();
		OutCachedGlyphData.Height = Face->height;
		OutCachedGlyphData.GlyphMetrics = Face->glyph->metrics;
		OutCachedGlyphData.SizeMetrics = Face->size->metrics;

		if (Face->glyph->outline.n_points > 0)
		{
			const int32 NumPoints = static_cast<int32>(Face->glyph->outline.n_points);
			OutCachedGlyphData.OutlinePoints.Reserve(NumPoints);
			for (int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
			{
				OutCachedGlyphData.OutlinePoints.Add(Face->glyph->outline.points[PointIndex]);
			}
		}

		GlyphDataMap.Emplace(InGlyphIndex, OutCachedGlyphData);
		return true;
	}

	return false;
}

FFreeTypeAdvanceCache::FFreeTypeAdvanceCache()
	: Face(0)
	, LoadFlags(0)
	, FontRenderSize(0)
	, AdvanceMap()
{
	// invalid cache always returns false
}

FFreeTypeAdvanceCache::FFreeTypeAdvanceCache(FT_Face InFace, const int32 InLoadFlags, const uint32 InFontRenderSize)
	: Face(InFace)
	, LoadFlags(InLoadFlags)
	, FontRenderSize((0 != (InLoadFlags & FT_LOAD_NO_SCALE))
			   ? static_cast<uint32>(FreeTypeUtils::DeterminePpemAndEmScale(InFace->units_per_EM, InFontRenderSize))
			   : InFontRenderSize) // compute once the EmScale and keep in FontSize when caching unscaled advances
	, AdvanceMap()
{
	check(((LoadFlags & FT_LOAD_NO_SCALE) == 0) || FreeTypeUtils::IsFaceEligibleForSdf(Face));
}

bool FFreeTypeAdvanceCache::FindOrCache(const uint32 InGlyphIndex, FT_Fixed& OutCachedAdvance)
{
	if (!Face)
	{
		return false;
	}
	// Try and find the advance from the cache...
	{
		const FT_Fixed* FoundCachedAdvance = AdvanceMap.Find(InGlyphIndex);
		if (FoundCachedAdvance)
		{
			OutCachedAdvance = *FoundCachedAdvance;
			return true;
		}
	}
	if (((LoadFlags & FT_LOAD_NO_SCALE) != 0))
	{
		const FT_Error Error = FT_Get_Advance(Face, InGlyphIndex, LoadFlags, &OutCachedAdvance);
		if (Error == 0)
		{													  // The emscale is computed once in the constructor and 
			FT_Long EmScale = static_cast<FT_Long>(FontRenderSize); // kept in FontSize when caching unscaled advances
			OutCachedAdvance = (FT_Fixed)FT_MulDiv((FT_Long)OutCachedAdvance, EmScale, FT_Long{ 64L });
			AdvanceMap.Add(InGlyphIndex, OutCachedAdvance);
			return true;
		}
	}
	else
	{
		FreeTypeUtils::ApplySizeAndScale(Face, FontRenderSize);

		// No cached data, go ahead and add an entry for it...
		const FT_Error Error = FT_Get_Advance(Face, InGlyphIndex, LoadFlags, &OutCachedAdvance);
		if (Error == 0)
		{
			if (!FT_IS_SCALABLE(Face) && FT_HAS_FIXED_SIZES(Face))
			{
				// Fixed size fonts don't support scaling, but we calculated the scale to use for the glyph in ApplySizeAndScale
				OutCachedAdvance = FT_MulFix(OutCachedAdvance, ((LoadFlags & FT_LOAD_VERTICAL_LAYOUT) ? Face->size->metrics.y_scale : Face->size->metrics.x_scale));
			}

			AdvanceMap.Add(InGlyphIndex, OutCachedAdvance);
			return true;
		}
	}

	return false;
}


FFreeTypeKerningCache::FFreeTypeKerningCache(FT_Face InFace, const int32 InKerningFlags, const uint32 InFontRenderSize)
	: Face(InFace)
	, KerningFlags(InKerningFlags)
	, FontRenderSize(InKerningFlags == FT_KERNING_UNSCALED ?
			   static_cast<uint32>(FreeTypeUtils::DeterminePpemAndEmScale(InFace->units_per_EM, InFontRenderSize)) :
			   InFontRenderSize) // compute once the EmScale and keep in FontSize when caching unscaled advances
	, KerningMap()
{
	check(Face);
	check(FT_HAS_KERNING(InFace));
	check(KerningFlags != FT_KERNING_UNSCALED || FreeTypeUtils::IsFaceEligibleForSdf(Face));
}
bool FFreeTypeKerningCache::FindOrCache(const uint32 InFirstGlyphIndex, const uint32 InSecondGlyphIndex, FT_Vector& OutKerning)
{
	const FKerningPair KerningPair(InFirstGlyphIndex, InSecondGlyphIndex);

	// Try and find the kerning from the cache...
	{
		const FT_Vector* FoundCachedKerning = KerningMap.Find(KerningPair);
		if (FoundCachedKerning)
		{
			OutKerning = *FoundCachedKerning;
			return true;
		}
	}

	if (KerningFlags == FT_KERNING_UNSCALED)
	{
		const FT_Error Error = FT_Get_Kerning(Face, InFirstGlyphIndex, InSecondGlyphIndex, KerningFlags, &OutKerning);
		if (Error == 0)
		{													  // The emscale is computed once in the constructor and 
			FT_Long EmScale = static_cast<FT_Long>(FontRenderSize); // kept in FontSize when caching unscaled kernings
			OutKerning.x = (FT_Pos)FT_MulFix(OutKerning.x, EmScale);
			OutKerning.y = (FT_Pos)FT_MulFix(OutKerning.y, EmScale);
			KerningMap.Add(KerningPair, OutKerning);
			return true;
		}
	}
	else
	{
		FreeTypeUtils::ApplySizeAndScale(Face, FontRenderSize);

		// No cached data, go ahead and add an entry for it...
		const FT_Error Error = FT_Get_Kerning(Face, InFirstGlyphIndex, InSecondGlyphIndex, KerningFlags, &OutKerning);
		if (Error == 0)
		{
			if (!FT_IS_SCALABLE(Face) && FT_HAS_FIXED_SIZES(Face))
			{
				// Fixed size fonts don't support scaling, but we calculated the scale to use for the glyph in ApplySizeAndScale
				OutKerning.x = FT_MulFix(OutKerning.x, Face->size->metrics.x_scale);
				OutKerning.y = FT_MulFix(OutKerning.y, Face->size->metrics.y_scale);
			}

			KerningMap.Add(KerningPair, OutKerning);
			return true;
		}
	}

	return false;
}

TSharedRef<FFreeTypeGlyphCache> FFreeTypeCacheDirectory::GetGlyphCache(FT_Face InFace, const int32 InLoadFlags, const uint32 InFontRenderSize)
{
	const FFontKey Key(InFace, InLoadFlags, InFontRenderSize);
	TSharedPtr<FFreeTypeGlyphCache>& Entry = GlyphCacheMap.FindOrAdd(Key);
	if (!Entry)
	{
		Entry = MakeShared<FFreeTypeGlyphCache>(InFace, InLoadFlags, InFontRenderSize);
	}
	return Entry.ToSharedRef();
}

TSharedRef<FFreeTypeAdvanceCache> FFreeTypeCacheDirectory::GetAdvanceCache(FT_Face InFace, const int32 InLoadFlags, const uint32 InFontRenderSize)
{
	if (!InFace || ((InLoadFlags & FT_LOAD_NO_SCALE) != 0 && !FreeTypeUtils::IsFaceEligibleForSdf(InFace)))
	{
		checkNoEntry(); // the slate clients can't reach these caches. Reaching this point is an internal error.
		return InvalidAdvanceCache.ToSharedRef();
	}
	const FFontKey Key(InFace, InLoadFlags, InFontRenderSize);
	TSharedPtr<FFreeTypeAdvanceCache>& Entry = AdvanceCacheMap.FindOrAdd(Key);
	if (!Entry)
	{
		Entry = MakeShared<FFreeTypeAdvanceCache>(InFace, InLoadFlags, InFontRenderSize);
	}
	return Entry.ToSharedRef();
}

TSharedPtr<FFreeTypeKerningCache> FFreeTypeCacheDirectory::GetKerningCache(FT_Face InFace, const int32 InKerningFlags, const uint32 InFontRenderSize)
{
	// Check if this font has kerning as not all fonts do.
	// We also can't perform kerning between two separate font faces
	if (InFace && FT_HAS_KERNING(InFace))
	{
		if ((InKerningFlags != FT_KERNING_UNSCALED || FreeTypeUtils::IsFaceEligibleForSdf(InFace)))
		{
			const FFontKey Key(InFace, InKerningFlags, InFontRenderSize);
			TSharedPtr<FFreeTypeKerningCache>& Entry = KerningCacheMap.FindOrAdd(Key);
			if (!Entry)
			{
				Entry = MakeShared<FFreeTypeKerningCache>(InFace, InKerningFlags, InFontRenderSize);
			}
			return Entry;
		}
		checkNoEntry();

	}
	return nullptr;
}

#endif // WITH_FREETYPE


FFreeTypeCacheDirectory::FFreeTypeCacheDirectory()
#if WITH_FREETYPE
	: InvalidAdvanceCache(MakeShared<FFreeTypeAdvanceCache>())
#endif
{

}

void FFreeTypeCacheDirectory::FlushCache()
{
#if WITH_FREETYPE
	GlyphCacheMap.Empty();
	AdvanceCacheMap.Empty();
	KerningCacheMap.Empty();
#endif // WITH_FREETYPE
}
