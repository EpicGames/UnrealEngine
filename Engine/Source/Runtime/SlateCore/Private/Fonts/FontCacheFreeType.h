// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/FontCache.h"
#include "Templates/IsIntegral.h"
#include "Templates/IsFloatingPoint.h"
#include "Misc/Optional.h"

#ifndef WITH_FREETYPE
	#define WITH_FREETYPE	0
#endif // WITH_FREETYPE


#if PLATFORM_COMPILER_HAS_GENERIC_KEYWORD
	#define generic __identifier(generic)
#endif	//PLATFORM_COMPILER_HAS_GENERIC_KEYWORD


#if WITH_FREETYPE
	THIRD_PARTY_INCLUDES_START
	#include "ft2build.h"

	// FreeType style include
	#include FT_FREETYPE_H
	#include FT_DRIVER_H
	#include FT_GLYPH_H
	#include FT_MODULE_H
	#include FT_BITMAP_H
	#include FT_ADVANCES_H
	#include FT_STROKER_H
	#include FT_SIZES_H
	THIRD_PARTY_INCLUDES_END
#endif // WITH_FREETYPE


#if PLATFORM_COMPILER_HAS_GENERIC_KEYWORD
	#undef generic
#endif	//PLATFORM_COMPILER_HAS_GENERIC_KEYWORD

namespace FreeTypeUtils
{

#if WITH_FREETYPE

/**
 * Get the eligibility of this face to generate SDF fonts
 */
bool IsFaceEligibleForSdf(FT_Face InFace);

/**
 * Get the eligibility of this glyph to generate SDF fonts
 */
bool IsGlyphEligibleForSdf(FT_GlyphSlot InGlyph);

/** Rounds towards -INF the given value in 26.6 space to the previous multiple of 64.
*/
FT_F26Dot6 Floor26Dot6(const FT_F26Dot6 InValue);

/** Rounds towards +INF the given value in 26.6 space to the next multiple of 64
*/
FT_F26Dot6 Ceil26Dot6(const FT_F26Dot6 InValue);

/** Round up to the nearest integer if the fractional part of the 26.6 value
*	is greater or equal to the half interval of 64th otherwise round down.
*/
FT_F26Dot6 Round26Dot6(const FT_F26Dot6 InValue);

/**
 * The EmScale maps design space distances relative to the em square (with resolution of InEmSize units),
 * to absolute 1/64th pixels distances in the device pixel plane (with a resolution of 96 dpi) with 
 * a character size of InPpem.
 */
FT_Fixed DetermineEmScale(const uint16 InEmSize, const FT_F26Dot6 InPpem);

/**
* Determine the rounded Ppem and then from it determine the EmScale, in one call.
*/
FT_Fixed DeterminePpemAndEmScale(const uint16 InEmSize, const float InFontSize, const float InFontScale);


/**
 * Compute the actual size that will be used by Freetype to render or do any process on glyphs.
 */
uint32 ComputeFontPixelSize(float InFontSize, float InFontScale);

/**
 * Apply the given point size and scale to the face.
 */
void ApplySizeAndScale(FT_Face InFace, const float InFontSize, const float InFontScale);

/**
 * Apply the given size in pixel to the face.
 */
void ApplySizeAndScale(FT_Face InFace, const uint32 RequiredFontPixelSize);

/**
 * Load the given glyph into the active slot of the given face.
 */
FT_Error LoadGlyph(FT_Face InFace, const uint32 InGlyphIndex, const int32 InLoadFlags, const float InFontSize, const float InFontScale);
FT_Error LoadGlyph(FT_Face InFace, const uint32 InGlyphIndex, const int32 InLoadFlags, const uint32 RequiredFontPixelSize);

/**
 * Get the height of the given face under the given layout method.
 * @note ApplySizeAndScale must have been called prior to this function to prepare the face.
 */
FT_Pos GetHeight(FT_Face InFace, const EFontLayoutMethod InLayoutMethod);

/**
 * Get the height of the given face under the given layout method, scaled by the face scale.
 * @note ApplySizeAndScale must have been called prior to this function to prepare the face.
 */
FT_Pos GetScaledHeight(FT_Face InFace, const EFontLayoutMethod InLayoutMethod);

/**
 * Get the ascender of the given face under the given layout method (ascenders are always scaled by the face scale).
 * @note ApplySizeAndScale must have been called prior to this function to prepare the face.
 */
FT_Pos GetAscender(FT_Face InFace, const EFontLayoutMethod InLayoutMethod);

/**
 * Get the descender of the given face under the given layout method (descenders are always scaled by the face scale).
 * @note ApplySizeAndScale must have been called prior to this function to prepare the face.
 */
FT_Pos GetDescender(FT_Face InFace, const EFontLayoutMethod InLayoutMethod);

/**
 * Get any additional scale that should be applied when producing the atlas glyphs from this face.
 * @note ApplySizeAndScale must have been called prior to this function to prepare the face.
 */
float GetBitmapAtlasScale(FT_Face InFace);

/**
 * Get any additional scale that should be applied when rendering glyphs from this face.
 * @note ApplySizeAndScale must have been called prior to this function to prepare the face.
 */
float GetBitmapRenderScale(FT_Face InFace);

#endif // WITH_FREETYPE

/** Convert the given value from 26.6 space into rounded pixel space */
template <typename TRetType, typename TParamType>
FORCEINLINE typename TEnableIf<TIsIntegral<TParamType>::Value, TRetType>::Type Convert26Dot6ToRoundedPixel(TParamType InValue)
{
	return static_cast<TRetType>((InValue + (1<<5)) >> 6);
}

/** Convert the given value from 26.6 space into rounded pixel space */
template <typename TRetType, typename TParamType>
FORCEINLINE typename TEnableIf<TIsFloatingPoint<TParamType>::Value, TRetType>::Type Convert26Dot6ToRoundedPixel(TParamType InValue)
{
	return static_cast<TRetType>(FMath::RoundToInt(InValue / 64.0f));
}

/** Convert the given value from pixel space into 26.6 space */
template <typename TRetType, typename TParamType>
FORCEINLINE typename TEnableIf<TIsIntegral<TParamType>::Value, TRetType>::Type ConvertPixelTo26Dot6(TParamType InValue)
{
	return static_cast<TRetType>(InValue << 6);
}

/** Convert the given value from pixel space into 26.6 space */
template <typename TRetType, typename TParamType>
FORCEINLINE typename TEnableIf<TIsFloatingPoint<TParamType>::Value, TRetType>::Type ConvertPixelTo26Dot6(TParamType InValue)
{
	return static_cast<TRetType>(InValue * 64);
}

/** Convert the given value from pixel space into 16.16 space */
template <typename TRetType, typename TParamType>
FORCEINLINE typename TEnableIf<TIsIntegral<TParamType>::Value, TRetType>::Type ConvertPixelTo16Dot16(TParamType InValue)
{
	return static_cast<TRetType>(InValue << 16);
}

/** Convert the given value from pixel space into 16.16 space */
template <typename TRetType, typename TParamType>
FORCEINLINE typename TEnableIf<TIsFloatingPoint<TParamType>::Value, TRetType>::Type ConvertPixelTo16Dot16(TParamType InValue)
{
	return static_cast<TRetType>(InValue * 65536);
}

}


/** 
 * Wrapper around a FreeType library instance.
 * This instance will be created using our memory allocator. 
 */
class FFreeTypeLibrary
{
public:
	FFreeTypeLibrary();
	~FFreeTypeLibrary();

#if WITH_FREETYPE
	FORCEINLINE FT_Library GetLibrary() const
	{
		return FTLibrary;
	}
#endif // WITH_FREETYPE

private:
	// Non-copyable
	FFreeTypeLibrary(const FFreeTypeLibrary&);
	FFreeTypeLibrary& operator=(const FFreeTypeLibrary&);

#if WITH_FREETYPE
	FT_Library FTLibrary;
	FT_Memory CustomMemory;
#endif // WITH_FREETYPE
};


/**
 * Wrapper around a FreeType face instance.
 * It will either steal the given buffer, or stream the given file from disk.
 */
class FFreeTypeFace
{
public:
	FFreeTypeFace(const FFreeTypeLibrary* InFTLibrary, FFontFaceDataConstRef InMemory, const int32 InFaceIndex, const EFontLayoutMethod InLayoutMethod);
	FFreeTypeFace(const FFreeTypeLibrary* InFTLibrary, const FString& InFilename, const int32 InFaceIndex, const EFontLayoutMethod InLayoutMethod);
	FFreeTypeFace(const EFontLayoutMethod InLayoutMethod);
	~FFreeTypeFace();

	FORCEINLINE bool IsFaceValid() const
	{
#if WITH_FREETYPE
		return FTFace != nullptr;
#else
		return false;
#endif // WITH_FREETYPE
	}

	FORCEINLINE bool IsFaceLoading() const
	{
#if WITH_FREETYPE
		return bPendingAsyncLoad;
#else
		return false;
#endif // WITH_FREETYPE
	}

	FORCEINLINE bool SupportsSdf() const
	{
#if WITH_FREETYPE
		return FreeTypeUtils::IsFaceEligibleForSdf(FTFace);
#else
		return false;
#endif // WITH_FREETYPE
	}

#if WITH_FREETYPE
	FORCEINLINE FT_Face GetFace() const
	{
#if WITH_ATLAS_DEBUGGING
		check(OwnerThread == GetCurrentSlateTextureAtlasThreadId());
#endif
		return FTFace;
	}

	FORCEINLINE FT_Pos GetHeight() const
	{
		return FreeTypeUtils::GetHeight(FTFace, LayoutMethod);
	}

	FORCEINLINE FT_Pos GetScaledHeight(bool bAllowOverride) const
	{
		if (bAllowOverride && (IsAscentOverridden || IsDescentOverridden))
			return GetAscender(true) - GetDescender(true);
		return FreeTypeUtils::GetScaledHeight(FTFace, LayoutMethod);
	}

	FORCEINLINE FT_Pos GetAscender(bool bAllowOverride) const
	{
		if (bAllowOverride && IsAscentOverridden)
		{
			FT_F26Dot6 ScaledAscender = FT_MulFix(AscentOverrideValue, FTFace->size->metrics.y_scale);
			return (ScaledAscender + 0b111111) & ~0b111111; //(26.6 fixed point ceil). Using ceiling of scaled ascend, as recommended by Freetype, to avoid grid fitting/hinting issues.
		}
		return FreeTypeUtils::GetAscender(FTFace, LayoutMethod);
	}

	FORCEINLINE FT_Pos GetDescender(bool bAllowOverride) const
	{
		if (bAllowOverride && IsDescentOverridden)
		{
			FT_F26Dot6 ScaledDescender =  FT_MulFix(DescentOverrideValue, FTFace->size->metrics.y_scale);
			return ScaledDescender & ~0b111111; //(26.6 fixed point floor). Using floor of scaled descend, as recommended by Freetype, to avoid grid fitting/hinting issues.
		}
		return FreeTypeUtils::GetDescender(FTFace, LayoutMethod);
	}

	FORCEINLINE float GetBitmapAtlasScale() const
	{
		return FreeTypeUtils::GetBitmapAtlasScale(FTFace);
	}

	FORCEINLINE float GetBitmapRenderScale() const
	{
		return FreeTypeUtils::GetBitmapRenderScale(FTFace);
	}
#endif // WITH_FREETYPE

	FORCEINLINE const TSet<FName>& GetAttributes() const
	{
		return Attributes;
	}

	FORCEINLINE EFontLayoutMethod GetLayoutMethod() const
	{
		return LayoutMethod;
	}

	/**
	 * Gets the memory size of the loaded font or 0 if the font is streamed
	 */
	SIZE_T GetAllocatedMemorySize() const;

	/**
	 * Returns the preprocessed font geometry object associated with the font data if available.
	 * It can only be available if this was initialized with a FontFaceData object.
	 */
	const UE::Slate::FPreprocessedFontGeometry* GetPreprocessedFontGeometry() const;

	void OverrideAscent(bool InOverride, int32 Value = 0)
	{
#if WITH_FREETYPE
		IsAscentOverridden = InOverride;
		AscentOverrideValue = FreeTypeUtils::ConvertPixelTo26Dot6<FT_F26Dot6>(Value);
#endif //WITH_FREETYPE
	}

	void OverrideDescent(bool InOverride, int32 Value = 0)
	{
#if WITH_FREETYPE
		IsDescentOverridden = InOverride;
		DescentOverrideValue = FreeTypeUtils::ConvertPixelTo26Dot6<FT_F26Dot6>(Value);
#endif //WITH_FREETYPE
	}

	void FailAsyncLoad();
	void CompleteAsyncLoad(const FFreeTypeLibrary* InFTLibrary, FFontFaceDataConstRef InMemory, const int32 InFaceIndex);

	/**
	 * Get the available sub-face data from the given font.
	 * Typically there will only be one face unless this is a TTC/OTC font.
	 * The index of the returned entry can be passed as InFaceIndex to the FFreeTypeFace constructor.
	 */
	static TArray<FString> GetAvailableSubFaces(const FFreeTypeLibrary* InFTLibrary, FFontFaceDataConstRef InMemory);
	static TArray<FString> GetAvailableSubFaces(const FFreeTypeLibrary* InFTLibrary, const FString& InFilename);

private:

#if WITH_FREETYPE
	void ParseAttributes();
#endif // WITH_FREETYPE

	// Non-copyable
	FFreeTypeFace(const FFreeTypeFace&);
	FFreeTypeFace& operator=(const FFreeTypeFace&);

#if WITH_FREETYPE
	FT_Face FTFace;
	FFontFaceDataConstPtr Memory;

	bool bPendingAsyncLoad = false;

	/** Custom FreeType stream handler for reading font data via the Unreal File System */
	struct FFTStreamHandler
	{
		FFTStreamHandler();
		FFTStreamHandler(const FString& InFilename);
		~FFTStreamHandler();
		static void CloseFile(FT_Stream InStream);
		static unsigned long ReadData(FT_Stream InStream, unsigned long InOffset, unsigned char* InBuffer, unsigned long InCount);

		IFileHandle* FileHandle;
		int64 FontSizeBytes;
	};

	FFTStreamHandler FTStreamHandler;
	FT_StreamRec FTStream;
	FT_Open_Args FTFaceOpenArgs;

	bool IsAscentOverridden = false;
	bool IsDescentOverridden = false;
	FT_F26Dot6 AscentOverrideValue = 0;
	FT_F26Dot6 DescentOverrideValue = 0;

#if WITH_ATLAS_DEBUGGING
	ESlateTextureAtlasThreadId OwnerThread;
#endif
#endif // WITH_FREETYPE

	TSet<FName> Attributes;

	EFontLayoutMethod LayoutMethod;
};


/**
 * Provides low-level glyph caching to avoid repeated calls to FT_Load_Glyph (see FCachedGlyphData).
 * Most of the data cached here is required for HarfBuzz, however a couple of things (such as the baseline and max character height) 
 * are used directly by the Slate font cache. Feel free to add more cached data if required, but please keep it in native FreeType 
 * format where possible - the goal here is to avoid calls to FT_Load_Glyph, *not* to perform data transformation to what Slate needs.
 */
class FFreeTypeGlyphCache
{
public:
#if WITH_FREETYPE
	FFreeTypeGlyphCache(FT_Face InFace, const int32 InLoadFlags, const uint32 InFontRenderSize);

	struct FCachedGlyphData
	{
		FT_Short Height;
		FT_Glyph_Metrics GlyphMetrics;
		FT_Size_Metrics SizeMetrics; 
		TArray<FT_Vector> OutlinePoints;
	};

	bool FindOrCache(const uint32 InGlyphIndex, FCachedGlyphData& OutCachedGlyphData);
#endif // WITH_FREETYPE

private:
#if WITH_FREETYPE
	FT_Face Face;
	const int32 LoadFlags;
	const uint32 FontRenderSize;
	TMap<uint32, FCachedGlyphData> GlyphDataMap;
#endif // WITH_FREETYPE
};


/**
 * Provides low-level advance caching to avoid repeated calls to FT_Get_Advance.
 */
class FFreeTypeAdvanceCache
{
public:
#if WITH_FREETYPE
	FFreeTypeAdvanceCache();
	FFreeTypeAdvanceCache(FT_Face InFace, const int32 InLoadFlags, const uint32 InFontRenderSize);

	bool FindOrCache(const uint32 InGlyphIndex, FT_Fixed& OutCachedAdvance);
#endif // WITH_FREETYPE

	void FlushCache();

private:
#if WITH_FREETYPE
	FT_Face Face;
	const int32 LoadFlags;
	const uint32 FontRenderSize;
	TMap<uint32, FT_Fixed> AdvanceMap;
#endif // WITH_FREETYPE
};


/**
 * Provides low-level kerning-pair caching for a set of font parameters to avoid repeated calls to FT_Get_Kerning.
 */
class FFreeTypeKerningCache
{
public:
#if WITH_FREETYPE
	FFreeTypeKerningCache(FT_Face InFace, const int32 InKerningFlags, const uint32 InFontRenderSize);

	/**
	 * Retrieve the kerning vector for a given pair of glyphs.
	 @param InFirstGlyphIndex		The first (left) glyph index
	 @param InSecondGlyphIndex		The second (right) glyph index
	 @param OutKerning				The vector to fill out with kerning amounts
	 @return True if the kerning value was found
	 */
	bool FindOrCache(const uint32 InFirstGlyphIndex, const uint32 InSecondGlyphIndex, FT_Vector& OutKerning);
#endif // WITH_FREETYPE

private:
#if WITH_FREETYPE
	struct FKerningPair
	{
		FKerningPair(const uint32 InFirstGlyphIndex, const uint32 InSecondGlyphIndex)
			: FirstGlyphIndex(InFirstGlyphIndex)
			, SecondGlyphIndex(InSecondGlyphIndex)
		{
		}

		FORCEINLINE bool operator==(const FKerningPair& Other) const
		{
			return FirstGlyphIndex == Other.FirstGlyphIndex 
				&& SecondGlyphIndex == Other.SecondGlyphIndex;
		}

		FORCEINLINE bool operator!=(const FKerningPair& Other) const
		{
			return !(*this == Other);
		}

		friend inline uint32 GetTypeHash(const FKerningPair& Key)
		{
			uint32 KeyHash = 0;
			KeyHash = HashCombine(KeyHash, Key.FirstGlyphIndex);
			KeyHash = HashCombine(KeyHash, Key.SecondGlyphIndex);
			return KeyHash;
		}

		uint32 FirstGlyphIndex;
		uint32 SecondGlyphIndex;
	};

	FT_Face Face;
	const int32 KerningFlags;
	const uint32 FontRenderSize;
	TMap<FKerningPair, FT_Vector> KerningMap;
#endif // WITH_FREETYPE
};


/**
 * Class that manages directory of caches for advances, kerning, and other parameters for fonts.
 */
class FFreeTypeCacheDirectory
{
public:
	FFreeTypeCacheDirectory();
#if WITH_FREETYPE
	/**
	 * Retrieve the glyph cache for a given set of font parameters.
	 * @return A reference to the font glyph cache.
	 */
	TSharedRef<FFreeTypeGlyphCache> GetGlyphCache(FT_Face InFace, const int32 InLoadFlags, const uint32 InFontRenderSize);

	/**
	 * Retrieve the advance cache for a given set of font parameters.
	 * @return A reference to the font advance cache.
	 */
	TSharedRef<FFreeTypeAdvanceCache> GetAdvanceCache(FT_Face InFace, const int32 InLoadFlags, const uint32 InFontRenderSize);

	/**
	 * Retrieve the kerning cache for a given set of font parameters.
	 * @return A pointer to the font kerning cache, invalid if the font does not perform kerning.
	 */
	TSharedPtr<FFreeTypeKerningCache> GetKerningCache(FT_Face InFace, const int32 InKerningFlags, const uint32 InFontRenderSize);
#endif // WITH_FREETYPE

	void FlushCache();

private:
#if WITH_FREETYPE
	/* Shared font key to look up internal caches; flag meaning changes but the internal type is the same. */
	class FFontKey
	{
	public:
		FFontKey(FT_Face InFace, const int32 InFlags, const uint32 InFontRenderSize)
			: Face(InFace)
			, Flags(InFlags)
			, FontRenderSize(InFontRenderSize)
			, KeyHash(0)
		{
			KeyHash = GetTypeHash(Face);
			KeyHash = HashCombine(KeyHash, GetTypeHash(Flags));
			KeyHash = HashCombine(KeyHash, GetTypeHash(FontRenderSize));
		}

		FORCEINLINE bool operator==(const FFontKey& Other) const
		{
			return Face == Other.Face
				&& Flags == Other.Flags
				&& FontRenderSize == Other.FontRenderSize;
		}

		FORCEINLINE bool operator!=(const FFontKey& Other) const
		{
			return !(*this == Other);
		}

		friend inline uint32 GetTypeHash(const FFontKey& Key)
		{
			return Key.KeyHash;
		}

	private:
		FT_Face Face;
		const int32 Flags;
		const uint32 FontRenderSize;
		uint32 KeyHash;
	};

	TMap<FFontKey, TSharedPtr<FFreeTypeGlyphCache>> GlyphCacheMap;
	TMap<FFontKey, TSharedPtr<FFreeTypeAdvanceCache>> AdvanceCacheMap;
	TMap<FFontKey, TSharedPtr<FFreeTypeKerningCache>> KerningCacheMap;
	TSharedPtr<FFreeTypeAdvanceCache> InvalidAdvanceCache;
#endif // WITH_FREETYPE
};
