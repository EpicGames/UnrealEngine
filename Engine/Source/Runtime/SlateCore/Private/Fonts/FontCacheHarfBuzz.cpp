// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/FontCacheHarfBuzz.h"
#include "Fonts/FontCache.h"
#include "Fonts/FontCacheFreeType.h"
#include "Fonts/SlateFontRenderer.h"
#include "Trace/SlateMemoryTags.h"
#include "Fonts/FontUtils.h"

#if WITH_FREETYPE && WITH_HARFBUZZ
DECLARE_CYCLE_STAT(TEXT("Create font"), STAT_HarfBuzzFontCache_ApplyFont, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Get_nominal_glyphs"), STAT_HarfBuzzFontFunctions_get_nominal_glyphs, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Get_glyph_h_advances"), STAT_HarfBuzzFontFunctions_get_glyph_h_advances, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Get_glyph_v_advances"), STAT_HarfBuzzFontFunctions_get_glyph_v_advances, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Get_glyph_v_origin"), STAT_HarfBuzzFontFunctions_get_glyph_v_origin, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Get_glyph_h_kerning"), STAT_HarfBuzzFontFunctions_get_glyph_h_kerning, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Get_glyph_extents"), STAT_HarfBuzzFontFunctions_get_glyph_extents, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Get_glyph_contour_point"), STAT_HarfBuzzFontFunctions_get_glyph_contour_point, STATGROUP_Slate);
#endif // #if WITH_FREETYPE && WITH_HARFBUZZ

#if WITH_HARFBUZZ

extern "C"
{

void* HarfBuzzMalloc(size_t InSizeBytes)
{
	LLM_SCOPE_BYTAG(UI_Text);
	return FMemory::Malloc(InSizeBytes);
}

void* HarfBuzzCalloc(size_t InNumItems, size_t InItemSizeBytes)
{
	LLM_SCOPE_BYTAG(UI_Text);
	const size_t AllocSizeBytes = InNumItems * InItemSizeBytes;
	if (AllocSizeBytes > 0)
	{
		void* Ptr = FMemory::Malloc(AllocSizeBytes);
		FMemory::Memzero(Ptr, AllocSizeBytes);
		return Ptr;
	}
	return nullptr;
}

void* HarfBuzzRealloc(void* InPtr, size_t InSizeBytes)
{
	LLM_SCOPE_BYTAG(UI_Text);
	return FMemory::Realloc(InPtr, InSizeBytes);
}

void HarfBuzzFree(void* InPtr)
{
	FMemory::Free(InPtr);
}

} // extern "C"

#endif // #if WITH_HARFBUZZ

namespace HarfBuzzUtils
{

#if WITH_HARFBUZZ

namespace Internal
{

template <bool IsUnicode, size_t TCHARSize>
void AppendStringToBuffer(const FStringView InString, const int32 InStartIndex, const int32 InLength, hb_buffer_t* InHarfBuzzTextBuffer)
{
	// todo: SHAPING - This is losing the context information that may be required to shape a sub-section of text.
	//				   In practice this may not be an issue as our platforms should all use the other functions, but to fix it we'd need UTF-8 iteration functions to find the correct points the buffer
	FStringView SubString = InString.Mid(InStartIndex, InLength);
	FTCHARToUTF8 SubStringUtf8(SubString.GetData(), SubString.Len());
	hb_buffer_add_utf8(InHarfBuzzTextBuffer, (const char*)SubStringUtf8.Get(), SubStringUtf8.Length(), 0, SubStringUtf8.Length());
}

template <>
void AppendStringToBuffer<true, 2>(const FStringView InString, const int32 InStartIndex, const int32 InLength, hb_buffer_t* InHarfBuzzTextBuffer)
{
	// A unicode encoding with a TCHAR size of 2 bytes is assumed to be UTF-16
	hb_buffer_add_utf16(InHarfBuzzTextBuffer, reinterpret_cast<const uint16_t*>(InString.GetData()), InString.Len(), InStartIndex, InLength);
}

template <>
void AppendStringToBuffer<true, 4>(const FStringView InString, const int32 InStartIndex, const int32 InLength, hb_buffer_t* InHarfBuzzTextBuffer)
{
	// A unicode encoding with a TCHAR size of 4 bytes is assumed to be UTF-32
	hb_buffer_add_utf32(InHarfBuzzTextBuffer, reinterpret_cast<const uint32_t*>(InString.GetData()), InString.Len(), InStartIndex, InLength);
}

} // namespace Internal

void AppendStringToBuffer(const FStringView InString, hb_buffer_t* InHarfBuzzTextBuffer)
{
	return Internal::AppendStringToBuffer<FPlatformString::IsUnicodeEncoded, sizeof(TCHAR)>(InString, 0, InString.Len(), InHarfBuzzTextBuffer);
}

void AppendStringToBuffer(const FStringView InString, const int32 InStartIndex, const int32 InLength, hb_buffer_t* InHarfBuzzTextBuffer)
{
	return Internal::AppendStringToBuffer<FPlatformString::IsUnicodeEncoded, sizeof(TCHAR)>(InString, InStartIndex, InLength, InHarfBuzzTextBuffer);
}

#endif // #if WITH_HARFBUZZ

} // namespace HarfBuzzUtils


#if WITH_FREETYPE && WITH_HARFBUZZ

namespace HarfBuzzFontFunctions
{

hb_user_data_key_t UserDataKey;

struct FUserData
{
	FUserData(uint32 InRenderSize, FFreeTypeCacheDirectory* InFTCacheDirectory, FT_Face InFreeTypeFace, int32 InFreeTypeFlags)
		: RenderSize(InRenderSize)
		, FTCacheDirectory(InFTCacheDirectory)
		, HarfBuzzFontExtents()
		, FreeTypeFace(InFreeTypeFace)
		, FreeTypeFlags(InFreeTypeFlags)
	{
		FMemory::Memzero(HarfBuzzFontExtents);
	}

	uint32 RenderSize;
	FFreeTypeCacheDirectory* FTCacheDirectory;
	hb_font_extents_t HarfBuzzFontExtents;
	FT_Face FreeTypeFace;
	int32 FreeTypeFlags;
};

FUserData* CreateUserData(uint32 InRenderSize, FFreeTypeCacheDirectory* InFTCacheDirectory, FT_Face InFreeTypeFace, int32 InFreeTypeFlags)
{
	return new FUserData(InRenderSize, InFTCacheDirectory, InFreeTypeFace, InFreeTypeFlags);
}

void DestroyUserData(void* UserData)
{
	FUserData* UserDataPtr = static_cast<FUserData*>(UserData);
	delete UserDataPtr;
}

namespace Internal
{
hb_bool_t get_font_h_extents(hb_font_t* InFont, void* InFontData, hb_font_extents_t* OutMetrics, void* InUserData)
{
	const FUserData* UserDataPtr = static_cast<FUserData*>(hb_font_get_user_data(InFont, &UserDataKey));
	*OutMetrics = UserDataPtr->HarfBuzzFontExtents;
	return true;
}

unsigned int get_nominal_glyphs(hb_font_t* InFont, void* InFontData, unsigned int InCount, const hb_codepoint_t *InUnicodeCharBuffer, unsigned int InUnicodeCharBufferStride, hb_codepoint_t *OutGlyphIndexBuffer, unsigned int InGlyphIndexBufferStride, void* InUserData)
{
//Too verbose for now, but will help in subsequent optimization
//	SCOPE_CYCLE_COUNTER(STAT_HarfBuzzFontFunctions_get_nominal_glyphs);

	const FUserData* UserDataPtr = static_cast<FUserData*>(hb_font_get_user_data(InFont, &UserDataKey));
	FT_Face FreeTypeFace = UserDataPtr->FreeTypeFace;

	const uint8* UnicodeCharRawBuffer = (const uint8*)InUnicodeCharBuffer;
	uint8* GlyphIndexRawBuffer = (uint8*)OutGlyphIndexBuffer;

	for (unsigned int ItemIndex = 0; ItemIndex < InCount; ++ItemIndex)
	{
		const hb_codepoint_t* UnicodeCharPtr = (const hb_codepoint_t*)UnicodeCharRawBuffer;
		hb_codepoint_t* OutGlyphIndexPtr = (hb_codepoint_t*)GlyphIndexRawBuffer;

		*OutGlyphIndexPtr = FT_Get_Char_Index(FreeTypeFace, *UnicodeCharPtr);

		// If the given font can't render that character (as the fallback font may be missing), try again with the fallback character
		if (*UnicodeCharPtr != 0 && *OutGlyphIndexPtr == 0)
		{
			*OutGlyphIndexPtr = FT_Get_Char_Index(FreeTypeFace, SlateFontRendererUtils::InvalidSubChar);
		}

		// If this resolution failed, return the number if items we managed to process
		if (*UnicodeCharPtr != 0 && *OutGlyphIndexPtr == 0)
		{
			return ItemIndex;
		}

		// Advance the buffers
		UnicodeCharRawBuffer += InUnicodeCharBufferStride;
		GlyphIndexRawBuffer += InGlyphIndexBufferStride;
	}

	// Processed everything - return the count
	return InCount;
}

hb_bool_t get_nominal_glyph(hb_font_t* InFont, void* InFontData, hb_codepoint_t InUnicodeChar, hb_codepoint_t* OutGlyphIndex, void* InUserData)
{
	return get_nominal_glyphs(InFont, InFontData, 1, &InUnicodeChar, sizeof(hb_codepoint_t), OutGlyphIndex, sizeof(hb_codepoint_t), InUserData) == 1;
}

void get_glyph_h_advances(hb_font_t* InFont, void* InFontData, unsigned int InCount, const hb_codepoint_t* InGlyphIndexBuffer, unsigned int InGlyphIndexBufferStride, hb_position_t* OutAdvanceBuffer, unsigned int InAdvanceBufferStride, void* InUserData)
{
//Too verbose for now, but will help in subsequent optimization
//	SCOPE_CYCLE_COUNTER(STAT_HarfBuzzFontFunctions_get_glyph_h_advances);

	const FUserData* UserDataPtr = static_cast<FUserData*>(hb_font_get_user_data(InFont, &UserDataKey));
	FT_Face FreeTypeFace = UserDataPtr->FreeTypeFace;
	const int32 FreeTypeFlags = UserDataPtr->FreeTypeFlags;

	int ScaleMultiplier = 1;
	{
		int FontXScale = 0;
		int FontYScale = 0;
		hb_font_get_scale(InFont, &FontXScale, &FontYScale);

		if (FontXScale < 0)
		{
			ScaleMultiplier = -1;
		}
	}

	const uint8* GlyphIndexRawBuffer = (const uint8*)InGlyphIndexBuffer;
	uint8* AdvanceRawBuffer = (uint8*)OutAdvanceBuffer;
	TSharedRef<FFreeTypeAdvanceCache> AdvanceCache = UserDataPtr->FTCacheDirectory->GetAdvanceCache(FreeTypeFace, FreeTypeFlags, UserDataPtr->RenderSize);

	for (unsigned int ItemIndex = 0; ItemIndex < InCount; ++ItemIndex)
	{
		const hb_codepoint_t* GlyphIndexPtr = (const hb_codepoint_t*)GlyphIndexRawBuffer;
		hb_position_t* OutAdvancePtr = (hb_position_t*)AdvanceRawBuffer;

		FT_Fixed CachedAdvanceData = 0;
		if (AdvanceCache->FindOrCache(*GlyphIndexPtr, CachedAdvanceData))
		{
			*OutAdvancePtr = ((CachedAdvanceData * ScaleMultiplier) + (1<<9)) >> 10;
		}
		else
		{
			*OutAdvancePtr = 0;
		}

		// Advance the buffers
		GlyphIndexRawBuffer += InGlyphIndexBufferStride;
		AdvanceRawBuffer += InAdvanceBufferStride;
	}
}

hb_position_t get_glyph_h_advance(hb_font_t* InFont, void* InFontData, hb_codepoint_t InGlyphIndex, void* InUserData)
{
	hb_position_t Advance = 0;
	get_glyph_h_advances(InFont, InFontData, 1, &InGlyphIndex, sizeof(hb_codepoint_t), &Advance, sizeof(hb_position_t), InUserData);
	return Advance;
}

void get_glyph_v_advances(hb_font_t* InFont, void* InFontData, unsigned int InCount, const hb_codepoint_t* InGlyphIndexBuffer, unsigned int InGlyphIndexBufferStride, hb_position_t* OutAdvanceBuffer, unsigned int InAdvanceBufferStride, void* InUserData)
{
//Too verbose for now, but will help in subsequent optimization
//	SCOPE_CYCLE_COUNTER(STAT_HarfBuzzFontFunctions_get_glyph_v_advances);

	const FUserData* UserDataPtr = static_cast<FUserData*>(hb_font_get_user_data(InFont, &UserDataKey));
	FT_Face FreeTypeFace = UserDataPtr->FreeTypeFace;
	const int32 FreeTypeFlags = UserDataPtr->FreeTypeFlags;

	int ScaleMultiplier = 1;
	{
		int FontXScale = 0;
		int FontYScale = 0;
		hb_font_get_scale(InFont, &FontXScale, &FontYScale);

		if (FontYScale < 0)
		{
			ScaleMultiplier = -1;
		}
	}

	const uint8* GlyphIndexRawBuffer = (const uint8*)InGlyphIndexBuffer;
	uint8* AdvanceRawBuffer = (uint8*)OutAdvanceBuffer;
	TSharedRef<FFreeTypeAdvanceCache> AdvanceCache = UserDataPtr->FTCacheDirectory->GetAdvanceCache(FreeTypeFace, FreeTypeFlags | FT_LOAD_VERTICAL_LAYOUT, UserDataPtr->RenderSize);

	for (unsigned int ItemIndex = 0; ItemIndex < InCount; ++ItemIndex)
	{
		const hb_codepoint_t* GlyphIndexPtr = (const hb_codepoint_t*)GlyphIndexRawBuffer;
		hb_position_t* OutAdvancePtr = (hb_position_t*)AdvanceRawBuffer;

		FT_Fixed CachedAdvanceData = 0;
		if (AdvanceCache->FindOrCache(*GlyphIndexPtr, CachedAdvanceData))
		{
			// Note: FreeType's vertical metrics grows downward while other FreeType coordinates have a Y growing upward. Hence the extra negation.
			*OutAdvancePtr = ((-CachedAdvanceData * ScaleMultiplier) + (1<<9)) >> 10;
		}
		else
		{
			*OutAdvancePtr = 0;
		}

		// Advance the buffers
		GlyphIndexRawBuffer += InGlyphIndexBufferStride;
		AdvanceRawBuffer += InAdvanceBufferStride;
	}
}

hb_position_t get_glyph_v_advance(hb_font_t* InFont, void* InFontData, hb_codepoint_t InGlyphIndex, void* InUserData)
{
	hb_position_t Advance = 0;
	get_glyph_v_advances(InFont, InFontData, 1, &InGlyphIndex, sizeof(hb_codepoint_t), &Advance, sizeof(hb_position_t), InUserData);
	return Advance;
}

hb_bool_t get_glyph_v_origin(hb_font_t* InFont, void* InFontData, hb_codepoint_t InGlyphIndex, hb_position_t* OutX, hb_position_t* OutY, void* InUserData)
{
//Too verbose for now, but will help in subsequent optimization
//	SCOPE_CYCLE_COUNTER(STAT_HarfBuzzFontFunctions_get_glyph_v_origin);

	const FUserData* UserDataPtr = static_cast<FUserData*>(hb_font_get_user_data(InFont, &UserDataKey));
	FT_Face FreeTypeFace = UserDataPtr->FreeTypeFace;
	const int32 FreeTypeFlags = UserDataPtr->FreeTypeFlags;

	TSharedRef<FFreeTypeGlyphCache> GlyphCache = UserDataPtr->FTCacheDirectory->GetGlyphCache(FreeTypeFace, FreeTypeFlags, UserDataPtr->RenderSize);
	FFreeTypeGlyphCache::FCachedGlyphData CachedGlyphData;
	if (GlyphCache->FindOrCache(InGlyphIndex, CachedGlyphData))
	{
		// Note: FreeType's vertical metrics grows downward while other FreeType coordinates have a Y growing upward. Hence the extra negation.
		*OutX = CachedGlyphData.GlyphMetrics.horiBearingX -   CachedGlyphData.GlyphMetrics.vertBearingX;
		*OutY = CachedGlyphData.GlyphMetrics.horiBearingY - (-CachedGlyphData.GlyphMetrics.vertBearingY);

		int FontXScale = 0;
		int FontYScale = 0;
		hb_font_get_scale(InFont, &FontXScale, &FontYScale);

		if (FontXScale < 0)
		{
			*OutX = -*OutX;
		}

		if (FontYScale < 0)
		{
			*OutY = -*OutY;
		}

		return true;
	}

	return false;
}

hb_position_t get_glyph_h_kerning(hb_font_t* InFont, void* InFontData, hb_codepoint_t InLeftGlyphIndex, hb_codepoint_t InRightGlyphIndex, void* InUserData)
{
//Too verbose for now, but will help in subsequent optimization
//	SCOPE_CYCLE_COUNTER(STAT_HarfBuzzFontFunctions_get_glyph_h_kerning);

	const FUserData* UserDataPtr = static_cast<FUserData*>(hb_font_get_user_data(InFont, &UserDataKey));
	FT_Face FreeTypeFace = UserDataPtr->FreeTypeFace;

	TSharedPtr<FFreeTypeKerningCache> KerningCache = UserDataPtr->FTCacheDirectory->GetKerningCache(FreeTypeFace, FT_KERNING_DEFAULT, UserDataPtr->RenderSize);
	if (KerningCache)
	{
		FT_Vector KerningVector;
		if (KerningCache->FindOrCache(InLeftGlyphIndex, InRightGlyphIndex, KerningVector))
		{
			return KerningVector.x;
		}
	}

	return 0;
}

hb_bool_t get_glyph_extents(hb_font_t* InFont, void* InFontData, hb_codepoint_t InGlyphIndex, hb_glyph_extents_t* OutExtents, void* InUserData)
{
//Too verbose for now, but will help in subsequent optimization
//	SCOPE_CYCLE_COUNTER(STAT_HarfBuzzFontFunctions_get_glyph_extents);

	const FUserData* UserDataPtr = static_cast<FUserData*>(hb_font_get_user_data(InFont, &UserDataKey));
	FT_Face FreeTypeFace = UserDataPtr->FreeTypeFace;
	const int32 FreeTypeFlags = UserDataPtr->FreeTypeFlags;

	TSharedRef<FFreeTypeGlyphCache> GlyphCache = UserDataPtr->FTCacheDirectory->GetGlyphCache(FreeTypeFace, FreeTypeFlags, UserDataPtr->RenderSize);
	FFreeTypeGlyphCache::FCachedGlyphData CachedGlyphData;
	if (GlyphCache->FindOrCache(InGlyphIndex, CachedGlyphData))
	{
		OutExtents->x_bearing	=  CachedGlyphData.GlyphMetrics.horiBearingX;
		OutExtents->y_bearing	=  CachedGlyphData.GlyphMetrics.horiBearingY;
		OutExtents->width		=  CachedGlyphData.GlyphMetrics.width;
		OutExtents->height		= -CachedGlyphData.GlyphMetrics.height;

		int FontXScale = 0;
		int FontYScale = 0;
		hb_font_get_scale(InFont, &FontXScale, &FontYScale);

		if (FontXScale < 0)
		{
			OutExtents->x_bearing = -OutExtents->x_bearing;
			OutExtents->width = -OutExtents->width;
		}

		if (FontYScale < 0)
		{
			OutExtents->y_bearing = -OutExtents->y_bearing;
			OutExtents->height = -OutExtents->height;
		}

		return true;
	}

	return false;
}

hb_bool_t get_glyph_contour_point(hb_font_t* InFont, void* InFontData, hb_codepoint_t InGlyphIndex, unsigned int InPointIndex, hb_position_t* OutX, hb_position_t* OutY, void* InUserData)
{
//Too verbose for now, but will help in subsequent optimization
//	SCOPE_CYCLE_COUNTER(STAT_HarfBuzzFontFunctions_get_glyph_contour_point);

	const FUserData* UserDataPtr = static_cast<FUserData*>(hb_font_get_user_data(InFont, &UserDataKey));
	FT_Face FreeTypeFace = UserDataPtr->FreeTypeFace;
	const int32 FreeTypeFlags = UserDataPtr->FreeTypeFlags;

	TSharedRef<FFreeTypeGlyphCache> GlyphCache = UserDataPtr->FTCacheDirectory->GetGlyphCache(FreeTypeFace, FreeTypeFlags, UserDataPtr->RenderSize);
	FFreeTypeGlyphCache::FCachedGlyphData CachedGlyphData;
	if (GlyphCache->FindOrCache(InGlyphIndex, CachedGlyphData))
	{
		if (InPointIndex < static_cast<unsigned int>(CachedGlyphData.OutlinePoints.Num()))
		{
			*OutX = CachedGlyphData.OutlinePoints[InPointIndex].x;
			*OutY = CachedGlyphData.OutlinePoints[InPointIndex].y;
			return true;
		}
	}

	return false;
}

} // namespace Internal

} // namespace HarfBuzzFontFunctions

#endif // WITH_FREETYPE && WITH_HARFBUZZ

FHarfBuzzFontCache::FHarfBuzzFontCache(FFreeTypeCacheDirectory* InFTCacheDirectory)
	: FTCacheDirectory(InFTCacheDirectory)
{
	check(FTCacheDirectory);

#if WITH_HARFBUZZ
	CustomHarfBuzzFuncs = hb_font_funcs_create();

	hb_font_funcs_set_font_h_extents_func(CustomHarfBuzzFuncs, &HarfBuzzFontFunctions::Internal::get_font_h_extents, nullptr, nullptr);
	hb_font_funcs_set_nominal_glyph_func(CustomHarfBuzzFuncs, &HarfBuzzFontFunctions::Internal::get_nominal_glyph, nullptr, nullptr);
#if WITH_HARFBUZZ_V24
	hb_font_funcs_set_nominal_glyphs_func(CustomHarfBuzzFuncs, &HarfBuzzFontFunctions::Internal::get_nominal_glyphs, nullptr, nullptr);
#endif // WITH_HARFBUZZ_V24
	hb_font_funcs_set_glyph_h_advance_func(CustomHarfBuzzFuncs, &HarfBuzzFontFunctions::Internal::get_glyph_h_advance, nullptr, nullptr);
#if WITH_HARFBUZZ_V24
	hb_font_funcs_set_glyph_h_advances_func(CustomHarfBuzzFuncs, &HarfBuzzFontFunctions::Internal::get_glyph_h_advances, nullptr, nullptr);
#endif // WITH_HARFBUZZ_V24
	hb_font_funcs_set_glyph_v_advance_func(CustomHarfBuzzFuncs, &HarfBuzzFontFunctions::Internal::get_glyph_v_advance, nullptr, nullptr);
#if WITH_HARFBUZZ_V24
	hb_font_funcs_set_glyph_v_advances_func(CustomHarfBuzzFuncs, &HarfBuzzFontFunctions::Internal::get_glyph_v_advances, nullptr, nullptr);
#endif // WITH_HARFBUZZ_V24
	hb_font_funcs_set_glyph_v_origin_func(CustomHarfBuzzFuncs, &HarfBuzzFontFunctions::Internal::get_glyph_v_origin, nullptr, nullptr);
	hb_font_funcs_set_glyph_h_kerning_func(CustomHarfBuzzFuncs, &HarfBuzzFontFunctions::Internal::get_glyph_h_kerning, nullptr, nullptr);
	hb_font_funcs_set_glyph_extents_func(CustomHarfBuzzFuncs, &HarfBuzzFontFunctions::Internal::get_glyph_extents, nullptr, nullptr);
	hb_font_funcs_set_glyph_contour_point_func(CustomHarfBuzzFuncs, &HarfBuzzFontFunctions::Internal::get_glyph_contour_point, nullptr, nullptr);

	hb_font_funcs_make_immutable(CustomHarfBuzzFuncs);
#endif // WITH_HARFBUZZ
}

FHarfBuzzFontCache::~FHarfBuzzFontCache()
{
	Flush();

#if WITH_HARFBUZZ
	hb_font_funcs_destroy(CustomHarfBuzzFuncs);
	CustomHarfBuzzFuncs = nullptr;
#endif // WITH_HARFBUZZ
}

#if WITH_HARFBUZZ

#if WITH_FREETYPE
FHarfBuzzFontCache::FFontKey::FFontKey(FT_Face InFace, const int32 InFlags)
: Face(InFace)
, Flags(InFlags)
, KeyHash(0)
{
	KeyHash = GetTypeHash(Face);
	KeyHash = HashCombine(KeyHash, GetTypeHash(Flags));
}
#endif //WITH_FREETYPE

// Shaping the glyph can be a very complex process, requiring many lookup to take the right decisions. Harfbuzz is using a caching system internally to avoid recomputing everything each time
// it shapes a text. In order to take advantage of it, it's really important to not recreate hb_font from scratch for each shape process, but unfortunately keeping all the hb_fonts in memory, with their cache,
// use too much memory. That's why the following function is keeping the hb_face (which is size agnostic and still keep the cache system of harfbuzz) in a cache, then for each call,
// it create the hb_font from the cached hb_face on the fly. This way, we speed up the shaping a lot thanks to the harfbuzz cache, while keeping memory low (because we don't keep 
// a font for each face and size combination).
hb_font_t* FHarfBuzzFontCache::CreateFont(const FFreeTypeFace& InFace, const uint32 InGlyphFlags, const FSlateFontInfo& InFontInfo, const float InFontScale)
{
	check(IsInGameThread()); //Would need some lock in map access if we got MT with that.

	hb_font_t* HarfBuzzFont = nullptr;

#if WITH_FREETYPE
	SCOPE_CYCLE_COUNTER(STAT_HarfBuzzFontCache_ApplyFont);

	FT_Face FreeTypeFace = InFace.GetFace();
	const uint32 FontRenderSize = FreeTypeUtils::ComputeFontPixelSize(InFontInfo.Size, InFontScale);
	FreeTypeUtils::ApplySizeAndScale(FreeTypeFace, FontRenderSize);

	FFontKey FontKey(FreeTypeFace, InGlyphFlags);
	hb_face_t*& CacheEntry = HarfBuzzFontCacheMap.FindOrAdd(FontKey);
	if (CacheEntry == nullptr)
	{
		hb_face_t* HarfBuzzFace = hb_ft_face_create(FreeTypeFace,nullptr);
		CacheEntry = HarfBuzzFace;
	}

	HarfBuzzFont = hb_font_create(CacheEntry);

	//The HarfBuzz face doesn't store the size information about the freetype face, so we need to provide the info to the font itself, as it needs it).
	const int HarfBuzzFontXScale = (int)(((uint64_t)FreeTypeFace->size->metrics.x_scale * (uint64_t)FreeTypeFace->units_per_EM + (1u << 15)) >> 16);
	const int HarfBuzzFontYScale = (int)(((uint64_t)FreeTypeFace->size->metrics.y_scale * (uint64_t)FreeTypeFace->units_per_EM + (1u << 15)) >> 16);
	hb_font_set_scale(HarfBuzzFont, HarfBuzzFontXScale, HarfBuzzFontYScale);

	hb_font_set_funcs(HarfBuzzFont, CustomHarfBuzzFuncs, nullptr, nullptr);

	HarfBuzzFontFunctions::FUserData* UserData = HarfBuzzFontFunctions::CreateUserData(FontRenderSize, FTCacheDirectory, FreeTypeFace, InGlyphFlags);

	// Apply the current settings of the font to the cache in order to allow harfbuzz to retrieve those values back when shaping the text.
	hb_font_extents_t& HarfBuzzFontExtents = UserData->HarfBuzzFontExtents;

	// Cache the font extents
	const bool bIsAscentDescentOverrideEnabled = UE::Slate::FontUtils::IsAscentDescentOverrideEnabled(InFontInfo.FontObject);
	HarfBuzzFontExtents.ascender = InFace.GetAscender(bIsAscentDescentOverrideEnabled);
	HarfBuzzFontExtents.descender = InFace.GetDescender(bIsAscentDescentOverrideEnabled);
	HarfBuzzFontExtents.line_gap = InFace.GetScaledHeight(bIsAscentDescentOverrideEnabled) - (HarfBuzzFontExtents.ascender - HarfBuzzFontExtents.descender);
	if (HarfBuzzFontYScale < 0)
	{
		HarfBuzzFontExtents.ascender = -HarfBuzzFontExtents.ascender;
		HarfBuzzFontExtents.descender = -HarfBuzzFontExtents.descender;
		HarfBuzzFontExtents.line_gap = -HarfBuzzFontExtents.line_gap;
	}

	hb_font_set_user_data(HarfBuzzFont,	&HarfBuzzFontFunctions::UserDataKey, UserData, &HarfBuzzFontFunctions::DestroyUserData,	true);

#endif // WITH_FREETYPE

	return HarfBuzzFont;
}

#endif // WITH_HARFBUZZ

void FHarfBuzzFontCache::Flush()
{
#if WITH_HARFBUZZ
	for (TTuple<FFontKey, hb_face_t*>& Entry : HarfBuzzFontCacheMap)
	{
		hb_face_destroy(Entry.Value);
	}
	HarfBuzzFontCacheMap.Empty();
#endif // WITH_HARFBUZZ
}

