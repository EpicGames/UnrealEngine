// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/StringView.h"

class FFreeTypeFace;
class FFreeTypeCacheDirectory;
struct FSlateFontInfo;

#ifndef WITH_HARFBUZZ
	#define WITH_HARFBUZZ 0
#endif // WITH_HARFBUZZ

#ifndef WITH_HARFBUZZ_V24
	#define WITH_HARFBUZZ_V24 0
#endif // WITH_HARFBUZZ_V24

#if PLATFORM_COMPILER_HAS_GENERIC_KEYWORD
	#define generic __identifier(generic)
#endif	//PLATFORM_COMPILER_HAS_GENERIC_KEYWORD

#if WITH_HARFBUZZ
	THIRD_PARTY_INCLUDES_START
	#include "hb.h"
	#include "hb-ft.h"
	THIRD_PARTY_INCLUDES_END
#endif // #if WITH_HARFBUZZ

#if PLATFORM_COMPILER_HAS_GENERIC_KEYWORD
	#undef generic
#endif	//PLATFORM_COMPILER_HAS_GENERIC_KEYWORD

namespace HarfBuzzUtils
{

#if WITH_HARFBUZZ

/** Utility function to append an FString into a hb_buffer_t in the most efficient way based on the string encoding method of the current platform */
void AppendStringToBuffer(const FStringView InString, hb_buffer_t* InHarfBuzzTextBuffer);

/** Utility function to append an FString into a hb_buffer_t in the most efficient way based on the string encoding method of the current platform */
void AppendStringToBuffer(const FStringView InString, const int32 InStartIndex, const int32 InLength, hb_buffer_t* InHarfBuzzTextBuffer);

#endif // #if WITH_HARFBUZZ

} // namespace HarfBuzzUtils

class FHarfBuzzFontCache
{
public:
	FHarfBuzzFontCache(FFreeTypeCacheDirectory* InFTCacheDirectory);
	~FHarfBuzzFontCache();

#if WITH_HARFBUZZ
	/** Create a HarfBuzz font from the given face - must be destroyed with hb_font_destroy when done */
	hb_font_t* CreateFont(const FFreeTypeFace& InFace, const uint32 InGlyphFlags, const FSlateFontInfo& InFontInfo, const float InFontScale);

#endif // WITH_HARFBUZZ
	/** Clear the cache containing all harfbuzz fonts */
	void Flush();

private:
	FFreeTypeCacheDirectory* FTCacheDirectory;

#if WITH_HARFBUZZ && WITH_FREETYPE
	class FFontKey
	{
	public:
		FFontKey(FT_Face InFace, const int32 InFlags);

		bool operator==(const FFontKey& Other) const
		{
			return Face == Other.Face
				&& Flags == Other.Flags;
		}

		bool operator!=(const FFontKey& Other) const
		{
			return !(*this == Other);
		}

		friend inline uint32 GetTypeHash(const FFontKey& Key)
		{
			return Key.KeyHash;
		}

	private:
		FT_Face Face;
		const int32 Flags;
		uint32 KeyHash;
	};
	TMap<FFontKey, hb_face_t*> HarfBuzzFontCacheMap;
#endif // WITH_HARFBUZZ && WITH_FREETYPE

#if WITH_HARFBUZZ
	hb_font_funcs_t* CustomHarfBuzzFuncs;
#endif // WITH_HARFBUZZ
};
