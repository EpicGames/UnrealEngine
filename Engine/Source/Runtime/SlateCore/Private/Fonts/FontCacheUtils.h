// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "SlateGlobals.h"

DECLARE_MEMORY_STAT_EXTERN(TEXT("Font Measure Memory"), STAT_SlateFontMeasureCacheMemory, STATGROUP_SlateMemory, SLATECORE_API);

/**
 * Basic Least Recently Used (LRU) cache
 */
class FLRUStringCache
{
public:
	FLRUStringCache( int32 InMaxNumElements )
		: LookupSet()
		, MostRecent(nullptr)
		, LeastRecent(nullptr)
		, MaxNumElements(InMaxNumElements)
	{ }

	~FLRUStringCache()
	{
		Empty();
	}

	/**
	 * Accesses an item in the cache.  
	 */
	FORCEINLINE const FVector2f* AccessItem(FStringView Key)
	{
		FCacheEntry** Entry = LookupSet.FindByHash(FLookupSet::KeyFuncsType::GetKeyHash(Key), Key);
		if( Entry )
		{
			MarkAsRecent( *Entry );
			return &((*Entry)->Value);
		}
		
		return nullptr;
	}

	void Add( FStringView Key, const FVector2f& Value )
	{
		FCacheEntry** Entry = LookupSet.FindByHash(FLookupSet::KeyFuncsType::GetKeyHash(Key), Key);
	
		// Make a new link
		if( !Entry )
		{
			if( LookupSet.Num() == MaxNumElements )
			{
				Eject();
				checkf( LookupSet.Num() < MaxNumElements, TEXT("Could not eject item from the LRU: (%d of %d), %s"), LookupSet.Num(), MaxNumElements, LeastRecent ? *LeastRecent->Key : TEXT("NULL"));
			}

			FCacheEntry* NewEntry = new FCacheEntry( FString(Key), Value );

			// Link before the most recent so that we become the most recent
			NewEntry->Link(MostRecent);
			MostRecent = NewEntry;

			if( LeastRecent == nullptr )
			{
				LeastRecent = NewEntry;
			}

			STAT( uint32 CurrentMemUsage = LookupSet.GetAllocatedSize() );
			LookupSet.Add( NewEntry );
			STAT( uint32 NewMemUsage = LookupSet.GetAllocatedSize() );
			INC_MEMORY_STAT_BY( STAT_SlateFontMeasureCacheMemory,  NewMemUsage - CurrentMemUsage );
		}
		else
		{
			// Trying to add an existing value 
			FCacheEntry* EntryPtr = *Entry;
			
			// Update the value
			EntryPtr->Value = Value;
			checkSlow(FLookupSet::KeyFuncsType::Matches(EntryPtr->Key, Key));
			// Mark as the most recent
			MarkAsRecent( EntryPtr );
		}
	}

	void Empty()
	{
		DEC_MEMORY_STAT_BY( STAT_SlateFontMeasureCacheMemory, LookupSet.GetAllocatedSize() );

		for( TSet<FCacheEntry*, FCaseSensitiveStringKeyFuncs >::TIterator It(LookupSet); It; ++It )
		{
			FCacheEntry* Entry = *It;
			// Note no need to unlink anything here. we are emptying the entire list
			delete Entry;
		}
		LookupSet.Empty();

		MostRecent = LeastRecent = nullptr;
	}
private:
	struct FCacheEntry
	{
		FString Key;
		FVector2f Value;
		FCacheEntry* Next;
		FCacheEntry* Prev;

		FCacheEntry( FString InKey, const FVector2f& InValue )
			: Key( MoveTemp(InKey) )
			, Value( InValue )
			, Next( nullptr )
			, Prev( nullptr )
		{
			INC_MEMORY_STAT_BY( STAT_SlateFontMeasureCacheMemory, Key.GetAllocatedSize()+sizeof(Value)+sizeof(Next)+sizeof(Prev) );
		}

		~FCacheEntry()
		{
			DEC_MEMORY_STAT_BY( STAT_SlateFontMeasureCacheMemory, Key.GetAllocatedSize()+sizeof(Value)+sizeof(Next)+sizeof(Prev) );
		}

		FORCEINLINE void Link( FCacheEntry* Before )
		{
			Next = Before;

			if( Before )
			{
				Before->Prev = this;
			}
		}

		FORCEINLINE void Unlink()
		{
			if( Prev )
			{
				Prev->Next = Next;
			}

			if( Next )
			{
				Next->Prev = Prev;
			}

			Prev = nullptr;
			Next = nullptr;
		}
	};

	struct FCaseSensitiveStringKeyFuncs : BaseKeyFuncs<FCacheEntry*, FString>
	{
		FORCEINLINE static const FString& GetSetKey( const FCacheEntry* Entry )
		{
			return Entry->Key;
		}

		FORCEINLINE static bool Matches(const FString& A, const FStringView& B)
		{
			return FStringView(A).Equals( B, ESearchCase::CaseSensitive );
		}

		FORCEINLINE static uint32 GetKeyHash(FStringView Identifier)
		{
			return CityHash32(reinterpret_cast<const char*>(Identifier.GetData()), Identifier.Len() * sizeof(TCHAR));
		}
	};
	

	/**
	 * Marks the link as the the most recent
	 *
	 * @param Entry	The link to mark as most recent
	 */
	FORCEINLINE void MarkAsRecent( FCacheEntry* Entry )
	{
		checkSlow( LeastRecent && MostRecent );

		// If we are the least recent entry we are no longer the least recent
		// The previous least recent item is now the most recent.  If it is nullptr then this entry is the only item in the list
		if( Entry == LeastRecent && LeastRecent->Prev != nullptr )
		{
			LeastRecent = LeastRecent->Prev;
		}

		// No need to relink if we happen to already be accessing the most recent value
		if( Entry != MostRecent )
		{
			// Unlink from its current spot
			Entry->Unlink();

			// Relink before the most recent so that we become the most recent
			Entry->Link(MostRecent);
			MostRecent = Entry;
		}
	}

	/**
	 * Removes the least recent item from the cache
	 */
	FORCEINLINE void Eject()
	{
		FCacheEntry* EntryToRemove = LeastRecent;
		// Eject the least recent, no more space
		check( EntryToRemove );
		STAT( uint32 CurrentMemUsage = LookupSet.GetAllocatedSize() );
		LookupSet.Remove( EntryToRemove->Key );
		STAT( uint32 NewMemUsage = LookupSet.GetAllocatedSize() );
		DEC_MEMORY_STAT_BY( STAT_SlateFontMeasureCacheMemory,  CurrentMemUsage - NewMemUsage );

		LeastRecent = LeastRecent->Prev;

		// Unlink the LRU
		EntryToRemove->Unlink();

		delete EntryToRemove;
	}

private:
	using FLookupSet = TSet< FCacheEntry*, FCaseSensitiveStringKeyFuncs >;
	FLookupSet LookupSet;
	/** Most recent item in the cache */
	FCacheEntry* MostRecent;
	/** Least recent item in the cache */
	FCacheEntry* LeastRecent;
	/** The maximum number of elements in the cache */
	int32 MaxNumElements;
};

