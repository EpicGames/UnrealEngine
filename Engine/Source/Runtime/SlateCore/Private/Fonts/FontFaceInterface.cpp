// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/FontFaceInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FontFaceInterface)

UFontFaceInterface::UFontFaceInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

