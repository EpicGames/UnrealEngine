// Copyright Epic Games, Inc. All Rights Reserved.

#include "FontGeometryPreprocessing.h"

#if WITH_EDITOR && WITH_FREETYPE

#if PLATFORM_MAC
#include <TargetConditionals.h>
#endif

#define MSDFGEN_USE_SKIA
#include "ThirdParty/skia/skia-simplify.h"

#define MSDFGEN_PARENT_NAMESPACE SlateMsdfgen
#include "ThirdParty/msdfgen/msdfgen.h"

namespace UE
{
namespace Slate
{

using namespace SlateMsdfgen;

// Corners with an angle greater than 3 radians (~171 degrees) won't be treated as corners.
static constexpr double SdfCornerAngleThreshold = 3.0;

static void ScanlineShape(const msdfgen::Shape& Shape, double Y, TArray<msdfgen::Scanline::Intersection>& OutIntersections)
{
	double X[3];
	int DY[3];
	OutIntersections.Reset();
	for (const msdfgen::Contour& Contour : Shape.contours)
	{
		for (const msdfgen::EdgeHolder& Edge : Contour.edges)
		{
			int N = Edge->scanlineIntersections(X, DY, Y);
			for (int I = 0; I < N; ++I) {
				msdfgen::Scanline::Intersection Intersection = { X[I], DY[I] };
				OutIntersections.Add(Intersection);
			}
		}
	}
	OutIntersections.Sort([](const msdfgen::Scanline::Intersection& A, const msdfgen::Scanline::Intersection& B) { return A.x < B.x; });
}

static bool ScanlineEquivalence(const msdfgen::Shape& A, const msdfgen::Shape& B)
{
	constexpr double VertexYEpsilon = .0001;
	TSet<double> VertY;
	for (const msdfgen::Contour& Contour : A.contours)
	{
		for (const msdfgen::EdgeHolder& Edge : Contour.edges)
		{
			VertY.Add(Edge->controlPoints()[0].y);
		}
	}
	for (const msdfgen::Contour& Contour : B.contours)
	{
		for (const msdfgen::EdgeHolder& Edge : Contour.edges)
		{
			VertY.Add(Edge->controlPoints()[0].y);
		}
	}
	TArray<msdfgen::Scanline::Intersection> AIntersections, BIntersections;
	for (double BaseY : VertY)
	{
		for (int Sign = -1; Sign <= 1; Sign += 2)
		{
			// Test scanline slightly above and below vertex Y.
			const double Y = BaseY+Sign*VertexYEpsilon;
			ScanlineShape(A, Y, AIntersections);
			ScanlineShape(B, Y, BIntersections);
			if (AIntersections.Num() != BIntersections.Num())
			{
				return false;
			}
			for (SIZE_T I = 0; I < AIntersections.Num(); ++I)
			{
				if (AIntersections[I].direction != BIntersections[I].direction)
				{
					return false;
				}
			}
		}
	}
	return true;
}

bool PreprocessFontGeometry(FPreprocessedFontGeometry& OutPreprocessedFontGeometry, FT_Face InFreeTypeFace)
{
	int StatNumFullGeometry = 0;
	int StatNumWindings = 0;
	int NumCorrectWindings = 0;
	int NumReverseWindings = 0;
	/** Maps winding directions of each glyph in font:
	 *      +1 = correct winding (and no preprocessing required)
	 *      -1 = reversed winding
	 *       0 = inconsistend winding or unnormalized geometry (individual glyph preprocessing data will be stored) or empty shape
	 */
	TArray<int8> GlyphWindingDirections;
	GlyphWindingDirections.SetNumZeroed(InFreeTypeFace->num_glyphs);
	for (FT_Long GlyphIndex = 0; GlyphIndex < InFreeTypeFace->num_glyphs; ++GlyphIndex)
	{
		// Load glyph into an MSDFgen Shape object
		if (FT_Load_Glyph(InFreeTypeFace, GlyphIndex, FT_LOAD_NO_SCALE))
		{
			continue;
		}
		msdfgen::Shape MsdfgenShape;
		if (msdfgen::readFreetypeOutline(MsdfgenShape, &InFreeTypeFace->glyph->outline, 1))
		{
			continue;
		}
		if (MsdfgenShape.contours.empty())
		{
			continue;
		}

		// Detect reversed or inconsistent contour windings
		bool bPreprocessed = false;
		bool bReversedWindings = false;
		FPreprocessedGlyphGeometry PreprocessedGlyphGeometry;
		FPreprocessedGlyphGeometry::FWindingFingerprint OriginalGlyphWinding(MsdfgenShape);
		MsdfgenShape.orientContours();
		FPreprocessedGlyphGeometry::FWindingFingerprint ResolvedGlyphWinding(MsdfgenShape);
		if (ResolvedGlyphWinding != OriginalGlyphWinding)
		{
			PreprocessedGlyphGeometry = FPreprocessedGlyphGeometry(OriginalGlyphWinding, ResolvedGlyphWinding);
			const FPreprocessedGlyphGeometryView PreprocessedGlyphGeometryView = PreprocessedGlyphGeometry.View();
			// Restore MsdfgenShape to its initial state by re-applying contour reversals
			PreprocessedGlyphGeometryView.UpdateWindings(MsdfgenShape);
			// If contour windings are fully reversed, do not store individual glyph preprocessing data yet in case this is true for the majority of glyphs
			if (PreprocessedGlyphGeometryView.HasAllContoursReversedWindings())
			{
				bReversedWindings = true;
			}
			else // Inconsistent windings case - list of contours to reverse must be stored either way
			{
				bPreprocessed = true;
				++StatNumWindings;
			}
		}

		// Detect self-intersecting geometry and store full geometry of the resolved shape
		FPreprocessedGlyphGeometry::FGeometryFingerprint OriginalGeometry(MsdfgenShape);
		msdfgen::resolveShapeGeometry(MsdfgenShape);
		FPreprocessedGlyphGeometry::FGeometryFingerprint ResolvedGeometry(MsdfgenShape);
		if (ResolvedGeometry != OriginalGeometry)
		{
			/* In many cases, the resolved geometry is slightly different but not in a consequential way.
			 * To save memory, we will do a secondary check by inspecting scanlines near vertices and compare them between resolved and unresolved shape.
			 * If these scanlines are equivalent, it can be assumed the rasterization output will be the same.
			 */
			// Reconstruct unresolved shape
			msdfgen::Shape UnresolvedMsdfgenShape;
			msdfgen::readFreetypeOutline(UnresolvedMsdfgenShape, &InFreeTypeFace->glyph->outline, 1);
			// Apply potential contour winding reversals from previous step
			PreprocessedGlyphGeometry.View().UpdateWindings(UnresolvedMsdfgenShape);

			if (!ScanlineEquivalence(MsdfgenShape, UnresolvedMsdfgenShape))
			{
				msdfgen::edgeColoringInkTrap(MsdfgenShape, SdfCornerAngleThreshold);
				PreprocessedGlyphGeometry = FPreprocessedGlyphGeometry(MsdfgenShape);
				bPreprocessed = true;
				++StatNumFullGeometry;
			}
		}

		if (bPreprocessed)
		{
			OutPreprocessedFontGeometry.AddGlyph((int32) GlyphIndex, PreprocessedGlyphGeometry.View());
		}
		else
		{
			if (bReversedWindings)
			{
				++NumReverseWindings;
				GlyphWindingDirections[GlyphIndex] = -1;
			}
			else
			{
				++NumCorrectWindings;
				GlyphWindingDirections[GlyphIndex] = 1;
			}
		}
	}

	int8 AtypicalWinding = -1;
	// If at least 2/3 of (non-preprocessed) glyphs have reversed windings, make that the default and mark correctly oriented glyphs as "reversed" to negate this.
	if (NumReverseWindings > 2*NumCorrectWindings)
	{
		OutPreprocessedFontGeometry.SetGlobalWindingReversal(true);
		AtypicalWinding = 1;
	}
	// Mark the winding direction deemed as "atypical" with the reversal flag
	const FPreprocessedGlyphGeometryView GlyphWindingReversalView(FPreprocessedGlyphGeometry::FLAG_REVERSE_WINDINGS, 0, nullptr, nullptr, 0, 0);
	for (int32 GlyphIndex = 0; GlyphIndex < GlyphWindingDirections.Num(); ++GlyphIndex)
	{
		if (GlyphWindingDirections[GlyphIndex] == AtypicalWinding)
		{
			OutPreprocessedFontGeometry.AddGlyph(GlyphIndex, GlyphWindingReversalView);
			++StatNumWindings;
		}
	}

	UE_LOGF(LogSlate, Log, "Preprocessed geometry for font %s is %d bytes, containing %d paths and %d windings out of %d total glyphs.", InFreeTypeFace->family_name, int(OutPreprocessedFontGeometry.GetDataSize()), StatNumFullGeometry, StatNumWindings, int(InFreeTypeFace->num_glyphs));

	return true;
}

} // namespace Slate
} // namespace UE

#endif
