// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/FontCacheFreeType.h"
#include "Fonts/PreprocessedFontGeometry.h"

#if WITH_EDITOR && WITH_FREETYPE

namespace UE
{
namespace Slate
{

/**
 * Produces preprocessed font geometry object for the given FreeType font face. Returns true on success.
 */
bool PreprocessFontGeometry(FPreprocessedFontGeometry& OutPreprocessedFontGeometry, FT_Face InFreeTypeFace);

}
}

#endif
