// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/FontMeasure.h"
#include "Fonts/FontCache.h"
#include "Fonts/FontCacheUtils.h"

#define USE_MEASURE_CACHING 1

namespace FontMeasureConstants
{
	/** Number of possible elements in each measurement cache */
	const uint32 MeasureCacheSize = 500;
}

struct FSlateFontMeasureCache
{
	FSlateFontMeasureCache( const FCompositeFont* const InCompositeFont )
		: MeasureCache( FontMeasureConstants::MeasureCacheSize )
#if WITH_EDITORONLY_DATA
		, CompositeFontHistoryRevision( INDEX_NONE )
#endif	// WITH_EDITORONLY_DATA
	{
#if WITH_EDITORONLY_DATA
		if( InCompositeFont )
		{
			CompositeFontHistoryRevision = InCompositeFont->HistoryRevision;
		}
#endif	// WITH_EDITORONLY_DATA
	}

#if WITH_EDITORONLY_DATA
	/** Check to see if our cached measure data is potentially stale for the given font */
	bool IsStale( const FCompositeFont* const InCompositeFont ) const
	{
		return CompositeFontHistoryRevision != (InCompositeFont ? InCompositeFont->HistoryRevision : INDEX_NONE);
	}
#endif	// WITH_EDITORONLY_DATA

	/** Internal measure cache */
	FMeasureCache MeasureCache;

#if WITH_EDITORONLY_DATA
	/** The history revision of the cached composite font */
	int32 CompositeFontHistoryRevision;
#endif	// WITH_EDITORONLY_DATA
};

TSharedRef< FSlateFontMeasure > FSlateFontMeasure::Create( const TSharedRef<class FSlateFontCache>& FontCache )
{
	return MakeShareable( new FSlateFontMeasure( FontCache ) );
}

FSlateFontMeasure::FSlateFontMeasure( const TSharedRef<class FSlateFontCache>& InFontCache )
	: FontCache( InFontCache )
{

}

UE::Slate::FDeprecateVector2DResult FSlateFontMeasure::Measure( FStringView Text, const FSlateFontInfo& InFontInfo, float FontScale ) const
{
	int32 DummyLastCharacterIndex;
	return MeasureStringInternal( Text, 0, Text.Len(), InFontInfo, false, FontScale, INDEX_NONE, ELastCharacterIndexFormat::Unused, DummyLastCharacterIndex);
}

UE::Slate::FDeprecateVector2DResult FSlateFontMeasure::Measure( const FText& Text, const FSlateFontInfo& InFontInfo, float FontScale ) const
{
	int32 DummyLastCharacterIndex;
	FStringView String = Text.ToString();
	return MeasureStringInternal( String, 0, String.Len(), InFontInfo, false, FontScale, INDEX_NONE, ELastCharacterIndexFormat::Unused, DummyLastCharacterIndex);
}

UE::Slate::FDeprecateVector2DResult FSlateFontMeasure::Measure( FStringView Text, int32 StartIndex, int32 EndIndex, const FSlateFontInfo& InFontInfo, bool IncludeKerningWithPrecedingChar, float FontScale ) const
{
	int32 DummyLastCharacterIndex;
	return MeasureStringInternal( Text, StartIndex, EndIndex, InFontInfo, IncludeKerningWithPrecedingChar, FontScale, INDEX_NONE, ELastCharacterIndexFormat::Unused, DummyLastCharacterIndex);
}

int32 FSlateFontMeasure::FindLastWholeCharacterIndexBeforeOffset( FStringView Text, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, float FontScale ) const
{
	return FindLastWholeCharacterIndexBeforeOffset( Text, 0, Text.Len(), InFontInfo, HorizontalOffset, false, FontScale );
}

int32 FSlateFontMeasure::FindLastWholeCharacterIndexBeforeOffset( const FText& Text, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, float FontScale ) const
{
	return FindLastWholeCharacterIndexBeforeOffset( Text.ToString(), InFontInfo, HorizontalOffset, FontScale );
}


int32 FSlateFontMeasure::FindLastWholeCharacterIndexBeforeOffset( FStringView Text, int32 StartIndex, int32 EndIndex, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, bool IncludeKerningWithPrecedingChar, float FontScale ) const
{
	int32 FoundLastCharacterIndex = INDEX_NONE;
	MeasureStringInternal( Text, StartIndex, EndIndex, InFontInfo, IncludeKerningWithPrecedingChar, FontScale, HorizontalOffset, ELastCharacterIndexFormat::LastWholeCharacterBeforeOffset, FoundLastCharacterIndex );
	return FoundLastCharacterIndex;
}


int32 FSlateFontMeasure::FindFirstWholeCharacterIndexAfterOffset( const FText& Text, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, float FontScale ) const
{
	return FindFirstWholeCharacterIndexAfterOffset( Text.ToString(), InFontInfo, HorizontalOffset, FontScale );
}

int32 FSlateFontMeasure::FindFirstWholeCharacterIndexAfterOffset( FStringView Text, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, float FontScale ) const
{
	return FindFirstWholeCharacterIndexAfterOffset( Text, 0, Text.Len(), InFontInfo, HorizontalOffset, false, FontScale );
}

int32 FSlateFontMeasure::FindFirstWholeCharacterIndexAfterOffset( FStringView Text, int32 StartIndex, int32 EndIndex, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, bool IncludeKerningWithPrecedingChar, float FontScale ) const
{
	int32 FoundLastCharacterIndex = FindCharacterIndexAtOffset( Text, StartIndex, EndIndex, InFontInfo, HorizontalOffset, IncludeKerningWithPrecedingChar, FontScale );

	float TextWidth = Measure( Text, StartIndex, EndIndex, InFontInfo, IncludeKerningWithPrecedingChar, FontScale ).X;
	float AvailableWidth = TextWidth - HorizontalOffset;

	float RightStringWidth = Measure( Text, FoundLastCharacterIndex, EndIndex, InFontInfo, IncludeKerningWithPrecedingChar, FontScale ).X;
	if ( AvailableWidth < RightStringWidth )
	{
		++FoundLastCharacterIndex;
	}

	return FoundLastCharacterIndex;
}

int32 FSlateFontMeasure::FindCharacterIndexAtOffset( FStringView Text, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, float FontScale ) const
{
	return FindCharacterIndexAtOffset( Text, 0, Text.Len(), InFontInfo, HorizontalOffset, false, FontScale );
}

int32 FSlateFontMeasure::FindCharacterIndexAtOffset( const FText& Text, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, float FontScale ) const
{
	return FindCharacterIndexAtOffset( Text.ToString(), InFontInfo, HorizontalOffset, FontScale );
}

int32 FSlateFontMeasure::FindCharacterIndexAtOffset( FStringView Text, int32 StartIndex, int32 EndIndex, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, bool IncludeKerningWithPrecedingChar, float FontScale ) const
{
	int32 FoundLastCharacterIndex = 0;
	MeasureStringInternal( Text, StartIndex, EndIndex, InFontInfo, IncludeKerningWithPrecedingChar, FontScale, HorizontalOffset, ELastCharacterIndexFormat::CharacterAtOffset, FoundLastCharacterIndex );
	return FoundLastCharacterIndex;
}

FVector2f FSlateFontMeasure::MeasureStringInternal( FStringView Text, int32 StartIndex, int32 EndIndex, const FSlateFontInfo& InFontInfo, bool IncludeKerningWithPrecedingChar, float FontScale, int32 StopAfterHorizontalOffset, ELastCharacterIndexFormat CharIndexFormat, int32& OutLastCharacterIndex ) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Slate_MeasureStringInternal)
	FCharacterList& CharacterList = FontCache->GetCharacterList( InFontInfo, FontScale );
	const uint16 MaxHeight = CharacterList.GetMaxHeight();

	const bool DoesStartAtBeginning = StartIndex == 0;
	const bool DoesFinishAtEnd = EndIndex == Text.Len();
	const int32 TextRangeLength = EndIndex - StartIndex;
	if ( Text.IsEmpty() || EndIndex - StartIndex <= 0 || EndIndex <= 0 || StartIndex < 0 || EndIndex <= StartIndex )
	{
		return FVector2f( 0.f, MaxHeight );
	}

#if USE_MEASURE_CACHING
	FMeasureCache* CurrentMeasureCache = nullptr;
	// Do not cache strings which have small sizes or which have complicated measure requirements
	if( DoesStartAtBeginning && DoesFinishAtEnd && !IncludeKerningWithPrecedingChar && TextRangeLength > 5 && StopAfterHorizontalOffset == INDEX_NONE )
	{
		CurrentMeasureCache = FindOrAddMeasureCache(InFontInfo, FontScale);

		if ( CurrentMeasureCache )
		{
			const FVector2f* CachedMeasurement = CurrentMeasureCache->AccessItem( Text );
			if( CachedMeasurement )
			{
				return *CachedMeasurement;
			}
		}
	}
#endif

	// The size of the string
	FVector2f Size(0.f,0.f);
	// Widest line encountered while drawing this text.
	int32 MaxLineWidth = 0;
	// The width of the current line so far.
	int32 CurrentX = 0;
	// Accumulated height of this block of text
	int32 StringSizeY = MaxHeight;
	// Character from last frame.
	TCHAR PreviousChar = 0;

	//If we are measuring a range then we should take into account the kerning with the character before the start of the range
	if ( !DoesStartAtBeginning && IncludeKerningWithPrecedingChar )
	{
		PreviousChar = Text[ StartIndex - 1 ];
	}

	int32 FinalPosX = 0;

	int32 CharIndex;
	for( CharIndex = StartIndex; CharIndex < EndIndex; ++CharIndex )
	{
		TCHAR CurrentChar = Text[CharIndex];

		const bool IsNewline = (CurrentChar == '\n');

		if (IsNewline)
		{
			// New line means
			// 1) we accumulate total height
			StringSizeY += MaxHeight;
			// 2) update the longest line we've encountered
			MaxLineWidth = FMath::Max(CurrentX, MaxLineWidth);
			// 3) the next line starts at the beginning
			CurrentX = 0;
		}
		else
		{
			// copy the value as we may be invoking GetCharacter() below and it could invalidate our reference
			const FCharacterEntry Entry = CharacterList.GetCharacter(CurrentChar, InFontInfo.FontFallback);

			int32 Kerning = 0;
			if( PreviousChar != 0 )
			{
				Kerning = CharacterList.GetKerning(CharacterList.GetCharacter(PreviousChar, InFontInfo.FontFallback), Entry);
			}

			PreviousChar = CurrentChar;

			const int32 TotalCharSpacing = 
				Kerning + Entry.HorizontalOffset +		// Width is any kerning plus how much to advance the position when drawing a new character
				Entry.XAdvance;	// How far we advance

			CurrentX += Kerning + Entry.XAdvance;

			// Were we asked to stop measuring after the specified horizontal offset in pixels?
			if( StopAfterHorizontalOffset != INDEX_NONE )
			{
				if( CharIndexFormat == ELastCharacterIndexFormat::CharacterAtOffset )
				{
					// Round our test toward the character's center position
					if( StopAfterHorizontalOffset < CurrentX - TotalCharSpacing / 2 )
					{
						// We've reached the stopping point, so bail
						break;
					}
				}
				else if( CharIndexFormat == ELastCharacterIndexFormat::LastWholeCharacterBeforeOffset )
				{
					if( StopAfterHorizontalOffset < CurrentX )
					{
						//LastWholeCharacterBeforeOffset is an inclusive index
						--CharIndex;
						if ( CharIndex < StartIndex )
						{
							CharIndex = INDEX_NONE;
						}

						// We've reached the stopping point, so bail
						break;
					}
				}
			}
		}
	}

	// We just finished a line, so need to update the longest line encountered.
	MaxLineWidth = FMath::Max(CurrentX, MaxLineWidth);

	Size.X = (float)MaxLineWidth;
	Size.Y = (float)StringSizeY;

	if (CharIndexFormat == ELastCharacterIndexFormat::LastWholeCharacterBeforeOffset)
	{
		if (CharIndex >= EndIndex)
		{
			//LastWholeCharacterBeforeOffset is an inclusive index
			CharIndex = EndIndex - 1;
		}
	}

	OutLastCharacterIndex = CharIndex;

#if USE_MEASURE_CACHING
	if( StopAfterHorizontalOffset == INDEX_NONE && CurrentMeasureCache )
	{
		CurrentMeasureCache->Add( Text, Size );
	}
#endif
	return Size;
}

FMeasureCache* FSlateFontMeasure::FindOrAddMeasureCache( const FSlateFontInfo& InFontInfo, const float InFontScale ) const
{
#if USE_MEASURE_CACHING
	FSlateFontKey FontKey(InFontInfo, FFontOutlineSettings::NoOutline, InFontScale);
	const FCompositeFont* const CompositeFont = InFontInfo.GetCompositeFont();

	TSharedPtr< FSlateFontMeasureCache > FoundMeasureCache = FontToMeasureCache.FindRef( FontKey );
	if( FoundMeasureCache.IsValid() )
	{
#if WITH_EDITORONLY_DATA
		// Clear out this entry if it's stale so that we make a new one
		if( FoundMeasureCache->IsStale( CompositeFont ) )
		{
			FoundMeasureCache.Reset();
			FontToMeasureCache.Remove( FontKey );
		}
		else
#endif	// WITH_EDITORONLY_DATA
		{
			return &FoundMeasureCache->MeasureCache;
		}
	}
	
	FoundMeasureCache = MakeShareable( new FSlateFontMeasureCache( CompositeFont ) );
	FontToMeasureCache.Add( FontKey, FoundMeasureCache );
	return &FoundMeasureCache->MeasureCache;
#else
	return nullptr;
#endif
}

uint16 FSlateFontMeasure::GetMaxCharacterHeight( const FSlateFontInfo& InFontInfo, float FontScale ) const
{
	FCharacterList& CharacterList = FontCache->GetCharacterList( InFontInfo, FontScale );

	const uint16 MaxHeight = CharacterList.GetMaxHeight();
	return MaxHeight;
}

int8 FSlateFontMeasure::GetKerning(const FSlateFontInfo& InFontInfo, float FontScale, TCHAR PreviousCharacter, TCHAR CurrentCharacter) const
{
	FCharacterList& CharacterList = FontCache->GetCharacterList( InFontInfo, FontScale );
	return CharacterList.GetKerning( PreviousCharacter, CurrentCharacter, InFontInfo.FontFallback );
}

int16 FSlateFontMeasure::GetBaseline( const FSlateFontInfo& InFontInfo, float FontScale ) const
{
	FCharacterList& CharacterList = FontCache->GetCharacterList( InFontInfo, FontScale );
	return CharacterList.GetBaseline();
}

void FSlateFontMeasure::FlushCache()
{
	FontToMeasureCache.Empty();
}
