// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/FontProviderInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FontProviderInterface)

UFontProviderInterface::UFontProviderInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

