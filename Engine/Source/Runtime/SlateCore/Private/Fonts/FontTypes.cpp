// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/FontTypes.h"


FSlateFontAtlas::FSlateFontAtlas(uint32 InWidth, uint32 InHeight, ESlateFontAtlasContentType InContentType, ESlateTextureAtlasPaddingStyle InPaddingStyle)
	: FSlateTextureAtlas(InWidth, InHeight, GetSlateFontAtlasContentBytesPerPixel(InContentType), InPaddingStyle, true), ContentType(InContentType)
{
}

FSlateFontAtlas::~FSlateFontAtlas()
{
}	

ESlateFontAtlasContentType FSlateFontAtlas::GetContentType() const
{
	return ContentType;
}

void FSlateFontAtlas::GetAtlasDataCopy(TArray<uint8>& OutData) const
{
	OutData = AtlasData;
}

const FAtlasedTextureSlot* FSlateFontAtlas::AddCharacter( const FCharacterRenderData& RenderData )
{
	check(RenderData.ContentType == GetContentType());
	return AddTexture( RenderData.SizeX, RenderData.SizeY, RenderData.RawPixels );
}

bool FSlateFontAtlas::BeginDeferredAddCharacter(const int16 InWidth, const int16 InHeight, FDeferredCharacterRenderData& OutCharInfo)
{
	if (InWidth > 0 && InHeight > 0)
	{
		if (const FAtlasedTextureSlot* NewSlot = FindSlotForTexture(InWidth, InHeight))
		{
			OutCharInfo.StartU = NewSlot->X + NewSlot->Padding;
			OutCharInfo.StartV = NewSlot->Y + NewSlot->Padding;
			OutCharInfo.USize = NewSlot->Width - 2 * NewSlot->Padding;
			OutCharInfo.VSize = NewSlot->Height - 2 * NewSlot->Padding;
			return true;
		}
	}
	return false;
}

void FSlateFontAtlas::EndDeferredAddCharacter(const FDeferredCharacterRenderData& CharInfo)
{
	const uint32 Padding = (PaddingStyle == ESlateTextureAtlasPaddingStyle::NoPadding) ? 0 : 1;
	if (CharInfo.USize > 0 && CharInfo.VSize > 0 && CharInfo.StartU >= (int16)Padding && CharInfo.StartV >= (int16)Padding && CharInfo.RawPixels.Num() == GetSlateFontAtlasContentBytesPerPixel(ContentType) * CharInfo.USize * CharInfo.VSize)
	{
		const FAtlasedTextureSlot Subregion(CharInfo.StartU - Padding,
											CharInfo.StartV - Padding,
											CharInfo.USize + 2 * Padding,
											CharInfo.VSize + 2 * Padding,
											Padding);
		CopyDataIntoSlot(&Subregion, CharInfo.RawPixels);
		MarkTextureDirty();
	}
}

void FSlateFontAtlas::Flush()
{
	// Empty the atlas
	EmptyAtlasData();

	// Recreate the data
	InitAtlasData();

	bNeedsUpdate = true;
	ConditionalUpdateTexture();
}
