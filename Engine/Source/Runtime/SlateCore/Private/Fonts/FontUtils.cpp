// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/FontUtils.h"
#include "Fonts/FontProviderInterface.h"
#include "Fonts/CompositeFont.h"

namespace UE
{
	namespace Slate
	{
		bool FontUtils::IsAscentDescentOverrideEnabled(const TObjectPtr<UObject const> FontObject)
		{
			const IFontProviderInterface* FontProvider = Cast<const IFontProviderInterface>(FontObject);
			if (FontProvider && FontProvider->GetCompositeFont())
			{
				return FontProvider->GetCompositeFont()->IsAscentDescentOverrideEnabled();
			}

			return true; //Default is enabled, unless we have find an explicit disabling earlier.
		}
	}
}