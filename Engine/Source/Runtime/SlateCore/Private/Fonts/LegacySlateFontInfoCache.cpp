// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/LegacySlateFontInfoCache.h"
#include "SlateGlobals.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "Misc/FileHelper.h"
#include "Fonts/FontProviderInterface.h"
#include "Fonts/UnicodeBlockRange.h"

FString FLegacySlateFontInfoCache::FFallbackContext::ToString() const
{
	FString FontName;
	if (FontData)
	{
		const UObject* FontAsset = FontData->GetFontFaceAsset();
		FontName = (FontAsset) ? FontAsset->GetPathName() : FontData->GetFontFilename();
	}
	else
	{
		FontName = TEXT("<unknown>");
	}

	FString CharacterInfo;
	if (Codepoint)
	{
		CharacterInfo = FString::Printf(TEXT("%c (U+%04x)"), (TCHAR)Codepoint, (int32)Codepoint);
	}
	else
	{
		CharacterInfo = TEXT("<unknown>");
	}

	return FString::Printf(TEXT("Font: '%s', Character: '%s'"), *FontName, *CharacterInfo);
}

TSharedPtr<FLegacySlateFontInfoCache> FLegacySlateFontInfoCache::Singleton;

FLegacySlateFontInfoCache& FLegacySlateFontInfoCache::Get()
{
	if (!Singleton.IsValid())
	{
		Singleton = MakeShareable(new FLegacySlateFontInfoCache());
	}
	return *Singleton;
}

FLegacySlateFontInfoCache::FLegacySlateFontInfoCache()
{
	LastResortFontPath = FPaths::EngineContentDir() / TEXT("SlateDebug/Fonts/LastResort.ttf");
	bIsLastResortFontAvailable = !FPlatformProperties::RequiresCookedData() && FPaths::FileExists(LastResortFontPath);
}

TSharedPtr<const FCompositeFont> FLegacySlateFontInfoCache::GetCompositeFont(const FName& InLegacyFontName, const EFontHinting InLegacyFontHinting)
{
	if (InLegacyFontName.IsNone())
	{
		return nullptr;
	}

	FString LegacyFontPath = InLegacyFontName.ToString();

	// Work out what the given path is supposed to be relative to
	if (!FPaths::FileExists(LegacyFontPath))
	{
		// UMG assets specify the path either relative to the game or engine content directories - test both
		LegacyFontPath = FPaths::ProjectContentDir() / InLegacyFontName.ToString();
		if (!FPaths::FileExists(LegacyFontPath))
		{
			LegacyFontPath = FPaths::EngineContentDir() / InLegacyFontName.ToString();
			if (!FPaths::FileExists(LegacyFontPath))
			{
				// Missing font file - just use what we were given
				LegacyFontPath = InLegacyFontName.ToString();
			}
		}
	}

	const FLegacyFontKey LegacyFontKey(*LegacyFontPath, InLegacyFontHinting);

	{
		TSharedPtr<const FCompositeFont>* const ExistingCompositeFont = LegacyFontNameToCompositeFont.Find(LegacyFontKey);
		if(ExistingCompositeFont)
		{
			return *ExistingCompositeFont;
		}
	}

	TSharedRef<const FCompositeFont> NewCompositeFont = MakeShared<FStandaloneCompositeFont>(NAME_None, LegacyFontPath, InLegacyFontHinting, EFontLoadingPolicy::LazyLoad);
	LegacyFontNameToCompositeFont.Add(LegacyFontKey, NewCompositeFont);
	return NewCompositeFont;
}

TSharedRef<const FCompositeFont> FLegacySlateFontInfoCache::GetDefaultFont()
{
	if (!DefaultFont.IsValid())
	{
		#define APPEND_FONT(TYPEFACE, NAME, FILENAME, HINTING) \
			(TYPEFACE).AppendFont(TEXT(NAME), FPaths::EngineContentDir() / TEXT("Slate") / TEXT("Fonts") / TEXT(FILENAME), HINTING, EFontLoadingPolicy::LazyLoad)

		#define APPEND_EDITOR_FONT(TYPEFACE, NAME, FILENAME, HINTING) \
			(TYPEFACE).AppendFont(TEXT(NAME), FPaths::EngineContentDir() / TEXT("Editor") / TEXT("Slate") / TEXT("Fonts") / TEXT(FILENAME), HINTING, EFontLoadingPolicy::LazyLoad)

		#define APPEND_RANGE(SUBFONT, RANGE) \
			(SUBFONT).CharacterRanges.Add(FUnicodeBlockRange::GetUnicodeBlockRange(EUnicodeBlockRange::RANGE).GetRange())

		TSharedRef<FCompositeFont> MutableDefaultFont = MakeShared<FStandaloneCompositeFont>();

		// Default
		{
			APPEND_FONT(MutableDefaultFont->DefaultTypeface, "Regular", "Roboto-Regular.ttf", EFontHinting::Default);
			APPEND_FONT(MutableDefaultFont->DefaultTypeface, "Italic", "Roboto-Italic.ttf", EFontHinting::Default);
			APPEND_FONT(MutableDefaultFont->DefaultTypeface, "Bold", "Roboto-Bold.ttf", EFontHinting::Default);
			APPEND_FONT(MutableDefaultFont->DefaultTypeface, "BoldItalic", "Roboto-BoldItalic.ttf", EFontHinting::Default);
			APPEND_FONT(MutableDefaultFont->DefaultTypeface, "BoldCondensed", "Roboto-BoldCondensed.ttf", EFontHinting::Default);
			APPEND_FONT(MutableDefaultFont->DefaultTypeface, "BoldCondensedItalic", "Roboto-BoldCondensedItalic.ttf", EFontHinting::Default);
			APPEND_FONT(MutableDefaultFont->DefaultTypeface, "Black", "Roboto-Black.ttf", EFontHinting::Default);
			APPEND_FONT(MutableDefaultFont->DefaultTypeface, "BlackItalic", "Roboto-BlackItalic.ttf", EFontHinting::Default);
			APPEND_FONT(MutableDefaultFont->DefaultTypeface, "Medium", "Roboto-Medium.ttf", EFontHinting::Default);
			APPEND_FONT(MutableDefaultFont->DefaultTypeface, "Light", "Roboto-Light.ttf", EFontHinting::Default);
			APPEND_FONT(MutableDefaultFont->DefaultTypeface, "VeryLight", "Roboto-Light.ttf", EFontHinting::Auto);
			APPEND_FONT(MutableDefaultFont->DefaultTypeface, "Mono", "DroidSansMono.ttf", EFontHinting::Default);
		}

		// Fallback
		{
			APPEND_FONT(MutableDefaultFont->FallbackTypeface.Typeface, "Regular", "DroidSansFallback.ttf", EFontHinting::Default);
		}

		// Explicit fallback for glyphs missing from Roboto.
		{
			FCompositeSubFont& SubFont = MutableDefaultFont->SubTypefaces[MutableDefaultFont->SubTypefaces.AddDefaulted()];
			APPEND_RANGE(SubFont, BoxDrawing);
			APPEND_RANGE(SubFont, BlockElements);
			APPEND_RANGE(SubFont, HalfwidthAndFullwidthForms);
			APPEND_FONT(SubFont.Typeface, "Regular", "DroidSansFallback.ttf", EFontHinting::Default);
		}

		// Arabic
		{
			FCompositeSubFont& SubFont = MutableDefaultFont->SubTypefaces[MutableDefaultFont->SubTypefaces.AddDefaulted()];
			APPEND_RANGE(SubFont, Arabic);
			APPEND_RANGE(SubFont, ArabicExtendedA);
			APPEND_RANGE(SubFont, ArabicMathematicalAlphabeticSymbols);
			APPEND_RANGE(SubFont, ArabicPresentationFormsA);
			APPEND_RANGE(SubFont, ArabicPresentationFormsB);
			APPEND_RANGE(SubFont, ArabicSupplement);
			APPEND_FONT(SubFont.Typeface, "Regular", "NotoNaskhArabicUI-Regular.ttf", EFontHinting::Default);
		}

		// Japanese (editor-only)
		if (GIsEditor)
		{
			FCompositeSubFont& SubFont = MutableDefaultFont->SubTypefaces[MutableDefaultFont->SubTypefaces.AddDefaulted()];
			SubFont.Cultures = TEXT("ja");
			APPEND_RANGE(SubFont, CJKCompatibility);
			APPEND_RANGE(SubFont, CJKCompatibilityForms);
			APPEND_RANGE(SubFont, CJKCompatibilityIdeographs);
			APPEND_RANGE(SubFont, CJKCompatibilityIdeographsSupplement);
			APPEND_RANGE(SubFont, CJKRadicalsSupplement);
			APPEND_RANGE(SubFont, CJKStrokes);
			APPEND_RANGE(SubFont, CJKSymbolsAndPunctuation);
			APPEND_RANGE(SubFont, CJKUnifiedIdeographs);
			APPEND_RANGE(SubFont, CJKUnifiedIdeographsExtensionA);
			APPEND_RANGE(SubFont, CJKUnifiedIdeographsExtensionB);
			APPEND_RANGE(SubFont, CJKUnifiedIdeographsExtensionC);
			APPEND_RANGE(SubFont, CJKUnifiedIdeographsExtensionD);
			APPEND_RANGE(SubFont, CJKUnifiedIdeographsExtensionE);
			APPEND_RANGE(SubFont, EnclosedCJKLettersAndMonths);
			APPEND_RANGE(SubFont, Hiragana);
			APPEND_RANGE(SubFont, Katakana);
			APPEND_RANGE(SubFont, KatakanaPhoneticExtensions);
			APPEND_RANGE(SubFont, Kanbun);
			APPEND_RANGE(SubFont, HalfwidthAndFullwidthForms);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Regular", "GenEiGothicPro-Regular.otf", EFontHinting::AutoLight);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Italic", "GenEiGothicPro-Regular.otf", EFontHinting::AutoLight);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Bold", "GenEiGothicPro-Bold.otf", EFontHinting::AutoLight);
			APPEND_EDITOR_FONT(SubFont.Typeface, "BoldItalic", "GenEiGothicPro-Bold.otf", EFontHinting::AutoLight);
			APPEND_EDITOR_FONT(SubFont.Typeface, "BoldCondensed", "GenEiGothicPro-SemiBold.otf", EFontHinting::AutoLight);
			APPEND_EDITOR_FONT(SubFont.Typeface, "BoldCondensedItalic", "GenEiGothicPro-SemiBold.otf", EFontHinting::AutoLight);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Black", "GenEiGothicPro-Heavy.otf", EFontHinting::AutoLight);
			APPEND_EDITOR_FONT(SubFont.Typeface, "BlackItalic", "GenEiGothicPro-Heavy.otf", EFontHinting::AutoLight);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Light", "GenEiGothicPro-Light.otf", EFontHinting::AutoLight);
			APPEND_EDITOR_FONT(SubFont.Typeface, "VeryLight", "GenEiGothicPro-Light.otf", EFontHinting::AutoLight);
		}

		// Simplified Chinese (editor-only)
		if (GIsEditor)
		{
			FCompositeSubFont& SubFont = MutableDefaultFont->SubTypefaces[MutableDefaultFont->SubTypefaces.AddDefaulted()];
			SubFont.Cultures = TEXT("zh-Hans");
			APPEND_RANGE(SubFont, CJKCompatibility);
			APPEND_RANGE(SubFont, CJKCompatibilityForms);
			APPEND_RANGE(SubFont, CJKCompatibilityIdeographs);
			APPEND_RANGE(SubFont, CJKCompatibilityIdeographsSupplement);
			APPEND_RANGE(SubFont, CJKRadicalsSupplement);
			APPEND_RANGE(SubFont, CJKStrokes);
			APPEND_RANGE(SubFont, CJKSymbolsAndPunctuation);
			APPEND_RANGE(SubFont, CJKUnifiedIdeographs);
			APPEND_RANGE(SubFont, CJKUnifiedIdeographsExtensionA);
			APPEND_RANGE(SubFont, CJKUnifiedIdeographsExtensionB);
			APPEND_RANGE(SubFont, CJKUnifiedIdeographsExtensionC);
			APPEND_RANGE(SubFont, CJKUnifiedIdeographsExtensionD);
			APPEND_RANGE(SubFont, CJKUnifiedIdeographsExtensionE);
			APPEND_RANGE(SubFont, EnclosedCJKLettersAndMonths);
			APPEND_FONT(SubFont.Typeface, "Regular", "DroidSansFallback.ttf", EFontHinting::Default);
			APPEND_FONT(SubFont.Typeface, "Italic", "DroidSansFallback.ttf", EFontHinting::Default);
			APPEND_FONT(SubFont.Typeface, "Bold", "DroidSansFallback.ttf", EFontHinting::Default);
			APPEND_FONT(SubFont.Typeface, "BoldItalic", "DroidSansFallback.ttf", EFontHinting::Default);
			APPEND_FONT(SubFont.Typeface, "BoldCondensed", "DroidSansFallback.ttf", EFontHinting::Default);
			APPEND_FONT(SubFont.Typeface, "BoldCondensedItalic", "DroidSansFallback.ttf", EFontHinting::Default);
			APPEND_FONT(SubFont.Typeface, "Black", "DroidSansFallback.ttf", EFontHinting::Default);
			APPEND_FONT(SubFont.Typeface, "BlackItalic", "DroidSansFallback.ttf", EFontHinting::Default);
			APPEND_FONT(SubFont.Typeface, "Light", "DroidSansFallback.ttf", EFontHinting::Default);
			APPEND_FONT(SubFont.Typeface, "VeryLight", "DroidSansFallback.ttf", EFontHinting::Default);
		}

		// Korean (editor-only)
		if (GIsEditor)
		{
			FCompositeSubFont& SubFont = MutableDefaultFont->SubTypefaces[MutableDefaultFont->SubTypefaces.AddDefaulted()];
			SubFont.Cultures = TEXT("ko");
			APPEND_RANGE(SubFont, HangulJamo);
			APPEND_RANGE(SubFont, HangulJamoExtendedA);
			APPEND_RANGE(SubFont, HangulJamoExtendedB);
			APPEND_RANGE(SubFont, HangulCompatibilityJamo);
			APPEND_RANGE(SubFont, HangulSyllables);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Regular", "NanumGothic.ttf", EFontHinting::Default);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Italic", "NanumGothic.ttf", EFontHinting::Default);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Bold", "NanumGothicBold.ttf", EFontHinting::Default);
			APPEND_EDITOR_FONT(SubFont.Typeface, "BoldItalic", "NanumGothicBold.ttf", EFontHinting::Default);
			APPEND_EDITOR_FONT(SubFont.Typeface, "BoldCondensed", "NanumGothicBold.ttf", EFontHinting::Default);
			APPEND_EDITOR_FONT(SubFont.Typeface, "BoldCondensedItalic", "NanumGothicBold.ttf", EFontHinting::Default);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Black", "NanumGothicExtraBold.ttf", EFontHinting::Default);
			APPEND_EDITOR_FONT(SubFont.Typeface, "BlackItalic", "NanumGothicExtraBold.ttf", EFontHinting::Default);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Light", "NanumGothic.ttf", EFontHinting::Default);
			APPEND_EDITOR_FONT(SubFont.Typeface, "VeryLight", "NanumGothic.ttf", EFontHinting::Default);
		}

		// Devanagari  (editor-only)
		if (GIsEditor)
		{
			FCompositeSubFont& SubFont = MutableDefaultFont->SubTypefaces.AddDefaulted_GetRef();
			APPEND_RANGE(SubFont, Devanagari);
			APPEND_RANGE(SubFont, DevanagariExtended);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Regular", "NotoSansDevanagari-Regular.ttf", EFontHinting::Default);
		}

		// Gurmukhi  (editor-only)
		if (GIsEditor)
		{
			FCompositeSubFont& SubFont = MutableDefaultFont->SubTypefaces.AddDefaulted_GetRef();
			APPEND_RANGE(SubFont, Gurmukhi);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Regular", "NotoSansGurmukhi-Regular.ttf", EFontHinting::Default);
		}

		// Math (editor-only)
		if (GIsEditor)
		{
			FCompositeSubFont& SubFont = MutableDefaultFont->SubTypefaces.AddDefaulted_GetRef();
			APPEND_RANGE(SubFont, MathematicalAlphanumericSymbols);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Regular", "NotoSansMath-Regular.ttf", EFontHinting::Default);
		}

		// Emoji (editor-only)
		if (GIsEditor)
		{
			FCompositeSubFont& SubFont = MutableDefaultFont->SubTypefaces.AddDefaulted_GetRef();
			APPEND_RANGE(SubFont, EmoticonsEmoji);
			APPEND_RANGE(SubFont, MiscellaneousSymbols);
			APPEND_RANGE(SubFont, MiscellaneousSymbolsAndArrows);
			APPEND_RANGE(SubFont, MiscellaneousSymbolsAndPictographs);
			APPEND_RANGE(SubFont, SupplementalSymbolsAndPictographs);
			APPEND_RANGE(SubFont, TransportAndMapSymbols);
			APPEND_RANGE(SubFont, Dingbats);
			APPEND_RANGE(SubFont, EnclosedAlphanumericSupplement);
			APPEND_EDITOR_FONT(SubFont.Typeface, "Regular", "NotoColorEmoji.ttf", EFontHinting::Default);
		}

		#undef APPEND_FONT
		#undef APPEND_EDITOR_FONT
		#undef APPEND_RANGE

		DefaultFont = MutableDefaultFont;
	}

	return DefaultFont.ToSharedRef();
}

TSharedPtr<const FCompositeFont> FLegacySlateFontInfoCache::GetSystemFont()
{
	if (!SystemFont.IsValid())
	{
		const TArray<uint8> FontBytes = FPlatformMisc::GetSystemFontBytes();
		if (FontBytes.Num() > 0)
		{
			const FString FontFilename = FPaths::EngineIntermediateDir() / TEXT("DefaultSystemFont.ttf");
			if (FFileHelper::SaveArrayToFile(FontBytes, *FontFilename))
			{
				SystemFont = MakeShared<FStandaloneCompositeFont>(NAME_None, FontFilename, EFontHinting::Default, EFontLoadingPolicy::LazyLoad);
			}
		}
	}

	return SystemFont;
}

bool FLegacySlateFontInfoCache::IsLastResortFontAvailable() const
{
	return bIsLastResortFontAvailable;
}

TSharedPtr<const FCompositeFont> FLegacySlateFontInfoCache::GetLastResortFont()
{
	// GetLastResortFont may be called from multiple threads at once
	FScopeLock Lock(&LastResortFontCS);

	if (!LastResortFont.IsValid() && bIsLastResortFontAvailable)
	{
		const FFontData& FontData = GetLastResortFontData(FFallbackContext());
		LastResortFont = MakeShared<FStandaloneCompositeFont>(NAME_None, FontData.GetFontFilename(), FontData.GetHinting(), FontData.GetLoadingPolicy());
	}

	return LastResortFont;
}

const FFontData& FLegacySlateFontInfoCache::GetLastResortFontData(const FFallbackContext& InContext)
{
	// GetLastResortFontData is called directly from the font cache, so may be called from multiple threads at once
	FScopeLock Lock(&LastResortFontDataCS);

	if (!LastResortFontData.IsValid())
	{
		LastResortFontData = MakeShared<FFontData>(bIsLastResortFontAvailable ? LastResortFontPath : FString(), EFontHinting::Default, EFontLoadingPolicy::LazyLoad);
		UE_LOGF(LogSlate, Log, "Last resort fallback font was requested. %ls", *InContext.ToString());
	}

	return *LastResortFontData;
}

void FLegacySlateFontInfoCache::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (LocalizedFallbackFontData.IsValid())
	{
		const_cast<FFontData&>(*LocalizedFallbackFontData).AddReferencedObjects(Collector);
	}

	if (LastResortFontData.IsValid())
	{
		const_cast<FFontData&>(*LastResortFontData).AddReferencedObjects(Collector);
	}
}

FString FLegacySlateFontInfoCache::GetReferencerName() const
{
	return TEXT("FLegacySlateFontInfoCache");
}

