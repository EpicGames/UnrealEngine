// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "Fonts/CompositeFont.h"

/**
 * Cache used to efficiently upgrade legacy FSlateFontInfo structs to use a composite font by reducing the amount of duplicate instances that are created
 */
class FLegacySlateFontInfoCache : public FGCObject, public TSharedFromThis<FLegacySlateFontInfoCache>
{
public:
	/**
	 * Context used to help debug font fallback requests
	 */
	struct FFallbackContext
	{
	public:
		FFallbackContext() = default;

		FFallbackContext(const FFontData* InFontData, const UTF32CHAR InCodepoint)
			: FontData(InFontData)
			, Codepoint(InCodepoint)
		{
		}

		FString ToString() const;

	private:
		const FFontData* FontData = nullptr;
		UTF32CHAR Codepoint = 0;
	};

	/**
	 * Get (or create) the singleton instance of this cache
	 */
	static FLegacySlateFontInfoCache& Get();

	/**
	 * Get (or create) an appropriate composite font from the legacy font name
	 */
	TSharedPtr<const FCompositeFont> GetCompositeFont(const FName& InLegacyFontName, const EFontHinting InLegacyFontHinting);

	/**
	 * Get (or create) the default font based on the current build configuration
	 */
	TSharedRef<const FCompositeFont> GetDefaultFont();

	/**
	 * Get (or create) the default system font
	 */
	TSharedPtr<const FCompositeFont> GetSystemFont();

	/**
	 * Is the last resort fallback font available (not all builds have it).
	 */
	bool IsLastResortFontAvailable() const;

	/**
	 * Get (or create) the last resort fallback font
	 */
	TSharedPtr<const FCompositeFont> GetLastResortFont();

	/**
	 * Get (or create) the last resort fallback font
	 */
	const FFontData& GetLastResortFontData(const FFallbackContext& InContext);

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

private:
	FLegacySlateFontInfoCache();

	struct FLegacyFontKey
	{
		FLegacyFontKey()
			: Name()
			, Hinting(EFontHinting::Default)
		{
		}

		FLegacyFontKey(const FName& InName, const EFontHinting InHinting)
			: Name(InName)
			, Hinting(InHinting)
		{
		}

		bool operator==(const FLegacyFontKey& Other) const 
		{
			return Name == Other.Name && Hinting == Other.Hinting;
		}

		bool operator!=(const FLegacyFontKey& Other) const 
		{
			return !(*this == Other);
		}

		friend inline uint32 GetTypeHash(const FLegacyFontKey& Key)
		{
			uint32 Hash = 0;
			Hash = HashCombine(Hash, GetTypeHash(Key.Name));
			Hash = HashCombine(Hash, uint32(Key.Hinting));
			return Hash;
		}

		FName Name;
		EFontHinting Hinting;
	};

	TMap<FLegacyFontKey, TSharedPtr<const FCompositeFont>> LegacyFontNameToCompositeFont;
	TSharedPtr<const FCompositeFont> DefaultFont;
	TSharedPtr<const FCompositeFont> SystemFont;
	TSharedPtr<const FCompositeFont> LastResortFont;
	
	TSharedPtr<const FFontData> LocalizedFallbackFontData;
	TSharedPtr<const FFontData> LastResortFontData;

	FCriticalSection LastResortFontCS;
	FCriticalSection LastResortFontDataCS;

	FString LastResortFontPath;
	bool bIsLastResortFontAvailable;

	static TSharedPtr<FLegacySlateFontInfoCache> Singleton;
};
