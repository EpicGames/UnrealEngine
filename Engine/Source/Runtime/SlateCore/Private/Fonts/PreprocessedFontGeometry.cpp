// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/PreprocessedFontGeometry.h"

#if PLATFORM_MAC
#include "TargetConditionals.h"
#endif

// Only enable Skia-dependent features of MSDFgen for Editor build
#if WITH_EDITOR
#define MSDFGEN_USE_SKIA
#include "ThirdParty/skia/skia-simplify.h"
#endif

#define MSDFGEN_PARENT_NAMESPACE SlateMsdfgen
#if !WITH_FREETYPE
#define MSDFGEN_NO_FREETYPE
#endif
#include "ThirdParty/msdfgen/msdfgen.h"

namespace UE
{
namespace Slate
{

using namespace SlateMsdfgen;

struct FPreprocessedGlyphGeometry::FWindingFingerprint::FContourFingerprint
{
	const msdfgen::EdgeSegment* FirstSegment;
	msdfgen::Point2 FirstPoint;
};

static int16 CompressCoordinate(double Value)
{
	return (int16) FMath::Clamp(FMath::RoundToInt(Value), -0x8000, 0x7fff);
}

static double DecompressCoordinate(int16 CompressedValue)
{
	return (double) CompressedValue;
}

static msdfgen::Point2 DecompressPoint(const int16* Coordinates)
{
	return msdfgen::Point2(DecompressCoordinate(Coordinates[0]), DecompressCoordinate(Coordinates[1]));
}

FPreprocessedGlyphGeometryView::FPreprocessedGlyphGeometryView(
	uint8 InFlags,
	int32 InContourCount,
	const uint8* InContourDataPtr,
	const int16* InCoordinateDataPtr,
	int32 InContourDataLength,
	int32 InCoordinateDataLength
)
	: Flags(InFlags)
	, ContourCount(InContourCount)
	, ContourDataPtr(InContourDataPtr)
	, CoordinateDataPtr(InCoordinateDataPtr)
	, ContourDataLength(InContourDataLength)
	, CoordinateDataLength(InCoordinateDataLength)
{
	uint8 FilteredValue = (Flags & (FPreprocessedGlyphGeometry::FLAG_REVERSE_WINDINGS | FPreprocessedGlyphGeometry::FLAG_CONTOUR_WINDINGS | FPreprocessedGlyphGeometry::FLAG_FULL_GEOMETRY));
	bool bHasSingleBit = FilteredValue != 0 && (FilteredValue & (FilteredValue - 1)) == 0;
	check(FilteredValue == 0 || bHasSingleBit); //Ensure we have no invalid combination of flags (FLAG_REVERSE_WINDINGS, FLAG_CONTOUR_WINDINGS and FLAG_FULL_GEOMETRY are mutually exclusive)
}

bool FPreprocessedGlyphGeometryView::UpdateGeometry(msdfgen::Shape& OutMsdfgenShape) const
{
	const bool bIsFullGeometry = (Flags & FPreprocessedGlyphGeometry::FLAG_FULL_GEOMETRY);
	if (!bIsFullGeometry)
	{
		return false;
	}

	check(!(Flags&(FPreprocessedGlyphGeometry::FLAG_REVERSE_WINDINGS|FPreprocessedGlyphGeometry::FLAG_CONTOUR_WINDINGS)));
	OutMsdfgenShape.inverseYAxis = (Flags&FPreprocessedGlyphGeometry::FLAG_INVERSE_Y_AXIS) != 0;
	OutMsdfgenShape.contours.clear();
	OutMsdfgenShape.contours.resize(ContourCount);
	const uint8* CurEdgePtr = ContourDataPtr;
	const int16* CurCoordinatePtr = CoordinateDataPtr;
	const uint8* const EdgeEndPtr = ContourDataPtr+ContourDataLength;
	const int16* const CoordinateEndPtr = CoordinateDataPtr+CoordinateDataLength;
	for (int32 ContourIndex = 0; ContourIndex < ContourCount; ++ContourIndex)
	{
		const int16* const InitialContourCoordinate = CurCoordinatePtr;
		msdfgen::Contour& OutContour = OutMsdfgenShape.contours[ContourIndex];
		msdfgen::Point2 DummyLastPoint;
		msdfgen::Point2* LastPointPtr = &DummyLastPoint;
		OutContour.edges.clear();
		bool ContourEnd = false;
		while (!ContourEnd)
		{
			checkSlow(CurEdgePtr < EdgeEndPtr);
			// Extract bits 4 through 6 as edge color (see FPreprocessedGlyphGeometry::ContourData description)
			msdfgen::EdgeColor edgeColor = msdfgen::EdgeColor(*CurEdgePtr>>4&0x07u);
			// The two least significant bits represent the number of control points in edge segment (see FPreprocessedGlyphGeometry::ContourData description)
			switch (*CurEdgePtr&0x03u) {
				case 1u:
					{
						checkSlow(CurCoordinatePtr+2 <= CoordinateEndPtr);
						// New msdfgen::LinearSegment will be freed by msdfgen::EdgeHolder which is implicitly constructed in the following statement
						msdfgen::LinearSegment* edgeSegment = new msdfgen::LinearSegment(
							*LastPointPtr = DecompressPoint(CurCoordinatePtr),
							msdfgen::Point2(),
							edgeColor
						);
						OutContour.addEdge(edgeSegment);
						LastPointPtr = &edgeSegment->p[1];
						CurCoordinatePtr += 2;
						break;
					}
				case 2u:
					{
						checkSlow(CurCoordinatePtr+4 <= CoordinateEndPtr);
						// New msdfgen::QuadraticSegment will be freed by msdfgen::EdgeHolder which is implicitly constructed in the following statement
						msdfgen::QuadraticSegment* edgeSegment = new msdfgen::QuadraticSegment(
							*LastPointPtr = DecompressPoint(CurCoordinatePtr),
							DecompressPoint(CurCoordinatePtr + 2),
							msdfgen::Point2(),
							edgeColor
						);
						OutContour.addEdge(edgeSegment);
						LastPointPtr = &edgeSegment->p[2];
						CurCoordinatePtr += 4;
						break;
					}
				case 3u:
					{
						checkSlow(CurCoordinatePtr+6 <= CoordinateEndPtr);
						// New msdfgen::CubicSegment will be freed by msdfgen::EdgeHolder which is implicitly constructed in the following statement
						msdfgen::CubicSegment* edgeSegment = new msdfgen::CubicSegment(
							*LastPointPtr = DecompressPoint(CurCoordinatePtr),
							DecompressPoint(CurCoordinatePtr + 2),
							DecompressPoint(CurCoordinatePtr + 4),
							msdfgen::Point2(),
							edgeColor
						);
						OutContour.addEdge(edgeSegment);
						LastPointPtr = &edgeSegment->p[3];
						CurCoordinatePtr += 6;
						break;
					}
			}
			ContourEnd = (*CurEdgePtr&FPreprocessedGlyphGeometry::FLAG_CONTOUR_END) != 0;
			++CurEdgePtr;
		}
		if (InitialContourCoordinate < CurCoordinatePtr)
		{
			*LastPointPtr = DecompressPoint(InitialContourCoordinate);
		}
	}
	return CurEdgePtr == EdgeEndPtr && CurCoordinatePtr == CoordinateEndPtr;
}

bool FPreprocessedGlyphGeometryView::UpdateWindings(msdfgen::Shape& InOutMsdfgenShape) const
{
	if (Flags&FPreprocessedGlyphGeometry::FLAG_REVERSE_WINDINGS)
	{
		check(!(Flags&(FPreprocessedGlyphGeometry::FLAG_CONTOUR_WINDINGS|FPreprocessedGlyphGeometry::FLAG_FULL_GEOMETRY)));
		for (msdfgen::Contour& Contour : InOutMsdfgenShape.contours)
		{
			Contour.reverse();
		}
		return true;
	}
	if (!((Flags&FPreprocessedGlyphGeometry::FLAG_CONTOUR_WINDINGS) && InOutMsdfgenShape.contours.size() == ContourCount))
	{
		return false;
	}
	check(!(Flags&(FPreprocessedGlyphGeometry::FLAG_REVERSE_WINDINGS|FPreprocessedGlyphGeometry::FLAG_FULL_GEOMETRY)));
	check(ContourDataLength == (ContourCount+7)/8);
	int32 ContourIndex = 0;
	const uint8* CurContourBits = ContourDataPtr;
	const uint8* const ContourBitsEnd = ContourDataPtr+ContourDataLength;
	while (ContourIndex < ContourCount && CurContourBits < ContourBitsEnd)
	{
		for (int32 Bit = 0; Bit < 8 && ContourIndex < ContourCount; ++Bit, ++ContourIndex)
		{
			if (*CurContourBits&1u<<Bit)
			{
				InOutMsdfgenShape.contours[ContourIndex].reverse();
			}
		}
		++CurContourBits;
	}
	check(CurContourBits == ContourBitsEnd);
	return ContourIndex == ContourCount;
}

bool FPreprocessedGlyphGeometryView::HasAllContoursReversedWindings() const
{
	return (Flags&FPreprocessedGlyphGeometry::FLAG_REVERSE_WINDINGS) != 0;
}

FPreprocessedGlyphGeometry::FWindingFingerprint::FWindingFingerprint(const msdfgen::Shape& MsdfgenShape)
{
	Contours.Reserve(MsdfgenShape.contours.size());
	for (const msdfgen::Contour& MsdfgenContour : MsdfgenShape.contours)
	{
		if (MsdfgenContour.edges.empty())
		{
			Contours.Add(FContourFingerprint());
		}
		else
		{
			Contours.Add(FContourFingerprint
				{
					(const msdfgen::EdgeSegment*) MsdfgenContour.edges.front(),
					MsdfgenContour.edges.front()->controlPoints()[1]
				}
			);
		}
	}
}

FPreprocessedGlyphGeometry::FWindingFingerprint::~FWindingFingerprint() = default;

bool FPreprocessedGlyphGeometry::FWindingFingerprint::operator==(const FWindingFingerprint& Other) const
{
	if (Contours.Num() != Other.Contours.Num())
	{
		return false;
	}
	for (int32 Index = 0; Index < Contours.Num(); ++Index)
	{
		if (!(Contours[Index].FirstSegment == Other.Contours[Index].FirstSegment && Contours[Index].FirstPoint == Other.Contours[Index].FirstPoint))
		{
			return false;
		}
	}
	return true;
}

bool FPreprocessedGlyphGeometry::FWindingFingerprint::operator!=(const FWindingFingerprint& Other) const
{
	return !operator==(Other);
}

bool FPreprocessedGlyphGeometry::FWindingFingerprint::Diff(bool& OutAllDiff, TArray<uint8>& OutDiffVector, const FWindingFingerprint &A, const FWindingFingerprint &B)
{
	if (A.Contours.Num() != B.Contours.Num())
	{
		return false;
	}
	OutAllDiff = true;
	OutDiffVector.SetNumZeroed((A.Contours.Num()+7)/8);
	for (int32 Index = 0; Index < A.Contours.Num(); ++Index)
	{
		if (!(A.Contours[Index].FirstSegment == B.Contours[Index].FirstSegment && A.Contours[Index].FirstPoint == B.Contours[Index].FirstPoint))
		{
			OutDiffVector[Index/8] |= 1u<<(Index%8);
		}
		else
		{
			OutAllDiff = false;
		}
	}
	return true;
}

FPreprocessedGlyphGeometry::FGeometryFingerprint::FGeometryFingerprint(const msdfgen::Shape& MsdfgenShape)
{
	Contours.SetNum(MsdfgenShape.contours.size());
	for (size_t ContourIndex = 0; ContourIndex < MsdfgenShape.contours.size(); ++ContourIndex)
	{
		for (const msdfgen::EdgeHolder& MsdfgenEdge : MsdfgenShape.contours[ContourIndex].edges)
		{
			const msdfgen::Point2 MsdfgenPoint = MsdfgenEdge->point(0);
			const TPair<int32, int32> RoundedPoint(
				FMath::RoundToInt32(MsdfgenPoint.x),
				FMath::RoundToInt32(MsdfgenPoint.y)
			);
			bool MultiContourPoint = false;
			for (size_t OtherContourIndex = 0; OtherContourIndex < ContourIndex; ++OtherContourIndex)
			{
				if (Contours[OtherContourIndex].Contains(RoundedPoint))
				{
					Contours[OtherContourIndex].Remove(RoundedPoint);
					MultiContourPoint = true;
					break;
				}
			}
			if (!MultiContourPoint)
			{
				Contours[ContourIndex].Add(RoundedPoint);
			}
		}
	}
}

bool FPreprocessedGlyphGeometry::FGeometryFingerprint::operator==(const FGeometryFingerprint& Other) const
{
	return OneWayMatch(*this, Other) && OneWayMatch(Other, *this);
}

bool FPreprocessedGlyphGeometry::FGeometryFingerprint::operator!=(const FGeometryFingerprint& Other) const
{
	return !operator==(Other);
}

bool FPreprocessedGlyphGeometry::FGeometryFingerprint::OneWayMatch(const FGeometryFingerprint& A, const FGeometryFingerprint& B)
{
	if (A.Contours.Num() != B.Contours.Num())
	{
		return false;
	}
	TArray<bool, TInlineAllocator<64> > UsedBContours;
	UsedBContours.SetNumZeroed(A.Contours.Num());
	for (const FContourVertices& AContour : A.Contours)
	{
		int32 MatchedBContourIndex = -1;
		for (const TPair<int32, int32>& AVertex : AContour)
		{
			bool bVertexMatched = false;
			for (int32 BContourIndex = 0; BContourIndex < B.Contours.Num(); ++BContourIndex)
			{
				if (B.Contours[BContourIndex].Contains(AVertex))
				{
					bVertexMatched = true;
					if (BContourIndex != MatchedBContourIndex)
					{
						if (MatchedBContourIndex < 0)
						{
							if (UsedBContours[BContourIndex])
							{
								return false;
							}
							UsedBContours[BContourIndex] = true;
							MatchedBContourIndex = BContourIndex;
						}
						else
						{
							return false;
						}
					}
				}
			}
			if (!bVertexMatched)
			{
				return false;
			}
		}
	}
	return true;
}

FPreprocessedGlyphGeometry::FPreprocessedGlyphGeometry()
	: Flags(0u)
	, ContourCount(0)
{ }

FPreprocessedGlyphGeometry::FPreprocessedGlyphGeometry(const FWindingFingerprint& WindingFingerprintA, const FWindingFingerprint& WindingFingerprintB)
	: Flags(0u)
	, ContourCount(0)
{
	bool bAllDiff = false;
	if (FWindingFingerprint::Diff(bAllDiff, ContourData, WindingFingerprintA, WindingFingerprintB))
	{
		if (bAllDiff)
		{
			Flags = FLAG_REVERSE_WINDINGS;
			ContourData.Reset();
		}
		else
		{
			Flags = FLAG_CONTOUR_WINDINGS;
			ContourCount = WindingFingerprintA.Contours.Num();
		}
	}
}

FPreprocessedGlyphGeometry::FPreprocessedGlyphGeometry(const msdfgen::Shape& MsdfgenShape)
{
	Flags = FLAG_FULL_GEOMETRY;
	if (MsdfgenShape.inverseYAxis)
	{
		Flags |= FLAG_INVERSE_Y_AXIS;
	}
	ContourCount = (int32) MsdfgenShape.contours.size();
	for (int32 ContourIndex = 0; ContourIndex < ContourCount; ++ContourIndex)
	{
		const msdfgen::Contour& MsdfgenContour = MsdfgenShape.contours[ContourIndex];
		if (!MsdfgenContour.edges.empty())
		{
			for (const msdfgen::EdgeHolder& srcEdge : MsdfgenContour.edges)
			{
				const msdfgen::Point2* const MsdfgenPoints = srcEdge->controlPoints();
				uint8 EdgeFlags = (uint8) ((unsigned) srcEdge->color<<4);
				int16 Coordinates[6];
				switch (srcEdge->type())
				{
					case (int) msdfgen::LinearSegment::EDGE_TYPE:
						Coordinates[0] = CompressCoordinate(MsdfgenPoints[0].x);
						Coordinates[1] = CompressCoordinate(MsdfgenPoints[0].y);
						CoordinateData.Append(Coordinates, 2);
						EdgeFlags |= 1u;
						break;
					case (int) msdfgen::QuadraticSegment::EDGE_TYPE:
						Coordinates[0] = CompressCoordinate(MsdfgenPoints[0].x);
						Coordinates[1] = CompressCoordinate(MsdfgenPoints[0].y);
						Coordinates[2] = CompressCoordinate(MsdfgenPoints[1].x);
						Coordinates[3] = CompressCoordinate(MsdfgenPoints[1].y);
						CoordinateData.Append(Coordinates, 4);
						EdgeFlags |= 2u;
						break;
					case (int) msdfgen::CubicSegment::EDGE_TYPE:
						Coordinates[0] = CompressCoordinate(MsdfgenPoints[0].x);
						Coordinates[1] = CompressCoordinate(MsdfgenPoints[0].y);
						Coordinates[2] = CompressCoordinate(MsdfgenPoints[1].x);
						Coordinates[3] = CompressCoordinate(MsdfgenPoints[1].y);
						Coordinates[4] = CompressCoordinate(MsdfgenPoints[2].x);
						Coordinates[5] = CompressCoordinate(MsdfgenPoints[2].y);
						CoordinateData.Append(Coordinates, 6);
						EdgeFlags |= 3u;
						break;
				}
				ContourData.Add(EdgeFlags);
			}
			ContourData.Last() |= FLAG_CONTOUR_END;
		}
	}
}

FPreprocessedGlyphGeometryView FPreprocessedGlyphGeometry::View() const
{
	return FPreprocessedGlyphGeometryView(
		Flags,
		ContourCount,
		ContourData.GetData(),
		CoordinateData.GetData(),
		ContourData.Num(),
		CoordinateData.Num()
	);
}

void FPreprocessedFontGeometry::AddGlyph(const int32 GlyphIndex, const FPreprocessedGlyphGeometryView& GlyphView)
{
	check(GlyphView.ContourCount >= 0 && GlyphView.ContourDataLength >= 0 && GlyphView.CoordinateDataLength >= 0);
	FGlyphHeader GlyphHeader = { };
	GlyphHeader.Flags = GlyphView.Flags;
	GlyphHeader.ContourCount = GlyphView.ContourCount;
	GlyphHeader.ContourDataStart = ContourData.Num();
	GlyphHeader.ContourDataLength = GlyphView.ContourDataLength;
	GlyphHeader.CoordinateDataStart = CoordinateData.Num();
	GlyphHeader.CoordinateDataLength = GlyphView.CoordinateDataLength;
	ContourData.Append(GlyphView.ContourDataPtr, GlyphView.ContourDataLength);
	CoordinateData.Append(GlyphView.CoordinateDataPtr, GlyphView.CoordinateDataLength);
	Glyphs.Add(GlyphIndex, GlyphHeader);
}

FPreprocessedGlyphGeometryView FPreprocessedFontGeometry::ViewGlyph(const int32 GlyphIndex) const
{
	if (const FGlyphHeader* GlyphHeader = Glyphs.Find(GlyphIndex))
	{
		uint8 GlyphFlags = GlyphHeader->Flags;
		if (GlobalWindingReversal && !(GlyphFlags&~FPreprocessedGlyphGeometry::FLAG_REVERSE_WINDINGS&0xffu))
		{
			GlyphFlags ^= FPreprocessedGlyphGeometry::FLAG_REVERSE_WINDINGS;
		}
		return FPreprocessedGlyphGeometryView(
			GlyphFlags,
			GlyphHeader->ContourCount,
			ContourData.GetData()+GlyphHeader->ContourDataStart,
			CoordinateData.GetData()+GlyphHeader->CoordinateDataStart,
			GlyphHeader->ContourDataLength,
			GlyphHeader->CoordinateDataLength
		);
	}
	return FPreprocessedGlyphGeometryView(
		GlobalWindingReversal ? FPreprocessedGlyphGeometry::FLAG_REVERSE_WINDINGS : 0,
		0,
		nullptr,
		nullptr,
		0,
		0
	);
}

SIZE_T FPreprocessedFontGeometry::GetAllocatedSize() const
{
	return Glyphs.GetAllocatedSize() + ContourData.GetAllocatedSize() + CoordinateData.GetAllocatedSize();
}

SIZE_T FPreprocessedFontGeometry::GetDataSize() const
{
	return sizeof(bool) + (sizeof(int32)+sizeof(FGlyphHeader)) * Glyphs.Num() + ContourData.NumBytes() + CoordinateData.NumBytes();
}

FArchive &operator<<(FArchive &Ar, FPreprocessedFontGeometry &FontGeometry)
{
	return Ar
		<< FontGeometry.GlobalWindingReversal
		<< FontGeometry.Glyphs
		<< FontGeometry.ContourData
		<< FontGeometry.CoordinateData;
}

} // namespace Slate
} // namespace UE
