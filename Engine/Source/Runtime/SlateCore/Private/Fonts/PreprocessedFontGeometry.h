// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Forward declaration of MSDFgen types
namespace SlateMsdfgen
{
namespace msdfgen
{
class Shape;
}
typedef msdfgen::Shape FShape;
}

namespace UE
{
namespace Slate
{

class FPreprocessedFontGeometry;

/** Representation of an FPreprocessedGlyphGeometry object stored elsewhere. */
class FPreprocessedGlyphGeometryView
{

public:
	friend FPreprocessedFontGeometry;

	FPreprocessedGlyphGeometryView()
		: Flags(0)
		, ContourCount(0)
		, ContourDataPtr(nullptr)
		, CoordinateDataPtr(nullptr)
		, ContourDataLength(0)
		, CoordinateDataLength(0)
	{ }

	FPreprocessedGlyphGeometryView(
		uint8 InFlags,
		int32 InContourCount,
		const uint8* InContourDataPtr,
		const int16* InCoordinateDataPtr,
		int32 InContourDataLength,
		int32 InCoordinateDataLength
	);

	/** If full geometry data is available, OutMsdfgenShape will be overwritten with this data. Otherwise, false is returned. */
	bool UpdateGeometry(SlateMsdfgen::FShape& OutMsdfgenShape) const;

	/** If winding data is available and the provided MSDFgen Shape is compatible, its contours will be reversed based on this data. Otherwise, false is returned. */
	bool UpdateWindings(SlateMsdfgen::FShape& InOutMsdfgenShape) const;

	/** Returns true if all contours have reversed windings. */
	bool HasAllContoursReversedWindings() const;

private:
	/** A bitfield of glyph flags (see FLAG_ constants in FPreprocessedGlyphGeometry) */
	uint8 Flags;
	/** The number of contours the glyph has. May be zero if preprocessed data is not present. */
	int32 ContourCount;
	/** Pointer to the beginning of FGlyphGeometryFingerprint::ContourData */
	const uint8* ContourDataPtr;
	/** Pointer to the beginning of FGlyphGeometryFingerprint::CoordinateData */
	const int16* CoordinateDataPtr;
	/** Number of elements of the contour data array */
	int32 ContourDataLength;
	/** Number of elements of the coordinate data array */
	int32 CoordinateDataLength;

};

/** Preprocessed geometry data for a single glyph. */
class FPreprocessedGlyphGeometry
{
public:

	/** Glyph flag which indicates that all contours need to be reversed (no contour data present) */
	static constexpr uint8 FLAG_REVERSE_WINDINGS = 0x01;
	/** Glyph flag which indicates that contour data represents a bit field of whether each contour's winding should be reversed. */
	static constexpr uint8 FLAG_CONTOUR_WINDINGS = 0x02;
	/** Glyph flag which indicates that contour data represents a sequence of edge segments. */
	static constexpr uint8 FLAG_FULL_GEOMETRY = 0x04;
	/** Glyph flag which indicates that the source msdfgen::Shape's inverseYAxis attribute is true. */
	static constexpr uint8 FLAG_INVERSE_Y_AXIS = 0x10;

	/** Edge segment flag which indicates that it is the last segment of the current contour and if another one follows, it initializes the next contour. */
	static constexpr uint8 FLAG_CONTOUR_END = 0x08;

	/**
	 * Captures the windings of the contours of a glyph so that it can be later determined which ones have been reversed.
	 */
	class FWindingFingerprint
	{

	public:
		friend FPreprocessedGlyphGeometry;

		explicit FWindingFingerprint(const SlateMsdfgen::FShape& MsdfgenShape);
		~FWindingFingerprint();

		bool operator==(const FWindingFingerprint& Other) const;
		bool operator!=(const FWindingFingerprint& Other) const;

		/** Compares two fingerprints, outputs the different contours in OutDiffVector, and sets OutAllDiff to whether all are different. Returns false if the fingerprints are incompatible. */
		static bool Diff(bool& OutAllDiff, TArray<uint8>& OutDiffVector, const FWindingFingerprint &A, const FWindingFingerprint &B);

	private:
		struct FContourFingerprint;

		/** The fingerprints of individual contours. */
		TArray<FContourFingerprint> Contours;

	};

	/**
	 * Captures the geometry of a glyph so that it can be determined if contours were split or merged by simplification.
	 */
	class FGeometryFingerprint
	{

	public:
		explicit FGeometryFingerprint(const SlateMsdfgen::FShape& MsdfgenShape);

		bool operator==(const FGeometryFingerprint& Other) const;
		bool operator!=(const FGeometryFingerprint& Other) const;

	private:
		typedef TSet<TPair<int32, int32> > FContourVertices;

		/** Checks that B matches A but not if A matches B. */
		static bool OneWayMatch(const FGeometryFingerprint& A, const FGeometryFingerprint& B);

		/** The vertices of individual contours. */
		TArray<FContourVertices> Contours;

	};

	FPreprocessedGlyphGeometry();
	FPreprocessedGlyphGeometry(const FWindingFingerprint& WindingFingerprintA, const FWindingFingerprint& WindingFingerprintB);
	explicit FPreprocessedGlyphGeometry(const SlateMsdfgen::FShape& MsdfgenShape);

	/** Returns a FPreprocessedGlyphGeometryView object corresponding to this object. */
	FPreprocessedGlyphGeometryView View() const;

private:
	/** A bitfield of glyph flags (see FLAG_ constants above) */
	uint8 Flags;
	/** The number of contours the glyph has. May be zero if preprocessed data is not present. */
	int32 ContourCount;

	/**
	 * If Flags contains FLAG_CONTOUR_WINDINGS, this is a sequence of bits (8 per array element),
	 * indicating which contours should be reversed.
	 * If Flags contains FLAG_FULL_GEOMETRY, this is a sequence of bitfields for each edge segment:
	 * With 0 being the least significant bit and 7 being the most significant bit of each array element,
	 * bits 0 and 1 contain the number of the edge segment's control points
	 *   (1 = linear edge segment, 2 = quadratic Bezier curve, 3 = cubic Bezier curve),
	 * bit 3 may contain FLAG_CONTOUR_END, indicating the end of one contour and start of the next one,
	 * and bits 4, 5, and 6 encode the edge's MSDF color (the red, green, and blue channel).
	 * Bits 2 and 7 are unused.
	 * Flags cannot contain both of the above, that would be an invalid state.
	 * If Flags contain neither, ContourData is empty.
	 */
	TArray<uint8> ContourData;

	/**
	 * If Flags contains FLAG_FULL_GEOMETRY, contains a sequence of coordinates for each edge segment.
	 * The number of coordinates to be read from CoordinateData is dictated by the number of control points
	 * indicated in ContourData. The coordinates are listed in the same order as the edge segments
	 * with the X coordinate always followed by the Y coordinate.
	 * The closing points of contours are not included, so for each contour the initial point must be remembered.
	 */
	TArray<int16> CoordinateData;

};

/**
 * This object represents additional per-glyph font geometry data, which override the base font file data
 * for "unnormalized" glyphs. The original font data are still required and this object serves as their extension.
 * If the font file data change for any reason, this object becomes invalid.
 *
 * A glyph's geometry is normalized when:
 *  - it contains no self-intersections, that is, no part of its edge intersects another part
 *  - and for each edge segment, looking from its initial point towards its next control point,
 *    the filled portion of the glyph is always on the right side of the edge and empty portion is on the left side.
 * 
 * Normalized geometry is required for generating signed distance fields (directly from vector geometry).
 *
 * For glyphs that already satisfy both conditions (are normalized), no data is stored in this object.
 * For glyphs that only satisfy the first condition and contain no false edge segments
 * (ones that do not lie at the glyph's boundary), only a boolean array will be stored,
 * which dictates which of its contours need to be reversed.
 * For other unnormalized glyphs, the entire shape geometry is stored.
 */
class FPreprocessedFontGeometry
{
public:

	/** Adds a single preprocessed glyph, identified by the numeric glyph index. */
	void AddGlyph(const int32 GlyphIndex, const FPreprocessedGlyphGeometryView& GlyphView);

	/**
	 * Returns the view of the glyph identified by the numeric glyph index.
	 * The view is only valid until this FPreprocessedFontGeometry object is modified.
	 * If the glyph is not present in the preprocessed data, a valid object will be returned,
	 * indicating no preprocessing is required.
	 */
	FPreprocessedGlyphGeometryView ViewGlyph(const int32 GlyphIndex) const;

	/** Calling this will cause all glyphs not added to be reported with the FLAG_REVERSE_WINDINGS flag and those added with that flag as clean */
	void SetGlobalWindingReversal(bool InGlobalWindingReversal)
	{
		GlobalWindingReversal = InGlobalWindingReversal;
	}

	/** Returns the object's total allocated size */
	SIZE_T GetAllocatedSize() const;

	/** Returns the object's actual data size */
	SIZE_T GetDataSize() const;

	/** Serializes the font geometry object to/from an archive */
	friend FArchive& operator<<(FArchive& Ar, FPreprocessedFontGeometry& FontGeometry);

private:

	/** Contains fixed-size glyph properties and maps the glyph's data within ContourData and CoordinateData arrays. */
	struct FGlyphHeader
	{
		/** Glyph flags - see constants in FPreprocessedGlyphGeometry */
		uint8 Flags;
		/** Number of glyph's contours */
		int32 ContourCount;
		/** Initial index of the glyph's contour data within the ContourData array */
		int32 ContourDataStart;
		/** Number of elements of the ContourData array for this glyph */
		int32 ContourDataLength;
		/** Initial index of the glyph's coordinates within the CoordinateData array */
		int32 CoordinateDataStart;
		/** Number of elements of the CoordinateData array for this glyph */
		int32 CoordinateDataLength;
	};

	friend FArchive& operator<<(FArchive& Ar, FGlyphHeader& GlyphHeader) {
		return Ar
			<< GlyphHeader.Flags
			<< GlyphHeader.ContourCount
			<< GlyphHeader.ContourDataStart
			<< GlyphHeader.ContourDataLength
			<< GlyphHeader.CoordinateDataStart
			<< GlyphHeader.CoordinateDataLength;
	}

	/** Indicates whether the entire font is encoded with the wrong contour winding.  */
	bool GlobalWindingReversal = false;
	/** Maps glyph headers of the included glyphs to glyph indices. */
	TMap<int32, FGlyphHeader> Glyphs;
	/** Concatenation of FPreprocessedGlyphGeometry::ContourData of all contained glyphs */
	TArray<uint8> ContourData;
	/** Concatenation of FPreprocessedGlyphGeometry::CoordinateData of all contained glyphs */
	TArray<int16> CoordinateData;

};

} // namespace Slate
} // namespace UE
