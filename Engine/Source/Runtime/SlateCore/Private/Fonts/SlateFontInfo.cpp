// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/SlateFontInfo.h"
#include "Templates/Casts.h"
#include "SlateGlobals.h"
#include "Fonts/FontProviderInterface.h"
#include "Fonts/LegacySlateFontInfoCache.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateFontInfo)

#include <limits>

/* FSlateFontInfo structors
 *****************************************************************************/

FFontOutlineSettings FFontOutlineSettings::NoOutline;

#if WITH_EDITORONLY_DATA
bool FFontOutlineSettings::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	// Don't actually serialize, just write the custom version for PostSerialize
	return false;
}

void FFontOutlineSettings::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FontOutlineDropShadowFixup)
	{
		// Set the default for drop shadow outlines to match the legacy behavior that font outlines applied to drop shadows by default
		// New assets will have the new default which is false
		bApplyOutlineToDropShadows = true;
	}
}
#endif

FSlateFontInfo::FSlateFontInfo( )
	: FontObject(nullptr)
	, FontMaterial(nullptr)
	, CompositeFont()
	, TypefaceFontName()
	, Size(24)
	, FontFallback(EFontFallback::FF_Max)
#if WITH_EDITORONLY_DATA
	, Hinting_DEPRECATED(EFontHinting::Default)
#endif
{
}


FSlateFontInfo::FSlateFontInfo( TSharedPtr<const FCompositeFont> InCompositeFont, const float InSize, const FName& InTypefaceFontName, const FFontOutlineSettings& InOutlineSettings )
	: FontObject(nullptr)
	, FontMaterial(nullptr)
	, OutlineSettings(InOutlineSettings)
	, CompositeFont(InCompositeFont)
	, TypefaceFontName(InTypefaceFontName)
	, Size(FMath::Clamp<float>(InSize, 0.f, std::numeric_limits<uint16>::max()))
	, FontFallback(EFontFallback::FF_Max)
#if WITH_EDITORONLY_DATA
	, Hinting_DEPRECATED(EFontHinting::Default)
#endif
{
	ensureMsgf(InSize >= 0.f && InSize <= std::numeric_limits<uint16>::max(), TEXT("The size provided is not supported by the renderer."));
	if (!InCompositeFont.IsValid())
	{
		UE_LOGF(LogSlate, Warning, "FSlateFontInfo was constructed with a null FCompositeFont. Slate will be forced to use the fallback font path which may be slower.");
	}
}


FSlateFontInfo::FSlateFontInfo( const UObject* InFontObject, const float InSize, const FName& InTypefaceFontName, const FFontOutlineSettings& InOutlineSettings )
	: FontObject(InFontObject)
	, FontMaterial(nullptr)
	, OutlineSettings(InOutlineSettings)
	, CompositeFont()
	, TypefaceFontName(InTypefaceFontName)
	, Size(FMath::Clamp<float>(InSize, 0.f, std::numeric_limits<uint16>::max()))
	, FontFallback(EFontFallback::FF_Max)
#if WITH_EDITORONLY_DATA
	, Hinting_DEPRECATED(EFontHinting::Default)
#endif
{
	ensureMsgf(InSize >= 0.f && InSize <= std::numeric_limits<uint16>::max(), TEXT("The size provided is not supported by the renderer."));
	if (InFontObject)
	{
		const IFontProviderInterface* FontProvider = Cast<const IFontProviderInterface>(InFontObject);
		if (!FontProvider || !FontProvider->GetCompositeFont())
		{
			UE_LOGF(LogSlate, Verbose, "'%ls' does not provide a composite font that can be used with Slate. Slate will be forced to use the fallback font path which may be slower.", *InFontObject->GetName());
		}
	}
	else
	{
		UE_LOGF(LogSlate, Warning, "FSlateFontInfo was constructed with a null UFont. Slate will be forced to use the fallback font path which may be slower.");
	}
}


FSlateFontInfo::FSlateFontInfo( const FString& InFontName, float InSize, EFontHinting InHinting, const FFontOutlineSettings& InOutlineSettings)
	: FontObject(nullptr)
	, FontMaterial(nullptr)
	, OutlineSettings(InOutlineSettings)
	, CompositeFont()
	, TypefaceFontName()
	, Size(FMath::Clamp<float>(InSize, 0.f, std::numeric_limits<uint16>::max()))
	, FontFallback(EFontFallback::FF_Max)
#if WITH_EDITORONLY_DATA
	, FontName_DEPRECATED(*InFontName)
	, Hinting_DEPRECATED(InHinting)
#endif
{
	ensureMsgf(InSize >= 0.f && InSize <= std::numeric_limits<uint16>::max(), TEXT("The size provided is not supported by the renderer."));

	//Useful for debugging style breakages
	//check( FPaths::FileExists( FontName.ToString() ) );

	UpgradeLegacyFontInfo(FName(*InFontName), InHinting);
}


FSlateFontInfo::FSlateFontInfo( const FName& InFontName, float InSize, EFontHinting InHinting )
	: FontObject(nullptr)
	, FontMaterial(nullptr)
	, CompositeFont()
	, TypefaceFontName()
	, Size(FMath::Clamp<float>(InSize, 0.f, std::numeric_limits<uint16>::max()))
	, FontFallback(EFontFallback::FF_Max)
#if WITH_EDITORONLY_DATA
	, FontName_DEPRECATED(InFontName)
	, Hinting_DEPRECATED(InHinting)
#endif
{
	ensureMsgf(InSize >= 0.f && InSize <= std::numeric_limits<uint16>::max(), TEXT("The size provided is not supported by the renderer."));

	//Useful for debugging style breakages
	//check( FPaths::FileExists( FontName.ToString() ) );

	UpgradeLegacyFontInfo(InFontName, InHinting);
}


FSlateFontInfo::FSlateFontInfo( const ANSICHAR* InFontName, float InSize, EFontHinting InHinting )
	: FontObject(nullptr)
	, FontMaterial(nullptr)
	, CompositeFont()
	, TypefaceFontName()
	, Size(FMath::Clamp<float>(InSize, 0.f, std::numeric_limits<uint16>::max()))
	, FontFallback(EFontFallback::FF_Max)
#if WITH_EDITORONLY_DATA
	, FontName_DEPRECATED(InFontName)
	, Hinting_DEPRECATED(InHinting)
#endif
{
	ensureMsgf(InSize >= 0.f && InSize <= std::numeric_limits<uint16>::max(), TEXT("The size provided is not supported by the renderer."));

	//Useful for debugging style breakages
	//check( FPaths::FileExists( FontName.ToString() ) );

	UpgradeLegacyFontInfo(FName(InFontName), InHinting);
}


FSlateFontInfo::FSlateFontInfo( const WIDECHAR* InFontName, float InSize, EFontHinting InHinting )
	: FontObject(nullptr)
	, FontMaterial(nullptr)
	, CompositeFont()
	, TypefaceFontName()
	, Size(FMath::Clamp<float>(InSize, 0.f, std::numeric_limits<uint16>::max()))
	, FontFallback(EFontFallback::FF_Max)
#if WITH_EDITORONLY_DATA
	, FontName_DEPRECATED(InFontName)
	, Hinting_DEPRECATED(InHinting)
#endif
{
	ensureMsgf(InSize >= 0.f && InSize <= std::numeric_limits<uint16>::max(), TEXT("The size provided is not supported by the renderer."));

	//Useful for debugging style breakages
	//check( FPaths::FileExists( FontName.ToString() ) );

	UpgradeLegacyFontInfo(FName(InFontName), InHinting);
}


bool FSlateFontInfo::HasValidFont() const
{
	return CompositeFont.IsValid() || FontObject != nullptr;
}


const FCompositeFont* FSlateFontInfo::GetCompositeFont() const
{
	const IFontProviderInterface* FontProvider = Cast<const IFontProviderInterface>(FontObject);
	if (FontProvider)
	{
		const FCompositeFont* const ProvidedCompositeFont = FontProvider->GetCompositeFont();
		return (ProvidedCompositeFont) ? ProvidedCompositeFont : FLegacySlateFontInfoCache::Get().GetLastResortFont().Get();
	}

	if (CompositeFont.IsValid())
	{
		return CompositeFont.Get();
	}

	return FLegacySlateFontInfoCache::Get().GetLastResortFont().Get();
}


#if WITH_EDITORONLY_DATA
void FSlateFontInfo::PostSerialize(const FArchive& Ar)
{
	if (Ar.UEVer() < VER_UE4_SLATE_COMPOSITE_FONTS && !FontObject)
	{
		UpgradeLegacyFontInfo(FontName_DEPRECATED, Hinting_DEPRECATED);
	}
}
#endif

void FSlateFontInfo::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(FontObject);
	Collector.AddReferencedObject(FontMaterial);
	Collector.AddReferencedObject(OutlineSettings.OutlineMaterial);
}

void FSlateFontInfo::UpgradeLegacyFontInfo(FName LegacyFontName, EFontHinting LegacyHinting)
{
	static const FName SpecialName_DefaultSystemFont("DefaultSystemFont");

	// Special case for using the default system font
	CompositeFont = (LegacyFontName == SpecialName_DefaultSystemFont)
		? FLegacySlateFontInfoCache::Get().GetSystemFont()
		: FLegacySlateFontInfoCache::Get().GetCompositeFont(LegacyFontName, LegacyHinting);
}

float FSlateFontInfo::GetClampSize() const
{
	return FMath::Clamp<float>(Size, 0.f, std::numeric_limits<uint16>::max());
}

float FSlateFontInfo::GetClampSkew() const
{
	return FMath::Clamp(SkewAmount, -5.f, 5.f);
}
