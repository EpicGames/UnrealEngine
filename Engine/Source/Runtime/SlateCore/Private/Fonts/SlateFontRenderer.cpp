// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/SlateFontRenderer.h"
#include "Fonts/FontCacheCompositeFont.h"
#include "Fonts/SlateTextShaper.h"
#include "Fonts/LegacySlateFontInfoCache.h"
#include "Fonts/CompositeFont.h"
#include "Fonts/FontProviderInterface.h"
#include "HAL/IConsoleManager.h"
#include "SlateGlobals.h"

#include <limits>
#include "Fonts/FontUtils.h"

DECLARE_CYCLE_STAT(TEXT("Freetype Render Glyph"), STAT_FreetypeRenderGlyph, STATGROUP_Slate);

/** The default miter limit for mitered outlines. The maximum size of a miter spike is this multiplied by the outline size. */
static const float DEFAULT_MITER_LIMIT = 2.f;

/**
 * Method for rendering fonts with the possibility of an outline.
 * 0 = freetype does everything and generates a bitmap for the base glyph. 
 * 1 = We override the freetype rasterizer.  Can help with some rendering anomalies on complicated fonts when freetype generates a wildly different stroke from the base glyph
 * Note: The font cache must be flushed if changing this in the middle of a running instance
 */ 
static int32 OutlineFontRenderMethod = 0;
FAutoConsoleVariableRef CVarOutlineFontRenderMethod(
	TEXT("Slate.OutlineFontRenderMethod"),
	OutlineFontRenderMethod,
	TEXT("Changes the render method for outline fonts.  0 = freetype does everything and generates a bitmap for the base glyph (default).  1 = We override the freetype rasterizer.  Can help with some rendering anomalies on complicated fonts."));

/**
 * Enable or disable anti-aliasing for font rendering.
 * Note: The font cache must be flushed if changing this in the middle of a running instance
 */ 
static int32 EnableFontAntiAliasing = 1;
FAutoConsoleVariableRef CVarEnableFontAntiAliasing(
	TEXT("Slate.EnableFontAntiAliasing"),
	EnableFontAntiAliasing,
	TEXT("Enable or disable anti-aliasing for font rendering (0 = off, 1 = on). Enabled by default."));

/**
 * Enable or disable detection of async loading font faces. This impacts whether or not FTextLayout, FSlateTextLayout etc would be able to detect if font faces are loading asynchronously and respond accordingly.
 * See FTextLayout::AreAnyFontFacesLoading() and FSlateTextLayout::OnPaint() to see how the detection of async loading font faces for a text widget has implications on rendering of text and execution of delegates.
 */
static bool GDetectAsyncLoadingFontFaces = false;
FAutoConsoleVariableRef CVarDetectAsyncLoadingFontFaces(
	TEXT("Slate.Font.DetectAsyncLoadingFontFaces"),
	GDetectAsyncLoadingFontFaces,
	TEXT("If set to true, text widgets will be able to detect if any of its font faces are loading asynchornously and respond accordingly.. This does nothing if Slate.Font.AsyncLazyLoad is set to false."));

namespace SlateFontRendererUtils
{

#if WITH_FREETYPE

void AppendGlyphFlags(const FFreeTypeFace& InFace, const FFontData& InFontData, uint32& InOutGlyphFlags)
{
	// Setup additional glyph flags
	InOutGlyphFlags |= FT_LOAD_COLOR;
	if (FT_IS_SCALABLE(InFace.GetFace()))
	{
		InOutGlyphFlags |= FT_LOAD_NO_BITMAP;
	}

	if (EnableFontAntiAliasing)
	{
		switch (InFontData.GetHinting())
		{
		case EFontHinting::Auto:		InOutGlyphFlags |= FT_LOAD_FORCE_AUTOHINT; break;
		case EFontHinting::AutoLight:	InOutGlyphFlags |= FT_LOAD_TARGET_LIGHT; break;
		case EFontHinting::Monochrome:	InOutGlyphFlags |= FT_LOAD_TARGET_MONO | FT_LOAD_FORCE_AUTOHINT; break;
		case EFontHinting::None:		InOutGlyphFlags |= FT_LOAD_NO_AUTOHINT | FT_LOAD_NO_HINTING; break;
		case EFontHinting::Default:
		default:						InOutGlyphFlags |= FT_LOAD_TARGET_NORMAL; break; //-V616
		}
	}
	else
	{
		InOutGlyphFlags |= FT_LOAD_TARGET_MONO | FT_LOAD_FORCE_AUTOHINT;
	}
}

#endif // WITH_FREETYPE

template <int32 BytesPerPixel>
void ResizeFontBitmap_SampleInputPixel(const TArray<uint8>& SrcData, const int32 PixelIndex, FVector4& OutStepColor);

template <>
void ResizeFontBitmap_SampleInputPixel<1>(const TArray<uint8>& SrcData, const int32 PixelIndex, FVector4& OutStepColor)
{
	OutStepColor.X += (float)SrcData[PixelIndex];
}

template <>
void ResizeFontBitmap_SampleInputPixel<4>(const TArray<uint8>& SrcData, const int32 PixelIndex, FVector4& OutStepColor)
{
	const int32 SrcDataIndex = PixelIndex * 4;
	OutStepColor.X += (float)SrcData[SrcDataIndex+0];
	OutStepColor.Y += (float)SrcData[SrcDataIndex+1];
	OutStepColor.Z += (float)SrcData[SrcDataIndex+2];
	OutStepColor.W += (float)SrcData[SrcDataIndex+3];
}

template <int32 BytesPerPixel>
void ResizeFontBitmap_StoreAverageOutputPixel(TArray<uint8>& DstData, const int32 PixelIndex, const FVector4& StepColor, const int32 PixelCount);

template <>
void ResizeFontBitmap_StoreAverageOutputPixel<1>(TArray<uint8>& DstData, const int32 PixelIndex, const FVector4& StepColor, const int32 PixelCount)
{
	ensure(PixelCount > 0);
	DstData[PixelIndex] = FMath::Clamp<uint8>((uint8)(StepColor.X / (float)PixelCount), 0, 255);
}

template <>
void ResizeFontBitmap_StoreAverageOutputPixel<4>(TArray<uint8>& DstData, const int32 PixelIndex, const FVector4& StepColor, const int32 PixelCount)
{
	ensure(PixelCount > 0);
	const int32 DstDataIndex = PixelIndex * 4;
	DstData[DstDataIndex + 0] = FMath::Clamp<uint8>((uint8)(StepColor.X / (float)PixelCount), 0, 255);
	DstData[DstDataIndex + 1] = FMath::Clamp<uint8>((uint8)(StepColor.Y / (float)PixelCount), 0, 255);
	DstData[DstDataIndex + 2] = FMath::Clamp<uint8>((uint8)(StepColor.Z / (float)PixelCount), 0, 255);
	DstData[DstDataIndex + 3] = FMath::Clamp<uint8>((uint8)(StepColor.W / (float)PixelCount), 0, 255);
}

// Note: Copied from FImageUtils and updated to work efficiently with 1 or 4-bytes per-pixel, and include the alpha channel (which the FImageUtils version just clobbers)
template <int32 BytesPerPixel>
void ResizeFontBitmap(const int32 SrcWidth, const int32 SrcHeight, const TArray<uint8>& SrcData, const int32 DstWidth, const int32 DstHeight, TArray<uint8>& DstData)
{
	static_assert(BytesPerPixel == 1 || BytesPerPixel == 4, "BytesPerPixel must be 1 or 4!");

	check(SrcData.Num() >= SrcWidth * SrcHeight * BytesPerPixel);
	DstData.SetNumZeroed(DstWidth * DstHeight * BytesPerPixel);

	float SrcX = 0;
	float SrcY = 0;

	const float StepSizeX = SrcWidth / (float)DstWidth;
	const float StepSizeY = SrcHeight / (float)DstHeight;

	for (int32 Y = 0; Y < DstHeight; ++Y)
	{
		int32 PixelPos = Y * DstWidth;
		SrcX = 0.0f;

		for (int32 X = 0; X < DstWidth; ++X)
		{
			float EndX = SrcX + StepSizeX;
			float EndY = SrcY + StepSizeY;

			// Generate a rectangular region of pixels and then find the average color of the region.
			int32 PosY = FMath::TruncToInt(SrcY + 0.5f);
			PosY = FMath::Clamp<int32>(PosY, 0, (SrcHeight - 1));

			int32 PosX = FMath::TruncToInt(SrcX + 0.5f);
			PosX = FMath::Clamp<int32>(PosX, 0, (SrcWidth - 1));

			int32 EndPosY = FMath::TruncToInt(EndY + 0.5f);
			EndPosY = FMath::Clamp<int32>(EndPosY, 0, (SrcHeight - 1));

			int32 EndPosX = FMath::TruncToInt(EndX + 0.5f);
			EndPosX = FMath::Clamp<int32>(EndPosX, 0, (SrcWidth - 1));

			int32 PixelCount = 0;
			FVector4 StepColor(0, 0, 0, 0);
			for (int32 PixelX = PosX; PixelX <= EndPosX; ++PixelX)
			{
				for (int32 PixelY = PosY; PixelY <= EndPosY; ++PixelY)
				{
					int32 StartPixel = PixelX + PixelY * SrcWidth;
					ResizeFontBitmap_SampleInputPixel<BytesPerPixel>(SrcData, StartPixel, StepColor);
					++PixelCount;
				}
			}

			// Store the final averaged pixel color value.
			ResizeFontBitmap_StoreAverageOutputPixel<BytesPerPixel>(DstData, PixelPos, StepColor, PixelCount);

			SrcX = EndX;
			++PixelPos;
		}

		SrcY += StepSizeY;
	}
}

} // namespace SlateFontRendererUtils


FSlateFontRenderer::FSlateFontRenderer(const FFreeTypeLibrary* InFTLibrary, FFreeTypeCacheDirectory* InFTCacheDirectory, FCompositeFontCache* InCompositeFontCache)
	: FTLibrary(InFTLibrary)
	, FTCacheDirectory(InFTCacheDirectory)
	, CompositeFontCache(InCompositeFontCache)
{
	check(FTLibrary);
	check(FTCacheDirectory);
	check(CompositeFontCache);
}

uint16 FSlateFontRenderer::GetMaxHeight(const FSlateFontInfo& InFontInfo, const float InScale) const
{
#if WITH_FREETYPE
	// Just get info for the null character 
	TCHAR Char = 0;
	const FFontData& FontData = CompositeFontCache->GetDefaultFontData(InFontInfo);
	const FFreeTypeFaceGlyphData FaceGlyphData = GetFontFaceForCodepoint(FontData, Char, InFontInfo.FontFallback);

	if (FaceGlyphData.FaceAndMemory.IsValid() && FaceGlyphData.FaceAndMemory->IsFaceValid() && !FaceGlyphData.FaceAndMemory->IsFaceLoading())
	{
		FreeTypeUtils::ApplySizeAndScale(FaceGlyphData.FaceAndMemory->GetFace(), InFontInfo.Size, InScale);

		// Adjust the height by the size of the outline that was applied.  
		const float HeightAdjustment = InFontInfo.OutlineSettings.OutlineSize * InScale;
		return static_cast<uint16>(FreeTypeUtils::Convert26Dot6ToRoundedPixel<int32>(FaceGlyphData.FaceAndMemory->GetScaledHeight(UE::Slate::FontUtils::IsAscentDescentOverrideEnabled(InFontInfo.FontObject))) + HeightAdjustment); 
	}

	return 0;
#else
	return 0;
#endif // WITH_FREETYPE
}

int16 FSlateFontRenderer::GetBaseline(const FSlateFontInfo& InFontInfo, const float InScale) const
{
#if WITH_FREETYPE
	// Just get info for the null character 
	TCHAR Char = 0;
	const FFontData& FontData = CompositeFontCache->GetDefaultFontData(InFontInfo);
	const FFreeTypeFaceGlyphData FaceGlyphData = GetFontFaceForCodepoint(FontData, Char, InFontInfo.FontFallback);

	if (FaceGlyphData.FaceAndMemory.IsValid() && FaceGlyphData.FaceAndMemory->IsFaceValid() && !FaceGlyphData.FaceAndMemory->IsFaceLoading())
	{
		FreeTypeUtils::ApplySizeAndScale(FaceGlyphData.FaceAndMemory->GetFace(), InFontInfo.Size, InScale);

		return FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>(FaceGlyphData.FaceAndMemory->GetDescender(UE::Slate::FontUtils::IsAscentDescentOverrideEnabled(InFontInfo.FontObject)));
	}

	return 0;
#else
	return 0;
#endif // WITH_FREETYPE
}

void FSlateFontRenderer::GetUnderlineMetrics(const FSlateFontInfo& InFontInfo, const float InScale, int16& OutUnderlinePos, int16& OutUnderlineThickness) const
{
#if WITH_FREETYPE
	const FFontData& FontData = CompositeFontCache->GetDefaultFontData(InFontInfo);
	FT_Face FontFace = GetFontFace(FontData);

	if (FontFace && FT_IS_SCALABLE(FontFace))
	{
		FreeTypeUtils::ApplySizeAndScale(FontFace, InFontInfo.Size, InScale);

		OutUnderlinePos = FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>(FT_MulFix(FontFace->underline_position, FontFace->size->metrics.y_scale));
		OutUnderlineThickness = FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>(FT_MulFix(FontFace->underline_thickness, FontFace->size->metrics.y_scale));
	}
	else
#endif // WITH_FREETYPE
	{
		OutUnderlinePos = 0;
		OutUnderlineThickness = 0;
	}
}

void FSlateFontRenderer::GetStrikeMetrics(const FSlateFontInfo& InFontInfo, const float InScale, int16& OutStrikeLinePos, int16& OutStrikeLineThickness) const
{
#if WITH_FREETYPE
	const FFontData& FontData = CompositeFontCache->GetDefaultFontData(InFontInfo);
	FT_Face FontFace = GetFontFace(FontData);

	if (FontFace && FT_IS_SCALABLE(FontFace))
	{
		FreeTypeUtils::ApplySizeAndScale(FontFace, InFontInfo.Size, InScale);

		// Place the strike at the given percent of the font's full height (the default is 60% from the bottom).
		OutStrikeLinePos = FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>(FT_MulFix(FT_DivFix(FT_MulFix(FreeTypeUtils::GetHeight(FontFace, FontData.GetLayoutMethod()), FontData.GetStrikeBrushHeightPercentage()), 100), FontFace->size->metrics.y_scale));
		OutStrikeLineThickness = FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>(FT_MulFix(FontFace->underline_thickness, FontFace->size->metrics.y_scale));
	}
	else
#endif // WITH_FREETYPE
	{
		OutStrikeLinePos = 0;
		OutStrikeLineThickness = 0;
	}
}

bool FSlateFontRenderer::HasKerning(const FFontData& InFontData) const
{
#if WITH_FREETYPE
	FT_Face FontFace = GetFontFace(InFontData);

	if (!FontFace)
	{
		return false;
	}

	return FT_HAS_KERNING(FontFace) != 0;
#else
	return false;
#endif // WITH_FREETYPE
}

int8 FSlateFontRenderer::GetKerning(const FFontData& InFontData, const float InSize, TCHAR First, TCHAR Second, const float InScale) const
{
#if WITH_FREETYPE
	FT_Face FontFace = GetFontFace(InFontData);

	// Check if this font has kerning as not all fonts do.
	// We also can't perform kerning between two separate font faces
	if (FontFace && FT_HAS_KERNING(FontFace))
	{
		const FT_UInt FirstIndex = FT_Get_Char_Index(FontFace, First);
		const FT_UInt SecondIndex = FT_Get_Char_Index(FontFace, Second);
		const uint32 FontRenderSize = FreeTypeUtils::ComputeFontPixelSize(InSize, InScale);
		TSharedPtr<FFreeTypeKerningCache> KerningCache = GetKerningCache(InFontData, FontRenderSize);
		if (KerningCache)
		{
			FT_Vector KerningVector;
			if (KerningCache->FindOrCache(FirstIndex, SecondIndex, KerningVector))
			{
				// Return pixel sizes
				return FreeTypeUtils::Convert26Dot6ToRoundedPixel<int8>(KerningVector.x);
			}
		}
	}
#endif // WITH_FREETYPE
	return 0;
}

TSharedPtr<FFreeTypeKerningCache> FSlateFontRenderer::GetKerningCache(const FFontData& InFontData, const uint32 FontRenderSize) const
{
#if WITH_FREETYPE
	FT_Face FontFace = GetFontFace(InFontData);
	return FTCacheDirectory->GetKerningCache(FontFace, FT_KERNING_DEFAULT, FontRenderSize);
#else
	return nullptr;
#endif // WITH_FREETYPE
}

bool FSlateFontRenderer::CanLoadCodepoint(const FFontData& InFontData, const UTF32CHAR InCodepoint, EFontFallback MaxFallbackLevel) const
{
	bool bReturnVal = false;

#if WITH_FREETYPE
	const FFreeTypeFaceGlyphData FaceGlyphData = GetFontFaceForCodepoint(InFontData, InCodepoint, MaxFallbackLevel);
	bReturnVal = FaceGlyphData.FaceAndMemory.IsValid() && !FaceGlyphData.FaceAndMemory->IsFaceLoading() && FaceGlyphData.GlyphIndex != 0;
#endif // WITH_FREETYPE

	return bReturnVal;
}

#if WITH_FREETYPE

FFreeTypeFaceGlyphData FSlateFontRenderer::GetFontFaceForCodepoint(const FFontData& InFontData, const UTF32CHAR InCodepoint, EFontFallback MaxFallbackLevel) const
{
	FFreeTypeFaceGlyphData ReturnVal;
	const bool bOverrideFallback = InCodepoint == SlateFontRendererUtils::InvalidSubChar;

	// Try the requested font first
	ReturnVal.FaceAndMemory = CompositeFontCache->GetFontFace(InFontData);

	if (ReturnVal.FaceAndMemory.IsValid())
	{
		// If we have valid face and memory but it just hasn't finished loading,
		// return like we found it, so that we don't immediately trigger falling back to yet another font
		// when it may have the glyph once it's actually done loading.
		// If we want to detect and handle still loading font faces, we avoid clearing the face in memory so text widgets are able to react accordingly when font faces are detected to still be loading.
		if (ReturnVal.FaceAndMemory->IsFaceLoading())
		{
			// This would prevent the detection of fonts that are still loading while text is being passed through the shaping pass
			// By clearing the face in memory, there is no way for text widgets to detect which font faces are loading and thus react accordingly  
			if (!GDetectAsyncLoadingFontFaces)
			{
				ReturnVal.FaceAndMemory.Reset();
			}
			ReturnVal.GlyphIndex = 0;
			ReturnVal.CharFallbackLevel = EFontFallback::FF_NoFallback;
			return ReturnVal;
		}
		else
		{
			ReturnVal.GlyphIndex = FT_Get_Char_Index(ReturnVal.FaceAndMemory->GetFace(), InCodepoint);
			ReturnVal.CharFallbackLevel = EFontFallback::FF_NoFallback;
		}
	}

	// If the requested glyph doesn't exist, use the last resort fallback font
	if (!ReturnVal.FaceAndMemory.IsValid() || (InCodepoint != 0 && ReturnVal.GlyphIndex == 0))
	{
		const bool bCanFallback = bOverrideFallback || MaxFallbackLevel >= EFontFallback::FF_LastResortFallback;

		if (bCanFallback && FLegacySlateFontInfoCache::Get().IsLastResortFontAvailable())
		{
			UE_LOGF(LogSlate, Warning, "Could not find Glyph Index %u with codepoint U+%x, getting last resort font data %ls", ReturnVal.GlyphIndex, InCodepoint, *InFontData.GetFontFilename());
			ReturnVal.FaceAndMemory = CompositeFontCache->GetFontFace(FLegacySlateFontInfoCache::Get().GetLastResortFontData(FLegacySlateFontInfoCache::FFallbackContext(&InFontData, InCodepoint)));

			if (ReturnVal.FaceAndMemory.IsValid())
			{
				ReturnVal.GlyphIndex = FT_Get_Char_Index(ReturnVal.FaceAndMemory->GetFace(), InCodepoint);

				if (ReturnVal.GlyphIndex != 0)
				{
					ReturnVal.CharFallbackLevel = EFontFallback::FF_LastResortFallback;
					ReturnVal.GlyphFlags |= FT_LOAD_FORCE_AUTOHINT;
				}
			}
		}
	}

	// Found an invalid glyph?
	if (InCodepoint != 0 && ReturnVal.GlyphIndex == 0)
	{
		ReturnVal.FaceAndMemory.Reset();
	}

	return ReturnVal;
}

#endif // WITH_FREETYPE

bool FSlateFontRenderer::GetRenderData(const FShapedGlyphEntry& InShapedGlyph, const FFontOutlineSettings& InOutlineSettings, FCharacterRenderData& OutRenderData) const
{
#if WITH_FREETYPE

#if	WITH_VERY_VERBOSE_SLATE_STATS
	SCOPE_CYCLE_COUNTER(STAT_FreetypeRenderGlyph);
#endif

	FFreeTypeFaceGlyphData FaceGlyphData;
	FaceGlyphData.FaceAndMemory = InShapedGlyph.FontFaceData->FontFace.Pin();
	FaceGlyphData.GlyphIndex = InShapedGlyph.GlyphIndex;
	FaceGlyphData.GlyphFlags = InShapedGlyph.FontFaceData->GlyphFlags;

	if (FaceGlyphData.FaceAndMemory.IsValid())
	{
		ensure(FaceGlyphData.FaceAndMemory->IsFaceValid());

		if (FaceGlyphData.FaceAndMemory->IsFaceValid())
		{
			if (FMath::IsNearlyEqual(InShapedGlyph.FontFaceData->FontSkew, 0.f)) 
			{
				FT_Set_Transform(FaceGlyphData.FaceAndMemory->GetFace(), nullptr, nullptr);
			}
			else 
			{
				// Skewing / Fake Italics (could do character rotation in future?).
				FT_Matrix TransformMatrix;
				TransformMatrix.xx = 0x10000L;
				TransformMatrix.xy = InShapedGlyph.FontFaceData->FontSkew * 0x10000L;
				TransformMatrix.yx = 0;
				TransformMatrix.yy = 0x10000L;
				FT_Set_Transform(FaceGlyphData.FaceAndMemory->GetFace(), &TransformMatrix, nullptr);
			}
			FT_Error Error = FreeTypeUtils::LoadGlyph(FaceGlyphData.FaceAndMemory->GetFace(), FaceGlyphData.GlyphIndex, FaceGlyphData.GlyphFlags, InShapedGlyph.FontFaceData->FontSize, InShapedGlyph.FontFaceData->FontScale);
			if (Error == 0)
			{
				return GetRenderDataInternal(FaceGlyphData, InShapedGlyph.FontFaceData->FontScale, InOutlineSettings, OutRenderData);
			}
		}
	}
#endif // WITH_FREETYPE
	return false;
}

#if WITH_FREETYPE

/**
 * Represents one or more pixels of a rasterized glyph that have the same coverage (filled amount)
 */
struct FRasterizerSpan
{
	/** Start x location of the span */
	int32 X;
	/** Start y location of the span  */
	int32 Y;
	
	/** Length of the span */
	int32 Width;
	
	/** How "filled" the span is where 0 is completely transparent and 255 is completely opaque */
	uint8 Coverage;

	FRasterizerSpan(int32 InX, int32 InY, int32 InWidth, uint8 InCoverage)
		: X(InX)
		, Y(InY)
		, Width(InWidth)
		, Coverage(InCoverage)
	{}
};

/**
 * Represents a single rasterized glyph
 */
struct FRasterizerSpanList
{
	/** List of spans in the glyph */
	TArray<FRasterizerSpan> Spans;
	/** Bounds around the glyph */
	FBox2D BoundingBox;

	FRasterizerSpanList()
		: BoundingBox(ForceInit)
	{}
};


/**
 * Rasterizes a font glyph
 */
void RenderOutlineRows(FT_Library Library, FT_Outline* Outline, FRasterizerSpanList& OutSpansList)
{
	auto RasterizerCallback = [](const int Y, const int Count, const FT_Span* const Spans, void* const UserData)
	{
		FRasterizerSpanList& UserDataSpanList = *static_cast<FRasterizerSpanList*>(UserData);

		TArray<FRasterizerSpan>& UserDataSpans = UserDataSpanList.Spans;
		FBox2D& BoundingBox = UserDataSpanList.BoundingBox;

		UserDataSpans.Reserve(UserDataSpans.Num() + Count);
		for(int32 SpanIndex = 0; SpanIndex < Count; ++SpanIndex)
		{
			const FT_Span& Span = Spans[SpanIndex];

			BoundingBox += FVector2D((float)Span.x, (float)Y);
			BoundingBox += FVector2D((float)(Span.x + Span.len - 1), (float)Y);

			UserDataSpans.Add(FRasterizerSpan(Span.x, Y, Span.len, Span.coverage));
		}
	};

	FT_Raster_Params RasterParams;
	FMemory::Memzero<FT_Raster_Params>(RasterParams);
	RasterParams.flags = FT_RASTER_FLAG_AA | FT_RASTER_FLAG_DIRECT | FT_RASTER_FLAG_CLIP;
	RasterParams.gray_spans = RasterizerCallback;
	RasterParams.user = &OutSpansList;

	// Bound clipping to the outline control bounds, truncated to integer pixels
	FT_Outline_Get_CBox(Outline, &RasterParams.clip_box);
	RasterParams.clip_box.xMin = RasterParams.clip_box.xMin >> 6;
	RasterParams.clip_box.yMin = RasterParams.clip_box.yMin >> 6;
	RasterParams.clip_box.xMax = (RasterParams.clip_box.xMax + 63) >> 6;
	RasterParams.clip_box.yMax = (RasterParams.clip_box.yMax + 63) >> 6;

	FT_Outline_Render(Library, Outline, &RasterParams);
}

bool FSlateFontRenderer::GetRenderDataInternal(const FFreeTypeFaceGlyphData& InFaceGlyphData, const float InScale, const FFontOutlineSettings& InOutlineSettings, FCharacterRenderData& OutRenderData) const
{
	SCOPE_CYCLE_COUNTER(STAT_FreetypeRenderGlyph);

	bool bIsMonochromeRendered = !EnableFontAntiAliasing;

	FT_Face Face = InFaceGlyphData.FaceAndMemory->GetFace();

	// Get the lot for the glyph.  This contains measurement info
	FT_GlyphSlot Slot = Face->glyph;

	const float BitmapAtlasScale = FreeTypeUtils::GetBitmapAtlasScale(Face);
	float ScaledOutlineSize = FMath::RoundToFloat(InOutlineSettings.OutlineSize * InScale);

	OutRenderData.ContentType = ESlateFontAtlasContentType::Alpha;
	OutRenderData.bSupportsOutline = FT_IS_SCALABLE(Face);

	int32 OutlineXOffset = 0;
	int32 OutlineYOffset = 0;

	if((ScaledOutlineSize > 0.0f || OutlineFontRenderMethod == 1) && Slot->format == FT_GLYPH_FORMAT_OUTLINE)
	{
		// Render the filled area first
		FRasterizerSpanList FillSpans;
		RenderOutlineRows(FTLibrary->GetLibrary(), &Slot->outline, FillSpans);

		FRasterizerSpanList OutlineSpans;
		
		// If there is an outline, render it second after applying a border stroke to the font to produce an outline
		if(ScaledOutlineSize > 0)
		{
			const FT_Stroker_LineCap LineCap = InOutlineSettings.bMiteredCorners ? FT_STROKER_LINECAP_SQUARE : FT_STROKER_LINECAP_ROUND;
			const FT_Stroker_LineJoin LineJoin = InOutlineSettings.bMiteredCorners ? FT_STROKER_LINEJOIN_MITER : FT_STROKER_LINEJOIN_ROUND;
			const FT_Fixed MiterLimit = InOutlineSettings.bMiteredCorners ? FMath::TruncToInt(FreeTypeUtils::ConvertPixelTo16Dot16<float>(DEFAULT_MITER_LIMIT)) : 0;

			FT_Stroker Stroker = nullptr;
			FT_Stroker_New(FTLibrary->GetLibrary(), &Stroker);
			FT_Stroker_Set(Stroker, FMath::TruncToInt(FreeTypeUtils::ConvertPixelTo26Dot6<float>(ScaledOutlineSize)), LineCap, LineJoin, MiterLimit);

			FT_Glyph Glyph = nullptr;
			FT_Get_Glyph(Slot, &Glyph);

			FT_Bool bInner = false;
			FT_Bool bDestroy = true; // FT_Glyph_StrokeBorder may reassign Glyph, so have it destroy the original if that happens to avoid leaking the glyph made via FT_Get_Glyph above
			FT_Glyph_StrokeBorder(&Glyph, Stroker, bInner, bDestroy);

			FT_Outline* Outline = &reinterpret_cast<FT_OutlineGlyph>(Glyph)->outline;

			RenderOutlineRows(FTLibrary->GetLibrary(), Outline, OutlineSpans);

			FT_Stroker_Done(Stroker);
			FT_Done_Glyph(Glyph);
		}

		const FBox2D BoundingBox = FillSpans.BoundingBox + OutlineSpans.BoundingBox;

		FVector2D Size = BoundingBox.GetSize();

		//Note: we add 1 to width and height because the size of the rect is inclusive
		const int32 Width = FMath::TruncToInt(Size.X)+1;
		const int32 Height = FMath::TruncToInt(Size.Y)+1;
		ensure(Width <= std::numeric_limits<int16>::max());
		ensure(Height <= std::numeric_limits<int16>::max());
		OutRenderData.SizeX = (int16)Width;
		OutRenderData.SizeY = (int16)Height;

		OutRenderData.RawPixels.Reset();
		OutRenderData.RawPixels.AddZeroed(Width*Height);

		const int32 XMin = FMath::TruncToInt(BoundingBox.Min.X);
		const int32 YMin = FMath::TruncToInt(BoundingBox.Min.Y);
		OutlineXOffset = FMath::RoundToInt32(BoundingBox.Min.X-FillSpans.BoundingBox.Min.X);
		OutlineYOffset = FMath::RoundToInt32(BoundingBox.Max.Y-FillSpans.BoundingBox.Max.Y);

		// Compute and copy the pixels for the total filled area of the glyph. 

		// Copy the outline area first
		const TArray<FRasterizerSpan>& FirstSpanList = OutlineSpans.Spans;
		{
			for(const FRasterizerSpan& Span : FirstSpanList)
			{
				for(int32 W = 0; W < Span.Width; ++W)
				{
					OutRenderData.RawPixels[((int32)((Height - 1 - (Span.Y - YMin)) * Width + Span.X - XMin + W))] = Span.Coverage;

				}
			}
		}

		// If there is an outline, it was rasterized by freetype with the filled area included
		// This code will eliminate the filled area if the user requests an outline with separate translucency for the fill area
		const TArray<FRasterizerSpan>& SecondSpanList = FillSpans.Spans;
		{
			if(ScaledOutlineSize > 0)
			{
				for (const FRasterizerSpan& Span : SecondSpanList)
				{
					for (int32 W = 0; W < Span.Width; ++W)
					{
						uint8& Src = OutRenderData.RawPixels[((int32)((Height - 1 - (Span.Y - YMin)) * Width + Span.X - XMin + W))];

						if(InOutlineSettings.bSeparateFillAlpha)
						{
							// this method is better for transparent fill areas
							Src = Span.Coverage ? FMath::Abs<uint8>(Src - Span.Coverage) : 0;
						}
						else
						{
							// This method is better for opaque fill areas 
							Src = Span.Coverage == 255 ? Span.Coverage : Src;
						}
					}
				}
			}
			else
			{
				for (const FRasterizerSpan& Span : SecondSpanList)
				{
					for (int32 w = 0; w < Span.Width; ++w)
					{
						OutRenderData.RawPixels[((int32)((Height - 1 - (Span.Y - YMin)) * Width + Span.X - XMin + w))] = Span.Coverage;
					}
				}
			}
		}

		// Note: in order to render the stroke properly AND to get proper measurements this must be done after rendering the stroke
		FT_Error Error = FT_Render_Glyph(Slot, bIsMonochromeRendered ? FT_RENDER_MODE_MONO : FT_RENDER_MODE_NORMAL);
		if (Error != 0)
		{
			UE_LOGF(LogSlate, Warning, "FreeType error %x while trying to render glyph index %u from face %ls %ls (from outline path)", Error, Slot->glyph_index, UTF8_TO_TCHAR(Face->family_name), UTF8_TO_TCHAR(Face->style_name));
			return false;
		}
	}
	else
	{
		// This path renders a standard font with no outline.
		FT_Error Error = FT_Render_Glyph(Slot, bIsMonochromeRendered ? FT_RENDER_MODE_MONO : FT_RENDER_MODE_NORMAL);
		if (Error != 0)
		{
			UE_LOGF(LogSlate, Warning, "FreeType error %x while trying to render glyph index %u from face %ls %ls (from standard path)", Error, Slot->glyph_index, UTF8_TO_TCHAR(Face->family_name), UTF8_TO_TCHAR(Face->style_name));
			return false;
		}

		FT_Bitmap* Bitmap = nullptr;
		FT_Bitmap TmpBitmap;
// Available from FreeType2-2.10.0 onwards 
		if (Slot->bitmap.pixel_mode == FT_PIXEL_MODE_BGRA)
		{
			OutRenderData.ContentType = ESlateFontAtlasContentType::Color;
			Bitmap = &Slot->bitmap;
		}
		else if (Slot->bitmap.pixel_mode != FT_PIXEL_MODE_GRAY)
		{
			// Convert the bitmap to 8bpp grayscale
			FT_Bitmap_New(&TmpBitmap);
			FT_Bitmap_Convert(FTLibrary->GetLibrary(), &Slot->bitmap, &TmpBitmap, 4);
			Bitmap = &TmpBitmap;
		}
		else
		{
			Bitmap = &Slot->bitmap;
		}

		check(Bitmap && ((Bitmap->pixel_mode == FT_PIXEL_MODE_GRAY && OutRenderData.ContentType == ESlateFontAtlasContentType::Alpha) || (Slot->bitmap.pixel_mode == FT_PIXEL_MODE_BGRA && OutRenderData.ContentType == ESlateFontAtlasContentType::Color)));

		const uint32 BytesPerPixel = GetSlateFontAtlasContentBytesPerPixel(OutRenderData.ContentType);
		OutRenderData.RawPixels.Reset();
		OutRenderData.RawPixels.AddUninitialized(Bitmap->rows * Bitmap->width * BytesPerPixel);

		// Nothing to do for zero width or height glyphs
		if(OutRenderData.RawPixels.Num())
		{
			// Copy the rendered bitmap to our raw pixels array
			for(uint32 Row = 0; Row < (uint32)Bitmap->rows; ++Row)
			{
				// Copy a single row. Note Bitmap.pitch contains the offset (in bytes) between rows.  Not always equal to Bitmap.width!
				FMemory::Memcpy(&OutRenderData.RawPixels[Row*Bitmap->width*BytesPerPixel], &Bitmap->buffer[Row*Bitmap->pitch], Bitmap->width*BytesPerPixel);
			}

			// Grayscale images not using 256 colors need to convert their gray range to a 0-255 range
			if(OutRenderData.ContentType == ESlateFontAtlasContentType::Alpha && Bitmap->num_grays != 256)
			{
				const int32 GrayBoost = 255 / (Bitmap->num_grays - 1);
				for(uint8& RawPixel : OutRenderData.RawPixels)
				{
					RawPixel *= GrayBoost;
				}
			}
		}

		OutRenderData.SizeX = (int16)Bitmap->width;
		OutRenderData.SizeY = (int16)Bitmap->rows;

		if(Bitmap == &TmpBitmap)
		{
			FT_Bitmap_Done(FTLibrary->GetLibrary(), Bitmap);
			Bitmap = nullptr;
		}

		// Resize the generated bitmap if required
		if (BitmapAtlasScale < 1.0f)
		{
			const int32 ScaledWidth = FMath::TruncToInt(OutRenderData.SizeX * BitmapAtlasScale);
			const int32 ScaledHeight = FMath::TruncToInt(OutRenderData.SizeY * BitmapAtlasScale);
			ensure(ScaledWidth <= std::numeric_limits<int16>::max());
			ensure(ScaledHeight <= std::numeric_limits<int16>::max());

			TArray<uint8> ScaledRawPixels;
			if (OutRenderData.ContentType == ESlateFontAtlasContentType::Alpha)
			{
				check(BytesPerPixel == 1);
				SlateFontRendererUtils::ResizeFontBitmap<1>(OutRenderData.SizeX, OutRenderData.SizeY, OutRenderData.RawPixels, ScaledWidth, ScaledHeight, ScaledRawPixels);
			}
			else
			{
				check(BytesPerPixel == 4);
				SlateFontRendererUtils::ResizeFontBitmap<4>(OutRenderData.SizeX, OutRenderData.SizeY, OutRenderData.RawPixels, ScaledWidth, ScaledHeight, ScaledRawPixels);
			}

			OutRenderData.SizeX = (int16)ScaledWidth;
			OutRenderData.SizeY = (int16)ScaledHeight;
			OutRenderData.RawPixels = MoveTemp(ScaledRawPixels);
		}

		// Reset the outline to zero.  If we are in this path, either the outline failed to generate because the font supported or there is no outline
		// We do not want to take it into account if it failed to generate
		ScaledOutlineSize = 0.0f;
		OutlineXOffset = 0;
		OutlineYOffset = 0;
	}

	const int32 HorizontalOffset = FMath::RoundToInt((Slot->bitmap_left + OutlineXOffset) * BitmapAtlasScale);
	const int32 VerticalOffset = FMath::TruncToInt((Slot->bitmap_top + OutlineYOffset) * BitmapAtlasScale);
	ensure(HorizontalOffset <= std::numeric_limits<int16>::max());
	ensure(VerticalOffset <= std::numeric_limits<int16>::max());
	OutRenderData.HorizontalOffset = (int16)HorizontalOffset;
	OutRenderData.VerticalOffset = (int16)VerticalOffset;

	return true;
}

FT_Face FSlateFontRenderer::GetFontFace(const FFontData& InFontData) const
{
	TSharedPtr<FFreeTypeFace> FaceAndMemory = CompositeFontCache->GetFontFace(InFontData);
	return (FaceAndMemory.IsValid() && !FaceAndMemory->IsFaceLoading()) ? FaceAndMemory->GetFace() : nullptr;
}

#endif // WITH_FREETYPE
