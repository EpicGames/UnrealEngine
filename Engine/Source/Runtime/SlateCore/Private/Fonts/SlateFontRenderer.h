// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Fonts/FontCacheFreeType.h"

class FCompositeFontCache;

/**
 * Internal struct for passing around information about loading a glyph 
 */
struct FFreeTypeFaceGlyphData
{
	/** The font face for the character */
	TSharedPtr<FFreeTypeFace> FaceAndMemory;

	/** The glyph index for the character */
	uint32 GlyphIndex;

	/** The glyph flags that should be used for loading the characters glyph */
	uint32 GlyphFlags;

	/** The fallback font set the character was loaded from */
	EFontFallback CharFallbackLevel;

	FFreeTypeFaceGlyphData()
		: FaceAndMemory()
		, GlyphIndex(0)
		, GlyphFlags(0)
		, CharFallbackLevel(EFontFallback::FF_NoFallback)
	{
	}
};

namespace SlateFontRendererUtils
{

/** Character used to substitute invalid font characters */
inline const TCHAR InvalidSubChar = TEXT('\uFFFD');

#if WITH_FREETYPE

/** Append the flags needed by the given font data to the given flags variable */
void AppendGlyphFlags(const FFreeTypeFace& InFace, const FFontData& InFontData, uint32& InOutGlyphFlags);

#endif // WITH_FREETYPE

} // namespace SlateFontRendererUtils


/**
 * Bridging point between FreeType and the Slate font system.
 * This class, via the instances you pass to its constructor, knows how to correctly render a Slate font.
 */
class FSlateFontRenderer
{
public:
	FSlateFontRenderer(const FFreeTypeLibrary* InFTLibrary, FFreeTypeCacheDirectory* InFTCacheDirectory, FCompositeFontCache* InCompositeFontCache);

	/**
	 * @return The global max height for any character in the default font
	 */
	uint16 GetMaxHeight(const FSlateFontInfo& InFontInfo, const float InScale) const;

	/** 
	 * @return the baseline for any character in the default font
	 */
	int16 GetBaseline(const FSlateFontInfo& InFontInfo, const float InScale) const;

	/**
	 * Get the underline metrics used by any character in the default font
	 *
	 * @param OutUnderlinePos		The offset from the baseline to the center of the underline bar
	 * @param OutUnderlineThickness	The thickness of the underline bar
	 */
	void GetUnderlineMetrics(const FSlateFontInfo& InFontInfo, const float InScale, int16& OutUnderlinePos, int16& OutUnderlineThickness) const;

	/**
	 * Get the strike metrics used by any character in the default font
	 *
	 * @param OutStrikeLinePos		The offset from the baseline to the center of the strike bar
	 * @param OutStrikeLineThickness The thickness of the strike bar
	 */
	void GetStrikeMetrics(const FSlateFontInfo& InFontInfo, const float InScale, int16& OutStrikeLinePos, int16& OutStrikeLineThickness) const;

	/**
	 * @param Whether or not the font has kerning
	 */
	bool HasKerning(const FFontData& InFontData) const;

	/**
	 * Calculates the kerning amount for a pair of characters
	 *
	 * @param InFontData	The font that used to draw the string with the first and second characters
	 * @param InSize		The size of the font to draw
	 * @param First			The first character in the pair
	 * @param Second		The second character in the pair
	 * @param InScale		The scale at which the font is being drawn
	 * @return The kerning amount, 0 if no kerning
	 */
	int8 GetKerning(const FFontData& InFontData, const float InSize, TCHAR First, TCHAR Second, const float InScale) const;

	/**
	 * Retrieves the kerning cache object used to retrieve kerning amounts between glyphs
	 *
	 * @param InFontData		The font that used to draw the string with the first and second characters
	 * @param InFontRenderSize	The final size of the font to render (including scaling)
	 * @return A pointer object referring to the kerning cache matching the given parameters, invalid/null if no kerning is performed
	 */
	TSharedPtr<FFreeTypeKerningCache> GetKerningCache(const FFontData& InFontData, const uint32 InFontRenderSize) const;

	/**
	 * Whether or not the specified character, within the specified font, can be loaded with the specified maximum font fallback level
	 *
	 * @param InFontData		Information about the font to load
	 * @param InCodepoint		The codepoint being loaded
	 * @param MaxFallbackLevel	The maximum fallback level to try for the font
	 * @return					Whether or not the character can be loaded
	 */
	bool CanLoadCodepoint(const FFontData& InFontData, const UTF32CHAR InCodepoint, EFontFallback MaxFallbackLevel) const;

#if WITH_FREETYPE
	/**
	 * Wrapper for GetFontFace, which reverts to fallback or last resort fonts if the face could not be loaded
	 *
	 * @param InFontData		Information about the font to load
	 * @param InCodepoint		The codepoint being loaded (required for checking if a fallback font is needed)
	 * @param MaxFallbackLevel	The maximum fallback level to try for the font
	 * @return					Returns the character font face data
	 */
	FFreeTypeFaceGlyphData GetFontFaceForCodepoint(const FFontData& InFontData, const UTF32CHAR InCodepoint, EFontFallback MaxFallbackLevel) const;
#endif // WITH_FREETYPE

	/** 
	 * Creates render data for a specific character 
	 * 
	 * @param InFontData		Raw font data to render the character with
	 * @param InSize			The size of the font to draw
	 * @param InOutlineSettings	Outline settings to apply when rendering the characer
	 * @param Char				The character to render
	 * @param OutRenderData		Will contain the created render data
	 * @param InScale			The scale of the font
	 * @param OutFallbackLevel	Outputs the fallback level of the font
	 */
	bool GetRenderData(const FShapedGlyphEntry& InShapedGlyph, const FFontOutlineSettings& InOutlineSettings, FCharacterRenderData& OutRenderData) const;

private:
#if WITH_FREETYPE
	/**
	 * Internal, shared implementation of GetRenderData
	 */
	bool GetRenderDataInternal(const FFreeTypeFaceGlyphData& InFaceGlyphData, const float InScale, const FFontOutlineSettings& InOutlineSettings, FCharacterRenderData& OutRenderData) const;

	/** 
	 * Gets or loads a FreeType font face 
	 */
	FT_Face GetFontFace(const FFontData& InFontData) const;
#endif // WITH_FREETYPE

	const FFreeTypeLibrary* FTLibrary;
	FFreeTypeCacheDirectory* FTCacheDirectory;
	FCompositeFontCache* CompositeFontCache;
};
