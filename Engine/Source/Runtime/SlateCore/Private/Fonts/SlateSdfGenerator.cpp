// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateSdfGenerator.h"
#include "Async/AsyncWork.h"
#include "Math/Vector2D.h"
#include "Math/IntPoint.h"
#include "Containers/MpscQueue.h"
#include "Containers/StaticArray.h"
#include "HAL/IConsoleManager.h"
#include "SlateFontRenderer.h"
#include "PreprocessedFontGeometry.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Stats/StatsMisc.h"

// This option will log the time it took to generate SDF for each glyph
//#define SLATE_LOG_SDF_GENERATOR_TIME 1

#if PLATFORM_MAC
#include "TargetConditionals.h"
#endif

// Only enable Skia-dependent features of MSDFgen for Editor build
#if WITH_EDITOR
#define MSDFGEN_USE_SKIA
// Include Windows.h before Skia so that its unnecessary defines are cleaned up and Skia's include will be ignored
#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif
#include "ThirdParty/skia/skia-simplify.h"
// Including source file directly, as seen in dlmalloc third-party dependency
#include "ThirdParty/skia/skia-simplify.cpp"
#endif // WITH_EDITOR

#define MSDFGEN_PARENT_NAMESPACE SlateMsdfgen
#if !WITH_FREETYPE
#define MSDFGEN_NO_FREETYPE
#endif
#include "ThirdParty/msdfgen/msdfgen.cpp"

CSV_DECLARE_CATEGORY_MODULE_EXTERN(SLATECORE_API, Slate);

#define MAX_GLYPH_SDF_SIDE 4096

static TAutoConsoleVariable<int32> CVarSlateSdfTextGeneratorPoolSize(TEXT("SlateSdfText.GeneratorPoolSize"), 1, TEXT("Sets the maximum number of concurrent tasks when generating multi-channel distance fields for Slate text glyphs"));

// Corners with an angle greater than 3 radians (~171 degrees) won't be treated as corners.
static constexpr double SDF_CORNER_ANGLE_THRESHOLD = 3.0;

// When a corner's angle tends towards zero, the size of its miter tends toward infinity. The miter limit filters extreme cases from being included in bounds.
static constexpr double SDF_BOUNDS_MITER_LIMIT = 1.0;

namespace SdfUtils
{

using namespace SlateMsdfgen;

/**
 * Specifies how the glyph's shape geometry will be mapped into the distance field's pixel coordinate system
 * and how the distance values are mapped to the 0 - 255 range.
 */
class FGlyphSdfMapping
{
public:
	/** Computes planar mapping for the minimum pixel-aligned bounding box for the glyph and the outside portion of its spread (non-mitered), origin stays at pixel boundaries */
	void WrapNonMitered(
		const msdfgen::Shape::Bounds& InMsdfgenBounds,
		const uint16 InUnitsPerEm,
		const int32 InPpem,
		const float InEmOuterSpread)
	{
		Wrap(nullptr, InMsdfgenBounds, InUnitsPerEm, InPpem, InEmOuterSpread, 0.0);
	}

	/** Computes planar mapping for the minimum pixel-aligned bounding box for the glyph and the outside portion of its spread (mitered), origin stays at pixel boundaries */
	void WrapMitered(const msdfgen::Shape& MsdfgenShape,
		const msdfgen::Shape::Bounds& InMsdfgenBounds,
		const uint16 InUnitsPerEm,
		const int32 InPpem,
		const float InEmOuterSpread,
		const double InMiterLimit)
	{
		Wrap(&MsdfgenShape, InMsdfgenBounds, InUnitsPerEm, InPpem, InEmOuterSpread, InMiterLimit);
	}

	void SetSpread(
		const uint16 InUnitsPerEm,
		const float InEmOuterSpread,
		const float InEmInnerSpread);

	const msdfgen::SDFTransformation& GetMsdfgenTransformation() const
	{
		return MsdfgenTransformation;
	}

	int32 GetSdfWidth() const
	{
		return SdfBounds.Width();
	}
	int32 GetSdfHeight() const
	{
		return SdfBounds.Height();
	}

	int32 GetBearingX() const
	{
		return SdfBounds.Min.X;
	}
	int32 GetBearingY() const
	{
		return SdfBounds.Max.Y;
	}

private:
	msdfgen::SDFTransformation MsdfgenTransformation;
	FIntRect SdfBounds;

	void Wrap(const msdfgen::Shape* MsdfgenShape,
		msdfgen::Shape::Bounds MsdfgenBounds,
		const uint16 InUnitsPerEm,
		const int32 InPpem,
		const float InEmOuterSpread,
		const double InMiterLimit);

};

void FGlyphSdfMapping::Wrap(const msdfgen::Shape* MsdfgenShape,
	msdfgen::Shape::Bounds MsdfgenBounds,
	const uint16 InUnitsPerEm,
	const int32 InPpem,
	const float InEmOuterSpread,
	const double InMiterLimit)
{
	const float UnitsPerEm = static_cast<float>(InUnitsPerEm);
	const float UnitScale = static_cast<float>(InPpem)/UnitsPerEm;

	// Add outer portion of spread to bounds
	const float MsdfgenOuterSpread = UnitsPerEm*InEmOuterSpread;
	MsdfgenBounds.l -= MsdfgenOuterSpread;
	MsdfgenBounds.b -= MsdfgenOuterSpread;
	MsdfgenBounds.r += MsdfgenOuterSpread;
	MsdfgenBounds.t += MsdfgenOuterSpread;

	// Find mitered bounds including outer portion of spread
	if (MsdfgenShape && InMiterLimit > 0)
	{
		MsdfgenShape->boundMiters(MsdfgenBounds.l, MsdfgenBounds.b, MsdfgenBounds.r, MsdfgenBounds.t, MsdfgenOuterSpread, InMiterLimit, 1);
	}

	// Convert to pixel bounds
	MsdfgenBounds.l *= UnitScale;
	MsdfgenBounds.b *= UnitScale;
	MsdfgenBounds.r *= UnitScale;
	MsdfgenBounds.t *= UnitScale;

	// Add 0.5 px to pixel bounds to make sure that spread does not extend beyond edge pixel centers
	MsdfgenBounds.l -= 0.5;
	MsdfgenBounds.b -= 0.5;
	MsdfgenBounds.r += 0.5;
	MsdfgenBounds.t += 0.5;

	// Normalize bounds if they are empty
	if (MsdfgenBounds.l > MsdfgenBounds.r)
	{
		MsdfgenBounds.l = 0;
		MsdfgenBounds.r = 0;
	}
	if (MsdfgenBounds.b > MsdfgenBounds.t)
	{
		MsdfgenBounds.b = 0;
		MsdfgenBounds.t = 0;
	}

	// Enlarge pixel bounds to nearest whole number
	SdfBounds.Min.X = FMath::FloorToInt32(MsdfgenBounds.l);
	SdfBounds.Min.Y = FMath::FloorToInt32(MsdfgenBounds.b);
	SdfBounds.Max.X = FMath::CeilToInt32(MsdfgenBounds.r);
	SdfBounds.Max.Y = FMath::CeilToInt32(MsdfgenBounds.t);

	MsdfgenTransformation = msdfgen::SDFTransformation(msdfgen::Projection(
		msdfgen::Vector2(UnitScale),
		msdfgen::Vector2(-SdfBounds.Min.X, -SdfBounds.Min.Y)/UnitScale),
		msdfgen::Range(2)
	);
}

void FGlyphSdfMapping::SetSpread(
	const uint16 InUnitsPerEm,
	const float InEmOuterSpread,
	const float InEmInnerSpread)
{
	const float UnitsPerEm = static_cast<float>(InUnitsPerEm);
	MsdfgenTransformation.distanceMapping = msdfgen::Range(-UnitsPerEm*InEmOuterSpread, UnitsPerEm*InEmInnerSpread);
}

/*
*
*/

#if WITH_FREETYPE

class FFreeTypeShapeBuilder
{
public:
	FFreeTypeShapeBuilder();

	bool Build(TSharedPtr<FFreeTypeFace> InFace,
		uint32 InGlyphIndex,
		float InEmOuterSpread,
		float InEmInnerSpread,
		int32 InPpem,
		msdfgen::Shape& OutMsdfgenShape,
		FGlyphSdfMapping& OutGlyphSdfMapping);

};

FFreeTypeShapeBuilder::FFreeTypeShapeBuilder()
{

}

bool FFreeTypeShapeBuilder::Build(TSharedPtr<FFreeTypeFace> InFace,
	uint32 InGlyphIndex, 
	float InEmOuterSpread,
	float InEmInnerSpread,
	int32 InPpem,
	msdfgen::Shape& OutMsdfgenShape,
	FGlyphSdfMapping& OutGlyphSdfMapping)
{
	if (!FreeTypeUtils::IsFaceEligibleForSdf(InFace->GetFace())) {
		return false;
	}

	UE::Slate::FPreprocessedGlyphGeometryView PreprocessedGlyphGeometry;
	if (const UE::Slate::FPreprocessedFontGeometry* PreprocessedFontGeometry = InFace->GetPreprocessedFontGeometry())
	{
		PreprocessedGlyphGeometry = PreprocessedFontGeometry->ViewGlyph(InGlyphIndex);
	}

	// Preprocessed glyph geometry may contain the full shape, only load it from FreeType if it does not
	if (!PreprocessedGlyphGeometry.UpdateGeometry(OutMsdfgenShape))
	{
		FT_Error Error = ::FT_Load_Glyph(InFace->GetFace(), 
										 InGlyphIndex, 
										 FT_LOAD_NO_SCALE
										 | FT_LOAD_IGNORE_TRANSFORM
										 | FT_LOAD_NO_HINTING
										 | FT_LOAD_NO_AUTOHINT
										 | FT_LOAD_NO_BITMAP);
		if (Error != 0)
		{
			return false;
		}
		if (!FreeTypeUtils::IsGlyphEligibleForSdf(InFace->GetFace()->glyph)
			|| InFace->GetFace()->glyph->metrics.width <= 0
			|| InFace->GetFace()->glyph->metrics.height <= 0
			|| InFace->GetFace()->glyph->outline.n_points <= 0)
		{
			return false;
		}

		Error = msdfgen::readFreetypeOutline(OutMsdfgenShape, &InFace->GetFace()->glyph->outline, 1);
		if (Error != 0 || OutMsdfgenShape.contours.empty())
		{
			return false;
		}

		// Preprocessed glyph data may also contain information about reversed contour windings. If not, this will have no effect.
		PreprocessedGlyphGeometry.UpdateWindings(OutMsdfgenShape);
	}

	// MSDFgen uses bottom-up Y coordinates but UE uses top-down.
	OutMsdfgenShape.inverseYAxis = !OutMsdfgenShape.inverseYAxis;

	// Find shape's geometry tight bounding box
	msdfgen::Shape::Bounds Bounds = OutMsdfgenShape.getBounds();

	// Check if shape is inverted and fix it
	msdfgen::Point2 OuterPoint(Bounds.l-(Bounds.r-Bounds.l)-1, Bounds.b-(Bounds.t-Bounds.b)-1);
	if (msdfgen::SimpleTrueShapeDistanceFinder::oneShotDistance(OutMsdfgenShape, OuterPoint) > 0)
	{
		for (msdfgen::Contour& Contour : OutMsdfgenShape.contours)
			Contour.reverse();
	}

	const uint16 UnitsPerEm = InFace->GetFace()->units_per_EM;
	// Compute glyph placement within the distance field bitmap
	OutGlyphSdfMapping.WrapMitered(OutMsdfgenShape, Bounds, UnitsPerEm, InPpem, InEmOuterSpread, SDF_BOUNDS_MITER_LIMIT);
	// Compute distance conversion to 8-bit color channel value
	OutGlyphSdfMapping.SetSpread(UnitsPerEm, InEmOuterSpread, InEmInnerSpread);

	if (!(OutGlyphSdfMapping.GetSdfWidth() > 0 && OutGlyphSdfMapping.GetSdfHeight() > 0 && OutGlyphSdfMapping.GetSdfWidth() < MAX_GLYPH_SDF_SIDE && OutGlyphSdfMapping.GetSdfHeight() < MAX_GLYPH_SDF_SIDE))
	{
		return false;
	}

	return true;
}

#endif // WITH_FREETYPE

/**
 * Performs the construction of multi-channel signed distance field for a single glyph.
 */
class alignas(PLATFORM_CACHE_LINE_SIZE) FSdfGeneratorTask
{
	FSlateSdfGenerator::FRequestDescriptor Descriptor;
	TArray<uint8> OutputPixels;
	msdfgen::Shape MsdfgenShape;
	FGlyphSdfMapping GlyphSdfMapping;
	#if SLATE_CSV_TRACKER
	FString DebugName;
	#endif

public:
	FSdfGeneratorTask()
	{
	}

	TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FSdfGeneratorTask, STATGROUP_ThreadPoolAsyncTasks);
	}

	void DoWork();
	bool CanAbandon()
	{
		return true;
	}
	void Abandon()
	{
	}

	void DoOutlineDecomposition();

	FSlateSdfGenerator::ERequestResponse Prepare(const FSlateSdfGenerator::FRequestDescriptor& InDescriptor, FSlateSdfGenerator::FRequestOutputInfo& OutOutputInfo);
	void End(const FSlateSdfGenerator::FForEachRequestDoneCallback& Callback);
	void Reset();

	void MakePlaceholder(TArray<uint8>& OutRawPixels) const;

};

static int32 GetNumSdfChannels(FSlateSdfGenerator::ESdfType InSdfType)
{
	switch (InSdfType)
	{
		case FSlateSdfGenerator::ESdfType::Simple:
		case FSlateSdfGenerator::ESdfType::Perpendicular:
			return 1;
		case FSlateSdfGenerator::ESdfType::MultichannelAndSimple:
			return 4;
		default:
			checkNoEntry();
	}
	return 0;
}

FSlateSdfGenerator::ERequestResponse FSdfGeneratorTask::Prepare(const FSlateSdfGenerator::FRequestDescriptor& InDescriptor, FSlateSdfGenerator::FRequestOutputInfo& OutOutputInfo)
{
	#if WITH_FREETYPE
	TSharedPtr<FFreeTypeFace> FontFace = InDescriptor.FontFace.Pin();
	if (FontFace && FontFace->IsFaceLoading())
	{
		return FSlateSdfGenerator::ERequestResponse::BUSY;
	}
	if (FontFace && FontFace->IsFaceValid())
	{
		if (SdfUtils::FFreeTypeShapeBuilder().Build(FontFace,
			InDescriptor.GlyphIndex,
			InDescriptor.EmOuterSpread,
			InDescriptor.EmInnerSpread,
			InDescriptor.Ppem,
			MsdfgenShape,
			GlyphSdfMapping))
		{
			Descriptor = InDescriptor;

			OutOutputInfo.ImageWidth = GlyphSdfMapping.GetSdfWidth();
			OutOutputInfo.ImageHeight = GlyphSdfMapping.GetSdfHeight();
			OutOutputInfo.BearingX = GlyphSdfMapping.GetBearingX();
			OutOutputInfo.BearingY = GlyphSdfMapping.GetBearingY();

			return FSlateSdfGenerator::ERequestResponse::SUCCESS;
		}
	}
	#endif
	return FSlateSdfGenerator::ERequestResponse::SDF_UNAVAILABLE;
}

void FSdfGeneratorTask::End(const FSlateSdfGenerator::FForEachRequestDoneCallback& Callback)
{
	Callback(Descriptor, MoveTemp(OutputPixels));
	Descriptor = {};
	MsdfgenShape.contours.clear();
}

void FSdfGeneratorTask::Reset()
{
	Descriptor = {};
	MsdfgenShape.contours.clear();
	OutputPixels.Empty();
}

void FSdfGeneratorTask::DoWork()
{
	#ifdef SLATE_LOG_SDF_GENERATOR_TIME
	SCOPE_LOG_TIME("FSdfGeneratorTask", NULL);
	#endif
	DoOutlineDecomposition();
}

void FSdfGeneratorTask::DoOutlineDecomposition()
{
	const bool OverlappedContourSupport = true;
	const int32 TargetWidth = GlyphSdfMapping.GetSdfWidth();
	const int32 TargetHeight = GlyphSdfMapping.GetSdfHeight();
	const int32 TargetChannels = GetNumSdfChannels(Descriptor.SdfType);
	TArray<float> FloatPixels;
	FloatPixels.SetNumUninitialized(TargetChannels * TargetWidth * TargetHeight);
	OutputPixels.SetNumUninitialized(TargetChannels * TargetWidth * TargetHeight);

	switch (Descriptor.SdfType)
	{
		case FSlateSdfGenerator::ESdfType::Simple:
		{
			msdfgen::BitmapRef<float, 1> FloatBitmap(FloatPixels.GetData(), TargetWidth, TargetHeight);
			msdfgen::generateSDF(
				FloatBitmap,
				MsdfgenShape,
				GlyphSdfMapping.GetMsdfgenTransformation(),
				msdfgen::GeneratorConfig(OverlappedContourSupport)
			);
			break;
		}
		case FSlateSdfGenerator::ESdfType::Perpendicular:
		{
			msdfgen::BitmapRef<float, 1> FloatBitmap(FloatPixels.GetData(), TargetWidth, TargetHeight);
			msdfgen::generatePSDF(
				FloatBitmap,
				MsdfgenShape,
				GlyphSdfMapping.GetMsdfgenTransformation(),
				msdfgen::GeneratorConfig(OverlappedContourSupport)
			);
			break;
		}
		case FSlateSdfGenerator::ESdfType::MultichannelAndSimple:
		{
			msdfgen::edgeColoringInkTrap(MsdfgenShape, SDF_CORNER_ANGLE_THRESHOLD);

			msdfgen::BitmapRef<float, 4> FloatBitmap(FloatPixels.GetData(), TargetWidth, TargetHeight);
			msdfgen::generateMTSDF(
				FloatBitmap,
				MsdfgenShape,
				GlyphSdfMapping.GetMsdfgenTransformation(),
				msdfgen::MSDFGeneratorConfig(
					OverlappedContourSupport,
					msdfgen::ErrorCorrectionConfig(
						msdfgen::ErrorCorrectionConfig::EDGE_PRIORITY,
						msdfgen::ErrorCorrectionConfig::CHECK_DISTANCE_AT_EDGE,
						msdfgen::ErrorCorrectionConfig::defaultMinDeviationRatio,
						msdfgen::ErrorCorrectionConfig::defaultMinImproveRatio,
						OutputPixels.GetData() // Temporarily repurpose output buffer as error correction buffer
					)
				)
			);
			break;
		}
		default:
			checkNoEntry();
	}

	const float* Src = FloatPixels.GetData();
	for (uint8* Dst = OutputPixels.GetData(), * End = Dst+TargetChannels*TargetWidth*TargetHeight; Dst < End; ++Dst, ++Src)
	{
		*Dst = msdfgen::pixelFloatToByte(*Src);
	}
}

void FSdfGeneratorTask::MakePlaceholder(TArray<uint8>& OutRawPixels) const
{
	const int32 TargetChannels = GetNumSdfChannels(Descriptor.SdfType);
	const int32 TargetWidth = GlyphSdfMapping.GetSdfWidth();
	const int32 TargetHeight = GlyphSdfMapping.GetSdfHeight();
	const int32 TargetArea = TargetWidth * TargetHeight;
	const int32 TotalSubpixels = TargetChannels * TargetArea;
	TArray<float> FloatPixels;
	FloatPixels.SetNumUninitialized(TargetArea);
	OutRawPixels.SetNumUninitialized(TotalSubpixels);
	const msdfgen::BitmapRef<float, 1> FloatBitmap(FloatPixels.GetData(), TargetWidth, TargetHeight);
	msdfgen::approximateSDF(
		FloatBitmap,
		MsdfgenShape,
		GlyphSdfMapping.GetMsdfgenTransformation()
	);

	const float* Src = FloatPixels.GetData();
	switch (TargetChannels)
	{
		case 1:
			for (uint8* Dst = OutRawPixels.GetData(), * End = Dst + TotalSubpixels; Dst < End; ++Dst, ++Src)
			{
				*Dst = msdfgen::pixelFloatToByte(*Src);
			}
			break;
		case 3:
			for (uint8* Dst = OutRawPixels.GetData(), * End = Dst + TotalSubpixels; Dst < End; Dst += 3, ++Src)
			{
				Dst[2] = Dst[1] = Dst[0] = msdfgen::pixelFloatToByte(*Src);
			}
			break;
		case 4:
			for (uint8* Dst = OutRawPixels.GetData(), * End = Dst + TotalSubpixels; Dst < End; Dst += 4, ++Src)
			{
				Dst[3] = Dst[2] = Dst[1] = Dst[0] = msdfgen::pixelFloatToByte(*Src);
			}
			break;
		default:
			checkNoEntry();
	}
}

} // namespace SdfUtils

  /*
  *
  */

class FSlateSdfGeneratorImpl : public FSlateSdfGenerator
{
public:
	FSlateSdfGeneratorImpl();
	~FSlateSdfGeneratorImpl();

	virtual ERequestResponse Spawn(const FRequestDescriptor& InRequest, FRequestOutputInfo& OutCharInfo) override;
	virtual ERequestResponse SpawnWithPlaceholder(const FRequestDescriptor& InRequest, FRequestOutputInfo& OutCharInfo, TArray<uint8>& OutRawPixels) override;
	virtual ERequestResponse Respawn(const FRequestDescriptor& InRequest, const FRequestOutputInfo& InCharInfo) override;
	virtual ERequestResponse MakePlaceholder(const FRequestDescriptor& InRequest, FRequestOutputInfo& OutCharInfo, TArray<uint8>& OutRawPixels) override;
	virtual void Update(const FForEachRequestDoneCallback& InEnumerator) override;
	virtual void Flush() override;
private:
	TArray<FAsyncTask<SdfUtils::FSdfGeneratorTask>*> FreeTasks;
	TArray<FAsyncTask<SdfUtils::FSdfGeneratorTask>*> StartedTasks;
	TArray<TUniquePtr<FAsyncTask<SdfUtils::FSdfGeneratorTask>>> TasksPool;
	FAutoConsoleVariableSink PoolSizeChangeSink;

	/** Enlarges the pool size according to the new value of the SlateSdfText.GeneratorPoolSize CVar. The pool cannot be shrinked and if the new value is lower, this will have no effect and false will be returned. */
	bool UpdatePoolSize();
};


FSlateSdfGeneratorImpl::FSlateSdfGeneratorImpl() :
	PoolSizeChangeSink(FConsoleCommandDelegate::CreateLambda([this]()
		{
			UpdatePoolSize();
		}
	))
{
	UpdatePoolSize();
}

FSlateSdfGeneratorImpl::~FSlateSdfGeneratorImpl()
{
	Flush();
	FreeTasks.Empty();
	TasksPool.Empty();
}

bool FSlateSdfGeneratorImpl::UpdatePoolSize()
{
	check(IsInGameThread());
	int32 PoolSize = 0;
	if (IsSlateSdfTextFeatureEnabled())
	{
		PoolSize = CVarSlateSdfTextGeneratorPoolSize.GetValueOnGameThread();
		if (PoolSize <= 0)
		{
			PoolSize = FGenericPlatformMisc::NumberOfWorkerThreadsToSpawn();
		}
	}
	int32 OldPoolSize = TasksPool.Num();
	if (PoolSize < OldPoolSize)
	{
		return false;
	}
	if (PoolSize > OldPoolSize)
	{
		FreeTasks.Reserve(PoolSize);
		StartedTasks.Reserve(PoolSize);
		TasksPool.Reserve(PoolSize);
		for (int32 Index = OldPoolSize; Index < PoolSize; ++Index)
		{
			int32 AddedIndex = TasksPool.Add(MakeUnique<FAsyncTask<SdfUtils::FSdfGeneratorTask>>());
			check(AddedIndex == Index);
			FreeTasks.Push(TasksPool[AddedIndex].Get());
		}
	}
	return true;
}

FSlateSdfGenerator::ERequestResponse FSlateSdfGeneratorImpl::Spawn(const FRequestDescriptor& InRequest, FRequestOutputInfo& OutCharInfo)
{
	if (FreeTasks.IsEmpty())
	{
		return ERequestResponse::BUSY;
	}
	FAsyncTask<SdfUtils::FSdfGeneratorTask>* Task = FreeTasks.Pop(EAllowShrinking::No);
	const ERequestResponse Result = Task->GetTask().Prepare(InRequest, OutCharInfo);
	if (Result == ERequestResponse::SUCCESS)
	{
		StartedTasks.Push(Task);
		Task->StartBackgroundTask();
	}
	else
	{
		FreeTasks.Push(Task);
	}
	return Result;
}

FSlateSdfGenerator::ERequestResponse FSlateSdfGeneratorImpl::SpawnWithPlaceholder(const FRequestDescriptor& InRequest, FRequestOutputInfo& OutCharInfo, TArray<uint8>& OutRawPixels)
{
	if (FreeTasks.IsEmpty())
	{
		const ERequestResponse Result = MakePlaceholder(InRequest, OutCharInfo, OutRawPixels);
		if (Result == ERequestResponse::SUCCESS)
		{
			return ERequestResponse::PLACEHOLDER_ONLY;
		}
		return Result;
	}
	FAsyncTask<SdfUtils::FSdfGeneratorTask>* Task = FreeTasks.Pop(EAllowShrinking::No);
	const ERequestResponse Result = Task->GetTask().Prepare(InRequest, OutCharInfo);
	if (Result == ERequestResponse::SUCCESS)
	{
		Task->GetTask().MakePlaceholder(OutRawPixels);
		StartedTasks.Push(Task);
		Task->StartBackgroundTask();
	}
	else
	{
		FreeTasks.Push(Task);
	}
	return Result;
}

FSlateSdfGenerator::ERequestResponse FSlateSdfGeneratorImpl::Respawn(const FRequestDescriptor& InRequest, const FRequestOutputInfo& InCharInfo)
{
	if (FreeTasks.IsEmpty())
	{
		return ERequestResponse::BUSY;
	}
	FRequestOutputInfo OutCharInfo = {};
	FAsyncTask<SdfUtils::FSdfGeneratorTask>* Task = FreeTasks.Pop(EAllowShrinking::No);
	ERequestResponse Result = Task->GetTask().Prepare(InRequest, OutCharInfo);
	if (Result == ERequestResponse::SUCCESS)
	{
		if (
			OutCharInfo.ImageWidth == InCharInfo.ImageWidth &&
			OutCharInfo.ImageHeight == InCharInfo.ImageHeight &&
			OutCharInfo.BearingX == InCharInfo.BearingX &&
			OutCharInfo.BearingY == InCharInfo.BearingY
		)
		{
			StartedTasks.Push(Task);
			Task->StartBackgroundTask();
			return ERequestResponse::SUCCESS;
		}
		else // Task spawn retried but output metrics differ, this shouldn't happen
		{
			checkNoEntry();
			Task->GetTask().Reset();
			Result = ERequestResponse::BAD_REQUEST;
		}
	}
	FreeTasks.Push(Task);
	return Result;
}

FSlateSdfGenerator::ERequestResponse FSlateSdfGeneratorImpl::MakePlaceholder(const FRequestDescriptor& InRequest, FRequestOutputInfo& OutCharInfo, TArray<uint8>& OutRawPixels)
{
	SdfUtils::FSdfGeneratorTask PlaceholderTask;
	const ERequestResponse Result = PlaceholderTask.Prepare(InRequest, OutCharInfo);
	if (Result == ERequestResponse::SUCCESS)
	{
		PlaceholderTask.MakePlaceholder(OutRawPixels);
	}
	return Result;
}

void FSlateSdfGeneratorImpl::Update(const FForEachRequestDoneCallback& InEnumerator)
{
	for (auto It = StartedTasks.CreateIterator(); It; ++It)
	{
		if ((*It)->IsDone())
		{
			(*It)->GetTask().End(InEnumerator);
			FreeTasks.Push(*It);
			It.RemoveCurrentSwap();
		}
	}
}

void FSlateSdfGeneratorImpl::Flush()
{	
	for (auto It = StartedTasks.CreateIterator(); It; ++It)
	{
		if ((*It)->Cancel() || (*It)->IsDone())
		{
			(*It)->GetTask().Reset();
			FreeTasks.Push(*It);
			It.RemoveCurrentSwap();
		}
	}
	for (auto It = StartedTasks.CreateIterator(); It; ++It)
	{
		(*It)->EnsureCompletion(false);
		(*It)->GetTask().Reset();
		FreeTasks.Push(*It);
	}
	StartedTasks.Empty(TasksPool.Num());
	check(FreeTasks.Num() == TasksPool.Num());
}

FSlateSdfGenerator::FSlateSdfGenerator()
{
	// no inline
}

FSlateSdfGenerator::~FSlateSdfGenerator()
{
	// no inline
}

FSlateSdfGenerator::Ptr FSlateSdfGenerator::create()
{
	return MakeUnique<FSlateSdfGeneratorImpl>();
}
