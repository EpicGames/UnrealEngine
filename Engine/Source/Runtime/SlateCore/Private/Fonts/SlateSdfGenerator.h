// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"

/**
 * Generates multi-channel signed distance fields for font glyphs.
 */
class FSlateSdfGenerator
{
public:
	/** A unique pointer to a FSlateSdfGenerator object. */
	using Ptr = TUniquePtr<FSlateSdfGenerator>;

	/** Generator's response to caller's request. */
	enum class ERequestResponse
	{
		/** Task spawned successfully (and placeholder generated if requested) */
		SUCCESS = 0,
		/** Glyph is available in the face but the SDF generation was not possible/successful */
		SDF_UNAVAILABLE,
		/** Task not spawned due to task pool being full (try again later) */
		BUSY,
		/** Task not spawned but placeholder and output info generated (respawn later) */
		PLACEHOLDER_ONLY,
		/** Task not spawned because the request data was not valid */
		BAD_REQUEST
	};

	/** Type of requested signed distance field. */
	enum class ESdfType
	{
		/** Simple single-channel signed distance field based on true Euclidean distance (1 channel/pixel) */
		Simple,
		/** Single-channel signed distance field based on the perpendicular distance metric, which is always orthogonal to an edge, and produces mitered rather than rounded outlines (1 channel/pixel) */
		Perpendicular,
		/** Multi-channel signed distance field with simple (true) signed distance field in additional channel (4 channels/pixel, also known as MTSDF) */
		MultichannelAndSimple
	};

	/** Glyph metrics made available immediately after spawning a new task. */
	struct FRequestOutputInfo
	{
		/** Raster Image Width */
		uint16 ImageWidth;
		/** Raster Image Height */
		uint16 ImageHeight;
		/** Position of left edge of image relative to glyph origin */
		int16 BearingX;
		/** Position of top edge of image relative to glyph origin */
		int16 BearingY;
	};

	/** Specifies the requested glyph and properties of the output distance field. */
	struct FRequestDescriptor
	{
		/** Pointer to the glyph's font face. */
		TWeakPtr<class FFreeTypeFace> FontFace;
		/** Numeric index of the requested glyph. */
		uint32 GlyphIndex;
		/** Type of requested distance field. */
		ESdfType SdfType;
		/** Outer portion of the width of representable distances in the output distance field expressed in em. */
		float EmOuterSpread;
		/** Inner portion of the width of representable distances in the output distance field expressed in em. */
		float EmInnerSpread;
		/** Pixels per em of the output distance field. */
		int32 Ppem;
	};

	/** Callback function for the finished tasks. */
	using FForEachRequestDoneCallback = TFunctionRef<void(const FRequestDescriptor/*RequestId*/, 
		TArray<uint8>/*RawPixels*/)>;

	virtual ~FSlateSdfGenerator();

	/** Starts generating a distance field for the requested glyph. */
	virtual ERequestResponse Spawn(const FRequestDescriptor& InRequest, FRequestOutputInfo& OutCharInfo) = 0;
	/** Starts generating a distance field and immediately provides an approximate distance field placeholder. */
	virtual ERequestResponse SpawnWithPlaceholder(const FRequestDescriptor& InRequest, FRequestOutputInfo& OutCharInfo, TArray<uint8>& OutRawPixels) = 0;
	/** Attempts to start generating again if previous attempt failed but produced a placeholder, whose FRequestOutputInfo must match. */
	virtual ERequestResponse Respawn(const FRequestDescriptor& InRequest, const FRequestOutputInfo& InCharInfo) = 0;
	/** Only generates a placeholder-quality signed distance field immediately into OutRawPixels without spawning. */
	virtual ERequestResponse MakePlaceholder(const FRequestDescriptor& InRequest, FRequestOutputInfo& OutCharInfo, TArray<uint8>& OutRawPixels) = 0;
	/** Checks for finished tasks and processes each of them by calling InEnumerator. */
	virtual void Update(const FForEachRequestDoneCallback& InEnumerator) = 0;
	/** Flushes all started tasks. */
	virtual void Flush() = 0;

	/** Creates an instance of FSlateSdfGenerator. */
	static Ptr create();

protected:
	FSlateSdfGenerator();

};
