// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/SlateTextShaper.h"
#include "Fonts/FontCacheCompositeFont.h"
#include "Fonts/SlateFontRenderer.h"
#include "Fonts/FontProviderInterface.h"
#include "Internationalization/BreakIterator.h"
#include "Internationalization/TextChar.h"
#include "SlateGlobals.h"

#include <limits>
#include "Fonts/FontUtils.h"

DECLARE_CYCLE_STAT(TEXT("Shape Bidirectional Text"), STAT_SlateShapeBidirectionalText, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Shape Unidirectional Text"), STAT_SlateShapeUnidirectionalText, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("HarfBuzz Text Shaping"), STAT_PerformHarfBuzzTextShaping, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Kerning Only Text Shaping"), STAT_PerformKerningOnlyTextShaping, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Split By FontFace"), STAT_SplitByFontFace, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Split By Script"), STAT_SplitByScript, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("Text Sequences Processing"), STAT_TextSequencesProcessing, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("HarfBuzz Shape"), STAT_HarfBuzzNativeShape, STATGROUP_Slate);

namespace
{

bool RenderCodepointAsWhitespace(const TCHAR InCodepoint)
{
	return FTextChar::IsWhitespace(InCodepoint)
		|| InCodepoint == TEXT('\u200B')	// Zero Width Space
		|| InCodepoint == TEXT('\u2009')	// Thin Space
		|| InCodepoint == TEXT('\u202F');	// Narrow No-Break Space
}

bool RenderCodepointAsWhitespace(const UTF32CHAR InCodepoint)
{
	return RenderCodepointAsWhitespace((TCHAR)InCodepoint);
}

struct FKerningOnlyTextSequenceEntry
{
	int32 TextStartIndex;
	int32 TextLength;
	const FFontData* FontDataPtr;
	TSharedPtr<FFreeTypeFace> FaceAndMemory;
	float SubFontScalingFactor;

	FKerningOnlyTextSequenceEntry(const int32 InTextStartIndex, const int32 InTextLength, const FFontData* InFontDataPtr, TSharedPtr<FFreeTypeFace> InFaceAndMemory, const float InSubFontScalingFactor)
		: TextStartIndex(InTextStartIndex)
		, TextLength(InTextLength)
		, FontDataPtr(InFontDataPtr)
		, FaceAndMemory(MoveTemp(InFaceAndMemory))
		, SubFontScalingFactor(InSubFontScalingFactor)
	{
	}
};

#if WITH_HARFBUZZ

struct FHarfBuzzTextSequenceEntry
{
	struct FSubSequenceEntry
	{
		int32 StartIndex;
		int32 Length;
		hb_script_t HarfBuzzScript;

		FSubSequenceEntry(const int32 InStartIndex, const int32 InLength, const hb_script_t InHarfBuzzScript)
			: StartIndex(InStartIndex)
			, Length(InLength)
			, HarfBuzzScript(InHarfBuzzScript)
		{
		}
	};

	int32 TextStartIndex;
	int32 TextLength;
	const FFontData* FontDataPtr;
	TSharedPtr<FFreeTypeFace> FaceAndMemory;
	float SubFontScalingFactor;
	TArray<FSubSequenceEntry> SubSequence;

	FHarfBuzzTextSequenceEntry(const int32 InTextStartIndex, const int32 InTextLength, const FFontData* InFontDataPtr, TSharedPtr<FFreeTypeFace> InFaceAndMemory, const float InSubFontScalingFactor)
		: TextStartIndex(InTextStartIndex)
		, TextLength(InTextLength)
		, FontDataPtr(InFontDataPtr)
		, FaceAndMemory(MoveTemp(InFaceAndMemory))
		, SubFontScalingFactor(InSubFontScalingFactor)
	{
	}
};

#endif // WITH_HARFBUZZ

} // anonymous namespace


FSlateTextShaper::FSlateTextShaper(FFreeTypeCacheDirectory* InFTCacheDirectory, FCompositeFontCache* InCompositeFontCache, FSlateFontRenderer* InFontRenderer, FSlateFontCache* InFontCache)
	: FTCacheDirectory(InFTCacheDirectory)
	, CompositeFontCache(InCompositeFontCache)
	, FontRenderer(InFontRenderer)
	, FontCache(InFontCache)
	, TextBiDiDetection(TextBiDi::CreateTextBiDi())
	, GraphemeBreakIterator(GSlateUseSharedBreakIterator ? FBreakIterator::GetCharacterBoundaryIterator() : FBreakIterator::CreateCharacterBoundaryIterator())
#if WITH_HARFBUZZ
	, HarfBuzzFontCache(FTCacheDirectory)
#endif // WITH_HARFBUZZ
{
	check(FTCacheDirectory);
	check(CompositeFontCache);
	check(FontRenderer);
	check(FontCache);
}

void FSlateTextShaper::FlushCache()
{
#if WITH_HARFBUZZ
	HarfBuzzFontCache.Flush();
#endif
}

FShapedGlyphSequenceRef FSlateTextShaper::ShapeBidirectionalText(const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo& InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InBaseDirection, const ETextShapingMethod TextShapingMethod, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces)
{
	SCOPE_CYCLE_COUNTER(STAT_SlateShapeBidirectionalText);

	TArray<TextBiDi::FTextDirectionInfo> TextDirectionInfos;
	TextBiDiDetection->ComputeTextDirection(InText, InTextStart, InTextLen, InBaseDirection, TextDirectionInfos);

	TArray<FShapedGlyphEntry> GlyphsToRender;
	for (const TextBiDi::FTextDirectionInfo& TextDirectionInfo : TextDirectionInfos)
	{
		PerformTextShaping(InText, TextDirectionInfo.StartIndex, TextDirectionInfo.Length, InFontInfo, InFontScale, TextDirectionInfo.TextDirection, TextShapingMethod, GlyphsToRender, OutLoadingFontFaces);
	}

	return FinalizeTextShaping(MoveTemp(GlyphsToRender), InFontInfo, InFontScale, FShapedGlyphSequence::FSourceTextRange(InTextStart, InTextLen));
}

FShapedGlyphSequenceRef FSlateTextShaper::ShapeUnidirectionalText(const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo& InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InTextDirection, const ETextShapingMethod TextShapingMethod, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces)
{
	SCOPE_CYCLE_COUNTER(STAT_SlateShapeUnidirectionalText);

	TArray<FShapedGlyphEntry> GlyphsToRender;
	PerformTextShaping(InText, InTextStart, InTextLen, InFontInfo, InFontScale, InTextDirection, TextShapingMethod, GlyphsToRender, OutLoadingFontFaces);
	return FinalizeTextShaping(MoveTemp(GlyphsToRender), InFontInfo, InFontScale, FShapedGlyphSequence::FSourceTextRange(InTextStart, InTextLen));
}

void FSlateTextShaper::PerformTextShaping(const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo& InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InTextDirection, const ETextShapingMethod TextShapingMethod, TArray<FShapedGlyphEntry>& OutGlyphsToRender, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces)
{
	check(InTextDirection != TextBiDi::ETextDirection::Mixed);

#if WITH_FREETYPE
	if (InTextLen > 0)
	{
#if WITH_HARFBUZZ
		auto TextRequiresFullShaping = [&]() -> bool
		{
			// RTL text always requires full shaping
			if (InTextDirection == TextBiDi::ETextDirection::RightToLeft)
			{
				return true;
			}

			// LTR text containing certain scripts or surrogate pairs requires full shaping
			{
				// Note: We deliberately avoid using HarfBuzz/ICU here as we don't care about the script itself, only that the character is within a shaped script range (and testing that is much faster!)
				auto CharRequiresFullShaping = [](const TCHAR InChar) -> bool
				{
					// Note: This isn't an exhaustive list, as it omits some "dead" or uncommon languages, and ranges outside the BMP
					#define RETURN_TRUE_IF_CHAR_WITHIN_RANGE(LOWER, UPPER) if (InChar >= (LOWER) && InChar <= (UPPER)) return true

					// Zero-width joiner
					if (InChar == TEXT('\u200D'))
					{
						return true;
					}

					// Combining characters
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0300'), TEXT('\u036F'));
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u1AB0'), TEXT('\u1AFF'));
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u1DC0'), TEXT('\u1DFF'));
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u20D0'), TEXT('\u20FF'));
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u3099'), TEXT('\u309A'));
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u31C0'), TEXT('\u31EF'));
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\uFE00'), TEXT('\uFE0F'));
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\uFE20'), TEXT('\uFE2F'));

					// Gurmukhi
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0A00'), TEXT('\u0A7F'));

					// Devanagari
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0900'), TEXT('\u097F'));
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\uA8E0'), TEXT('\uA8FF'));
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u1CD0'), TEXT('\u1CFF'));

					// Bengali
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0980'), TEXT('\u09FF'));

					// Gujarati
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0A80'), TEXT('\u0AFF'));

					// Odia
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0B00'), TEXT('\u0B7F'));

					// Tamil
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0B80'), TEXT('\u0BFF'));

					// Telugu
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0C00'), TEXT('\u0C7F'));

					// Kannada
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0C80'), TEXT('\u0CFF'));

					// Malayalam
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0D00'), TEXT('\u0D7F'));

					// Thai
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0E00'), TEXT('\u0E7F'));
					
					// Tibetan
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0F00'), TEXT('\u0FFF'));

					// Khmer
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u1780'), TEXT('\u17FF'));
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u19E0'), TEXT('\u19FF'));

					// Sinhala
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u0D80'), TEXT('\u0DFF'));

					// Limbu
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u1900'), TEXT('\u194F'));

					// Tai Tham
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u1A20'), TEXT('\u1AAF'));

					// Tai Viet
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\uAA80'), TEXT('\uAADF'));

					// Batak
					RETURN_TRUE_IF_CHAR_WITHIN_RANGE(TEXT('\u1BC0'), TEXT('\u1BFF'));

					#undef RETURN_TRUE_IF_CHAR_WITHIN_RANGE

					return false;
				};

				const int32 TextEndIndex = InTextStart + InTextLen;
				for (int32 RunningTextIndex = InTextStart; RunningTextIndex < TextEndIndex; ++RunningTextIndex)
				{
					const TCHAR Char = InText[RunningTextIndex];

					if (Char <= TEXT('\u007F'))
					{
						continue;
					}

					if (CharRequiresFullShaping(Char))
					{
						return true;
					}

					{
						const int32 NextTextIndex = RunningTextIndex + 1;
						if (NextTextIndex < TextEndIndex && StringConv::IsHighSurrogate(Char) && StringConv::IsLowSurrogate(InText[NextTextIndex]))
						{
							return true;
						}
					}
				}
			}

			return false;
		};

		if (TextShapingMethod == ETextShapingMethod::FullShaping || (TextShapingMethod == ETextShapingMethod::Auto && TextRequiresFullShaping()))
		{
			PerformHarfBuzzTextShaping(InText, InTextStart, InTextLen, InFontInfo, InFontScale, InTextDirection, OutGlyphsToRender, OutLoadingFontFaces);
		}
		else
#endif // WITH_HARFBUZZ
		{
			PerformKerningOnlyTextShaping(InText, InTextStart, InTextLen, InFontInfo, InFontScale, OutGlyphsToRender, OutLoadingFontFaces);
		}
	}
#endif // WITH_FREETYPE
}

FShapedGlyphSequenceRef FSlateTextShaper::FinalizeTextShaping(TArray<FShapedGlyphEntry> InGlyphsToRender, const FSlateFontInfo& InFontInfo, const float InFontScale, const FShapedGlyphSequence::FSourceTextRange& InSourceTextRange) const
{
	int16 TextBaseline = 0;
	uint16 MaxHeight = 0;

#if WITH_FREETYPE
	{
		// Just get info for the null character
		TCHAR Char = 0;
		const FFontData& FontData = CompositeFontCache->GetDefaultFontData(InFontInfo);
		const FFreeTypeFaceGlyphData FaceGlyphData = FontRenderer->GetFontFaceForCodepoint(FontData, Char, InFontInfo.FontFallback);

		if (FaceGlyphData.FaceAndMemory.IsValid() && FaceGlyphData.FaceAndMemory->IsFaceValid() && !FaceGlyphData.FaceAndMemory->IsFaceLoading())
		{
			if (FMath::IsNearlyEqual(InFontInfo.GetClampSkew(), 0.f))
			{
				FT_Set_Transform(FaceGlyphData.FaceAndMemory->GetFace(), nullptr, nullptr);
			}
			else
			{
				// Skewing / Fake Italics (could do character rotation in future?).
				FT_Matrix TransformMatrix;
				TransformMatrix.xx = 0x10000L;
				TransformMatrix.xy = InFontInfo.GetClampSkew() * 0x10000L;
				TransformMatrix.yx = 0;
				TransformMatrix.yy = 0x10000L;
				FT_Set_Transform(FaceGlyphData.FaceAndMemory->GetFace(), &TransformMatrix, nullptr);
			}

			FreeTypeUtils::ApplySizeAndScale(FaceGlyphData.FaceAndMemory->GetFace(), InFontInfo.Size, InFontScale);

			const bool IsAscentDescentOverridenEnabled = UE::Slate::FontUtils::IsAscentDescentOverrideEnabled(InFontInfo.FontObject);
			TextBaseline = FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>(FaceGlyphData.FaceAndMemory->GetDescender(IsAscentDescentOverridenEnabled));
			MaxHeight = FreeTypeUtils::Convert26Dot6ToRoundedPixel<uint16>(FaceGlyphData.FaceAndMemory->GetScaledHeight(IsAscentDescentOverridenEnabled));
		}
	}
#endif // WITH_FREETYPE

	return MakeShared<FShapedGlyphSequence>(MoveTemp(InGlyphsToRender), 
											TextBaseline, 
											MaxHeight, 
											InFontInfo.FontMaterial.Get(),
											InFontInfo.bMaterialIsStencil,
											InFontInfo.OutlineSettings,
											InSourceTextRange);
}

#if WITH_FREETYPE

void FSlateTextShaper::PerformKerningOnlyTextShaping(const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo& InFontInfo, const float InFontScale, TArray<FShapedGlyphEntry>& OutGlyphsToRender, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces) const
{
	SCOPE_CYCLE_COUNTER(STAT_PerformKerningOnlyTextShaping);

	// We need to work out the correct FFontData for everything so that we can build accurate FShapedGlyphFaceData for rendering later on
	TArray<FKerningOnlyTextSequenceEntry, TInlineAllocator<4>> KerningOnlyTextSequence;

	// Step 1) Split the text into sections that are using the same font face (composite fonts may contain different faces for different character ranges)
	{
		SCOPE_CYCLE_COUNTER(STAT_SplitByFontFace);

		// Data used while detecting font face boundaries
		int32 SplitStartIndex = InTextStart;
		int32 RunningTextIndex = InTextStart;
		const FFontData* RunningFontDataPtr = nullptr;
		TSharedPtr<FFreeTypeFace> RunningFaceAndMemory;
		float RunningSubFontScalingFactor = 1.0f;

		auto AppendPendingFontDataToSequence = [&]()
		{
			if (RunningFontDataPtr)
			{
				KerningOnlyTextSequence.Emplace(
					SplitStartIndex,						// InTextStartIndex
					RunningTextIndex - SplitStartIndex,		// InTextLength
					RunningFontDataPtr,						// InFontDataPtr
					RunningFaceAndMemory,					// InFaceAndMemory
					RunningSubFontScalingFactor				// InSubFontScalingFactor
					);

				RunningFontDataPtr = nullptr;
				RunningFaceAndMemory.Reset();
				RunningSubFontScalingFactor = 1.0f;
			}
		};

		const int32 TextEndIndex = InTextStart + InTextLen;
		for (; RunningTextIndex < TextEndIndex; ++RunningTextIndex)
		{
			const TCHAR CurrentChar = InText[RunningTextIndex];
			const bool bShouldRenderAsWhitespace = RenderCodepointAsWhitespace(CurrentChar);

			// First try with the actual character
			float SubFontScalingFactor = 1.0f;
			const FFontData* FontDataPtr = &CompositeFontCache->GetFontDataForCodepoint(InFontInfo, CurrentChar, SubFontScalingFactor);
			FFreeTypeFaceGlyphData FaceGlyphData = FontRenderer->GetFontFaceForCodepoint(*FontDataPtr, CurrentChar, bShouldRenderAsWhitespace ? EFontFallback::FF_NoFallback : InFontInfo.FontFallback);
			if (OutLoadingFontFaces && FaceGlyphData.FaceAndMemory.IsValid() && FaceGlyphData.FaceAndMemory->IsFaceLoading())
			{
				OutLoadingFontFaces->AddUnique(FaceGlyphData.FaceAndMemory);
			}
			

			// If none of our fonts can render that character (as the fallback font may be missing), 
			// try again with the fallback character, or a normal space if this character was supposed to 
			// be whitespace (as we don't render whitespace anyway)
			if (!FaceGlyphData.FaceAndMemory.IsValid() || FaceGlyphData.FaceAndMemory->IsFaceLoading())
			{
				const TCHAR FallbackChar = bShouldRenderAsWhitespace ? TEXT(' ') : SlateFontRendererUtils::InvalidSubChar;
				FontDataPtr = &CompositeFontCache->GetFontDataForCodepoint(InFontInfo, FallbackChar, SubFontScalingFactor);
				FaceGlyphData = FontRenderer->GetFontFaceForCodepoint(*FontDataPtr, FallbackChar, InFontInfo.FontFallback);
			}

			// Only scalable font types can use sub-font scaling
			if (FaceGlyphData.FaceAndMemory.IsValid() && !FaceGlyphData.FaceAndMemory->IsFaceLoading() && !FT_IS_SCALABLE(FaceGlyphData.FaceAndMemory->GetFace()))
			{
				SubFontScalingFactor = 1.0f;
			}

			if (!RunningFontDataPtr || RunningFontDataPtr != FontDataPtr || RunningFaceAndMemory != FaceGlyphData.FaceAndMemory || RunningSubFontScalingFactor != SubFontScalingFactor)
			{
				AppendPendingFontDataToSequence();

				SplitStartIndex = RunningTextIndex;
				RunningFontDataPtr = FontDataPtr;
				RunningFaceAndMemory = FaceGlyphData.FaceAndMemory;
				RunningSubFontScalingFactor = SubFontScalingFactor;
			}
		}

		AppendPendingFontDataToSequence();
	}

	// Step 2) Now we use the font cache to get the size for each character, and kerning for each character pair
	{
		SCOPE_CYCLE_COUNTER(STAT_TextSequencesProcessing);

		OutGlyphsToRender.Reserve(OutGlyphsToRender.Num() + InTextLen);
		for (const FKerningOnlyTextSequenceEntry& KerningOnlyTextSequenceEntry : KerningOnlyTextSequence)
		{
			if (!KerningOnlyTextSequenceEntry.FaceAndMemory.IsValid() || KerningOnlyTextSequenceEntry.FaceAndMemory->IsFaceLoading())
			{
				continue;
			}

			const bool bHasKerning = FT_HAS_KERNING(KerningOnlyTextSequenceEntry.FaceAndMemory->GetFace()) != 0 || InFontInfo.LetterSpacing != 0;

			uint32 GlyphFlags = 0;
			SlateFontRendererUtils::AppendGlyphFlags(*KerningOnlyTextSequenceEntry.FaceAndMemory, *KerningOnlyTextSequenceEntry.FontDataPtr, GlyphFlags);
			const float FinalFontScale = InFontScale * KerningOnlyTextSequenceEntry.SubFontScalingFactor;

			// Letter spacing should scale proportional to font size / 1000 (to roughly mimic Photoshop tracking)
			const float LetterSpacingScaledAsFloat = InFontInfo.LetterSpacing != 0 ? InFontInfo.LetterSpacing * FinalFontScale * InFontInfo.Size / 1000.f : 0.f;
			ensure(LetterSpacingScaledAsFloat <= std::numeric_limits<int16>::max());
			const int16 LetterSpacingScaled = (int16)LetterSpacingScaledAsFloat;

			// Used for monospacing
			int16 FixedAdvance = INDEX_NONE;
			if (InFontInfo.bForceMonospaced)
			{
				const float MonospacingScaledAsFloat = InFontInfo.MonospacedWidth != 0 ? InFontInfo.MonospacedWidth * InFontInfo.Size * FinalFontScale : 0.f;
				ensure(MonospacingScaledAsFloat <= std::numeric_limits<int16>::max());
				FixedAdvance = (int16)MonospacingScaledAsFloat;
			}

			FreeTypeUtils::ApplySizeAndScale(KerningOnlyTextSequenceEntry.FaceAndMemory->GetFace(), InFontInfo.Size, FinalFontScale);
			const FFontRasterizationSettings FontRasterizationSettings = KerningOnlyTextSequenceEntry.FontDataPtr->GetFontRasterizationSettings();
			const uint32 FontRenderSize = FreeTypeUtils::ComputeFontPixelSize(InFontInfo.Size, FinalFontScale);
			TSharedRef<FShapedGlyphFaceData> ShapedGlyphFaceData = MakeShared<FShapedGlyphFaceData>(KerningOnlyTextSequenceEntry.FaceAndMemory, GlyphFlags, InFontInfo.Size, FinalFontScale, InFontInfo.GetClampSkew(), FontRasterizationSettings.Mode, FontRasterizationSettings.DistanceFieldPpem);
			TSharedPtr<FFreeTypeKerningCache> KerningCache = FTCacheDirectory->GetKerningCache(KerningOnlyTextSequenceEntry.FaceAndMemory->GetFace(), FT_KERNING_DEFAULT, FontRenderSize);
			TSharedRef<FFreeTypeAdvanceCache> AdvanceCache = FTCacheDirectory->GetAdvanceCache(KerningOnlyTextSequenceEntry.FaceAndMemory->GetFace(), GlyphFlags, FontRenderSize);

			for (int32 SequenceCharIndex = 0; SequenceCharIndex < KerningOnlyTextSequenceEntry.TextLength; ++SequenceCharIndex)
			{
				const int32 CurrentCharIndex = KerningOnlyTextSequenceEntry.TextStartIndex + SequenceCharIndex;
				const TCHAR CurrentChar = InText[CurrentCharIndex];

				if (!InsertSubstituteGlyphs(InText, CurrentCharIndex, ShapedGlyphFaceData, AdvanceCache, OutGlyphsToRender, LetterSpacingScaled))
				{
					uint32 GlyphIndex = FT_Get_Char_Index(KerningOnlyTextSequenceEntry.FaceAndMemory->GetFace(), CurrentChar);

					// If the given font can't render that character (as the fallback font may be missing), try again with the fallback character
					if (CurrentChar != 0 && GlyphIndex == 0)
					{
						GlyphIndex = FT_Get_Char_Index(KerningOnlyTextSequenceEntry.FaceAndMemory->GetFace(), SlateFontRendererUtils::InvalidSubChar);
					}

					int16 XAdvance = 0;
					{
						FT_Fixed CachedAdvanceData = 0;
						if (AdvanceCache->FindOrCache(GlyphIndex, CachedAdvanceData))
						{
							XAdvance = FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>((CachedAdvanceData + (1<<9)) >> 10);
						}
					}

					const int32 CurrentGlyphEntryIndex = OutGlyphsToRender.AddDefaulted();
					FShapedGlyphEntry& ShapedGlyphEntry = OutGlyphsToRender[CurrentGlyphEntryIndex];
					ShapedGlyphEntry.FontFaceData = ShapedGlyphFaceData;
					ShapedGlyphEntry.GlyphIndex = GlyphIndex;
					ShapedGlyphEntry.SourceIndex = CurrentCharIndex;
					ShapedGlyphEntry.XAdvance = !InFontInfo.bForceMonospaced ? XAdvance : FixedAdvance;
					ShapedGlyphEntry.YAdvance = 0;
					ShapedGlyphEntry.XOffset = !InFontInfo.bForceMonospaced ? 0 : (FixedAdvance - XAdvance) / 2;
					ShapedGlyphEntry.YOffset = 0;
					ShapedGlyphEntry.Kerning = 0;
					ShapedGlyphEntry.NumCharactersInGlyph = 1;
					ShapedGlyphEntry.NumGraphemeClustersInGlyph = 1;
					ShapedGlyphEntry.TextDirection = TextBiDi::ETextDirection::LeftToRight;
					ShapedGlyphEntry.bIsVisible = !RenderCodepointAsWhitespace(CurrentChar);

					// Apply the letter spacing and font kerning against the previous entry
					if (CurrentGlyphEntryIndex > 0 && bHasKerning)
					{
						FShapedGlyphEntry& PreviousShapedGlyphEntry = OutGlyphsToRender[CurrentGlyphEntryIndex - 1];

						if (LetterSpacingScaled != 0)
						{
							PreviousShapedGlyphEntry.XAdvance += LetterSpacingScaled;
						}

						if (ShapedGlyphEntry.bIsVisible && !InFontInfo.bForceMonospaced)
						{
							FT_Vector KerningVector;
							if (KerningCache && KerningCache->FindOrCache(PreviousShapedGlyphEntry.GlyphIndex, ShapedGlyphEntry.GlyphIndex, KerningVector))
							{
								const int8 Kerning = FreeTypeUtils::Convert26Dot6ToRoundedPixel<int8>(KerningVector.x);
								PreviousShapedGlyphEntry.XAdvance += Kerning;
								PreviousShapedGlyphEntry.Kerning = Kerning;
							}
						}
					}
				}
			}
		}
	}
}

#endif // WITH_FREETYPE

#if WITH_HARFBUZZ

void FSlateTextShaper::PerformHarfBuzzTextShaping(const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo& InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InTextDirection, TArray<FShapedGlyphEntry>& OutGlyphsToRender, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces)
{
	SCOPE_CYCLE_COUNTER(STAT_PerformHarfBuzzTextShaping);

	// HarfBuzz can only shape data that uses the same font face, reads in the same direction, and uses the same script so we need to split the given text...
	TArray<FHarfBuzzTextSequenceEntry, TInlineAllocator<12>> HarfBuzzTextSequence;

	GraphemeBreakIterator->SetString(InText + InTextStart, InTextLen);

	// HarfBuzz does not currently support letter spacing/tracking in conjunction with certain combinations of grapheme clusters, so we will bypass letter spacing under certain conditions (such as with Arabic script).
	bool bBypassLetterSpacing = false;

	// Step 1) Split the text into sections that are using the same font face (composite fonts may contain different faces for different character ranges)
	{
		SCOPE_CYCLE_COUNTER(STAT_SplitByFontFace);

		// Data used while detecting font face boundaries
		int32 SplitStartIndex = InTextStart;
		int32 RunningTextIndex = InTextStart;
		const FFontData* RunningFontDataPtr = nullptr;
		TSharedPtr<FFreeTypeFace> RunningFaceAndMemory;
		float RunningSubFontScalingFactor = 1.0f;

		auto AppendPendingFontDataToSequence = [&]()
		{
			if (RunningFontDataPtr)
			{
				HarfBuzzTextSequence.Emplace(
					SplitStartIndex,						// InTextStartIndex
					RunningTextIndex - SplitStartIndex,		// InTextLength
					RunningFontDataPtr,						// InFontDataPtr
					RunningFaceAndMemory,					// InFaceAndMemory
					RunningSubFontScalingFactor				// InSubFontScalingFactor
					);

				RunningFontDataPtr = nullptr;
				RunningFaceAndMemory.Reset();
				RunningSubFontScalingFactor = 1.0f;
			}
		};

		// Process the text as its grapheme clusters, and have each cluster use the font of its first codepoint (eg, for Emoji combination)
		for (int32 PreviousBreak = 0, CurrentBreak = 0; (CurrentBreak = GraphemeBreakIterator->MoveToNext()) != INDEX_NONE; PreviousBreak = CurrentBreak)
		{
			const int32 ClusterSize = CurrentBreak - PreviousBreak;
			
			// Combine any surrogate pairs into a complete codepoint
			UTF32CHAR CurrentCodepoint = InText[RunningTextIndex];
			if (ClusterSize > 1)
			{
				const int32 NextTextIndex = RunningTextIndex + 1;
				if (StringConv::IsHighSurrogate(InText[RunningTextIndex]) && StringConv::IsLowSurrogate(InText[NextTextIndex]))
				{
					CurrentCodepoint = StringConv::EncodeSurrogate(InText[RunningTextIndex], InText[NextTextIndex]);
				}
				else
				{
					bBypassLetterSpacing = true;
				}
			}

			const bool bShouldRenderAsWhitespace = RenderCodepointAsWhitespace(CurrentCodepoint);

			// First try with the actual character
			float SubFontScalingFactor = 1.0f;
			const FFontData* FontDataPtr = &CompositeFontCache->GetFontDataForCodepoint(InFontInfo, CurrentCodepoint, SubFontScalingFactor);
			FFreeTypeFaceGlyphData FaceGlyphData = FontRenderer->GetFontFaceForCodepoint(*FontDataPtr, CurrentCodepoint, bShouldRenderAsWhitespace ? EFontFallback::FF_NoFallback : InFontInfo.FontFallback);

			if (OutLoadingFontFaces && FaceGlyphData.FaceAndMemory.IsValid() && FaceGlyphData.FaceAndMemory->IsFaceLoading())
			{
				OutLoadingFontFaces->AddUnique(FaceGlyphData.FaceAndMemory);
			}
			// If none of our fonts can render that character (as the fallback font may be missing), try again with the fallback character, or a normal space if this character was supposed to be whitespace (as we don't render whitespace anyway)
			if (!FaceGlyphData.FaceAndMemory.IsValid() || FaceGlyphData.FaceAndMemory->IsFaceLoading())
			{
				const UTF32CHAR FallbackChar = bShouldRenderAsWhitespace ? TEXT(' ') : SlateFontRendererUtils::InvalidSubChar;
				FontDataPtr = &CompositeFontCache->GetFontDataForCodepoint(InFontInfo, FallbackChar, SubFontScalingFactor);
				FaceGlyphData = FontRenderer->GetFontFaceForCodepoint(*FontDataPtr, FallbackChar, InFontInfo.FontFallback);
			}

			// Only scalable font types can use sub-font scaling
			if (FaceGlyphData.FaceAndMemory.IsValid() && !FaceGlyphData.FaceAndMemory->IsFaceLoading() && !FT_IS_SCALABLE(FaceGlyphData.FaceAndMemory->GetFace()))
			{
				SubFontScalingFactor = 1.0f;
			}

			if (!RunningFontDataPtr || RunningFontDataPtr != FontDataPtr || RunningFaceAndMemory != FaceGlyphData.FaceAndMemory || RunningSubFontScalingFactor != SubFontScalingFactor)
			{
				AppendPendingFontDataToSequence();

				SplitStartIndex = RunningTextIndex;
				RunningFontDataPtr = FontDataPtr;
				RunningFaceAndMemory = FaceGlyphData.FaceAndMemory;
				RunningSubFontScalingFactor = SubFontScalingFactor;
			}

			RunningTextIndex += ClusterSize;
		}

		AppendPendingFontDataToSequence();
	}

	// Step 2) Split the font face sections by their script code
	{
		SCOPE_CYCLE_COUNTER(STAT_SplitByScript);
		hb_unicode_funcs_t* HarfBuzzUnicodeFuncs = hb_unicode_funcs_get_default();

		for (FHarfBuzzTextSequenceEntry& HarfBuzzTextSequenceEntry : HarfBuzzTextSequence)
		{
			// Data used while detecting script code boundaries
			int32 SplitStartIndex = HarfBuzzTextSequenceEntry.TextStartIndex;
			int32 RunningTextIndex = HarfBuzzTextSequenceEntry.TextStartIndex;
			TOptional<hb_script_t> RunningHarfBuzzScript;

			auto StartNewPendingTextSequence = [&](const hb_script_t InHarfBuzzScript)
			{
				SplitStartIndex = RunningTextIndex;
				RunningHarfBuzzScript = InHarfBuzzScript;
			};

			auto AppendPendingTextToSequence = [&]()
			{
				if (RunningHarfBuzzScript.IsSet())
				{
					HarfBuzzTextSequenceEntry.SubSequence.Emplace(
						SplitStartIndex,					// InStartIndex
						RunningTextIndex - SplitStartIndex,	// InLength
						RunningHarfBuzzScript.GetValue()	// InHarfBuzzScript
						);

					RunningHarfBuzzScript.Reset();
				}
			};

			auto IsSpecialCharacter = [](const hb_script_t InHarfBuzzScript) -> bool
			{
				// Characters in the common, inherited, and unknown scripts are allowed (and in the case of inherited, required) to merge with the script 
				// of the character(s) that preceded them. This also helps to minimize shaping batches, as spaces are within the common script.
				return InHarfBuzzScript == HB_SCRIPT_COMMON || InHarfBuzzScript == HB_SCRIPT_INHERITED || InHarfBuzzScript == HB_SCRIPT_UNKNOWN;
			};

			const int32 TextEndIndex = HarfBuzzTextSequenceEntry.TextStartIndex + HarfBuzzTextSequenceEntry.TextLength;
			for (; RunningTextIndex < TextEndIndex; ++RunningTextIndex)
			{
				const hb_script_t CharHarfBuzzScript = hb_unicode_script(HarfBuzzUnicodeFuncs, InText[RunningTextIndex]);

				if (!RunningHarfBuzzScript.IsSet() || RunningHarfBuzzScript.GetValue() != CharHarfBuzzScript)
				{
					if (!RunningHarfBuzzScript.IsSet())
					{
						// Always start a new run if we're currently un-set
						StartNewPendingTextSequence(CharHarfBuzzScript);
					}
					else if (!IsSpecialCharacter(CharHarfBuzzScript))
					{
						if (IsSpecialCharacter(RunningHarfBuzzScript.GetValue()))
						{
							// If we started our run on a special character, we need to swap the script type to the non-special type as soon as we can
							RunningHarfBuzzScript = CharHarfBuzzScript;
						}
						else
						{
							// Transitioned a non-special character; end the current run and create a new one
							AppendPendingTextToSequence();
							StartNewPendingTextSequence(CharHarfBuzzScript);
						}
					}
				}
			}

			AppendPendingTextToSequence();
		}
	}

	if (InTextDirection == TextBiDi::ETextDirection::RightToLeft)
	{
		// Need to flip the sequence here to mimic what HarfBuzz would do if the text had been a single sequence of right-to-left text
		Algo::Reverse(HarfBuzzTextSequence);
		bBypassLetterSpacing = true;
	}

	// Step 3) Now we use HarfBuzz to shape each font data sequence using its FreeType glyph
	{
		SCOPE_CYCLE_COUNTER(STAT_TextSequencesProcessing);

		hb_buffer_t* HarfBuzzTextBuffer = hb_buffer_create();
		const FStringView InTextView = InText; // This will do a strlen, so do it once at the start
		const bool bUseLetterSpacing = !bBypassLetterSpacing && InFontInfo.LetterSpacing != 0;

		for (const FHarfBuzzTextSequenceEntry& HarfBuzzTextSequenceEntry : HarfBuzzTextSequence)
		{
			if (!HarfBuzzTextSequenceEntry.FaceAndMemory.IsValid())
			{
				continue;
			}

#if WITH_FREETYPE
			const bool bHasKerning = FT_HAS_KERNING(HarfBuzzTextSequenceEntry.FaceAndMemory->GetFace()) != 0 || bUseLetterSpacing;
#else  // WITH_FREETYPE
			const bool bHasKerning = false;
#endif // WITH_FREETYPE

			// Letter spacing should scale proportional to font size / 1000 (to roughly mimic Photoshop tracking)
			const int32 LetterSpacingScaledAsInt = bUseLetterSpacing ? InFontInfo.LetterSpacing * InFontInfo.Size / 1000 : 0;
			ensure(LetterSpacingScaledAsInt <= std::numeric_limits<int16>::max());
			const int16 LetterSpacingScaled = bUseLetterSpacing ? (int16)(LetterSpacingScaledAsInt) : 0;

			const hb_feature_t HarfBuzzFeatures[] = {
				{ HB_TAG('k','e','r','n'), bHasKerning, 0, uint32(-1) },
				{ HB_TAG('l','i','g','a'), LetterSpacingScaled == 0, 0, uint32(-1) } // Disable standard ligatures if we have non-zero letter spacing to allow the individual characters to flow freely
			};
			const int32 HarfBuzzFeaturesCount = UE_ARRAY_COUNT(HarfBuzzFeatures);

			uint32 GlyphFlags = 0;
			SlateFontRendererUtils::AppendGlyphFlags(*HarfBuzzTextSequenceEntry.FaceAndMemory, *HarfBuzzTextSequenceEntry.FontDataPtr, GlyphFlags);
			const float FinalFontScale = InFontScale * HarfBuzzTextSequenceEntry.SubFontScalingFactor;

			hb_font_t* HarfBuzzFont = HarfBuzzFontCache.CreateFont(*HarfBuzzTextSequenceEntry.FaceAndMemory, GlyphFlags, InFontInfo, FinalFontScale);
			if (!HarfBuzzFont)
			{
				continue;
			}
			const FFontRasterizationSettings FontRasterizationSettings = HarfBuzzTextSequenceEntry.FontDataPtr->GetFontRasterizationSettings();
			TSharedRef<FShapedGlyphFaceData> ShapedGlyphFaceData = MakeShared<FShapedGlyphFaceData>(HarfBuzzTextSequenceEntry.FaceAndMemory, 
																									GlyphFlags, 
																									InFontInfo.Size, 
																									FinalFontScale, 
																									InFontInfo.GetClampSkew(),
																									FontRasterizationSettings.Mode,
																									FontRasterizationSettings.DistanceFieldPpem);
			const uint32 FontRenderSize = FreeTypeUtils::ComputeFontPixelSize(InFontInfo.Size, FinalFontScale);
			TSharedPtr<FFreeTypeKerningCache> KerningCache = FTCacheDirectory->GetKerningCache(HarfBuzzTextSequenceEntry.FaceAndMemory->GetFace(), 
																							   FT_KERNING_DEFAULT,
																							   FontRenderSize);
			TSharedRef<FFreeTypeAdvanceCache> AdvanceCache = FTCacheDirectory->GetAdvanceCache(HarfBuzzTextSequenceEntry.FaceAndMemory->GetFace(), 
																							   GlyphFlags,
																							   FontRenderSize);

			for (const FHarfBuzzTextSequenceEntry::FSubSequenceEntry& HarfBuzzTextSubSequenceEntry : HarfBuzzTextSequenceEntry.SubSequence)
			{
				const int32 InitialNumGlyphsToRender = OutGlyphsToRender.Num();

				hb_buffer_set_cluster_level(HarfBuzzTextBuffer, HB_BUFFER_CLUSTER_LEVEL_MONOTONE_GRAPHEMES);
				hb_buffer_set_direction(HarfBuzzTextBuffer, (InTextDirection == TextBiDi::ETextDirection::LeftToRight) ? HB_DIRECTION_LTR : HB_DIRECTION_RTL);
				hb_buffer_set_script(HarfBuzzTextBuffer, HarfBuzzTextSubSequenceEntry.HarfBuzzScript);

				HarfBuzzUtils::AppendStringToBuffer(InTextView, HarfBuzzTextSubSequenceEntry.StartIndex, HarfBuzzTextSubSequenceEntry.Length, HarfBuzzTextBuffer);
				{
					SCOPE_CYCLE_COUNTER(STAT_HarfBuzzNativeShape);
					hb_shape(HarfBuzzFont, HarfBuzzTextBuffer, HarfBuzzFeatures, HarfBuzzFeaturesCount);
				}

				uint32 HarfBuzzGlyphCount = 0;
				hb_glyph_info_t* HarfBuzzGlyphInfos = hb_buffer_get_glyph_infos(HarfBuzzTextBuffer, &HarfBuzzGlyphCount);
				hb_glyph_position_t* HarfBuzzGlyphPositions = hb_buffer_get_glyph_positions(HarfBuzzTextBuffer, &HarfBuzzGlyphCount);

				OutGlyphsToRender.Reserve(OutGlyphsToRender.Num() + static_cast<int32>(HarfBuzzGlyphCount));
				for (uint32 HarfBuzzGlyphIndex = 0; HarfBuzzGlyphIndex < HarfBuzzGlyphCount; ++HarfBuzzGlyphIndex)
				{
					const hb_glyph_info_t& HarfBuzzGlyphInfo = HarfBuzzGlyphInfos[HarfBuzzGlyphIndex];
					const hb_glyph_position_t& HarfBuzzGlyphPosition = HarfBuzzGlyphPositions[HarfBuzzGlyphIndex];

					const int32 CurrentCharIndex = static_cast<int32>(HarfBuzzGlyphInfo.cluster);
					const TCHAR CurrentChar = InText[CurrentCharIndex];
					if (!InsertSubstituteGlyphs(InText, CurrentCharIndex, ShapedGlyphFaceData, AdvanceCache, OutGlyphsToRender, LetterSpacingScaled))
					{
						const int32 CurrentGlyphEntryIndex = OutGlyphsToRender.AddDefaulted();
						FShapedGlyphEntry& ShapedGlyphEntry = OutGlyphsToRender[CurrentGlyphEntryIndex];
						ShapedGlyphEntry.FontFaceData = ShapedGlyphFaceData;
						ShapedGlyphEntry.GlyphIndex = HarfBuzzGlyphInfo.codepoint;
						ShapedGlyphEntry.SourceIndex = CurrentCharIndex;
						ShapedGlyphEntry.XAdvance = FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>(HarfBuzzGlyphPosition.x_advance);
						ShapedGlyphEntry.YAdvance = -FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>(HarfBuzzGlyphPosition.y_advance);
						ShapedGlyphEntry.XOffset = FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>(HarfBuzzGlyphPosition.x_offset);
						ShapedGlyphEntry.YOffset = -FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>(HarfBuzzGlyphPosition.y_offset);
						ShapedGlyphEntry.Kerning = 0;
						ShapedGlyphEntry.NumCharactersInGlyph = 0; // Filled in later once we've processed each cluster
						ShapedGlyphEntry.NumGraphemeClustersInGlyph = 0; // Filled in later once we have an accurate character count
						ShapedGlyphEntry.TextDirection = InTextDirection;
						ShapedGlyphEntry.bIsVisible = !RenderCodepointAsWhitespace(CurrentChar);

						// Apply the letter spacing and font kerning against the previous entry
						if (CurrentGlyphEntryIndex > 0 && bHasKerning)
						{
							FShapedGlyphEntry& PreviousShapedGlyphEntry = OutGlyphsToRender[CurrentGlyphEntryIndex - 1];

							if (LetterSpacingScaled != 0)
							{
								PreviousShapedGlyphEntry.XAdvance += LetterSpacingScaled;
							}
#if WITH_FREETYPE
							if (ShapedGlyphEntry.bIsVisible)
							{
								FT_Vector KerningVector;
								if (KerningCache && KerningCache->FindOrCache(PreviousShapedGlyphEntry.GlyphIndex, ShapedGlyphEntry.GlyphIndex, KerningVector))
								{
									PreviousShapedGlyphEntry.Kerning = FreeTypeUtils::Convert26Dot6ToRoundedPixel<int8>(KerningVector.x);
								}
							}
#endif // WITH_FREETYPE
						}
					}
				}

				hb_buffer_clear_contents(HarfBuzzTextBuffer);

				// Count the characters and grapheme clusters that belong to each glyph in this sub-sequence (if they haven't already been set)
				{
					const int32 NumGlyphsRendered = OutGlyphsToRender.Num() - InitialNumGlyphsToRender;
					if (NumGlyphsRendered > 0)
					{
						auto ConditionalUpdateGlyphCountsForRange = [this, InTextStart](FShapedGlyphEntry& ShapedGlyphEntry, const int32 TextStartIndex, const int32 TextEndIndex)
						{
							check(TextStartIndex <= TextEndIndex);

							if (ShapedGlyphEntry.NumCharactersInGlyph == 0 && ShapedGlyphEntry.NumGraphemeClustersInGlyph == 0)
							{
								ensure(TextEndIndex - TextStartIndex <= std::numeric_limits<uint8>::max());
								ShapedGlyphEntry.NumCharactersInGlyph = (uint8)(TextEndIndex - TextStartIndex);

								if (ShapedGlyphEntry.NumCharactersInGlyph > 0)
								{
									const int32 FirstCharacterIndex = TextStartIndex - InTextStart;
									const int32 LastCharacterIndex = TextEndIndex - InTextStart;

									// Only count grapheme clusters if this glyph starts on a grapheme boundary
									int32 PreviousBreak = FirstCharacterIndex;
									{
										GraphemeBreakIterator->MoveToCandidateAfter(FirstCharacterIndex);
										PreviousBreak = GraphemeBreakIterator->MoveToPrevious();
									}

									if (PreviousBreak == FirstCharacterIndex)
									{
										int32 CurrentBreak = LastCharacterIndex;
										for (CurrentBreak = GraphemeBreakIterator->MoveToCandidateAfter(FirstCharacterIndex);
											CurrentBreak != INDEX_NONE;
											CurrentBreak = GraphemeBreakIterator->MoveToNext()
											)
										{
											++ShapedGlyphEntry.NumGraphemeClustersInGlyph;
											if (CurrentBreak >= LastCharacterIndex)
											{
												break;
											}
										}

										// Only count grapheme clusters if this glyph ends on a grapheme boundary
										if (CurrentBreak != LastCharacterIndex)
										{
											ShapedGlyphEntry.NumGraphemeClustersInGlyph = 0;
										}
									}
								}
							}
						};

						auto GetNextGlyphToRenderIndex = [&OutGlyphsToRender](const int32 GlyphToRenderIndex) -> int32
						{
							FShapedGlyphEntry& ShapedGlyphEntry = OutGlyphsToRender[GlyphToRenderIndex];
							int32 NextGlyphToRenderIndex = GlyphToRenderIndex + 1;

							// Walk forward to find the first glyph in the next cluster; the number of characters in this glyph is the difference between their two source indices
							for (; NextGlyphToRenderIndex < OutGlyphsToRender.Num(); ++NextGlyphToRenderIndex)
							{
								const FShapedGlyphEntry& NextShapedGlyphEntry = OutGlyphsToRender[NextGlyphToRenderIndex];
								if (ShapedGlyphEntry.SourceIndex != NextShapedGlyphEntry.SourceIndex)
								{
									break;
								}
							}

							return NextGlyphToRenderIndex;
						};

						// The glyphs in the array are in render order, so LTR and RTL text use different start and end points in the source string
						if (InTextDirection == TextBiDi::ETextDirection::LeftToRight)
						{
							for (int32 GlyphToRenderIndex = InitialNumGlyphsToRender; GlyphToRenderIndex < OutGlyphsToRender.Num();)
							{
								FShapedGlyphEntry& ShapedGlyphEntry = OutGlyphsToRender[GlyphToRenderIndex];

								const int32 NextGlyphToRenderIndex = GetNextGlyphToRenderIndex(GlyphToRenderIndex);
								if (NextGlyphToRenderIndex < OutGlyphsToRender.Num())
								{
									const FShapedGlyphEntry& NextShapedGlyphEntry = OutGlyphsToRender[NextGlyphToRenderIndex];
									ConditionalUpdateGlyphCountsForRange(ShapedGlyphEntry, ShapedGlyphEntry.SourceIndex, NextShapedGlyphEntry.SourceIndex);
								}
								else
								{
									ConditionalUpdateGlyphCountsForRange(ShapedGlyphEntry, ShapedGlyphEntry.SourceIndex, HarfBuzzTextSubSequenceEntry.StartIndex + HarfBuzzTextSubSequenceEntry.Length);
								}

								GlyphToRenderIndex = NextGlyphToRenderIndex;
							}
						}
						else
						{
							int32 PreviousSourceIndex = HarfBuzzTextSubSequenceEntry.StartIndex + HarfBuzzTextSubSequenceEntry.Length;
							for (int32 GlyphToRenderIndex = InitialNumGlyphsToRender; GlyphToRenderIndex < OutGlyphsToRender.Num();)
							{
								FShapedGlyphEntry& ShapedGlyphEntry = OutGlyphsToRender[GlyphToRenderIndex];

								ConditionalUpdateGlyphCountsForRange(ShapedGlyphEntry, ShapedGlyphEntry.SourceIndex, PreviousSourceIndex);
								GlyphToRenderIndex = GetNextGlyphToRenderIndex(GlyphToRenderIndex);
								PreviousSourceIndex = ShapedGlyphEntry.SourceIndex;
							}
						}
					}
				}
			}

			hb_font_destroy(HarfBuzzFont);
		}

		hb_buffer_destroy(HarfBuzzTextBuffer);
	}

	GraphemeBreakIterator->ClearString();
}

#endif // WITH_HARFBUZZ

bool FSlateTextShaper::InsertSubstituteGlyphs(const TCHAR* InText, const int32 InCharIndex, const TSharedRef<FShapedGlyphFaceData>& InShapedGlyphFaceData, const TSharedRef<FFreeTypeAdvanceCache>& AdvanceCache, TArray<FShapedGlyphEntry>& OutGlyphsToRender, const int16 InLetterSpacingScaled)
{
	auto GetSpecifiedGlyphIndexAndAdvance = [&InShapedGlyphFaceData, &AdvanceCache](TCHAR Char, uint32& OutSpaceGlyphIndex, int16& OutSpaceXAdvance)
	{
		OutSpaceGlyphIndex = 0;
		OutSpaceXAdvance = 0;
#if WITH_FREETYPE
		{
			TSharedPtr<FFreeTypeFace> FTFace = InShapedGlyphFaceData->FontFace.Pin();
			if (FTFace.IsValid())
			{
				OutSpaceGlyphIndex = FT_Get_Char_Index(FTFace->GetFace(), Char);

				FT_Fixed CachedAdvanceData = 0;
				if (AdvanceCache->FindOrCache(OutSpaceGlyphIndex, CachedAdvanceData))
				{
					OutSpaceXAdvance = FreeTypeUtils::Convert26Dot6ToRoundedPixel<int16>((CachedAdvanceData + (1<<9)) >> 10);
				}
			}
		}
#endif // WITH_FREETYPE
	};

	const TCHAR Char = InText[InCharIndex];

	if (TextBiDi::IsControlCharacter(Char) || Char == TEXT('\u200B'))	// Zero Width Space
	{
		// We insert a stub entry for control characters and zero-width spaces to avoid them being drawn as a visual glyph with size
		FShapedGlyphEntry& ShapedGlyphEntry = OutGlyphsToRender.AddDefaulted_GetRef();
		ShapedGlyphEntry.FontFaceData = InShapedGlyphFaceData;
		ShapedGlyphEntry.GlyphIndex = 0;
		ShapedGlyphEntry.SourceIndex = InCharIndex;
		ShapedGlyphEntry.XAdvance = 0;
		ShapedGlyphEntry.YAdvance = 0;
		ShapedGlyphEntry.XOffset = 0;
		ShapedGlyphEntry.YOffset = 0;
		ShapedGlyphEntry.Kerning = 0;
		ShapedGlyphEntry.NumCharactersInGlyph = 1;
		ShapedGlyphEntry.NumGraphemeClustersInGlyph = 1;
		ShapedGlyphEntry.TextDirection = TextBiDi::ETextDirection::LeftToRight;
		ShapedGlyphEntry.bIsVisible = false;
		return true;
	}
	
	if (Char == TEXT('\t'))
	{
		uint32 SpaceGlyphIndex = 0;
		int16 SpaceXAdvance = 0;
		GetSpecifiedGlyphIndexAndAdvance(TEXT(' '), SpaceGlyphIndex, SpaceXAdvance);

		// We insert a spacer glyph with (up-to) the width of 4 space glyphs in-place of a tab character
		// TODO: Tabulation handling should be refactored to work properly: the tabbing is currently relative to the last characters,
		// while it should be relative to the beginning of a line). This existing implementation work properly only with leading tabulation.
		static const int16 TabWidthInSpaces = 4;
		int16 NumSpacesToIgnore = 0;
		for (int32 Idx = FMath::Max(0, InCharIndex - TabWidthInSpaces); Idx < InCharIndex; ++Idx)
		{
			NumSpacesToIgnore = InText[Idx] == TEXT('\t') ? 0 : (NumSpacesToIgnore + 1) % TabWidthInSpaces;
		}

		const int16 NumSpacesToInsert = TabWidthInSpaces - NumSpacesToIgnore;
		if (NumSpacesToInsert > 0)
		{
			FShapedGlyphEntry& ShapedGlyphEntry = OutGlyphsToRender.AddDefaulted_GetRef();
			ShapedGlyphEntry.FontFaceData = InShapedGlyphFaceData;
			ShapedGlyphEntry.GlyphIndex = SpaceGlyphIndex;
			ShapedGlyphEntry.SourceIndex = InCharIndex;
			int32 XAdvance = (SpaceXAdvance + InLetterSpacingScaled) * NumSpacesToInsert;
			ensure(XAdvance <= std::numeric_limits<int16>::max());
			ShapedGlyphEntry.XAdvance = (int16)XAdvance;
			ShapedGlyphEntry.YAdvance = 0;
			ShapedGlyphEntry.XOffset = 0;
			ShapedGlyphEntry.YOffset = 0;
			ShapedGlyphEntry.Kerning = 0;
			ShapedGlyphEntry.NumCharactersInGlyph = 1;
			ShapedGlyphEntry.NumGraphemeClustersInGlyph = 1;
			ShapedGlyphEntry.TextDirection = TextBiDi::ETextDirection::LeftToRight;
			ShapedGlyphEntry.bIsVisible = false;
		}

		return true;
	}

	if (Char == TEXT('\u2009') ||	// Thin Space
		Char == TEXT('\u202F') ||	// Narrow No-Break Space
		Char == TEXT('\u2026'))		// Ellipsis character
	{
		// Not all fonts support these characters
#if WITH_FREETYPE
		{
			TSharedPtr<FFreeTypeFace> FTFace = InShapedGlyphFaceData->FontFace.Pin();
			if (FTFace.IsValid())
			{
				const uint32 GlyphIndex = FT_Get_Char_Index(FTFace->GetFace(), Char);
				if (GlyphIndex != 0)
				{
					// If it does, then let it render the character itself as it may have better metrics
					return false;
				}
			}
		}
#endif // WITH_FREETYPE

		if (Char == TEXT('\u2026'))
		{
			// If the ellipsis character doesn't exist in the font replace with 3 dots
			uint32 DotGlyphIndex = 0;
			int16 DotXAdvance = 0;
			GetSpecifiedGlyphIndexAndAdvance(TEXT('.'), DotGlyphIndex, DotXAdvance);

			// Insert 3 dots
			for (int32 Dot = 0; Dot < 3; ++Dot)
			{
				FShapedGlyphEntry& ShapedGlyphEntry = OutGlyphsToRender.AddDefaulted_GetRef();
				ShapedGlyphEntry.FontFaceData = InShapedGlyphFaceData;
				ShapedGlyphEntry.GlyphIndex = DotGlyphIndex;
				ShapedGlyphEntry.SourceIndex = InCharIndex;
				ShapedGlyphEntry.XAdvance = DotXAdvance + InLetterSpacingScaled;
				ShapedGlyphEntry.YAdvance = 0;
				ShapedGlyphEntry.XOffset = 0;
				ShapedGlyphEntry.YOffset = 0;
				ShapedGlyphEntry.Kerning = 0;
				ShapedGlyphEntry.NumCharactersInGlyph = 1; // should this be 3 instead of a loop three times?
				ShapedGlyphEntry.NumGraphemeClustersInGlyph = 1;
				ShapedGlyphEntry.TextDirection = TextBiDi::ETextDirection::LeftToRight;
				ShapedGlyphEntry.bIsVisible = true;
			}
		}
		else
		{
			// If it doesn't, then make these 2/3rd the width of a normal space
			uint32 SpaceGlyphIndex = 0;
			int16 SpaceXAdvance = 0;
			GetSpecifiedGlyphIndexAndAdvance(TEXT(' '), SpaceGlyphIndex, SpaceXAdvance);

			FShapedGlyphEntry& ShapedGlyphEntry = OutGlyphsToRender.AddDefaulted_GetRef();
			ShapedGlyphEntry.FontFaceData = InShapedGlyphFaceData;
			ShapedGlyphEntry.GlyphIndex = SpaceGlyphIndex;
			ShapedGlyphEntry.SourceIndex = InCharIndex;
			ShapedGlyphEntry.XAdvance = ((SpaceXAdvance + InLetterSpacingScaled) * 2) / 3;
			ShapedGlyphEntry.YAdvance = 0;
			ShapedGlyphEntry.XOffset = 0;
			ShapedGlyphEntry.YOffset = 0;
			ShapedGlyphEntry.Kerning = 0;
			ShapedGlyphEntry.NumCharactersInGlyph = 1;
			ShapedGlyphEntry.NumGraphemeClustersInGlyph = 1;
			ShapedGlyphEntry.TextDirection = TextBiDi::ETextDirection::LeftToRight;
			ShapedGlyphEntry.bIsVisible = false;
		}
		return true;
	}

	return false;
}
