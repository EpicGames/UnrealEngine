// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/FontCache.h"
#include "Fonts/FontCacheFreeType.h"
#include "Fonts/FontCacheHarfBuzz.h"
#include "Fonts/FontRasterizationMode.h"

class FCompositeFontCache;
class FSlateFontRenderer;
class IBreakIterator;

/**
 * Internal class used to hold the FreeType data within the shaped glyph sequence.
 */
class FShapedGlyphFaceData
{
public:
	FShapedGlyphFaceData(TWeakPtr<FFreeTypeFace> InFontFace, const uint32 InGlyphFlags, const float InFontSize, const float InFontScale, const float InFontSkew, const EFontRasterizationMode InRasterizationMode, const int16 InSdfPpem)
		: FontFace(MoveTemp(InFontFace))
		, GlyphFlags(InGlyphFlags)
		, FontSize(InFontSize)
		, FontScale(InFontScale)
		, BitmapRenderScale(1.0f)
		, FontSkew(InFontSkew)
		, RasterizationMode(InRasterizationMode)
		, SdfPpem(InSdfPpem)
	{
#if WITH_FREETYPE
		if (TSharedPtr<FFreeTypeFace> FontFacePin = FontFace.Pin())
		{
			BitmapRenderScale = FontFacePin->GetBitmapRenderScale();
		}
#endif	// WITH_FREETYPE
	}

	/** Weak pointer to the FreeType face to render with */
	TWeakPtr<FFreeTypeFace> FontFace;
	/** Provides the glyph flags used to render the font */
	uint32 GlyphFlags;
	/** Provides the point size used to render the font */
	float FontSize;
	/** Provides the final scale used to render to the font */
	float FontScale;
	/** Any additional scale that should be applied when rendering this glyph */
	float BitmapRenderScale;
	/** The skew transform amount for the rendered font */
	float FontSkew;
	/** The glyph rasterization mode (bitmap / distance field) */
	EFontRasterizationMode RasterizationMode;
	/** Distance field px/em resolution, not used if RasterizationMode is Bitmap */
	int16 SdfPpem;

};


/**
 * Bridging point between HarfBuzz and the Slate font system.
 * This class, via the instances you pass to its constructor, knows how to correctly shape a Slate font.
 */
class FSlateTextShaper
{
public:
	FSlateTextShaper(FFreeTypeCacheDirectory* InFTCacheDirectory, FCompositeFontCache* InCompositeFontCache, FSlateFontRenderer* InFontRenderer, FSlateFontCache* InFontCache);

	FShapedGlyphSequenceRef ShapeBidirectionalText(const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo& InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InBaseDirection, const ETextShapingMethod TextShapingMethod, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces);

	FShapedGlyphSequenceRef ShapeUnidirectionalText(const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo& InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InTextDirection, const ETextShapingMethod TextShapingMethod, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces);

	/** Get rid of the harfbuzz font cache, wich store the harfbuzz fonts. */
	void FlushCache();

private:
	void PerformTextShaping(const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo& InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InTextDirection, const ETextShapingMethod TextShapingMethod, TArray<FShapedGlyphEntry>& OutGlyphsToRender, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces);

	FShapedGlyphSequenceRef FinalizeTextShaping(TArray<FShapedGlyphEntry> InGlyphsToRender, const FSlateFontInfo& InFontInfo, const float InFontScale, const FShapedGlyphSequence::FSourceTextRange& InSourceTextRange) const;

#if WITH_FREETYPE
	void PerformKerningOnlyTextShaping(const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo& InFontInfo, const float InFontScale, TArray<FShapedGlyphEntry>& OutGlyphsToRender, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces) const;
#endif // WITH_FREETYPE

#if WITH_HARFBUZZ
	void PerformHarfBuzzTextShaping(const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo& InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InTextDirection, TArray<FShapedGlyphEntry>& OutGlyphsToRender, TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces);
#endif // WITH_HARFBUZZ

	static bool InsertSubstituteGlyphs(const TCHAR* InText, const int32 InCharIndex, const TSharedRef<FShapedGlyphFaceData>& InShapedGlyphFaceData, const TSharedRef<FFreeTypeAdvanceCache>& AdvanceCache, TArray<FShapedGlyphEntry>& OutGlyphsToRender, const int16 InLetterSpacingScaled = 0);

	FFreeTypeCacheDirectory* FTCacheDirectory;
	FCompositeFontCache* CompositeFontCache;
	FSlateFontRenderer* FontRenderer;
	FSlateFontCache* FontCache;

	/** Unicode BiDi text detection */
	TUniquePtr<TextBiDi::ITextBiDi> TextBiDiDetection;

	/** Iterator to use to detect grapheme cluster boundaries */
	TSharedRef<IBreakIterator> GraphemeBreakIterator;

#if WITH_HARFBUZZ
	/** HarfBuzz font cache (using our cached functions rather than FreeType directly) */
	FHarfBuzzFontCache HarfBuzzFontCache;
#endif // WITH_HARFBUZZ
};
