// Copyright Epic Games, Inc. All Rights Reserved.

#include "Fonts/UnicodeBlockRange.h"

TArrayView<const FUnicodeBlockRange> FUnicodeBlockRange::GetUnicodeBlockRanges()
{
	static const FUnicodeBlockRange UnicodeBlockRanges[] = {
		#define REGISTER_UNICODE_BLOCK_RANGE(LOWERBOUND, UPPERBOUND, SYMBOLNAME, LOCTEXT_KEY, LOCTEXT_LITERAL) { EUnicodeBlockRange::SYMBOLNAME, TEXT(LOCTEXT_KEY), TEXT(LOCTEXT_LITERAL), LOWERBOUND, UPPERBOUND },
		#include "Fonts/UnicodeBlockRange.inl"
		#undef REGISTER_UNICODE_BLOCK_RANGE
	};

	return UnicodeBlockRanges;
}

FUnicodeBlockRange FUnicodeBlockRange::GetUnicodeBlockRange(const EUnicodeBlockRange InBlockIndex)
{
	TArrayView<const FUnicodeBlockRange> UnicodeBlockRanges = GetUnicodeBlockRanges();
	return UnicodeBlockRanges[(int32)InBlockIndex];
}
