// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/DragAndDrop.h"
#include "Layout/Visibility.h"
#include "Application/SlateApplicationBase.h"
#include "Widgets/SWindow.h"


/* FDragDropOperation structors
 *****************************************************************************/

FDragDropOperation::FDragDropOperation()
	: bCreateNewWindow(true)
{
}

FDragDropOperation::~FDragDropOperation()
{
	DestroyCursorDecoratorWindow();
}


/* 
 *****************************************************************************/

void FDragDropOperation::OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent )
{
	DestroyCursorDecoratorWindow();
}

void FDragDropOperation::OnDragged( const class FDragDropEvent& DragDropEvent )
{
	if (CursorDecoratorWindow.IsValid())
	{
		FSlateApplicationBase& SlateApplication = FSlateApplicationBase::Get();
						
		FVector2f Position = DragDropEvent.GetScreenSpacePosition() + FSlateApplicationBase::Get().GetCursorSize();

		if (Position != CursorDecoratorWindow->GetPositionInScreen())
		{
			FSlateRect Anchor(Position.X, Position.Y, Position.X, Position.Y);
			Position = SlateApplication.CalculateTooltipWindowPosition(Anchor, CursorDecoratorWindow->GetDesiredSizeDesktopPixels(), false);
		}

		CursorDecoratorWindow->MoveWindowTo(Position);
	}
}

FCursorReply FDragDropOperation::OnCursorQuery()
{
	if ( MouseCursorOverride.IsSet() )
	{
		return FCursorReply::Cursor( MouseCursorOverride.GetValue() );
	}

	if ( MouseCursor.IsSet() )
	{
		return FCursorReply::Cursor(MouseCursor.GetValue());
	}

	return FCursorReply::Unhandled();
}

void FDragDropOperation::SetDecoratorVisibility(bool bVisible)
{
	if (CursorDecoratorWindow.IsValid())
	{
		if (bVisible)
		{
			CursorDecoratorWindow->ShowWindow();
		}
		else
		{
			CursorDecoratorWindow->HideWindow();
		}
	}
	else if ( bCreateNewWindow == false )
	{
		TSharedPtr<SWidget> DecoratorToUse = GetDefaultDecorator();

		if ( DecoratorToUse.IsValid() )
		{
			if ( bVisible )
			{
				DecoratorToUse->SetVisibility(EVisibility::HitTestInvisible);
			}
			else
			{
				DecoratorToUse->SetVisibility(EVisibility::Hidden);
			}
		}
	}
}

void FDragDropOperation::SetCursorOverride( TOptional<EMouseCursor::Type> CursorType )
{
	MouseCursorOverride = CursorType;
}


/* FDragDropOperation implementation
 *****************************************************************************/

void FDragDropOperation::Construct()
{
	if ( bCreateNewWindow )
	{
		CreateCursorDecoratorWindow();
	}
}

void FDragDropOperation::CreateCursorDecoratorWindow()
{
	TSharedPtr<SWidget> DecoratorToUse = GetDefaultDecorator();

	if ( DecoratorToUse.IsValid() )
	{
		CursorDecoratorWindow = SWindow::MakeCursorDecorator();
		CursorDecoratorWindow->SetContent(DecoratorToUse.ToSharedRef());

		FSlateApplicationBase::Get().AddWindow(CursorDecoratorWindow.ToSharedRef(), true);
	}
}

void FDragDropOperation::DestroyCursorDecoratorWindow()
{
	if (CursorDecoratorWindow.IsValid())
	{
		CursorDecoratorWindow->RequestDestroyWindow();
		CursorDecoratorWindow.Reset();
	}
}


/* FExternalDragOperation implementation
 *****************************************************************************/

TSharedRef<FExternalDragOperation> FExternalDragOperation::NewText( FString InText )
{
	TSharedRef<FExternalDragOperation> Operation = MakeShareable(new FExternalDragOperation);
	Operation->DragType = DragText;
	Operation->DraggedText = MoveTemp(InText);
	Operation->Construct();
	return Operation;
}

TSharedRef<FExternalDragOperation> FExternalDragOperation::NewFiles( TArray<FString> InFileNames )
{
	TSharedRef<FExternalDragOperation> Operation = MakeShareable(new FExternalDragOperation);
	Operation->DragType = DragFiles;
	Operation->DraggedFileNames = MoveTemp(InFileNames);
	Operation->Construct();
	return Operation;
}

TSharedRef<FExternalDragOperation> FExternalDragOperation::NewOperation( FString InText, TArray<FString> InFileNames )
{
	TSharedRef<FExternalDragOperation> Operation = MakeShareable(new FExternalDragOperation);
	Operation->DragType = 0 | (InText.IsEmpty() ? 0 : DragText) | (InFileNames.Num() == 0 ? 0 : DragFiles);
	Operation->DraggedText = MoveTemp(InText);
	Operation->DraggedFileNames = MoveTemp(InFileNames);
	Operation->Construct();
	return Operation;
}


/* FGameDragDropOperation implementation
*****************************************************************************/

FGameDragDropOperation::FGameDragDropOperation()
	: DecoratorPosition(0,0)
{
	bCreateNewWindow = false;
}

FVector2D FGameDragDropOperation::GetDecoratorPosition() const
{
	return DecoratorPosition;
}
