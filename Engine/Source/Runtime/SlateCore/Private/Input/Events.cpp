// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/Events.h"
#include "Layout/ArrangedWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Layout/WidgetPath.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Events)


/* Static initialization
 *****************************************************************************/

const FTouchKeySet FTouchKeySet::StandardSet(EKeys::LeftMouseButton);
const FTouchKeySet FTouchKeySet::EmptySet(EKeys::Invalid);

FInputEvent::~FInputEvent() = default;

FGeometry FInputEvent::FindGeometry(const TSharedRef<SWidget>& WidgetToFind) const
{
	return EventPath->FindArrangedWidget(WidgetToFind).Get(FArrangedWidget::GetNullWidget()).Geometry;
}

TSharedRef<SWindow> FInputEvent::GetWindow() const
{
	return EventPath->GetWindow();
}

FText FInputEvent::ToText() const
{
	return NSLOCTEXT("Events", "Unimplemented", "Unimplemented");
}

bool FInputEvent::IsPointerEvent() const
{
	return false;
}

bool FInputEvent::IsKeyEvent() const
{
	return false;
}

FCharacterEvent::~FCharacterEvent() = default;

FText FCharacterEvent::ToText() const
{
	return FText::Format( NSLOCTEXT("Events", "Char", "Char({0})"), FText::FromString(FString::ConstructFromPtrSize(&Character, 1)) );
}

FKeyEvent::FKeyEvent()
	: FInputEvent(FModifierKeysState(), 0, false)
	, Key()
	, CharacterCode(0)
	, KeyCode(0)
{
}

FKeyEvent::FKeyEvent(const FKey InKey,
	const FModifierKeysState& InModifierKeys,
	const uint32 InUserIndex,
	const bool bInIsRepeat,
	const uint32 InCharacterCode,
	const uint32 InKeyCode
)
	: FInputEvent(InModifierKeys, InUserIndex, bInIsRepeat)
	, Key(InKey)
	, CharacterCode(InCharacterCode)
	, KeyCode(InKeyCode)
{
}

FKeyEvent::FKeyEvent(const FKey InKey,
	const FModifierKeysState& InModifierKeys,
	const FInputDeviceId InDeviceId,
	const bool bInIsRepeat,
	const uint32 InCharacterCode,
	const uint32 InKeyCode,
	const TOptional<int32> InOptionalSlateUserIndex
)
	: FInputEvent(InModifierKeys, InDeviceId, bInIsRepeat)
	, Key(InKey)
	, CharacterCode(InCharacterCode)
	, KeyCode(InKeyCode)
{
	if (InOptionalSlateUserIndex.IsSet())
	{
		UserIndex = InOptionalSlateUserIndex.GetValue();
	}
}

FKeyEvent::~FKeyEvent() = default;

FText FKeyEvent::ToText() const
{
	return FText::Format( NSLOCTEXT("Events", "Key", "Key({0})"), Key.GetDisplayName() );
}

bool FKeyEvent::IsKeyEvent() const
{
	return true;
}

FAnalogInputEvent::~FAnalogInputEvent() = default;

FText FAnalogInputEvent::ToText() const
{
	return FText::Format(NSLOCTEXT("Events", "AnalogInput", "AnalogInput(key:{0}, value:{1}"), GetKey().GetDisplayName(), AnalogValue);
}

FPointerEvent::~FPointerEvent() = default;

FText FPointerEvent::ToText() const
{
	return FText::Format( NSLOCTEXT("Events", "Pointer", "Pointer(key:{0}, pos:{1}x{2}, delta:{3}x{4})"), EffectingButton.GetDisplayName(), ScreenSpacePosition.X, ScreenSpacePosition.Y, CursorDelta.X, CursorDelta.Y);
}

bool FPointerEvent::IsPointerEvent() const
{
	return true;
}

