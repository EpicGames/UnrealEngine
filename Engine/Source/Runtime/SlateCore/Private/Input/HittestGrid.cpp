// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/HittestGrid.h"
#include "Input/NavigationMethod.h"
#include "Input/NavigationMethodOrthogonal.h"
#include "Input/NavigationMethodProximity.h"
#include "Input/NavigationRouting.h"
#include "Rendering/RenderingCommon.h"
#include "SlateGlobals.h"
#include "StructUtils/InstancedStruct.h"
#include "HAL/IConsoleManager.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"
#include "Types/NavigationMetaData.h"

DEFINE_LOG_CATEGORY_STATIC(LogHittestDebug, Display, All);

#define UE_SLATE_ENABLE_HITTEST_STATS !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#if UE_SLATE_ENABLE_HITTEST_STATS
DECLARE_CYCLE_STAT(TEXT("HitTestGrid AddWidget"), STAT_SlateHTG_AddWidget, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("HitTestGrid RemoveWidget"), STAT_SlateHTG_RemoveWidget, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("HitTestGrid Clear"), STAT_SlateHTG_Clear, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("HitTestGrid GetCollapsedWidgets"), STAT_SlateHTG_GetCollapsedWidgets, STATGROUP_Slate);
#endif

#define LOCTEXT_NAMESPACE "HittestGrid"
#define UE_SLATE_HITTESTGRID_ARRAYSIZEMAX 0

#if UE_SLATE_HITTESTGRID_ARRAYSIZEMAX
int32 HittestGrid_CollapsedHittestGridArraySizeMax = 0;
int32 HittestGrid_CollapsedWidgetsArraySizeMax = 0;
#endif

static bool GEnableNavigationExperimental = false;
static FAutoConsoleVariableRef CVarEnableNavigationExperimental(
	TEXT("Slate.Navigation.Experimental"),
	GEnableNavigationExperimental,
	TEXT("If true, will use experimental navigation system"));

#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
FHittestGrid::FDebuggingFindNextFocusableWidget FHittestGrid::OnFindNextFocusableWidgetExecuted;
#endif //WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING

//
// Helper Functions
//

FVector2f ClosestPointOnSlateRotatedRect(const FVector2f &Point, const FSlateRotatedRect& RotatedRect)
{
	//no need to do any testing if we are inside of the rect
	if (RotatedRect.IsUnderLocation(Point))
	{
		return Point;
	}

	const static int32 NumOfCorners = 4;
	FVector2f Corners[NumOfCorners];
	Corners[0] = RotatedRect.TopLeft;
	Corners[1] = Corners[0] + RotatedRect.ExtentX;
	Corners[2] = Corners[1] + RotatedRect.ExtentY;
	Corners[3] = Corners[0] + RotatedRect.ExtentY;

	FVector2f RetPoint = FVector2f::ZeroVector;
	float ClosestDistSq = FLT_MAX;
	for (int32 i = 0; i < NumOfCorners; ++i)
	{
		//grab the closest point along the line segment
		const FVector2f ClosestPoint = FMath::ClosestPointOnSegment2D(Point, Corners[i], Corners[(i + 1) % NumOfCorners]);

		//get the distance between the two
		const float TestDist = FVector2f::DistSquared(Point, ClosestPoint);

		//if the distance is smaller than the current smallest, update our closest
		if (TestDist < ClosestDistSq)
		{
			RetPoint = FVector2f(UE_REAL_TO_FLOAT(ClosestPoint.X), UE_REAL_TO_FLOAT(ClosestPoint.Y));
			ClosestDistSq = TestDist;
		}
	}

	return RetPoint;
}

FORCEINLINE float DistanceSqToSlateRotatedRect(const FVector2f &Point, const FSlateRotatedRect& RotatedRect)
{
	return FVector2f::DistSquared(ClosestPointOnSlateRotatedRect(Point, RotatedRect), Point);
}

FORCEINLINE bool IsOverlappingSlateRotatedRect(const FVector2f& Point, const float Radius, const FSlateRotatedRect& RotatedRect)
{
	return DistanceSqToSlateRotatedRect( Point, RotatedRect ) <= (Radius * Radius);
}

bool ContainsInteractableWidget(const TArray<FWidgetAndPointer>& PathToTest)
{
	for (int32 i = PathToTest.Num() - 1; i >= 0; --i)
	{
		const FWidgetAndPointer& WidgetAndPointer = PathToTest[i];
		if (WidgetAndPointer.Widget->IsInteractable())
		{
			return true;
		}
	}
	return false;
}


const FVector2f FHittestGrid::CellSize(128.0f, 128.0f);

//
// FHittestGrid::FWidgetIndex
//
const FHittestGrid::FWidgetData& FHittestGrid::FWidgetIndex::GetWidgetData() const
{
	check(Grid->WidgetArray.IsValidIndex(WidgetIndex));
	return Grid->WidgetArray[WidgetIndex];
}

//
// FHittestGrid::FGridTestingParams
//
struct FHittestGrid::FGridTestingParams
{
	/** Ctor */
	FGridTestingParams()
	: CellCoord(-1, -1)
	, CursorPositionInGrid(FVector2f::ZeroVector)
	, Radius(-1.0f)
	, bTestWidgetIsInteractive(false)
	{}

	FIntPoint CellCoord;
	FVector2f CursorPositionInGrid;
	float Radius;
	bool bTestWidgetIsInteractive;
};

//
// FHittestGrid::FCell
//

void FHittestGrid::FCell::AddIndex(int32 WidgetIndex)
{
	check(!WidgetIndexes.Contains(WidgetIndex));
	WidgetIndexes.Add(WidgetIndex);
}

void FHittestGrid::FCell::RemoveIndex(int32 WidgetIndex)
{
	WidgetIndexes.RemoveSingleSwap(WidgetIndex);
}

void FHittestGrid::FCell::Reset()
{
	WidgetIndexes.Reset();
}

//
// FHittestGrid
//

FHittestGrid::FHittestGrid()
	: WidgetMap()
	, WidgetArray()
	, Cells()
	, AppendedGridArray()
	, Owner(nullptr)
	, CullingRect()
	, NumCells(0, 0)
	, GridOrigin(0, 0)
	, GridSize(0, 0)
	, CurrentUserIndex(INDEX_NONE)
	, PaintToHitTestTransform() // Default identity
#if WITH_SLATE_DEBUGGING
	, LastNavigationMethod()
#endif
{
}

bool FHittestGrid::IsWidgetDataInLayerRange(const FWidgetData& WidgetData, const TInterval<int32>& LayerRange)
{
	// top 32 bits are InBatchPriorityGroup, which we don't worry about
	int32 WidgetLayer = static_cast<int32>(WidgetData.PrimarySort);
	
	return LayerRange.Min <= WidgetLayer && LayerRange.Max > WidgetLayer;
}

TArray<FWidgetAndPointer> FHittestGrid::GetBubblePath(UE::Slate::FDeprecateVector2DParameter DesktopSpaceCoordinate, float CursorRadius, bool bIgnoreEnabledStatus, int32 UserIndex, TInterval<int32> LayerRange)
{
	checkSlow(IsInGameThread());

	const FVector2f CursorPositionInGrid = DesktopSpaceCoordinate - GridOrigin;

	if (WidgetArray.Num() > 0 && Cells.Num() > 0)
	{
		FGridTestingParams TestingParams;
		TestingParams.CursorPositionInGrid = CursorPositionInGrid;
		TestingParams.CellCoord = GetCellCoordinate(CursorPositionInGrid);
		TestingParams.Radius = 0.0f;
		TestingParams.bTestWidgetIsInteractive = false;

		// First add the exact point test results
		const FIndexAndDistance BestHit = GetHitIndexFromCellIndex(TestingParams);
		if (BestHit.IsValid())
		{
			const FWidgetData& BestHitWidgetData = BestHit.GetWidgetData();
			const TSharedPtr<SWidget> FirstHitWidget = BestHitWidgetData.GetWidget();
			// Make Sure we landed on a valid widget
			if (FirstHitWidget.IsValid() && IsCompatibleUserIndex(UserIndex, BestHitWidgetData.UserIndex))
			{
				TArray<FWidgetAndPointer> Path;

				TSharedPtr<SWidget> CurWidget = FirstHitWidget;
				while (CurWidget.IsValid())
				{
					FGeometry DesktopSpaceGeometry = CurWidget->GetPaintSpaceGeometry();
					DesktopSpaceGeometry.AppendTransform(FSlateLayoutTransform(GridOrigin - GridWindowOrigin));

					if(IsWidgetDataInLayerRange(BestHitWidgetData, LayerRange))
					{
						Path.Emplace(FArrangedWidget(CurWidget.ToSharedRef(), DesktopSpaceGeometry));
					}
					
					CurWidget = CurWidget->Advanced_GetPaintParentWidget();
				}

				if (Path.IsEmpty() || !Path.Last().Widget->Advanced_IsWindow())
				{
					return TArray<FWidgetAndPointer>();
				}

				Algo::Reverse(Path);

				bool bRemovedDisabledWidgets = false;
				if (!bIgnoreEnabledStatus)
				{
					// @todo It might be more correct to remove all disabled widgets and non-hit testable widgets.  It doesn't make sense to have a hit test invisible widget as a leaf in the path
					// and that can happen if we remove a disabled widget. Furthermore if we did this we could then append custom paths in all cases since the leaf most widget would be hit testable
					// For backwards compatibility changing this could be risky
					const int32 DisabledWidgetIndex = Path.IndexOfByPredicate([](const FArrangedWidget& SomeWidget) { return !SomeWidget.Widget->IsEnabled(); });
					if (DisabledWidgetIndex != INDEX_NONE)
					{
						bRemovedDisabledWidgets = true;
						Path.RemoveAt(DisabledWidgetIndex, Path.Num() - DisabledWidgetIndex);
					}
				}

				if (!bRemovedDisabledWidgets && Path.Num() > 0)
				{
					if (BestHitWidgetData.CustomPath.IsValid())
					{
						FVector2d DesktopSpaceCoordinate2d(DesktopSpaceCoordinate.X, DesktopSpaceCoordinate.Y);
						TArray<FWidgetAndPointer> BubblePathExtension = BestHitWidgetData.CustomPath.Pin()->GetBubblePathAndVirtualCursors(FirstHitWidget->GetTickSpaceGeometry(), DesktopSpaceCoordinate2d, bIgnoreEnabledStatus);
						Path.Append(MoveTemp(BubblePathExtension));
					}
				}

				return Path;
			}
		}
	}

	return TArray<FWidgetAndPointer>();
}

bool FHittestGrid::SetHittestArea(const UE::Slate::FDeprecateVector2DParameter& HittestPositionInDesktop, const UE::Slate::FDeprecateVector2DParameter& HittestDimensions, const UE::Slate::FDeprecateVector2DParameter& HitestOffsetInWindow)
{
	bool bWasCleared = false;

	// If the size of the hit test area changes we need to clear it out
	if (GridSize != HittestDimensions)
	{
		GridSize = HittestDimensions;
		NumCells = FIntPoint(FMath::CeilToInt(GridSize.X / CellSize.X), FMath::CeilToInt(GridSize.Y / CellSize.Y));

		const int32 NewTotalCells = NumCells.X * NumCells.Y;
		if (NewTotalCells < 0)
		{
			ensureMsgf(false, TEXT("The gridsize is invalid. (%f, %f) (%d, %d)"), GridSize.X, GridSize.Y, NumCells.X, NumCells.Y);
			GridSize = FVector2f(0.f, 0.f);
			NumCells = 0;
			ClearInternal(0);
		}
		else
		{
			ClearInternal(NewTotalCells);
		}

		bWasCleared = true;
	}

	GridOrigin = HittestPositionInDesktop;
	GridWindowOrigin = HitestOffsetInWindow;

	return bWasCleared;
}

void FHittestGrid::Clear()
{
	const int32 TotalCells = Cells.Num();
	ClearInternal(TotalCells);
}

void FHittestGrid::ClearInternal(int32 TotalCells)
{
#if UE_SLATE_ENABLE_HITTEST_STATS
	SCOPE_CYCLE_COUNTER(STAT_SlateHTG_Clear);
#endif
	for (FCell& Cell : Cells)
		Cell.Reset();
	Cells.SetNumZeroed(TotalCells);

	WidgetMap.Reset();
	WidgetArray.Reset();
	AppendedGridArray.Reset();
}

TSharedPtr<SWidget> FHittestGrid::FindNextFocusableWidget(const FArrangedWidget& StartingWidget, const EUINavigation Direction, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget, int32 UserIndex)
{
	if (GEnableNavigationExperimental)
	{
		return FindNextFocusableWidgetExperimental(StartingWidget, Direction, NavigationReply, RuleWidget, UserIndex);
	}
	return FindNextFocusableWidgetDefault(StartingWidget, Direction, NavigationReply, RuleWidget, UserIndex);
}


// TODO: NavigationMethodOrthogonal class deprecates the following and can be eventually removed. 
//		UE::Slate::Private, FindFocusableWidget, FindNextFocusableWidgetDefault

namespace UE::Slate::Private
{
	bool IsParentsEnabled(const SWidget* Widget)
	{
		while (Widget)
		{
			if (!Widget->IsEnabled())
			{
				return false;
			}
			Widget = Widget->Advanced_GetPaintParentWidget().Get();
		}
		return true;
	}
}

#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
namespace HittestGridDebuggingText
{
	/*extern*/ const FText Valid = LOCTEXT("StateValid", "Valid"); //~ The widget is valid will be consider as the result
	/*extern*/ const FText NotCompatibleWithUserIndex = LOCTEXT("StateNotCompatibleWithUserIndex", "User Index not compatible"); //~ The widget is not compatible with the requested user index
	/*extern*/ const FText DoesNotIntersect = LOCTEXT("StateDoesNotIntersect", "Does not intersect"); //~ The widget rect is not in the correct direction or is not intersecting with the "swept" rectangle
	/*extern*/ const FText PreviousWidgetIsBetter = LOCTEXT("StatePreviousWidgetIsBetter", "Previous Widget was better"); //~ The widget would be valid but the previous valid is closer
	/*extern*/ const FText NotADescendant = LOCTEXT("StateNotADescendant", "Not a descendant"); //~ We have a non escape boundary condition and the widget isn't a descendant of our boundary
	/*extern*/ const FText Disabled = LOCTEXT("StateNotEnabled", "Disabled"); //~ The widget is not enabled
	/*extern*/ const FText ParentDisabled = LOCTEXT("StateParentNotEnabled", "ParentDisabled"); //~ A parent of the widget is disabled
	/*extern*/ const FText DoesNotSuportKeyboardFocus = LOCTEXT("StateDoesNotSuportKeyboardFocus", "Keyboard focus unsupported"); //~ THe widget does not support keyboard focus
}
#define AddToNextFocusableWidgetCondidateDebugResults(Candidate, Result) { if (IntermediateResultsPtr) { IntermediateResultsPtr->Emplace((Candidate), (Result)); } }
#else
#define AddToNextFocusableWidgetCondidateDebugResults(Candidate, Result) CA_ASSUME(Candidate)
#endif

template<typename TCompareFunc, typename TSourceSideFunc, typename TDestSideFunc>
TSharedPtr<SWidget> FHittestGrid::FindFocusableWidget(FSlateRect WidgetRect, const FSlateRect SweptRect, int32 AxisIndex, int32 Increment, const EUINavigation Direction, const FNavigationReply& NavigationReply, TCompareFunc CompareFunc, TSourceSideFunc SourceSideFunc, TDestSideFunc DestSideFunc, int32 UserIndex, TArray<FDebuggingFindNextFocusableWidgetArgs::FWidgetResult>* IntermediateResultsPtr, TSet<TSharedPtr<SWidget>>* DisabledDestinations) const
{
	FIntPoint CurrentCellPoint = GetCellCoordinate(WidgetRect.GetCenter());

	int32 StartingIndex = CurrentCellPoint[AxisIndex];

	float CurrentSourceSide = SourceSideFunc(WidgetRect);

	int32 StrideAxis, StrideAxisMin, StrideAxisMax;

	// Ensure that the hit test grid is valid before proceeding
	if (NumCells.X < 1 || NumCells.Y < 1)
	{
		return TSharedPtr<SWidget>();
	}

	if (AxisIndex == 0)
	{
		StrideAxis = 1;
		StrideAxisMin = FMath::Min(FMath::Max(FMath::FloorToInt(SweptRect.Top / CellSize.Y), 0), NumCells.Y - 1);
		StrideAxisMax = FMath::Min(FMath::Max(FMath::FloorToInt(SweptRect.Bottom / CellSize.Y), 0), NumCells.Y - 1);
	}
	else
	{
		StrideAxis = 0;
		StrideAxisMin = FMath::Min(FMath::Max(FMath::FloorToInt(SweptRect.Left / CellSize.X), 0), NumCells.X - 1);
		StrideAxisMax = FMath::Min(FMath::Max(FMath::FloorToInt(SweptRect.Right / CellSize.X), 0), NumCells.X - 1);
	}

	bool bWrapped = false;
	while (CurrentCellPoint[AxisIndex] >= 0 && CurrentCellPoint[AxisIndex] < NumCells[AxisIndex])
	{
		FIntPoint StrideCellPoint = CurrentCellPoint;
		int32 CurrentCellProcessed = CurrentCellPoint[AxisIndex];

		// Increment before the search as a wrap case will change our current cell.
		CurrentCellPoint[AxisIndex] += Increment;

		FSlateRect BestWidgetRect;
		TSharedPtr<SWidget> BestWidget = TSharedPtr<SWidget>();

		for (StrideCellPoint[StrideAxis] = StrideAxisMin; StrideCellPoint[StrideAxis] <= StrideAxisMax; ++StrideCellPoint[StrideAxis])
		{
			FCollapsedWidgetsArray WidgetIndexes;
			GetCollapsedWidgets(WidgetIndexes, StrideCellPoint.X, StrideCellPoint.Y);

			for (int32 i = WidgetIndexes.Num() - 1; i >= 0; --i)
			{
				const FWidgetData& TestCandidate = WidgetIndexes[i].GetWidgetData();
				const TSharedPtr<SWidget> TestWidget = TestCandidate.GetWidget();
				if (!TestWidget.IsValid())
				{
					continue;
				}

				if (!IsCompatibleUserIndex(UserIndex, TestCandidate.UserIndex))
				{
					AddToNextFocusableWidgetCondidateDebugResults(TestWidget, HittestGridDebuggingText::NotCompatibleWithUserIndex);
					continue;
				}

				FGeometry TestCandidateGeo = TestWidget->GetPaintSpaceGeometry();
				TestCandidateGeo.AppendTransform(FSlateLayoutTransform(-GridWindowOrigin));
				const FSlateRect TestCandidateRect = TestCandidateGeo.GetRenderBoundingRect();
				if (!(CompareFunc(DestSideFunc(TestCandidateRect), CurrentSourceSide) && FSlateRect::DoRectanglesIntersect(SweptRect, TestCandidateRect)))
				{
					AddToNextFocusableWidgetCondidateDebugResults(TestWidget, HittestGridDebuggingText::DoesNotIntersect);
					continue;
				}

				if (BestWidget.IsValid())
				{
					const float BestDestSide = DestSideFunc(BestWidgetRect);
					const float TestDestSide = DestSideFunc(TestCandidateRect);

					// If the test candidate is further on the navigation axis, keep the current best.
					if (!CompareFunc(BestDestSide, TestDestSide))
					{
						AddToNextFocusableWidgetCondidateDebugResults(TestWidget, HittestGridDebuggingText::PreviousWidgetIsBetter);
						continue;
					}

					// Equidistant on the navigation axis: prefer the candidate whose center
					// is closest to the source center.
					if (CompareFunc(TestDestSide, BestDestSide))
					{
						const float BestDistSq = FVector2f::DistSquared(WidgetRect.GetCenter(), BestWidgetRect.GetCenter());
						const float TestDistSq = FVector2f::DistSquared(WidgetRect.GetCenter(), TestCandidateRect.GetCenter());
						if (TestDistSq >= BestDistSq)
						{
							AddToNextFocusableWidgetCondidateDebugResults(TestWidget, HittestGridDebuggingText::PreviousWidgetIsBetter);
							continue;
						}
					}
				}

				// If we have a non escape boundary condition and this widget isn't a descendant of our boundary condition widget then it's invalid so we keep looking.
				if (NavigationReply.GetBoundaryRule() != EUINavigationRule::Escape
					&& NavigationReply.GetHandler().IsValid()
					&& !IsDescendantOf(NavigationReply.GetHandler().Get(), TestCandidate))
				{
					AddToNextFocusableWidgetCondidateDebugResults(TestWidget, HittestGridDebuggingText::NotADescendant);
					continue;
				}

				if (!TestWidget->IsEnabled())
				{
					AddToNextFocusableWidgetCondidateDebugResults(TestWidget, HittestGridDebuggingText::Disabled);
					continue;
				}

				if (!TestWidget->SupportsKeyboardFocus())
				{
					AddToNextFocusableWidgetCondidateDebugResults(TestWidget, HittestGridDebuggingText::DoesNotSuportKeyboardFocus);
					continue;
				}

				if (DisabledDestinations->Contains(TestWidget))
				{
					AddToNextFocusableWidgetCondidateDebugResults(TestWidget, HittestGridDebuggingText::ParentDisabled);
					continue;
				}

				BestWidgetRect = TestCandidateRect;
				BestWidget = TestWidget;
				AddToNextFocusableWidgetCondidateDebugResults(TestWidget, HittestGridDebuggingText::Valid);
			}
		}

		if (BestWidget.IsValid())
		{
			// Check for the need to apply our rule
			if (CompareFunc(DestSideFunc(BestWidgetRect), SourceSideFunc(SweptRect)))
			{
				switch (NavigationReply.GetBoundaryRule())
				{
				case EUINavigationRule::Explicit:
					return NavigationReply.GetFocusRecipient();
				case EUINavigationRule::Custom:
				case EUINavigationRule::CustomBoundary:
				{
					const FNavigationDelegate& FocusDelegate = NavigationReply.GetFocusDelegate();
					if (FocusDelegate.IsBound())
					{
						return FocusDelegate.Execute(Direction);
					}
					return TSharedPtr<SWidget>();
				}
				case EUINavigationRule::Stop:
					return TSharedPtr<SWidget>();
				case EUINavigationRule::Wrap:
					CurrentSourceSide = DestSideFunc(SweptRect);
					FVector2f SampleSpot = WidgetRect.GetCenter();
					SampleSpot[AxisIndex] = CurrentSourceSide;
					CurrentCellPoint = GetCellCoordinate(SampleSpot);
					bWrapped = true;
					break;
				}
			}

			// Make sure all parents of the chosen widget are enabled before returning.
			// Note that IsParentsEnabled is a costly function. We call it here as the last step to minimize the number of calls to it.
			if (!UE::Slate::Private::IsParentsEnabled(BestWidget.Get()))
			{
				// Find the next best widget because this one has disabled parents.
				DisabledDestinations->Add(BestWidget);
				return FindFocusableWidget(WidgetRect, SweptRect, AxisIndex, Increment, Direction, NavigationReply, CompareFunc, SourceSideFunc, DestSideFunc, UserIndex, IntermediateResultsPtr, DisabledDestinations);
			}
			else
			{
				return BestWidget;
			}
		}

		// break if we have looped back to where we started.
		if (bWrapped && StartingIndex == CurrentCellProcessed) { break; }

		// If were going to fail our bounds check and our rule is to a boundary condition (Wrap or CustomBoundary) handle appropriately
		if (!(CurrentCellPoint[AxisIndex] >= 0 && CurrentCellPoint[AxisIndex] < NumCells[AxisIndex]))
		{
			if (NavigationReply.GetBoundaryRule() == EUINavigationRule::Wrap)
			{
				if (bWrapped)
				{
					// If we've already wrapped, unfortunately it must be that the starting widget wasn't within the boundary
					break;
				}
				CurrentSourceSide = DestSideFunc(SweptRect);
				FVector2f SampleSpot = WidgetRect.GetCenter();
				SampleSpot[AxisIndex] = CurrentSourceSide;
				CurrentCellPoint = GetCellCoordinate(SampleSpot);
				bWrapped = true;
			}
			else if (NavigationReply.GetBoundaryRule() == EUINavigationRule::CustomBoundary)
			{
				const FNavigationDelegate& FocusDelegate = NavigationReply.GetFocusDelegate();
				if (FocusDelegate.IsBound())
				{
					return FocusDelegate.Execute(Direction);
				}
			}
		}
	}

	return TSharedPtr<SWidget>();
}

#undef AddToNextFocusableWidgetCondidateDebugResults

TSharedPtr<SWidget> FHittestGrid::FindNextFocusableWidgetDefault(const FArrangedWidget& StartingWidget, const EUINavigation Direction, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget, int32 UserIndex)
{
	FGeometry StartingWidgetGeo = StartingWidget.Widget->GetPaintSpaceGeometry();
	StartingWidgetGeo.AppendTransform(FSlateLayoutTransform(-GridWindowOrigin));
	FSlateRect WidgetRect = StartingWidgetGeo.GetRenderBoundingRect();

	FGeometry BoundingRuleWidgetGeo = RuleWidget.Widget->GetPaintSpaceGeometry();
	BoundingRuleWidgetGeo.AppendTransform(FSlateLayoutTransform(-GridWindowOrigin));
	FSlateRect BoundingRuleRect = BoundingRuleWidgetGeo.GetRenderBoundingRect();

	FSlateRect SweptWidgetRect = WidgetRect;

	TSharedPtr<SWidget> Widget = TSharedPtr<SWidget>();

#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
	TArray<FDebuggingFindNextFocusableWidgetArgs::FWidgetResult> IntermediateResults;
	TArray<FDebuggingFindNextFocusableWidgetArgs::FWidgetResult>* IntermediateResultsPtr = OnFindNextFocusableWidgetExecuted.IsBound() ? &IntermediateResults : nullptr;
#else
	TArray<FDebuggingFindNextFocusableWidgetArgs::FWidgetResult>* IntermediateResultsPtr = nullptr;
#endif

	TSet<TSharedPtr<SWidget>> DisabledDestinations = TSet<TSharedPtr<SWidget>>();

	switch (Direction)
	{
	case EUINavigation::Left:
		SweptWidgetRect.Left = BoundingRuleRect.Left;
		SweptWidgetRect.Right = BoundingRuleRect.Right;
		SweptWidgetRect.Top += 0.5f;
		SweptWidgetRect.Bottom -= 0.5f;
		Widget = FindFocusableWidget(WidgetRect, SweptWidgetRect, 0, -1, Direction, NavigationReply,
			[](float A, float B) { return A - 0.1f < B; }, // Compare function
			[](FSlateRect SourceRect) { return SourceRect.Left; }, // Source side function
			[](FSlateRect DestRect) { return DestRect.Right; }, // Dest side function
			UserIndex, IntermediateResultsPtr, &DisabledDestinations);
		break;
	case EUINavigation::Right:
		SweptWidgetRect.Left = BoundingRuleRect.Left;
		SweptWidgetRect.Right = BoundingRuleRect.Right;
		SweptWidgetRect.Top += 0.5f;
		SweptWidgetRect.Bottom -= 0.5f;
		Widget = FindFocusableWidget(WidgetRect, SweptWidgetRect, 0, 1, Direction, NavigationReply,
			[](float A, float B) { return A + 0.1f > B; }, // Compare function
			[](FSlateRect SourceRect) { return SourceRect.Right; }, // Source side function
			[](FSlateRect DestRect) { return DestRect.Left; }, // Dest side function
			UserIndex, IntermediateResultsPtr, &DisabledDestinations);
		break;
	case EUINavigation::Up:
		SweptWidgetRect.Top = BoundingRuleRect.Top;
		SweptWidgetRect.Bottom = BoundingRuleRect.Bottom;
		SweptWidgetRect.Left += 0.5f;
		SweptWidgetRect.Right -= 0.5f;
		Widget = FindFocusableWidget(WidgetRect, SweptWidgetRect, 1, -1, Direction, NavigationReply,
			[](float A, float B) { return A - 0.1f < B; }, // Compare function
			[](FSlateRect SourceRect) { return SourceRect.Top; }, // Source side function
			[](FSlateRect DestRect) { return DestRect.Bottom; }, // Dest side function
			UserIndex, IntermediateResultsPtr, &DisabledDestinations);
		break;
	case EUINavigation::Down:
		SweptWidgetRect.Top = BoundingRuleRect.Top;
		SweptWidgetRect.Bottom = BoundingRuleRect.Bottom;
		SweptWidgetRect.Left += 0.5f;
		SweptWidgetRect.Right -= 0.5f;
		Widget = FindFocusableWidget(WidgetRect, SweptWidgetRect, 1, 1, Direction, NavigationReply,
			[](float A, float B) { return A + 0.1f > B; }, // Compare function
			[](FSlateRect SourceRect) { return SourceRect.Bottom; }, // Source side function
			[](FSlateRect DestRect) { return DestRect.Top; }, // Dest side function
			UserIndex, IntermediateResultsPtr, &DisabledDestinations);
		break;

	default:
		break;
	}

#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
	if (IntermediateResultsPtr)
	{
		FDebuggingFindNextFocusableWidgetArgs Args{ StartingWidget, Direction, NavigationReply, RuleWidget, UserIndex, Widget , MoveTemp(*IntermediateResultsPtr) };
		OnFindNextFocusableWidgetExecuted.Broadcast(this, Args);
	}
#endif

	return Widget;
}

bool FHittestGrid::IsDescendantOf(const SWidget* ParentWidget, const FWidgetData& ChildData) const
{
	const TSharedPtr<SWidget> ChildWidgetPtr = ChildData.GetWidget();
	const SWidget* CurWidget = ChildWidgetPtr.Get();
	if (CurWidget == ParentWidget)
	{
		return false;
	}

	while (CurWidget)
	{
		if (ParentWidget == CurWidget)
		{
			return true;
		}
		CurWidget = CurWidget->Advanced_GetPaintParentWidget().Get();
	}

	return false;
}

TSharedPtr<SWidget> FHittestGrid::FindNextFocusableWidgetExperimental(const FArrangedWidget& StartingWidget, const EUINavigation Direction, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget, int32 UserIndex)
{
	// Ensure that the hit test grid is valid before proceeding
	if (NumCells.X < 1 || NumCells.Y < 1)
	{
		return TSharedPtr<SWidget>();
	}

#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
	TArray<FDebuggingFindNextFocusableWidgetArgs::FWidgetResult> IntermediateResults;
	TArray<FDebuggingFindNextFocusableWidgetArgs::FWidgetResult>* IntermediateResultsPtr = OnFindNextFocusableWidgetExecuted.IsBound() ? &IntermediateResults : nullptr;
#else
	TArray<FDebuggingFindNextFocusableWidgetArgs::FWidgetResult>* IntermediateResultsPtr = nullptr;
#endif

	TSharedPtr<SWidget> NextFocusedWidget;
	TInstancedStruct<FNavigationMethod> NavigationMethod = GetNavigationMethod(StartingWidget);
	if (NavigationMethod.IsValid())
	{
		FNavigationMethod* NavigationMethodPtr = NavigationMethod.GetMutablePtr();

		NavigationMethodPtr->Initialize(this, IntermediateResultsPtr);
		NextFocusedWidget = NavigationMethodPtr->FindNextFocusableWidget(StartingWidget, Direction, NavigationReply, RuleWidget, UserIndex);
	}
	else
	{
		// Fallback to existing behavior
		NextFocusedWidget = FindNextFocusableWidgetDefault(StartingWidget, Direction, NavigationReply, RuleWidget, UserIndex);
	}

	FNavigationRoutingParams RoutingParams(StartingWidget.Widget, NextFocusedWidget, this, NavigationReply, UserIndex);
	NextFocusedWidget = FNavigationRouting::RouteNavigationFocus(RoutingParams);

#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
	if (IntermediateResultsPtr)
	{
		FDebuggingFindNextFocusableWidgetArgs Args{ StartingWidget, Direction, NavigationReply, RuleWidget, UserIndex, NextFocusedWidget, MoveTemp(*IntermediateResultsPtr) };
		OnFindNextFocusableWidgetExecuted.Broadcast(this, Args);
	}

	
#endif

#if WITH_SLATE_DEBUGGING
	
	if (GSlateNavigationDebugging && NavigationMethod.IsValid())
	{
		// Duplicate, as this pointer may outlive the navigation method associated with raw instanced struct pointer
		LastNavigationMethod = NavigationMethod;
	}

	NavigationMethod.Reset();

#endif

	return NextFocusedWidget;
}

FSlateRect FHittestGrid::GetWidgetRenderBoundingRect(const TSharedPtr<SWidget>& Widget) const
{
	if (!Widget.IsValid())
	{
		return FSlateRect();
	}

	return Widget->GetHitTestBoundingRect().OffsetBy(-GridWindowOrigin);
}

FIntPoint FHittestGrid::GetCellCoordinate(UE::Slate::FDeprecateVector2DParameter Position) const
{
	return FIntPoint(
		FMath::Min(FMath::Max(FMath::FloorToInt(Position.X / CellSize.X), 0), NumCells.X - 1),
		FMath::Min(FMath::Max(FMath::FloorToInt(Position.Y / CellSize.Y), 0), NumCells.Y - 1));
}

bool FHittestGrid::IsValidCellCoord(const FIntPoint& CellCoord) const
{
	return IsValidCellCoord(CellCoord.X, CellCoord.Y);
}

bool FHittestGrid::IsValidCellCoord(const int32 XCoord, const int32 YCoord) const
{
	return XCoord >= 0 && XCoord < NumCells.X && YCoord >= 0 && YCoord < NumCells.Y;
}

void FHittestGrid::AddGrid(const TSharedRef<const FHittestGrid>& OtherGrid)
{
	auto GetCollapsedHittestGrid_NoTests = [](const FHittestGrid* HittestGrid, FCollapsedHittestGridArray& OutResult)
		{
			OutResult.Add(HittestGrid);
			for (int32 Index = 0; Index < OutResult.Num(); ++Index)
			{
				for (const FAppendedGridData& AppendedGridData : OutResult[Index]->AppendedGridArray)
				{
					if (const TSharedPtr<const FHittestGrid> Grid = AppendedGridData.Grid.Pin())
					{
						OutResult.Add(Grid.Get());
					}
				}
			}
		};

	const bool bIsContains = AppendedGridArray.ContainsByPredicate([OtherGrid](const FAppendedGridData& Other) { return Other.Grid == OtherGrid; });
	if (ensure(CanBeAppended(&OtherGrid.Get())))
	{
		if (!bIsContains)
		{
			// Check for recursion
			FCollapsedHittestGridArray AllHittestGrid;
			GetCollapsedHittestGrid_NoTests(this, AllHittestGrid); // we are building a new array, do not perform size check
			const bool bIsInCollapsed = AllHittestGrid.ContainsByPredicate([OtherGrid](const FHittestGrid* Other) { return &*OtherGrid == Other; });
			ensure(!bIsInCollapsed);
			if (bIsInCollapsed)
			{
				return;
			}

			AppendedGridArray.Emplace(OtherGrid->Owner, OtherGrid);
		}
	}
	else
	{
		RemoveGrid(OtherGrid);
	}
}

void FHittestGrid::RemoveGrid(const TSharedRef<const FHittestGrid>& OtherGrid)
{
	const int32 AppendedGridIndex = AppendedGridArray.IndexOfByPredicate(
		[OtherGrid](const FAppendedGridData& Other)
		{
			return Other.Grid == OtherGrid;
		});
	if (AppendedGridIndex != INDEX_NONE)
	{
		AppendedGridArray.RemoveAtSwap(AppendedGridIndex);
	}
}

void FHittestGrid::RemoveGrid(const SWidget* OtherGridOwner)
{
	const int32 AppendedGridIndex = AppendedGridArray.IndexOfByPredicate(
		[OtherGridOwner](const FAppendedGridData& Other)
		{
			return Other.CachedOwner == OtherGridOwner;
		});
	if (AppendedGridIndex != INDEX_NONE)
	{
#if WITH_SLATE_DEBUGGING
		// Confirmed that the cached grid is valid
		if (const TSharedPtr<const FHittestGrid> Pin = AppendedGridArray[AppendedGridIndex].Grid.Pin())
		{
			ensure(Pin->Owner == OtherGridOwner);
		}
#endif
		AppendedGridArray.RemoveAtSwap(AppendedGridIndex);
	}
}

void FHittestGrid::SetPaintToHitTestTransform(const FSlateRenderTransform& InTransform)
{
	PaintToHitTestTransform = InTransform;
	if (PaintToHitTestTransform.IsSet() && PaintToHitTestTransform.GetValue().IsIdentity())
	{
		PaintToHitTestTransform = TOptional<FSlateRenderTransform>();
	}
}

bool FHittestGrid::CanBeAppended(const FHittestGrid* OtherGrid) const
{
	return OtherGrid && OtherGrid->Owner && this != OtherGrid && SameSize(OtherGrid);
}

bool FHittestGrid::SameSize(const FHittestGrid* OtherGrid) const
{
	return GridOrigin == OtherGrid->GridOrigin && GridWindowOrigin == OtherGrid->GridWindowOrigin && GridSize == OtherGrid->GridSize;
}

void FHittestGrid::AddWidget(const SWidget* InWidget, int32 InBatchPriorityGroup, int32 InLayerId, FSlateInvalidationWidgetSortOrder InSecondarySort)
{
	check(InWidget);
	if (!InWidget->GetVisibility().IsHitTestVisible())
	{
		return;
	}

#if UE_SLATE_ENABLE_HITTEST_STATS
	SCOPE_CYCLE_COUNTER(STAT_SlateHTG_AddWidget);
#endif

	// Track the widget and identify it's Widget Index
	FGeometry GridSpaceGeometry = InWidget->GetPaintSpaceGeometry();
	GridSpaceGeometry.AppendTransform(FSlateLayoutTransform(-GridWindowOrigin));

	// Currently using grid offset because the grid covers all desktop space.
	FSlateRect BoundingRect = GridSpaceGeometry.GetRenderBoundingRect();

	if (PaintToHitTestTransform.IsSet())
	{
		// Ensure the cells check correspond to the hittest to paint remapping
		const FSlateRotatedRect WindowOrientedClipRect = TransformRect(
			Concatenate(
				Inverse(GridSpaceGeometry.GetAccumulatedLayoutTransform()),
				Concatenate(GridSpaceGeometry.GetAccumulatedRenderTransform(), PaintToHitTestTransform.GetValue())),
			FSlateRotatedRect(GridSpaceGeometry.GetLayoutBoundingRect())
		);
		BoundingRect = WindowOrientedClipRect.ToBoundingRect();
	}

	// Starting and ending cells covered by this widget.	
	const FIntPoint UpperLeftCell = GetCellCoordinate(BoundingRect.GetTopLeft());
	const FIntPoint LowerRightCell = GetCellCoordinate(BoundingRect.GetBottomRight());

	const int64 PrimarySort = (((int64)InBatchPriorityGroup << 32) | InLayerId);

	bool bAddWidget = true;
	if (int32* FoundIndex = WidgetMap.Find(InWidget))
	{
		FWidgetData& WidgetData = WidgetArray[*FoundIndex];
		if (WidgetData.UpperLeftCell != UpperLeftCell || WidgetData.LowerRightCell != LowerRightCell)
		{
			// Need to be updated
			RemoveWidget(InWidget);
		}
		else
		{
			// Only update
			bAddWidget = false;
			WidgetData.PrimarySort = PrimarySort;
			WidgetData.SecondarySort = InSecondarySort;
			WidgetData.UserIndex = CurrentUserIndex;
		}
	}

	if (bAddWidget)
	{
		int32& WidgetIndex = WidgetMap.Add(InWidget);
		WidgetIndex = WidgetArray.Emplace(const_cast<SWidget*>(InWidget)->AsShared(), UpperLeftCell, LowerRightCell, PrimarySort, InSecondarySort, CurrentUserIndex);
		for (int32 XIndex = UpperLeftCell.X; XIndex <= LowerRightCell.X; ++XIndex)
		{
			for (int32 YIndex = UpperLeftCell.Y; YIndex <= LowerRightCell.Y; ++YIndex)
			{
				if (IsValidCellCoord(XIndex, YIndex))
				{
					CellAt(XIndex, YIndex).AddIndex(WidgetIndex);
				}
			}
		}
	}
}

void FHittestGrid::RemoveWidget(const SWidget* InWidget)
{
#if UE_SLATE_ENABLE_HITTEST_STATS
	SCOPE_CYCLE_COUNTER(STAT_SlateHTG_RemoveWidget);
#endif

	int32 WidgetIndex = INDEX_NONE;
	if (WidgetMap.RemoveAndCopyValue(InWidget, WidgetIndex))
	{
		const FWidgetData& WidgetData = WidgetArray[WidgetIndex];

		// Starting and ending cells covered by this widget.	
		const FIntPoint& UpperLeftCell = WidgetData.UpperLeftCell;
		const FIntPoint& LowerRightCell = WidgetData.LowerRightCell;

		for (int32 XIndex = UpperLeftCell.X; XIndex <= LowerRightCell.X; ++XIndex)
		{
			for (int32 YIndex = UpperLeftCell.Y; YIndex <= LowerRightCell.Y; ++YIndex)
			{
				checkSlow(IsValidCellCoord(XIndex, YIndex));
				CellAt(XIndex, YIndex).RemoveIndex(WidgetIndex);
			}
		}

		WidgetArray.RemoveAt(WidgetIndex);
	}

	RemoveGrid(InWidget);
}

void FHittestGrid::UpdateWidget(const SWidget* InWidget, FSlateInvalidationWidgetSortOrder InSecondarySort)
{
	check(InWidget);
	if (int32* FoundWidgetIndex = WidgetMap.Find(InWidget))
	{
		WidgetArray[*FoundWidgetIndex].SecondarySort = InSecondarySort;
	}
}

bool FHittestGrid::ContainsWidget(const SWidget* InWidget) const
{
	return WidgetMap.Contains(InWidget);
}

void FHittestGrid::InsertCustomHitTestPath(const SWidget* InWidget, const TSharedRef<ICustomHitTestPath>& CustomHitTestPath)
{
	int32* WidgetIndex = WidgetMap.Find(InWidget);
	if (ensureMsgf(WidgetIndex, TEXT("The widget is not in the hittest grid.")))
	{
		FWidgetData& WidgetData = WidgetArray[*WidgetIndex];
		WidgetData.CustomPath = CustomHitTestPath;
	}
}

FHittestGrid::FIndexAndDistance FHittestGrid::GetHitIndexFromCellIndex(const FGridTestingParams& Params) const
{
	//check if the cell coord 
	if (IsValidCellCoord(Params.CellCoord))
	{
		// Get the cell and sort it 
		FCollapsedWidgetsArray WidgetIndexes;
		GetCollapsedWidgets(WidgetIndexes, Params.CellCoord.X, Params.CellCoord.Y);

#if 0 //Unroll some data for debugging if necessary
		struct FDebugData
		{
			FWidgetData WidgetData;
			FName WidgetType;
			FName WidgetLoc;
			TSharedPtr<SWidget> Widget;
		};

		TArray<FDebugData> DebugData;
		for (int32 i = 0; i < WidgetIndexes.Num(); ++i)
		{
			FDebugData& Cur = DebugData.AddDefaulted_GetRef();
			Cur.WidgetData = WidgetIndexes[i].GetWidgetData();
			Cur.Widget = Cur.WidgetData.GetWidget();
			Cur.WidgetType = Cur.Widget.IsValid() ? Cur.Widget->GetType() : NAME_None;
			Cur.WidgetLoc = Cur.Widget.IsValid() ? Cur.Widget->GetCreatedInLocation() : NAME_None;
		}
#endif

		// Consider front-most widgets first for hittesting.
		for (int32 i = WidgetIndexes.Num() - 1; i >= 0; --i)
		{
			check(WidgetIndexes[i].IsValid());
			const FWidgetData& TestCandidate = WidgetIndexes[i].GetWidgetData();
			const TSharedPtr<SWidget> TestWidget = TestCandidate.GetWidget();

			// When performing a point hittest, accept all hittestable widgets.
			// When performing a hittest with a radius, only grab interactive widgets.
			const bool bIsValidWidget = TestWidget.IsValid() && (!Params.bTestWidgetIsInteractive || TestWidget->IsInteractable());
			if (bIsValidWidget)
			{
				const FVector2f WindowSpaceCoordinate = Params.CursorPositionInGrid + GridWindowOrigin;

				const FGeometry& TestGeometry = TestWidget->GetPaintSpaceGeometry();

				bool bPointInsideClipMasks = true;

				if (WidgetIndexes[i].GetCullingRect().IsValid())
				{
					bPointInsideClipMasks = WidgetIndexes[i].GetCullingRect().ContainsPoint(WindowSpaceCoordinate);
				}

				if (bPointInsideClipMasks)
				{
					const TOptional<FSlateClippingState>& WidgetClippingState = TestWidget->GetCurrentClippingState();
					if (WidgetClippingState.IsSet())
					{
						// TODO: Solve non-zero radius cursors?
						bPointInsideClipMasks = WidgetClippingState->IsPointInside(WindowSpaceCoordinate);
					}
				}

				if (bPointInsideClipMasks)
				{
					// Compute the render space clipping rect (FGeometry exposes a layout space clipping rect).
					FSlateRotatedRect WindowOrientedClipRect;
					if (WidgetIndexes[i].GetGrid()->PaintToHitTestTransform.IsSet())
					{
						// If the hittestgrid contains an additional hittest rotation, apply this.
						const FTransform2D GridPaintToHitTestTransform = WidgetIndexes[i].GetGrid()->PaintToHitTestTransform.GetValue();
						WindowOrientedClipRect = TransformRect(
							Concatenate(
								Inverse(TestGeometry.GetAccumulatedLayoutTransform()),
								Concatenate(TestGeometry.GetAccumulatedRenderTransform(), GridPaintToHitTestTransform)),
							FSlateRotatedRect(TestGeometry.GetLayoutBoundingRect())
						);
					}
					else
					{
						WindowOrientedClipRect = TransformRect(
							Concatenate(
								Inverse(TestGeometry.GetAccumulatedLayoutTransform()),
								TestGeometry.GetAccumulatedRenderTransform()),
							FSlateRotatedRect(TestGeometry.GetLayoutBoundingRect())
						);
					}

					if (IsOverlappingSlateRotatedRect(WindowSpaceCoordinate, Params.Radius, WindowOrientedClipRect))
					{
						// For non-0 radii also record the distance to cursor's center so that we can pick the closest hit from the results.
						const bool bNeedsDistanceSearch = Params.Radius > 0.0f;
						const float DistSq = (bNeedsDistanceSearch) ? DistanceSqToSlateRotatedRect(WindowSpaceCoordinate, WindowOrientedClipRect) : 0.0f;
						return FIndexAndDistance(WidgetIndexes[i], DistSq);
					}
				}
			}
		}
	}

	return FIndexAndDistance();
}

 void FHittestGrid::GetCollapsedHittestGrid(FCollapsedHittestGridArray& OutResult) const
{
	OutResult.Add(this);
	for (const FAppendedGridData& AppendedGridData : AppendedGridArray)
	{
		if (const TSharedPtr<const FHittestGrid>& AppendedGrid = AppendedGridData.Grid.Pin())
		{
			if (ensure(!OutResult.Contains(AppendedGrid.Get()))) // Check for recursion
			{
				if (ensure(SameSize(AppendedGrid.Get())))
				{
					AppendedGrid->GetCollapsedHittestGrid(OutResult);
				}
			}
		}
	}

#if UE_SLATE_HITTESTGRID_ARRAYSIZEMAX
	HittestGrid_CollapsedHittestGridArraySizeMax = FMath::Max(OutResult.Num(), HittestGrid_CollapsedHittestGridArraySizeMax);
#endif
}

#define UE_VERIFY_WIDGET_VALIDITE 0
 void FHittestGrid::GetCollapsedWidgets(FCollapsedWidgetsArray& OutResult, const int32 X, const int32 Y) const
 {
#if UE_SLATE_ENABLE_HITTEST_STATS
	 SCOPE_CYCLE_COUNTER(STAT_SlateHTG_GetCollapsedWidgets);
#endif
	 const int32 CellIndex = Y * NumCells.X + X;
	 check(Cells.IsValidIndex(CellIndex));

	 FCollapsedHittestGridArray AllHitTestGrids;
	 GetCollapsedHittestGrid(AllHitTestGrids);

	 // N.B. it would be more efficient if we only rebuild if the cell has changed
	 //but the PrimarySort and SecondaySort are not updated directly and the cell doesn't know about it

	 {
		 for (const FHittestGrid* HittestGrid : AllHitTestGrids)
		 {
			 const TArray<int32>& WidgetsIndexes = HittestGrid->CellAt(X, Y).GetWidgetIndexes();
			 for (int32 WidgetIndex : WidgetsIndexes)
			 {
#if UE_VERIFY_WIDGET_VALIDITE
				 ensureAlways(HittestGrid->WidgetArray.IsValidIndex(WidgetIndex));
#endif
				 OutResult.Emplace(HittestGrid, WidgetIndex);
			 }
		 }

		 OutResult.StableSort([](const FWidgetIndex& A, const FWidgetIndex& B)
			 {
				 const FWidgetData& WidgetDataA = A.GetWidgetData();
				 const FWidgetData& WidgetDataB = B.GetWidgetData();
				 return WidgetDataA.PrimarySort < WidgetDataB.PrimarySort || (WidgetDataA.PrimarySort == WidgetDataB.PrimarySort && WidgetDataA.SecondarySort < WidgetDataB.SecondarySort);
			 });
	 }

#if UE_SLATE_HITTESTGRID_ARRAYSIZEMAX
	 HittestGrid_CollapsedWidgetsArraySizeMax = FMath::Max(OutResult.Num(), HittestGrid_CollapsedWidgetsArraySizeMax);
#endif
 }
#undef UE_VERIFY_WIDGET_VALIDITE

void FHittestGrid::RemoveStaleAppendedHittestGrid()
{
	for (int32 AppendedGridIndex = AppendedGridArray.Num() - 1; AppendedGridIndex >= 0; --AppendedGridIndex)
	{
		bool bToRemove = false;
		const TWeakPtr<const FHittestGrid>& WeakAppendedGrid = AppendedGridArray[AppendedGridIndex].Grid;
		if (const TSharedPtr<const FHittestGrid>& AppendedGrid = WeakAppendedGrid.Pin())
		{
			if (!SameSize(AppendedGrid.Get()))
			{
				bToRemove = true;
			}
		}
		else
		{
			// grid was removed without notifying the SlateInvalidationRoot.
			//That can happened when the widget containing the owner of the grid is removed in the same frame as it was added.
			bToRemove = true;
		}

		if (bToRemove)
		{
			AppendedGridArray.RemoveAtSwap(AppendedGridIndex);
		}
	}
}

TInstancedStruct<FNavigationMethod> FHittestGrid::GetNavigationMethod(const FArrangedWidget& StartingWidget)
{
	TInstancedStruct<FNavigationMethod> NavigationMethodStruct;
	TSharedPtr<SWidget> Parent = StartingWidget.Widget;
	while (!NavigationMethodStruct.IsValid() && Parent != nullptr)
	{
		TSharedPtr<FNavigationMetaData> NavigationMetaData = Parent->GetMetaData<FNavigationMetaData>();
		if (NavigationMetaData.IsValid())
		{
			NavigationMethodStruct = NavigationMetaData->GetNavigationMethodStruct();
		}
		Parent = Parent->GetParentWidget();
	}

	return NavigationMethodStruct;
}

#if WITH_SLATE_DEBUGGING
void FHittestGrid::LogGrid() const
{
	FString TempString;
	for (int32 y = 0; y < NumCells.Y; ++y)
	{
		for (int x = 0; x < NumCells.X; ++x)
		{
			TempString += "\t";
			TempString += "[";
			for (int32 i : CellAt(x, y).GetWidgetIndexes())
			{
				TempString += FString::Printf(TEXT("%d,"), i);
			}
			TempString += "]";
		}
		TempString += "\n";
	}

	TempString += "\n";

	UE_LOGF(LogHittestDebug, Warning, "\n%ls", *TempString);

	for (TSparseArray<FWidgetData>::TConstIterator It(WidgetArray); It; ++It)
	{
		const FWidgetData& CurWidgetData = *It;
		const TSharedPtr<SWidget> CachedWidget = CurWidgetData.GetWidget();
		UE_LOGF(LogHittestDebug, Warning, "  [%d][%lld] => %ls @ %ls",
			It.GetIndex(),
			CurWidgetData.PrimarySort,
			CachedWidget.IsValid() ? *CachedWidget->ToString() : TEXT("Invalid Widget"),
			CachedWidget.IsValid() ? *CachedWidget->GetPaintSpaceGeometry().ToString() : TEXT("Invalid Widget"));
	}
}

constexpr FLinearColor GetUserColorFromIndex(int32 Index)
{
	switch (Index)
	{
	case 0:
		return FLinearColor::Red;
	case 1:
		return FLinearColor::Green;
	case 2:
		return FLinearColor::Blue;
	case 3:
		return FLinearColor::Yellow;
	default:
		return FLinearColor::Transparent;
	}
};


void FHittestGrid::DisplayGrid(int32 InLayer, const FGeometry& AllottedGeometry, FSlateWindowElementList& WindowElementList, EDisplayGridFlags DisplayFlags) const
{
	static const FSlateBrush* FocusRectangleBrush = FCoreStyle::Get().GetBrush(TEXT("FocusRectangle"));
	static const FSlateBrush* BorderBrush = FCoreStyle::Get().GetBrush(TEXT("Border"));

	const FSlateBrush* Brush = EnumHasAnyFlags(DisplayFlags, EDisplayGridFlags::UseFocusBrush) ? FocusRectangleBrush : BorderBrush;

	auto DisplayGrid = [&] (const FHittestGrid* HittestGrid)
	{
		for (TSparseArray<FWidgetData>::TConstIterator It(HittestGrid->WidgetArray); It; ++It)
		{
			const FWidgetData& CurWidgetData = *It;
			const TSharedPtr<SWidget> CachedWidget = CurWidgetData.GetWidget();
			if (ensure(CachedWidget.IsValid())) // Widget should always be valid
			{
				if (EnumHasAnyFlags(DisplayFlags, EDisplayGridFlags::HideDisabledWidgets) && !CachedWidget->IsEnabled())
				{
					continue;
				}
				if (EnumHasAnyFlags(DisplayFlags, EDisplayGridFlags::HideUnsupportedKeyboardFocusWidgets) && !CachedWidget->SupportsKeyboardFocus())
				{
					continue;
				}

				FSlateDrawElement::MakeBox(
					WindowElementList,
					InLayer,
					CachedWidget->GetPaintSpaceGeometry().ToPaintGeometry(),
					Brush,
					ESlateDrawEffect::None,
					GetUserColorFromIndex(CurWidgetData.UserIndex).CopyWithNewOpacity(.005f)
				);
			}
		}
	};

	FCollapsedHittestGridArray AllHitTestGrids;
	GetCollapsedHittestGrid(AllHitTestGrids);
	for(const FHittestGrid* HittestGrid : AllHitTestGrids)
	{
		DisplayGrid(HittestGrid);
	}

	// Appended grid should always be valid
	FCollapsedHittestGridArray TestHitTestGrids;
	TestHitTestGrids.Add(this);
	for (int32 Index = 0; Index < TestHitTestGrids.Num(); ++Index)
	{
		const FHittestGrid* Grid = TestHitTestGrids[Index];
		for (const FAppendedGridData& OtherGridData : Grid->AppendedGridArray)
		{
			const TSharedPtr<const FHittestGrid> OtherGridPin = OtherGridData.Grid.Pin();
			if (ensureAlways(OtherGridPin.IsValid()))
			{
				TestHitTestGrids.Add(&*OtherGridPin);
			}
		}
	}
}

void FHittestGrid::DisplayNavigation(int32 InLayer, const FGeometry& AllottedGeometry, FSlateWindowElementList& WindowElementList)
{
	if (GSlateNavigationDebugging && LastNavigationMethod.IsValid())
	{
		FNavigationMethod* NavigationMethodPtr = LastNavigationMethod.GetMutablePtr();
		NavigationMethodPtr->DrawDebug(InLayer, AllottedGeometry, WindowElementList);
	}
}

TArray<FHittestGrid::FWidgetSortData> FHittestGrid::GetAllWidgetSortDatas() const
{
	TArray<FWidgetSortData> Result;
	Result.Reserve(WidgetArray.Num());
	for (const FWidgetData& WidgetData : WidgetArray)
	{
		FWidgetSortData WidgetSortData = { WidgetData.WeakWidget, WidgetData.PrimarySort, WidgetData.SecondarySort };
		Result.Add(MoveTemp(WidgetSortData));
	}
	return Result;
}
#endif // WITH_SLATE_DEBUGGING

#undef UE_SLATE_HITTESTGRID_ARRAYSIZEMAX
#undef LOCTEXT_NAMESPACE
#undef UE_SLATE_ENABLE_HITTEST_STATS
