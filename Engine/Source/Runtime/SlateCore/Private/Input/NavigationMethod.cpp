// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/NavigationMethod.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationMethod)

#define LOCTEXT_NAMESPACE "NavigationMethod"

void FNavigationMethod::Initialize(const FHittestGrid* InHittestGrid, TArray<FDebugWidgetResult>* InIntermediateResultsPtr)
{
	HittestGrid = InHittestGrid;
	IntermediateResultsPtr = InIntermediateResultsPtr;
	DisabledDestinations.Empty();
}

TSharedPtr<SWidget> FNavigationMethod::FindNextFocusableWidget(const FArrangedWidget& StartingWidget, const EUINavigation Direction, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget, int32 InUserIndex) 
{ 
	return StartingWidget.Widget; 
}

FIntPoint FNavigationMethod::GetCellCoordinate(FVector2f Position) const
{
	check(HittestGrid);
	return HittestGrid->GetCellCoordinate(Position);
}

bool FNavigationMethod::IsValidCellCoordinate(int32 X, int32 Y) const
{	
	check(HittestGrid);
	return HittestGrid->IsValidCellCoord(X, Y);
}

bool FNavigationMethod::IsParentsEnabled(const SWidget* Widget)
{
	while (Widget)
	{
		if (!Widget->IsEnabled())
		{
			return false;
		}
		Widget = Widget->Advanced_GetPaintParentWidget().Get();
	}
	return true;
}

void FNavigationMethod::ForEachFocusableWidgetsInCell(int32 X, int32 Y, const FNavigationReply& NavigationReply, int32 UserIndex, FWidgetFunc WidgetFunc)
{
	check(HittestGrid);

	FHittestGrid::FCollapsedWidgetsArray WidgetIndexes;
	HittestGrid->GetCollapsedWidgets(WidgetIndexes, X, Y);

	for (int32 WidgetIndex = WidgetIndexes.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
	{
		const FHittestGrid::FWidgetData& WidgetData = WidgetIndexes[WidgetIndex].GetWidgetData();
		const TSharedPtr<SWidget> Widget = WidgetData.GetWidget();
		if (!Widget.IsValid())
		{
			continue;
		}

		if (!HittestGrid->IsCompatibleUserIndex(UserIndex, WidgetData.UserIndex))
		{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
			AddDebugIntermediateResult(Widget, HittestGridDebuggingText::NotCompatibleWithUserIndex);
#endif
			continue;
		}

		// If we have a non escape boundary condition and this widget isn't a descendant of our boundary condition widget then it's invalid so we keep looking.
		if (NavigationReply.GetBoundaryRule() != EUINavigationRule::Escape
			&& NavigationReply.GetHandler().IsValid()
			&& !HittestGrid->IsDescendantOf(NavigationReply.GetHandler().Get(), WidgetData))
		{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
			AddDebugIntermediateResult(Widget, HittestGridDebuggingText::NotADescendant);
#endif
			continue;
		}

		if (!Widget->IsEnabled())
		{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
			AddDebugIntermediateResult(Widget, HittestGridDebuggingText::Disabled);
#endif
			continue;
		}

		if (!Widget->SupportsKeyboardFocus())
		{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
			AddDebugIntermediateResult(Widget, HittestGridDebuggingText::DoesNotSuportKeyboardFocus);
#endif
			continue;
		}

		if (DisabledDestinations.Contains(Widget))
		{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
			AddDebugIntermediateResult(Widget, HittestGridDebuggingText::ParentDisabled);
#endif
			continue;
		}

		bool bValid = WidgetFunc(Widget);
		if (bValid)
		{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
			AddDebugIntermediateResult(Widget, HittestGridDebuggingText::Valid);
#endif
		}
	}
}

#if WITH_SLATE_DEBUGGING

void FNavigationMethod::DrawDebug(int32 InLayer, const FGeometry& AllottedGeometry, FSlateWindowElementList& WindowElementList)
{
	(void)InLayer;
	(void)AllottedGeometry;
	(void)WindowElementList;
}
#endif //WITH_SLATE_DEBUGGING

#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
void FNavigationMethod::AddDebugIntermediateResult(const TSharedPtr<const SWidget>& InWidget, const FText Result) 
{ 
	if (IntermediateResultsPtr) 
	{ 
		IntermediateResultsPtr->Emplace(InWidget, Result); 
	} 
}
#endif //WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING

#undef LOCTEXT_NAMESPACE
