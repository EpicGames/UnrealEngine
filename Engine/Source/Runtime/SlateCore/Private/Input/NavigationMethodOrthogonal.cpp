// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/NavigationMethodOrthogonal.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationMethodOrthogonal)

#define LOCTEXT_NAMESPACE "NavigationMethodOrthogonal"

template<typename TCompareFunc, typename TSourceSideFunc, typename TDestSideFunc>
TSharedPtr<SWidget> FNavigationMethodOrthogonal::FindFocusableWidget(const FSlateRect WidgetRect, const FSlateRect SweptRect, int32 AxisIndex, int32 Increment, const EUINavigation Direction, const FNavigationReply& NavigationReply, TCompareFunc CompareFunc, TSourceSideFunc SourceSideFunc, TDestSideFunc DestSideFunc, int32 UserIndex)
{
	const FIntPoint NumCells = HittestGrid->NumCells;
	const FVector2f CellSize = HittestGrid->CellSize;

	FIntPoint CurrentCellPoint = HittestGrid->GetCellCoordinate(WidgetRect.GetCenter());

	int32 StartingIndex = CurrentCellPoint[AxisIndex];

	float CurrentSourceSide = SourceSideFunc(WidgetRect);

	int32 StrideAxis, StrideAxisMin, StrideAxisMax;

	// Ensure that the hit test grid is valid before proceeding
	if (NumCells.X < 1 || NumCells.Y < 1)
	{
		return TSharedPtr<SWidget>();
	}

	if (AxisIndex == 0)
	{
		StrideAxis = 1;
		StrideAxisMin = FMath::Min(FMath::Max(FMath::FloorToInt(SweptRect.Top / CellSize.Y), 0), NumCells.Y - 1);
		StrideAxisMax = FMath::Min(FMath::Max(FMath::FloorToInt(SweptRect.Bottom / CellSize.Y), 0), NumCells.Y - 1);
	}
	else
	{
		StrideAxis = 0;
		StrideAxisMin = FMath::Min(FMath::Max(FMath::FloorToInt(SweptRect.Left / CellSize.X), 0), NumCells.X - 1);
		StrideAxisMax = FMath::Min(FMath::Max(FMath::FloorToInt(SweptRect.Right / CellSize.X), 0), NumCells.X - 1);
	}

	bool bWrapped = false;
	while (CurrentCellPoint[AxisIndex] >= 0 && CurrentCellPoint[AxisIndex] < NumCells[AxisIndex])
	{
		FIntPoint StrideCellPoint = CurrentCellPoint;
		int32 CurrentCellProcessed = CurrentCellPoint[AxisIndex];

		// Increment before the search as a wrap case will change our current cell.
		CurrentCellPoint[AxisIndex] += Increment;

		FSlateRect BestWidgetRect;
		TSharedPtr<SWidget> BestWidget = TSharedPtr<SWidget>();

		for (StrideCellPoint[StrideAxis] = StrideAxisMin; StrideCellPoint[StrideAxis] <= StrideAxisMax; ++StrideCellPoint[StrideAxis])
		{
			FHittestGrid::FCollapsedWidgetsArray WidgetIndexes;
			HittestGrid->GetCollapsedWidgets(WidgetIndexes, StrideCellPoint.X, StrideCellPoint.Y);

			for (int32 i = WidgetIndexes.Num() - 1; i >= 0; --i)
			{
				const FHittestGrid::FWidgetData& TestCandidate = WidgetIndexes[i].GetWidgetData();
				const TSharedPtr<SWidget> TestWidget = TestCandidate.GetWidget();
				if (!TestWidget.IsValid())
				{
					continue;
				}

				if (!HittestGrid->IsCompatibleUserIndex(UserIndex, TestCandidate.UserIndex))
				{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
					AddDebugIntermediateResult(TestWidget, HittestGridDebuggingText::NotCompatibleWithUserIndex);
#endif
					continue;
				}

				const FSlateRect TestCandidateRect = HittestGrid->GetWidgetRenderBoundingRect(TestWidget);
				if (!(CompareFunc(DestSideFunc(TestCandidateRect), CurrentSourceSide) && FSlateRect::DoRectanglesIntersect(SweptRect, TestCandidateRect)))
				{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
					AddDebugIntermediateResult(TestWidget, HittestGridDebuggingText::DoesNotIntersect);
#endif
					continue;
				}

				// If this found widget isn't closer then the previously found widget then keep looking.
				if (BestWidget.IsValid() && !CompareFunc(DestSideFunc(BestWidgetRect), DestSideFunc(TestCandidateRect)))
				{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
					AddDebugIntermediateResult(TestWidget, HittestGridDebuggingText::PreviousWidgetIsBetter);
#endif
					continue;
				}

				// If we have a non escape boundary condition and this widget isn't a descendant of our boundary condition widget then it's invalid so we keep looking.
				if (NavigationReply.GetBoundaryRule() != EUINavigationRule::Escape
					&& NavigationReply.GetHandler().IsValid()
					&& !HittestGrid->IsDescendantOf(NavigationReply.GetHandler().Get(), TestCandidate))
				{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
					AddDebugIntermediateResult(TestWidget, HittestGridDebuggingText::NotADescendant);
#endif
					continue;
				}

				if (!TestWidget->IsEnabled())
				{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
					AddDebugIntermediateResult(TestWidget, HittestGridDebuggingText::Disabled);
#endif
					continue;
				}
				
				if (!TestWidget->SupportsKeyboardFocus())
				{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
					AddDebugIntermediateResult(TestWidget, HittestGridDebuggingText::DoesNotSuportKeyboardFocus);
#endif
					continue;
				}

				if (DisabledDestinations.Contains(TestWidget))
				{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
					AddDebugIntermediateResult(TestWidget, HittestGridDebuggingText::ParentDisabled);
#endif
					continue;
				}

				BestWidgetRect = TestCandidateRect;
				BestWidget = TestWidget;

#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
				AddDebugIntermediateResult(TestWidget, HittestGridDebuggingText::Valid);
#endif
			}
		}

		if (BestWidget.IsValid())
		{
			// Check for the need to apply our rule
			if (CompareFunc(DestSideFunc(BestWidgetRect), SourceSideFunc(SweptRect)))
			{
				switch (NavigationReply.GetBoundaryRule())
				{
				case EUINavigationRule::Explicit:
					return NavigationReply.GetFocusRecipient();
				case EUINavigationRule::Custom:
				case EUINavigationRule::CustomBoundary:
				{
					const FNavigationDelegate& FocusDelegate = NavigationReply.GetFocusDelegate();
					if (FocusDelegate.IsBound())
					{
						return FocusDelegate.Execute(Direction);
					}
					return TSharedPtr<SWidget>();
				}
				case EUINavigationRule::Stop:
					return TSharedPtr<SWidget>();
				case EUINavigationRule::Wrap:
					CurrentSourceSide = DestSideFunc(SweptRect);
					FVector2f SampleSpot = WidgetRect.GetCenter();
					SampleSpot[AxisIndex] = CurrentSourceSide;
					CurrentCellPoint = HittestGrid->GetCellCoordinate(SampleSpot);
					bWrapped = true;
					break;
				}
			}

			// Make sure all parents of the chosen widget are enabled before returning.
			// Note that IsParentsEnabled is a costly function. We call it here as the last step to minimize the number of calls to it.
			if (!IsParentsEnabled(BestWidget.Get()))
			{	
				// Find the next best widget because this one has disabled parents.
				DisabledDestinations.Add(BestWidget);
				return FindFocusableWidget(WidgetRect, SweptRect, AxisIndex, Increment, Direction, NavigationReply, CompareFunc, SourceSideFunc, DestSideFunc, UserIndex);
			}
			else
			{
				return BestWidget;
			}
		}

		// break if we have looped back to where we started.
		if (bWrapped && StartingIndex == CurrentCellProcessed) { break; }

		// If were going to fail our bounds check and our rule is to a boundary condition (Wrap or CustomBoundary) handle appropriately
		if (!(CurrentCellPoint[AxisIndex] >= 0 && CurrentCellPoint[AxisIndex] < NumCells[AxisIndex]))
		{
			if (NavigationReply.GetBoundaryRule() == EUINavigationRule::Wrap)
			{
				if (bWrapped)
				{
					// If we've already wrapped, unfortunately it must be that the starting widget wasn't within the boundary
					break;
				}
				CurrentSourceSide = DestSideFunc(SweptRect);
				FVector2f SampleSpot = WidgetRect.GetCenter();
				SampleSpot[AxisIndex] = CurrentSourceSide;
				CurrentCellPoint = HittestGrid->GetCellCoordinate(SampleSpot);
				bWrapped = true;
			}
			else if (NavigationReply.GetBoundaryRule() == EUINavigationRule::CustomBoundary)
			{
				const FNavigationDelegate& FocusDelegate = NavigationReply.GetFocusDelegate();
				if (FocusDelegate.IsBound())
				{
					return FocusDelegate.Execute(Direction);
				}
			}
		}
	}

	return TSharedPtr<SWidget>();
}

TSharedPtr<SWidget> FNavigationMethodOrthogonal::FindNextFocusableWidget(const FArrangedWidget& StartingWidget, const EUINavigation Direction, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget, int32 UserIndex)
{
	FSlateRect WidgetRect = HittestGrid->GetWidgetRenderBoundingRect(StartingWidget.Widget);
	FSlateRect BoundingRuleRect = HittestGrid->GetWidgetRenderBoundingRect(RuleWidget.Widget);
	FSlateRect SweptWidgetRect = WidgetRect;

	switch (Direction)
	{
	case EUINavigation::Left:
		SweptWidgetRect.Left = BoundingRuleRect.Left;
		SweptWidgetRect.Right = BoundingRuleRect.Right;
		SweptWidgetRect.Top += 0.5f;
		SweptWidgetRect.Bottom -= 0.5f;
		return FindFocusableWidget(WidgetRect, SweptWidgetRect, 0, -1, Direction, NavigationReply,
			[](float A, float B) { return A - 0.1f < B; }, // Compare function
			[](FSlateRect SourceRect) { return SourceRect.Left; }, // Source side function
			[](FSlateRect DestRect) { return DestRect.Right; }, // Dest side function
			UserIndex);
	case EUINavigation::Right:
		SweptWidgetRect.Left = BoundingRuleRect.Left;
		SweptWidgetRect.Right = BoundingRuleRect.Right;
		SweptWidgetRect.Top += 0.5f;
		SweptWidgetRect.Bottom -= 0.5f;
		return FindFocusableWidget(WidgetRect, SweptWidgetRect, 0, 1, Direction, NavigationReply,
			[](float A, float B) { return A + 0.1f > B; }, // Compare function
			[](FSlateRect SourceRect) { return SourceRect.Right; }, // Source side function
			[](FSlateRect DestRect) { return DestRect.Left; }, // Dest side function
			UserIndex);
	case EUINavigation::Up:
		SweptWidgetRect.Top = BoundingRuleRect.Top;
		SweptWidgetRect.Bottom = BoundingRuleRect.Bottom;
		SweptWidgetRect.Left += 0.5f;
		SweptWidgetRect.Right -= 0.5f;
		return FindFocusableWidget(WidgetRect, SweptWidgetRect, 1, -1, Direction, NavigationReply,
			[](float A, float B) { return A - 0.1f < B; }, // Compare function
			[](FSlateRect SourceRect) { return SourceRect.Top; }, // Source side function
			[](FSlateRect DestRect) { return DestRect.Bottom; }, // Dest side function
			UserIndex);
	case EUINavigation::Down:
		SweptWidgetRect.Top = BoundingRuleRect.Top;
		SweptWidgetRect.Bottom = BoundingRuleRect.Bottom;
		SweptWidgetRect.Left += 0.5f;
		SweptWidgetRect.Right -= 0.5f;
		return FindFocusableWidget(WidgetRect, SweptWidgetRect, 1, 1, Direction, NavigationReply,
			[](float A, float B) { return A + 0.1f > B; }, // Compare function
			[](FSlateRect SourceRect) { return SourceRect.Bottom; }, // Source side function
			[](FSlateRect DestRect) { return DestRect.Top; }, // Dest side function
			UserIndex);

	default:
		break;
	}

	return TSharedPtr<SWidget>(nullptr);
}

#undef LOCTEXT_NAMESPACE
