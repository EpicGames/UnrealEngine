// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Input/NavigationMethod.h"

#include "NavigationMethodOrthogonal.generated.h"

USTRUCT(DisplayName="Orthogonal")
struct FNavigationMethodOrthogonal : public FNavigationMethod
{
	GENERATED_BODY()

public:
	virtual TSharedPtr<SWidget> FindNextFocusableWidget(const FArrangedWidget& StartingWidget, const EUINavigation Direction, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget, int32 InUserIndex);
	
private:
	/** Default Navigation Behavior. Sweeps hittestgrid cells orthogonal to direction of movement. Finds widget whoe edge is nearest starting widget opposing edge **/
	template<typename TCompareFunc, typename TSourceSideFunc, typename TDestSideFunc>
	TSharedPtr<SWidget> FindFocusableWidget(const FSlateRect WidgetRect, const FSlateRect SweptRect, int32 AxisIndex, int32 Increment, const EUINavigation Direction, const FNavigationReply& NavigationReply, TCompareFunc CompareFunc, TSourceSideFunc SourceSideFunc, TDestSideFunc DestSideFunc, int32 UserIndex);
};
