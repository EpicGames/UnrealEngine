// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/NavigationMethodProximity.h"
#include "Styling/CoreStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationMethodProximity)

static bool GNavigationSearchFromCorners = false;
static FAutoConsoleVariableRef CVarNavigationSearchFromCorners(
	TEXT("Slate.Navigation.SearchFromCorners"),
	GNavigationSearchFromCorners,
	TEXT("If true, will scan for candidate widget from the corners the source widget. This will increase with width of widgets that can be navigated to"));

static bool GNavigationIgnoreCustomRules = false;
static FAutoConsoleVariableRef CVarNavigationIgnoreCustomRules(
	TEXT("Slate.Navigation.IgnoreCustomRules"),
	GNavigationIgnoreCustomRules,
	TEXT("Will disable custom navigation rules. Used for testing purposes."));

#define LOCTEXT_NAMESPACE "NavigationMethodProximity"

namespace UE::Slate::Private
{
	float MinkowskiDistance(const FVector2f& A, const FVector2f& B, float P)
	{
		if (P <= 0.0f) P = FLT_MIN;

		float sum = 0.0f;
		sum += FMath::Pow(FMath::Abs(A.X - B.X), P);
		sum += FMath::Pow(FMath::Abs(A.Y - B.Y), P);
		return FMath::Pow(sum, 1.0f / P);
	}
}

void FNavigationMethodProximity::Initialize(const FHittestGrid* InHittestGrid, TArray<FDebugWidgetResult>* InIntermediateResultsPtr)
{
	FNavigationMethod::Initialize(InHittestGrid, InIntermediateResultsPtr);
	VisitedWidgets.Empty();

#if WITH_SLATE_DEBUGGING
	DrawDebugData = {};
#endif //WITH_SLATE_DEBUGGING
}

TSharedPtr<SWidget> FNavigationMethodProximity::FindNextFocusableWidget(const FArrangedWidget& StartingWidget, const EUINavigation Direction, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget, int32 UserIndex)
{
	// Prevent self navigation
	DisabledDestinations.Add(StartingWidget.Widget);

	const FSlateRect WidgetRect = HittestGrid->GetWidgetRenderBoundingRect(StartingWidget.Widget);
	const FSlateRect BoundaryRect = HittestGrid->GetWidgetRenderBoundingRect(RuleWidget.Widget);

#if WITH_SLATE_DEBUGGING
	PrepareDrawDebugData(WidgetRect, Direction);
#endif //WITH_SLATE_DEBUGGING

	return FindFocusableWidgetFromRect(WidgetRect, BoundaryRect, Direction, NavigationReply, UserIndex);
}

TSharedPtr<SWidget> FNavigationMethodProximity::FindFocusableWidgetFromRect(const FSlateRect WidgetRect, const FSlateRect BoundaryRect, const EUINavigation Direction, const FNavigationReply& NavigationReply, int32 UserIndex)
{
	// Get the cell scan direction. Will scan in the direction of navigation
	const float SearchDirectionRadians = GetSearchDirectionRadians(Direction);
	FIntPoint CellDelta(FMath::RoundToInt32(FMath::Cos(SearchDirectionRadians)), FMath::RoundToInt32(FMath::Sin(SearchDirectionRadians)));

	TArray<FCellSearchInfo> CellsToSearch;
	PopulateCellsToSearch(WidgetRect, Direction, false, CellsToSearch);

	TSharedPtr<SWidget> BestWidget;
	float BestWidgetDistance = FLT_MAX;

	TSharedPtr<SWidget> BestBoundaryWidget;
	float BestBoundaryWidgetDistance = FLT_MAX;

	while (CellsToSearch.Num())
	{
		// Scan in reverse to support removal during iteration
		for (int32 CellsToSearchIndex = CellsToSearch.Num() - 1; CellsToSearchIndex >= 0; --CellsToSearchIndex )
		{
			FCellSearchInfo& SearchInfo = CellsToSearch[CellsToSearchIndex];
			FCellSearchResult CellResult = FindBestWidgetInCell(SearchInfo, BoundaryRect, Direction, NavigationReply, UserIndex);	
			SearchInfo.CurrentCellPoint += CellDelta;

			if (CellResult.Widget.IsValid())
			{
				// Make sure all parents of the chosen widget are enabled before returning.
				// Note that IsParentsEnabled is a costly function. We call it here as the last step to minimize the number of calls to it.
				if (!IsParentsEnabled(CellResult.Widget.Get()))
				{
					// Find the next best widget because this one has disabled parents.
					DisabledDestinations.Add(CellResult.Widget);
					return FindFocusableWidgetFromRect(WidgetRect, BoundaryRect, Direction, NavigationReply, UserIndex);
				}
			}
	
			switch(CellResult.Action)
			{
			case ECellSearchResultAction::EndSearch:
			{
				CellsToSearch.RemoveAt(CellsToSearchIndex);
				break;
			}
			case ECellSearchResultAction::ApplyBoundaryRule:
			{
				CellsToSearch.RemoveAt(CellsToSearchIndex);
				HandleBoundaryRule(WidgetRect, BoundaryRect, Direction, NavigationReply, CellResult, SearchInfo, CellsToSearch);
				break;
			}
			case ECellSearchResultAction::ContinueSearch:
				break;		
			}

			if (CellResult.Widget.IsValid())
			{
				// if the widget was set via a boundary rule, or discovered after a wrapped search. 
				if (!SearchInfo.bWrapped && CellResult.Action != ECellSearchResultAction::ApplyBoundaryRule)
				{
					if (CellResult.Distance < BestWidgetDistance)
					{
						BestWidget = CellResult.Widget;
						BestWidgetDistance = CellResult.Distance;
					}
				}
				else
				{
					if (CellResult.Distance < BestBoundaryWidgetDistance)
					{
						BestBoundaryWidget = CellResult.Widget;
						BestBoundaryWidgetDistance = CellResult.Distance;
					}
				}
			}
		}
	}

	// Standard navigation will take precedence over boundary rules. 
	// This is to avoid wrapped widgets very close to the boundary edge from take priority over further focusablke widgets within the bounds
	if (BestWidget == nullptr)
	{
		return BestBoundaryWidget;
	}

	return BestWidget;
}

FNavigationMethodProximity::FCellSearchResult FNavigationMethodProximity::FindBestWidgetInCell(const FCellSearchInfo& SearchInfo, const FSlateRect BoundaryRect, const EUINavigation Direction, const FNavigationReply& NavigationReply, int32 UserIndex)
{
	FCellSearchResult CellResult;
	CellResult.Action = ECellSearchResultAction::ContinueSearch;

	// If cell to check is beyond the grid, apply boundary rule
	if (!HittestGrid->IsValidCellCoord(SearchInfo.CurrentCellPoint))
	{
		CellResult.Action = ECellSearchResultAction::ApplyBoundaryRule;
		return CellResult;
	}

	auto WidgetFunc = [&](TSharedPtr<SWidget> Candidate) 
	{
		if (VisitedWidgets.Contains(Candidate.Get()))
		{
			return false;
		}
		VisitedWidgets.Add(Candidate.Get());

		const FSlateRect CandidateRect = HittestGrid->GetWidgetRenderBoundingRect(Candidate);
				
		if (!IsOutsideNavigationEdge(Direction, SearchInfo.CurrentSearchRect, CandidateRect))
		{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
			AddDebugIntermediateResult(Candidate, HittestGridDebuggingText::DoesNotIntersect);
#endif
			return false;
		}

		float Distance = CalculateDistance(Direction, SearchInfo.CurrentSearchRect, CandidateRect);

#if WITH_SLATE_DEBUGGING
		DebugCandidateDistance(Direction, SearchInfo.CurrentSearchRect, CandidateRect, Distance);
#endif

		// If this found widget isn't closer then the previously found widget then keep looking.
		if (CellResult.Widget.IsValid() && Distance >= CellResult.Distance)
		{
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
			AddDebugIntermediateResult(Candidate, HittestGridDebuggingText::PreviousWidgetIsBetter);
#endif
			return false;
		}

		// If the candidate is better, ensure its hierarchy is valid
		if (!IsParentsEnabled(Candidate.Get()))
		{
			DisabledDestinations.Add(Candidate);
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
			AddDebugIntermediateResult(Candidate, HittestGridDebuggingText::ParentDisabled);
#endif
			return false;
		}

		CellResult.Widget = Candidate;
		CellResult.WidgetRect = CandidateRect;
		CellResult.Distance = Distance;
		CellResult.SearchRect = SearchInfo.CurrentSearchRect;
		CellResult.Action = ECellSearchResultAction::EndSearch;
		return true;
	};

	const FIntPoint CurrentCellPoint = SearchInfo.CurrentCellPoint;
	ForEachFocusableWidgetsInCell(CurrentCellPoint.X, CurrentCellPoint.Y, NavigationReply, UserIndex, WidgetFunc);

	// If best widget is outside the bounds, apply boundary rule
	if (CellResult.Widget.IsValid() && !FSlateRect::DoRectanglesIntersect(BoundaryRect, CellResult.WidgetRect))
	{	
		CellResult.Action = ECellSearchResultAction::ApplyBoundaryRule;
	}

	return CellResult;
}

void FNavigationMethodProximity::HandleBoundaryRule(const FSlateRect SourceRect, const FSlateRect BoundaryRect, const EUINavigation Direction, const FNavigationReply& NavigationReply, FCellSearchResult& OutCellResult, FCellSearchInfo& OutSearchInfo, TArray<FCellSearchInfo>& OutCellsToSearch)
{
	switch (NavigationReply.GetBoundaryRule())
	{
	case EUINavigationRule::Explicit:
	{
		if (GNavigationIgnoreCustomRules)
		{
			break;
		}

		OutCellResult.Widget = NavigationReply.GetFocusRecipient();
		break;
	}
	case EUINavigationRule::Custom:
	case EUINavigationRule::CustomBoundary:
	{
		if (GNavigationIgnoreCustomRules)
		{
			break;
		}

		const FNavigationDelegate& FocusDelegate = NavigationReply.GetFocusDelegate();
		if (FocusDelegate.IsBound())
		{
			OutCellResult.Widget = FocusDelegate.Execute(Direction);
		}
		break;
	}
	case EUINavigationRule::Wrap:
	{
		if (!OutSearchInfo.bWrapped)
		{
			// Rescan search space by adding CellsToSearch from this new point. Add as wrapped so they do not take precedence of non-wrapped
			const FSlateRect WrappedRect = WrapRectWithinBoundary(Direction, BoundaryRect, SourceRect);
			PopulateCellsToSearch(WrappedRect, Direction, true, OutCellsToSearch);

			OutSearchInfo.bWrapped = true;
			
			// Invalidate widget, will wrap to next
			OutCellResult.Widget = nullptr;
		}
		break;
	}
	case EUINavigationRule::Stop:
	{
		OutCellResult.Widget = nullptr;
		break;
	}
	case EUINavigationRule::Escape:
	{
		break;
	}
	default:
	{
		break;
	}
	}

	if (OutCellResult.Widget.IsValid())
	{
		OutCellResult.WidgetRect = HittestGrid->GetWidgetRenderBoundingRect(OutCellResult.Widget);
		OutCellResult.Distance = CalculateDistance(Direction, OutCellResult.SearchRect, OutCellResult.WidgetRect);
	}
	else
	{
		OutCellResult.Distance = FLT_MAX;
	}
}

void FNavigationMethodProximity::CalculateSamplePoints(EUINavigation Direction, const FSlateRect& SourceRect, const FSlateRect& DestinationRect, FVector2f& OutSourcePoint, FVector2f& OutDestinationPoint) const
{
	// NOTE: Current implementation is naive, and could have improved accuracy. 
	//		Using nearest points on rectangles could get a better metric for movement distance. 
	//      However, this may cause neighboring perpindicular widgets to have a greater influence than desired. 
	//		As a workaround, perhaps take into account the distance of the point on the target to the distance of the cone (or direction of travel)
	const FVector2f SearchDirection = GetSearchDirection(Direction);

	OutSourcePoint = SourceRect.GetCenter() + SourceRect.GetSize() * 0.5f * SearchDirection;
	OutDestinationPoint = DestinationRect.GetCenter() + DestinationRect.GetSize() * -0.5f * SearchDirection; // The target rect offset is the inverse of the source
}

float FNavigationMethodProximity::CalculateDistance(EUINavigation Direction, const FSlateRect& SourceRect, const FSlateRect& DestinationRect) const
{
	FVector2f SourcePoint;
	FVector2f DestinationPoint; 
	CalculateSamplePoints(Direction, SourceRect, DestinationRect, SourcePoint, DestinationPoint);
	return UE::Slate::Private::MinkowskiDistance(SourcePoint, DestinationPoint, AlignmentFactor);
}

void FNavigationMethodProximity::PopulateCellsToSearch(const FSlateRect& SourceRect, EUINavigation Direction, bool bWrapped, TArray<FCellSearchInfo>& OutCellsToSearch)
{
	// Find the starting cells by raymarching two mirror rays at an angle from the search point
	const float SearchDirectionRadians = GetSearchDirectionRadians(Direction);

	FVector2f SearchPointMin, SearchPointMax;
	GetSearchPoints(Direction, SourceRect, SearchPointMin, SearchPointMax);

	FVector2f CellPointMin = GetCellCoordinate(SearchPointMin);
	FVector2f CellPointMax = GetCellCoordinate(SearchPointMax);

	// Fill cells between edges
	FVector2f EdgeDelta = (CellPointMax - CellPointMin);
	EdgeDelta.Normalize();
	for (float Y = CellPointMin.Y; Y < CellPointMin.Y + EdgeDelta.Y; ++Y)
	{
		for (float X = CellPointMin.X; X < CellPointMin.X + EdgeDelta.X; ++X)
		{
			const FVector2f EdgePoint(X, Y);
			FCellSearchInfo SearchInfo(this, SourceRect, EdgePoint, bWrapped);

			OutCellsToSearch.Add(SearchInfo);
		}
	}

	// Apply the search angle to the angle of the navigation
	const float SearchAngleRadians = FMath::DegreesToRadians(SearchAngleDegrees);
	const FVector2f DeltaMin = FVector2f(FMath::Cos(SearchDirectionRadians + SearchAngleRadians), FMath::Sin(SearchDirectionRadians + SearchAngleRadians));
	const FVector2f DeltaMax = FVector2f(FMath::Cos(SearchDirectionRadians - SearchAngleRadians), FMath::Sin(SearchDirectionRadians - SearchAngleRadians));

	bool CheckCellPointMin = true;
	bool CheckCellPointMax = true;
	while (CheckCellPointMin || CheckCellPointMax)
	{
		CheckCellPointMin = IsValidCellCoordinate(CellPointMin.X, CellPointMin.Y);
		CheckCellPointMax = IsValidCellCoordinate(CellPointMax.X, CellPointMax.Y);

		if (CheckCellPointMin)
		{
			FCellSearchInfo SearchInfo(this, SourceRect, CellPointMin, bWrapped);
			OutCellsToSearch.Add(SearchInfo);

			CellPointMin += DeltaMin;
		}
		if (CheckCellPointMax)
		{
			FCellSearchInfo SearchInfo(this, SourceRect, CellPointMax, bWrapped);

			OutCellsToSearch.Add(SearchInfo);
			CellPointMax += DeltaMax;
		}
	}

#if WITH_SLATE_DEBUGGING
	for (const FCellSearchInfo& Cell : OutCellsToSearch)
	{
		DebugSearchCell(Cell);
	}
#endif //WITH_SLATE_DEBUGGING
}

FVector2f FNavigationMethodProximity::GetSearchDirection(const EUINavigation Direction) const
{	
	switch (Direction)
	{
	case EUINavigation::Left:
		return FVector2f(-1.0f, 0.0f);
	case EUINavigation::Right:
		return FVector2f(1.0f, 0.0f);
	case EUINavigation::Up:
		return FVector2f(0.0f, -1.0f);
	case EUINavigation::Down:
		return FVector2f(0.0f, 1.0f);
	default:
		check(false);
	}
	return FVector2f();
}

void FNavigationMethodProximity::GetSearchPoints(const EUINavigation Direction, const FSlateRect& SourceRect, FVector2f& OutCellSearchPointMin, FVector2f& OutCellSearchPointMax) const
{
	if (!GNavigationSearchFromCorners)
	{
		// Return from the center of the search edge
		OutCellSearchPointMin = SourceRect.GetCenter() + SourceRect.GetSize() * 0.5f * GetSearchDirection(Direction);
		OutCellSearchPointMax = OutCellSearchPointMin;
		return;
	}

	switch (Direction)
	{
	case EUINavigation::Left:
		OutCellSearchPointMin = FVector2f(SourceRect.Left, SourceRect.Top);
		OutCellSearchPointMax = FVector2f(SourceRect.Left, SourceRect.Bottom);
		break;
	case EUINavigation::Right:
		OutCellSearchPointMin = FVector2f(SourceRect.Right, SourceRect.Top);
		OutCellSearchPointMax = FVector2f(SourceRect.Right, SourceRect.Bottom);
		break;
	case EUINavigation::Up:
		OutCellSearchPointMin = FVector2f(SourceRect.Left, SourceRect.Top);
		OutCellSearchPointMax = FVector2f(SourceRect.Right, SourceRect.Top);
		break;
	case EUINavigation::Down:
		OutCellSearchPointMin = FVector2f(SourceRect.Left, SourceRect.Bottom);
		OutCellSearchPointMax = FVector2f(SourceRect.Right, SourceRect.Bottom);
		break;
	}
}

float FNavigationMethodProximity::GetSearchDirectionRadians(const EUINavigation Direction) const
{
	switch (Direction)
	{
	case EUINavigation::Left:
		return static_cast<float>(PI);
	case EUINavigation::Right:
		return 0.0f;
	case EUINavigation::Up:
		return static_cast<float>(-PI) / 2.0f;
	case EUINavigation::Down:
		return static_cast<float>(PI) / 2.0f;
	default:
		check(false);
	}
	return 0;
}

bool FNavigationMethodProximity::IsOutsideNavigationEdge(const EUINavigation Direction, const FSlateRect& SourceRect, const FSlateRect& DestinationRect) const
{
	switch (Direction)
	{
	case EUINavigation::Left: 
		return DestinationRect.Right - OverlapThreshold < SourceRect.Left;
	case EUINavigation::Right:
		return DestinationRect.Left + OverlapThreshold > SourceRect.Right;
	case EUINavigation::Up:
		return DestinationRect.Bottom - OverlapThreshold < SourceRect.Top;
	case EUINavigation::Down:
		return DestinationRect.Top + OverlapThreshold > SourceRect.Bottom;
	default:
		check(false);
	}
	return 0;
}

FSlateRect FNavigationMethodProximity::WrapRectWithinBoundary(const EUINavigation Direction, const FSlateRect& BoundaryRect, const FSlateRect& SourceRect) const
{
	switch (Direction)
	{
	case EUINavigation::Left:
	{
		FSlateRect OutRect = SourceRect;
		float Width = OutRect.GetSize().X;
		OutRect.Left = BoundaryRect.Right;
		OutRect.Right = OutRect.Left + Width;
		return OutRect;
	}
	case EUINavigation::Right:
	{
		FSlateRect OutRect = SourceRect;
		float Width = OutRect.GetSize().X;
		OutRect.Right = BoundaryRect.Left;
		OutRect.Left = OutRect.Right - Width;
		return OutRect;
	}
	case EUINavigation::Up:
	{
		FSlateRect OutRect = SourceRect;
		float Height = OutRect.GetSize().Y;
		OutRect.Top = BoundaryRect.Bottom;
		OutRect.Bottom = OutRect.Top + Height;
		return OutRect;
	}
	case EUINavigation::Down:
	{
		FSlateRect OutRect = SourceRect;
		float Height = OutRect.GetSize().Y;
		OutRect.Bottom = BoundaryRect.Top;
		OutRect.Top = OutRect.Bottom - Height;
		return OutRect;
	}
	default:
		check(false);
	}

	return FSlateRect();
}

#if WITH_SLATE_DEBUGGING

void FNavigationMethodProximity::PrepareDrawDebugData(const FSlateRect& SourceRect, EUINavigation Direction)
{
	// Draw wireframe rect
	DrawDebugData.Points.Add(SourceRect.GetTopLeft());
	DrawDebugData.Points.Add(SourceRect.GetTopRight());
	DrawDebugData.Colors.Add(FLinearColor::Green);
	DrawDebugData.Colors.Add(FLinearColor::Green);

	DrawDebugData.Points.Add(SourceRect.GetTopRight());
	DrawDebugData.Points.Add(SourceRect.GetBottomRight());
	DrawDebugData.Colors.Add(FLinearColor::Green);
	DrawDebugData.Colors.Add(FLinearColor::Green);

	DrawDebugData.Points.Add(SourceRect.GetBottomRight());
	DrawDebugData.Points.Add(SourceRect.GetBottomLeft());
	DrawDebugData.Colors.Add(FLinearColor::Green);
	DrawDebugData.Colors.Add(FLinearColor::Green);

	DrawDebugData.Points.Add(SourceRect.GetTopLeft());
	DrawDebugData.Points.Add(SourceRect.GetBottomLeft());
	DrawDebugData.Colors.Add(FLinearColor::Green);
	DrawDebugData.Colors.Add(FLinearColor::Green);

	// Draw Search Cone
	FVector2f SearchPointMin, SearchPointMax;
	GetSearchPoints(Direction, SourceRect, SearchPointMin, SearchPointMax);
	
	const float SearchDirectionRadians = GetSearchDirectionRadians(Direction);
	const float SearchAngleRadians = FMath::DegreesToRadians(SearchAngleDegrees);
	const FVector2f DirectionMin = FVector2f(FMath::Cos(SearchDirectionRadians + SearchAngleRadians), FMath::Sin(SearchDirectionRadians + SearchAngleRadians));
	const FVector2f DirectionMax = FVector2f(FMath::Cos(SearchDirectionRadians - SearchAngleRadians), FMath::Sin(SearchDirectionRadians - SearchAngleRadians));

	const float RayLength = 500;
	DrawDebugData.Points.Add(SearchPointMin);
	DrawDebugData.Points.Add(SearchPointMin + DirectionMin * RayLength);
	DrawDebugData.Colors.Add(FLinearColor::Blue);
	DrawDebugData.Colors.Add(FLinearColor::Blue);
	
	DrawDebugData.Points.Add(SearchPointMax);
	DrawDebugData.Points.Add(SearchPointMax + DirectionMax * RayLength);
	DrawDebugData.Colors.Add(FLinearColor::Blue);
	DrawDebugData.Colors.Add(FLinearColor::Blue);
	
}

void FNavigationMethodProximity::DebugCandidateDistance(EUINavigation Direction, const FSlateRect& SourceRect, const FSlateRect& DestinationRect, float Distance)
{
	FVector2f SourcePoint, TargetPoint;
	CalculateSamplePoints(Direction, SourceRect, DestinationRect, SourcePoint, TargetPoint);

	DrawDebugData.Points.Add(SourcePoint);
	DrawDebugData.Points.Add(TargetPoint);
	DrawDebugData.LabelPositions.Add((SourcePoint + TargetPoint) * 0.5f);
	DrawDebugData.Colors.Add(FLinearColor::Green);
	DrawDebugData.Colors.Add(FLinearColor::Green);
	DrawDebugData.Distances.Add(Distance);
}

void FNavigationMethodProximity::DebugSearchCell(const FCellSearchInfo& Cell)
{
	FVector2f CellWorldSpace = FVector2f(static_cast<float>(Cell.CurrentCellPoint.X), static_cast<float>(Cell.CurrentCellPoint.Y)) * HittestGrid->GetCellSize();
	FSlateRect Rect(CellWorldSpace, CellWorldSpace + HittestGrid->GetCellSize());

	DrawDebugData.Points.Add(Rect.GetTopLeft());
	DrawDebugData.Points.Add(Rect.GetTopRight());
	DrawDebugData.Colors.Add(FLinearColor::Red);
	DrawDebugData.Colors.Add(FLinearColor::Red);

	DrawDebugData.Points.Add(Rect.GetTopRight());
	DrawDebugData.Points.Add(Rect.GetBottomRight());
	DrawDebugData.Colors.Add(FLinearColor::Red);
	DrawDebugData.Colors.Add(FLinearColor::Red);

	DrawDebugData.Points.Add(Rect.GetBottomRight());
	DrawDebugData.Points.Add(Rect.GetBottomLeft());
	DrawDebugData.Colors.Add(FLinearColor::Red);
	DrawDebugData.Colors.Add(FLinearColor::Red);

	DrawDebugData.Points.Add(Rect.GetTopLeft());
	DrawDebugData.Points.Add(Rect.GetBottomLeft());
	DrawDebugData.Colors.Add(FLinearColor::Red);
	DrawDebugData.Colors.Add(FLinearColor::Red);
}

void FNavigationMethodProximity::DrawDebug(int32 InLayer, const FGeometry& AllottedGeometry, FSlateWindowElementList& WindowElementList) 
{
	ensure(DrawDebugData.Colors.Num() == DrawDebugData.Points.Num());
	
	FSlateDrawElement::MakeLines(WindowElementList, InLayer, AllottedGeometry.ToPaintGeometry(), DrawDebugData.Points, DrawDebugData.Colors);
	
	const FSlateFontInfo FontInfo = FCoreStyle::GetDefaultFontStyle("Regular", 8);	
	for (int32 Index = 0; Index < DrawDebugData.Distances.Num(); ++Index)
	{
		const float Distance = DrawDebugData.Distances[Index];
		const FVector2f LabelPosition = DrawDebugData.LabelPositions[Index];
		const FLinearColor Color = FLinearColor(0, 0.89f, 0.05f, 1); // Greenish
		const FText Label = FText::Format(LOCTEXT("NavigationLabel", "{0}"), Distance);
		const FPaintGeometry LabelGeometry = AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(LabelPosition));
		FSlateDrawElement::MakeText(WindowElementList, InLayer, LabelGeometry, Label, FontInfo, ESlateDrawEffect::None, Color);
	}
}

#endif //WITH_SLATE_DEBUGGING


#undef LOCTEXT_NAMESPACE
