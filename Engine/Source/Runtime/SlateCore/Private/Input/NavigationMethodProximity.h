// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Input/HittestGrid.h"
#include "Input/NavigationMethod.h"

#include "NavigationMethodProximity.generated.h"

USTRUCT(DisplayName="Proximity")
struct FNavigationMethodProximity : public FNavigationMethod
{
	GENERATED_BODY()

public:		
	TSharedPtr<SWidget> FindNextFocusableWidget(const FArrangedWidget& StartingWidget, const EUINavigation Direction, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget, int32 InUserIndex) override;

	void Initialize(const FHittestGrid* InHittestGrid, TArray<FDebugWidgetResult>* InIntermediateResultsPtr) override;

private:
	/** Controls the preference for off-axis widgets. Values 0.0 to 2.0. Lower values increases preference for axis aligned widgets **/
	UPROPERTY(EditAnywhere, Category = Navigation, meta = (ClampMin = "0.0", ClampMax = "2.0"))
	float AlignmentFactor = 1.0f;

	/**	Controls the view angle in degrees from source widget to candidate widget search region. Values 0 to 90 **/
	UPROPERTY(EditAnywhere, Category = Navigation, meta = (ClampMin = "0.0", ClampMax = "90.0", Units = "Degrees"))
	float SearchAngleDegrees = 45.0f;

	/**	In slate units, controls the allowed the amount of widget edge overlap widget when considering candidate widget **/
	UPROPERTY(EditAnywhere, Category = Navigation, meta = (ClampMin = "0.0"))
	float OverlapThreshold = 20.0f;

private:

	struct FCellSearchInfo
	{
		FCellSearchInfo(const FNavigationMethodProximity* NavigationMethod, const FSlateRect& CurrentSearchRect, const FVector2f& SamplePoint, bool bWrapped)
			: NavigationMethod(NavigationMethod)
			, CurrentSearchRect(CurrentSearchRect)
			, CurrentCellPoint(FMath::RoundToInt(SamplePoint.X), FMath::RoundToInt(SamplePoint.Y))
			, StartingCellPoint(FMath::RoundToInt(SamplePoint.X), FMath::RoundToInt(SamplePoint.Y))
			, bWrapped(bWrapped)
		{}

		const FNavigationMethodProximity* NavigationMethod;
		FSlateRect CurrentSearchRect;
		FIntPoint CurrentCellPoint;
		FIntPoint StartingCellPoint;
		bool bWrapped : 1 ;
	};

	enum class ECellSearchResultAction
	{
		ContinueSearch,
		EndSearch,
		ApplyBoundaryRule
	};

	struct FCellSearchResult
	{
		TSharedPtr<SWidget> Widget;
		FSlateRect WidgetRect;
		FSlateRect SearchRect;
		float Distance = FLT_MAX;
		ECellSearchResultAction Action;
	};

	TSharedPtr<SWidget> FindFocusableWidgetFromRect(const FSlateRect WidgetRect, const FSlateRect BoundaryRect, const EUINavigation Direction, const FNavigationReply& NavigationReply, int32 UserIndex);
	FCellSearchResult FindBestWidgetInCell(const FCellSearchInfo& SearchInfo, const FSlateRect BoundaryRect, const EUINavigation Direction, const FNavigationReply& NavigationReply, int32 UserIndex);	
	void HandleBoundaryRule(const FSlateRect SourceRect, const FSlateRect BoundaryRect, const EUINavigation Direction, const FNavigationReply& NavigationReply, FCellSearchResult& OutCellResult, FCellSearchInfo& SearchInfo, TArray<FCellSearchInfo>& OutCellsToSearch);
	void PopulateCellsToSearch(const FSlateRect& SourceRect, EUINavigation Direction, bool bWrapped, TArray<FCellSearchInfo>& OutCellsToSearch);
	
	void CalculateSamplePoints(EUINavigation Direction, const FSlateRect& SourceRect, const FSlateRect& DestinationRect, FVector2f& OutSourcePoint, FVector2f& OutDestinationPoint) const;
	float CalculateDistance(EUINavigation Direction, const FSlateRect& SourceRect, const FSlateRect& DestinationRect) const;

	FVector2f GetSearchDirection(const EUINavigation Direction) const;
	void GetSearchPoints(const EUINavigation Direction, const FSlateRect& SourceRect, FVector2f& OutCellSearchPointMin, FVector2f& OutCellSearchPointMax) const;
	float GetSearchDirectionRadians(EUINavigation Direction) const;

	bool IsOutsideNavigationEdge(const EUINavigation Direction, const FSlateRect& SourceRect, const FSlateRect& DestinationRect) const;
	FSlateRect WrapRectWithinBoundary(const EUINavigation Direction, const FSlateRect& BoundaryRect, const FSlateRect& SourceRect) const;

	TSet<SWidget*> VisitedWidgets;

private:
#if WITH_SLATE_DEBUGGING

	struct FDrawDebugData
	{
		TArray<FVector2f> LabelPositions;
		TArray<float> Distances;
		TArray<FVector2f> Points;
		TArray<FLinearColor> Colors;
	};

	FDrawDebugData DrawDebugData;

	void PrepareDrawDebugData(const FSlateRect& SourceRect, EUINavigation Direction);
	void DebugCandidateDistance(EUINavigation Direction, const FSlateRect& SourceRect, const FSlateRect& DestinationRect, float Distance);
	void DebugSearchCell(const FCellSearchInfo& Cell);
	void DrawDebug(int32 InLayer, const FGeometry& AllottedGeometry, FSlateWindowElementList& WindowElementList);

#endif //WITH_SLATE_DEBUGGING
};