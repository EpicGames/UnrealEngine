// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/NavigationRouting.h"

#include "Input/HittestGrid.h"
#include "Types/NavigationMetaData.h"
#include "Math/UnrealMathUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NavigationRouting)


TSharedPtr<SWidget> FNavigationRouting::RouteNavigationFocus(const FNavigationRoutingParams& Params)
{
	EWidgetNavigationRoutingPolicy Policy = EWidgetNavigationRoutingPolicy::Default;

	// Find owning policy
	TSharedPtr<SWidget> PolicyWidget = Params.FocusedWidget;
	while (PolicyWidget.IsValid())
	{
		TSharedPtr<FNavigationMetaData> NavigationMetaData = PolicyWidget->GetMetaData<FNavigationMetaData>();
		if (NavigationMetaData.IsValid())
		{
			Policy = NavigationMetaData->GetNavigationRoutingPolicy();
			if (Policy != EWidgetNavigationRoutingPolicy::Default)
			{
				break;
			}
		}

		PolicyWidget = PolicyWidget->GetParentWidget();
	}

	if (!PolicyWidget.IsValid())
	{
		return Params.FocusedWidget;
	}

	// If starting widget is within policy widget, do not apply routing policy. Only apply when focus enters the policy widget
	const FSlateRect WidgetRect = Params.HittestGrid->GetWidgetRenderBoundingRect(Params.StartingWidget);
	const FSlateRect BoundaryRect = Params.HittestGrid->GetWidgetRenderBoundingRect(PolicyWidget);
	if (FSlateRect::DoRectanglesIntersect(WidgetRect, BoundaryRect))
	{
		return Params.FocusedWidget;
	}

	switch (Policy)
	{
	case EWidgetNavigationRoutingPolicy::AcceptFocus:
	{
		return Params.FocusedWidget;
	}
	case EWidgetNavigationRoutingPolicy::RouteToTopMostChild:
	{
		auto GetEdgeFunc = [](const FSlateRect& Rect) { return Rect.Top; };
		return RouteNavigationFocusToEdge(Params, 1, 1, BoundaryRect, BoundaryRect.GetTopLeft(), GetEdgeFunc);
	}
	case EWidgetNavigationRoutingPolicy::RouteToBottomMostChild:
	{
		auto GetEdgeFunc = [](const FSlateRect& Rect) { return Rect.Bottom; };
		return RouteNavigationFocusToEdge(Params, -1, 1, BoundaryRect, BoundaryRect.GetBottomLeft(), GetEdgeFunc);
	}
	case EWidgetNavigationRoutingPolicy::RouteToLeftMostChild:
	{
		auto GetEdgeFunc = [](const FSlateRect& Rect) { return Rect.Left; };
		return RouteNavigationFocusToEdge(Params, 1, 0,  BoundaryRect, BoundaryRect.GetTopLeft(), GetEdgeFunc);
	}
	case EWidgetNavigationRoutingPolicy::RouteToRightMostChild:
	{
		auto GetEdgeFunc = [](const FSlateRect& Rect) { return Rect.Right; };
		return RouteNavigationFocusToEdge(Params, -1, 0,  BoundaryRect, BoundaryRect.GetTopRight(), GetEdgeFunc);
	}
	case EWidgetNavigationRoutingPolicy::RouteToTopLeftChild:
	{
		// Routing to top left corner, scan from the top of the boundary, find widget nearest the left edge of rect
		auto GetEdgeFunc = [](const FSlateRect& Rect) { return Rect.Left; };
		return RouteNavigationFocusToEdge(Params, 1, 1, BoundaryRect, BoundaryRect.GetTopLeft(), GetEdgeFunc);
	}
	case EWidgetNavigationRoutingPolicy::RouteToTopRightChild:
	{
		// Routing to top right corner, scan from the top of the boundary, find widget nearest the right edge of rect
		auto GetEdgeFunc = [](const FSlateRect& Rect) { return Rect.Right; };
		return RouteNavigationFocusToEdge(Params, 1, 1, BoundaryRect, BoundaryRect.GetTopLeft(), GetEdgeFunc);
	}
	case EWidgetNavigationRoutingPolicy::RouteToBottomLeftChild:
	{
		// Routing to bottom left corner, scan from the bottom of the boundary, find widget nearest the left edge of rect
		auto GetEdgeFunc = [](const FSlateRect& Rect) { return Rect.Left; };
		return RouteNavigationFocusToEdge(Params, -1, 1, BoundaryRect, BoundaryRect.GetBottomLeft(), GetEdgeFunc);
	}
	case EWidgetNavigationRoutingPolicy::RouteToBottomRightChild:
	{
		// Routing to bottom right corner, scan from the bottom of the boundary, find widget nearest the right edge of rect
		auto GetEdgeFunc = [](const FSlateRect& Rect) { return Rect.Right; };
		return RouteNavigationFocusToEdge(Params, -1, 1, BoundaryRect, BoundaryRect.GetBottomLeft(), GetEdgeFunc);
	}
	default:
	{
		// Fallback to default. Do no routing.
		return Params.FocusedWidget;
	}
	}
}


TSharedPtr<SWidget> FNavigationRouting::RouteNavigationFocusToEdge(const FNavigationRoutingParams& Params, int Direction, int AxisIndex, const FSlateRect& BoundaryRect, const FVector2f& StartingPoint, FGetEdgeFunc GetEdgeFunc)
{
	const float BoundaryEdge = GetEdgeFunc(BoundaryRect);

	const FIntPoint NumCells = Params.HittestGrid->NumCells;
	const FVector2f CellSize = Params.HittestGrid->CellSize;
	FIntPoint CurrentCellPoint = Params.HittestGrid->GetCellCoordinate(StartingPoint);

	int32 StrideAxis, StrideAxisMin, StrideAxisMax;
	if (AxisIndex == 0)
	{
		StrideAxis = 1;
		StrideAxisMin = FMath::Min(FMath::Max(FMath::FloorToInt(BoundaryRect.Top / CellSize.Y), 0), NumCells.Y - 1);
		StrideAxisMax = FMath::Min(FMath::Max(FMath::FloorToInt(BoundaryRect.Bottom / CellSize.Y), 0), NumCells.Y - 1);
	}
	else
	{
		StrideAxis = 0;
		StrideAxisMin = FMath::Min(FMath::Max(FMath::FloorToInt(BoundaryRect.Left / CellSize.X), 0), NumCells.X - 1);
		StrideAxisMax = FMath::Min(FMath::Max(FMath::FloorToInt(BoundaryRect.Right / CellSize.X), 0), NumCells.X - 1);
	}

	while (CurrentCellPoint[AxisIndex] >= 0 && CurrentCellPoint[AxisIndex] < NumCells[AxisIndex])
	{
		FIntPoint StrideCellPoint = CurrentCellPoint;

		// Find first nearest widget within stride
		float DistanceNearestEdge = FLT_MAX;
		TSharedPtr<SWidget> WidgetNearestEdge;

		for (StrideCellPoint[StrideAxis] = StrideAxisMin; StrideCellPoint[StrideAxis] <= StrideAxisMax; ++StrideCellPoint[StrideAxis])
		{
			FHittestGrid::FCollapsedWidgetsArray WidgetIndexes;
			Params.HittestGrid->GetCollapsedWidgets(WidgetIndexes, StrideCellPoint.X, StrideCellPoint.Y);

			for (int32 WidgetIndex = WidgetIndexes.Num() - 1; WidgetIndex >= 0; --WidgetIndex)
			{
				const FHittestGrid::FWidgetData& CandidateWidgetData = WidgetIndexes[WidgetIndex].GetWidgetData();
				const TSharedPtr<SWidget> CandidateWidget = CandidateWidgetData.GetWidget();
				if (!CandidateWidget.IsValid())
				{
					continue;
				}

				if (!Params.HittestGrid->IsCompatibleUserIndex(Params.UserIndex, CandidateWidgetData.UserIndex))
				{
					continue;
				}

				// If we have a non escape boundary condition and this widget isn't a descendant of our boundary condition widget then it's invalid so we keep looking.
				if (Params.NavigationReply.GetBoundaryRule() != EUINavigationRule::Escape
					&& Params.NavigationReply.GetHandler().IsValid()
					&& !Params.HittestGrid->IsDescendantOf(Params.NavigationReply.GetHandler().Get(), CandidateWidgetData))
				{
					continue;
				}

				if (!CandidateWidget->IsEnabled())
				{
					continue;
				}

				if (!CandidateWidget->SupportsKeyboardFocus())
				{
					continue;
				}

				const FSlateRect& WidgetRect = Params.HittestGrid->GetWidgetRenderBoundingRect(CandidateWidget);
				if (!FSlateRect::DoRectanglesIntersect(BoundaryRect, WidgetRect))
				{
					continue;
				}

				const float Edge = GetEdgeFunc(WidgetRect);
				const float Distance = FMath::Abs(Edge - BoundaryEdge);
				if (!WidgetNearestEdge.IsValid() || Distance < DistanceNearestEdge)
				{
					DistanceNearestEdge = Distance;
					WidgetNearestEdge = CandidateWidget;
				}
			}
		}

		// Return the first widget found when searching from the edge
		if (WidgetNearestEdge.IsValid())
		{
			return WidgetNearestEdge;
		}

		CurrentCellPoint[AxisIndex] += Direction;
	}

	// No widget found.
	return Params.FocusedWidget;
}
