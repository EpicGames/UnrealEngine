// Copyright Epic Games, Inc. All Rights Reserved.

#include "Input/Reply.h"

FReply::FReply( bool bIsHandled )
	: TReplyBase<FReply>(bIsHandled)
	, RequestedMousePos()
	, EventHandler(nullptr)
	, MouseCaptor(nullptr)
	, FocusRecipient(nullptr)
	, MouseLockWidget(nullptr)
	, DetectDragForWidget(nullptr)
	, NavigationDestination(nullptr)
	, DragDropContent(nullptr)
	, FocusChangeReason(EFocusCause::SetDirectly)
	, NavigationType(EUINavigation::Invalid)
	, NavigationGenesis(ENavigationGenesis::User)
	, NavigationSource(ENavigationSource::FocusedWidget)
	, bReleaseMouseCapture(false)
	, bSetUserFocus(false)
	, bReleaseUserFocus(false)
	, bAllUsers(false)
	, bShouldReleaseMouseLock(false)
	, bUseHighPrecisionMouse(false)
	, bPreventThrottling(false)
	, bEndDragDrop(false)
{
}

FReply::FReply(const FReply&) = default;

FReply& FReply::operator =(const FReply&) = default;

FReply& FReply::SetMousePos(const FIntPoint& NewMousePos)
{
	this->RequestedMousePos = NewMousePos;
	return Me();
}

FReply& FReply::SetUserFocus(TSharedRef<SWidget> GiveMeFocus, EFocusCause ReasonFocusIsChanging, bool bInAllUsers)
{
	this->bSetUserFocus = true;
	this->FocusRecipient = GiveMeFocus;
	this->FocusChangeReason = ReasonFocusIsChanging;
	this->bReleaseUserFocus = false;
	this->bAllUsers = bInAllUsers;
	return Me();
}

FReply& FReply::ClearUserFocus(EFocusCause ReasonFocusIsChanging, bool bInAllUsers)
{
	this->FocusRecipient = nullptr;
	this->FocusChangeReason = ReasonFocusIsChanging;
	this->bReleaseUserFocus = true;
	this->bSetUserFocus = false;
	this->bAllUsers = bInAllUsers;
	return Me();
}

FReply& FReply::CancelFocusRequest()
{
	this->bSetUserFocus = false;
	this->FocusRecipient = nullptr;
	this->bReleaseUserFocus = false;

	return Me();
}

FString FReply::ToString()
{
	FString HandledStr = IsEventHandled() ? TEXT("Handled") : TEXT("Unhandled");

	if (bReleaseMouseCapture)
	{
		HandledStr += TEXT("+ReleaseMouseCapture");
	}
	if (bSetUserFocus)
	{
		HandledStr += TEXT("+SetUserFocus");
	}

	return HandledStr;
}
