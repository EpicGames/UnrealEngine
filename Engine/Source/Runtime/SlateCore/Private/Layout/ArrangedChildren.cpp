// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/ArrangedChildren.h"
#include "Widgets/SWidget.h"


/* FArrangedChildren interface
 *****************************************************************************/

void FArrangedChildren::AddWidget(const FArrangedWidget& InWidgetGeometry)
{
	AddWidget(InWidgetGeometry.Widget->GetVisibility(), InWidgetGeometry);
}

void FArrangedChildren::InsertWidget(const FArrangedWidget& InWidgetGeometry, int32 Index)
{
	InsertWidget(InWidgetGeometry.Widget->GetVisibility(), InWidgetGeometry, Index);
}