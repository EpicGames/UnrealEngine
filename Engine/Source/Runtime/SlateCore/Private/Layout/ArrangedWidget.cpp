// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/ArrangedWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

// Use function to initialize because SNullWidget::NullWidget is not statically initialized yet
const FArrangedWidget& FArrangedWidget::GetNullWidget()
{
	static FArrangedWidget NullArrangedWidget(SNullWidget::NullWidget, FGeometry());
	checkSlow(&SNullWidget::NullWidget.Get() != nullptr);
	return NullArrangedWidget;
}

FString FArrangedWidget::ToString( ) const
{
	return FString::Printf(TEXT("%s @ %s"), *Widget->ToString(), *Geometry.ToString());
}


FWidgetAndPointer::FWidgetAndPointer()
	: FArrangedWidget(FArrangedWidget::GetNullWidget())
{}

FWidgetAndPointer::FWidgetAndPointer(const FArrangedWidget& InWidget)
	: FArrangedWidget(InWidget)
	, OptionalPointerPosition()
{}

FWidgetAndPointer::FWidgetAndPointer(const FArrangedWidget& InWidget, TOptional<FVirtualPointerPosition> InPosition)
	: FArrangedWidget(InWidget)
	, OptionalPointerPosition(InPosition)
{}
