// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/Clipping.h"
#include "Misc/AssertionMacros.h"
#include "Logging/LogVerbosity.h"
#include "SlateGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Clipping)

FSlateClippingZone::FSlateClippingZone(const FShortRect& AxisAlignedRect)
	: bIsAxisAligned(true)
	, bIntersect(true)
	, bAlwaysClip(false)
{
	int16 Left = FMath::Min(AxisAlignedRect.Left, AxisAlignedRect.Right);
	int16 Right = FMath::Max(AxisAlignedRect.Left, AxisAlignedRect.Right);
	int16 Top = FMath::Min(AxisAlignedRect.Top, AxisAlignedRect.Bottom);
	int16 Bottom = FMath::Max(AxisAlignedRect.Top, AxisAlignedRect.Bottom);

	TopLeft = FVector2f(Left, Top);
	TopRight = FVector2f(Right, Top);
	BottomLeft = FVector2f(Left, Bottom);
	BottomRight = FVector2f(Right, Bottom);
}

FSlateClippingZone::FSlateClippingZone(const FSlateRect& AxisAlignedRect)
	: bIsAxisAligned(true)
	, bIntersect(true)
	, bAlwaysClip(false)
{
	FSlateRect RoundedAxisAlignedRect = AxisAlignedRect.Round();
	float Left = FMath::Min(RoundedAxisAlignedRect.Left, RoundedAxisAlignedRect.Right);
	float Right = FMath::Max(RoundedAxisAlignedRect.Left, RoundedAxisAlignedRect.Right);
	float Top = FMath::Min(RoundedAxisAlignedRect.Top, RoundedAxisAlignedRect.Bottom);
	float Bottom = FMath::Max(RoundedAxisAlignedRect.Top, RoundedAxisAlignedRect.Bottom);

	TopLeft = FVector2f(Left, Top);
	TopRight = FVector2f(Right, Top);
	BottomLeft = FVector2f(Left, Bottom);
	BottomRight = FVector2f(Right, Bottom);
}

FSlateClippingZone::FSlateClippingZone(const FGeometry& BooundingGeometry)
	: bIntersect(true)
	, bAlwaysClip(false)
{
	const FSlateRenderTransform& Transform = BooundingGeometry.GetAccumulatedRenderTransform();
	FVector2f LocalSize = BooundingGeometry.GetLocalSize();

	InitializeFromArbitraryPoints(
		Transform.TransformPoint(FVector2f(0.f, 0.f)),
		Transform.TransformPoint(FVector2f(LocalSize.X, 0.f)),
		Transform.TransformPoint(FVector2f(0.f, LocalSize.Y)),
		Transform.TransformPoint(LocalSize)
	);
}

FSlateClippingZone::FSlateClippingZone(const FPaintGeometry& PaintingGeometry)
	: bIntersect(true)
	, bAlwaysClip(false)
{
	const FSlateRenderTransform& Transform = PaintingGeometry.GetAccumulatedRenderTransform();
	FVector2f LocalSize = PaintingGeometry.GetLocalSize();

	InitializeFromArbitraryPoints(
		Transform.TransformPoint(FVector2f(0.f, 0.f)),
		Transform.TransformPoint(FVector2f(LocalSize.X, 0.f)),
		Transform.TransformPoint(FVector2f(0.f, LocalSize.Y)),
		Transform.TransformPoint(LocalSize)
	);
}

FSlateClippingZone::FSlateClippingZone(const UE::Slate::FDeprecateVector2DParameter& InTopLeft, const UE::Slate::FDeprecateVector2DParameter& InTopRight, const UE::Slate::FDeprecateVector2DParameter& InBottomLeft, const UE::Slate::FDeprecateVector2DParameter& InBottomRight)
	: bIntersect(true)
	, bAlwaysClip(false)
{
	InitializeFromArbitraryPoints(InTopLeft, InTopRight, InBottomLeft, InBottomRight);
}

void FSlateClippingZone::InitializeFromArbitraryPoints(const UE::Slate::FDeprecateVector2DParameter& InTopLeft, const UE::Slate::FDeprecateVector2DParameter& InTopRight, const UE::Slate::FDeprecateVector2DParameter& InBottomLeft, const UE::Slate::FDeprecateVector2DParameter& InBottomRight)
{
	bIsAxisAligned = false;

	// Clipping is in pixel space, accept a very high tolerance
	const float Tolerance = .1f;

	// Since this is a rectangle check to edges.  If their points are equal they are aligned to the same axis and thus the whole rect is aligned
	if (FMath::IsNearlyEqual(InTopLeft.X, InBottomLeft.X, Tolerance))
	{
		if (FMath::IsNearlyEqual(InBottomLeft.Y, InBottomRight.Y, Tolerance))
		{
			bIsAxisAligned = true;
		}
	}
	else if (FMath::IsNearlyEqual(InTopLeft.Y, InBottomLeft.Y, Tolerance))
	{
		if (FMath::IsNearlyEqual(InBottomLeft.X, InBottomRight.X, Tolerance))
		{
			bIsAxisAligned = true;
		}
	}

	if ( bIsAxisAligned )
	{
		// Determine the true left, right, top bottom points.
		const FSlateRect RoundedAxisAlignedRect = FSlateRect(InTopLeft.X, InTopLeft.Y, InBottomRight.X, InBottomRight.Y).Round();
		const float Left = FMath::Min(RoundedAxisAlignedRect.Left, RoundedAxisAlignedRect.Right);
		const float Right = FMath::Max(RoundedAxisAlignedRect.Left, RoundedAxisAlignedRect.Right);
		const float Top = FMath::Min(RoundedAxisAlignedRect.Top, RoundedAxisAlignedRect.Bottom);
		const float Bottom = FMath::Max(RoundedAxisAlignedRect.Top, RoundedAxisAlignedRect.Bottom);

		TopLeft = FVector2f(Left, Top);
		TopRight = FVector2f(Right, Top);
		BottomLeft = FVector2f(Left, Bottom);
		BottomRight = FVector2f(Right, Bottom);
	}
	else
	{
		TopLeft = FVector2f(InTopLeft);
		TopRight = FVector2f(InTopRight);
		BottomLeft = FVector2f(InBottomLeft);
		BottomRight = FVector2f(InBottomRight);
	}
}

FSlateClippingZone FSlateClippingZone::Intersect(const FSlateClippingZone& Other) const
{
	check(IsAxisAligned());
	check(Other.IsAxisAligned());

	FSlateRect Intersected(
		FMath::Max(TopLeft.X, Other.TopLeft.X),
		FMath::Max(TopLeft.Y, Other.TopLeft.Y), 
		FMath::Min(BottomRight.X, Other.BottomRight.X), 
		FMath::Min(BottomRight.Y, Other.BottomRight.Y));

	if ( ( Intersected.Bottom < Intersected.Top ) || ( Intersected.Right < Intersected.Left ) )
	{
		return FSlateClippingZone(FSlateRect(0, 0, 0, 0));
	}
	else
	{
		return FSlateClippingZone(Intersected);
	}
}

FSlateRect FSlateClippingZone::GetBoundingBox() const
{
	FVector2f Points[4] =
	{
		TopLeft,
		TopRight,
		BottomLeft,
		BottomRight
	};

	return FSlateRect(
		FMath::Min(Points[0].X, FMath::Min3(Points[1].X, Points[2].X, Points[3].X)),
		FMath::Min(Points[0].Y, FMath::Min3(Points[1].Y, Points[2].Y, Points[3].Y)),
		FMath::Max(Points[0].X, FMath::Max3(Points[1].X, Points[2].X, Points[3].X)),
		FMath::Max(Points[0].Y, FMath::Max3(Points[1].Y, Points[2].Y, Points[3].Y))
	);
}

static float VectorSign(const FVector2f& Vec, const FVector2f& A, const FVector2f& B)
{
	return FMath::Sign((B.X - A.X) * (Vec.Y - A.Y) - (B.Y - A.Y) * (Vec.X - A.X));
}

// Returns true when the point is inside the triangle
// Should not return true when the point is on one of the edges
static bool IsPointInTriangle(const FVector2f& TestPoint, const FVector2f& A, const FVector2f& B, const FVector2f& C)
{
	float BA = VectorSign(B, A, TestPoint);
	float CB = VectorSign(C, B, TestPoint);
	float AC = VectorSign(A, C, TestPoint);

	// point is in the same direction of all 3 tri edge lines
	// must be inside, regardless of tri winding
	return BA == CB && CB == AC;
}

bool FSlateClippingZone::IsPointInside(const UE::Slate::FDeprecateVector2DParameter& Point) const
{
	if (IsAxisAligned())
	{
		return Point.X >= TopLeft.X && Point.X <= TopRight.X && Point.Y >= TopLeft.Y && Point.Y <= BottomLeft.Y;
	}
	else
	{
		if (IsPointInTriangle(Point, TopLeft, TopRight, BottomLeft) || IsPointInTriangle(Point, BottomLeft, TopRight, BottomRight))
		{
			return true;
		}

		return false;
	}
}

//-------------------------------------------------------------------

FSlateClippingState::FSlateClippingState(EClippingFlags InFlags /*= EClippingFlags::None*/)
	: Flags(InFlags)
#if WITH_SLATE_DEBUGGING
	, Debugging_StateIndex(INDEX_NONE)
	, Debugging_StateIndexFromFrame(INDEX_NONE)
#endif
{
}

bool FSlateClippingState::IsPointInside(const UE::Slate::FDeprecateVector2DParameter& Point) const
{
	if (ScissorRect.IsSet())
	{
		return ScissorRect->IsPointInside(Point);
	}
	
	check(StencilQuads.Num() > 0);
	for (const FSlateClippingZone& Quad : StencilQuads)
	{
		if (!Quad.IsPointInside(Point))
		{
			return false;
		}
	}

	return true;
}

//-------------------------------------------------------------------

FSlateClippingManager::FSlateClippingManager()
{
}

const FSlateClippingState* FSlateClippingManager::GetPreviousClippingState(bool bWillIntersectWithParent) const
{
	const FSlateClippingState* PreviousClippingState = nullptr;

	if (!bWillIntersectWithParent)
	{
		for (int32 StackIndex = ClippingStack.Num() - 1; StackIndex >= 0; StackIndex--)
		{
			const FSlateClippingState& ClippingState = ClippingStates[ClippingStack[StackIndex]];

			if (ClippingState.GetAlwaysClip())
			{
				PreviousClippingState = &ClippingState;
				break;
			}
		}
	}
	else if (ClippingStack.Num() > 0)
	{
		PreviousClippingState = &ClippingStates[ClippingStack.Top()];
	}

	return PreviousClippingState;
}

FSlateClippingState FSlateClippingManager::CreateClippingState(const FSlateClippingZone& InClipRect) const
{
	const FSlateClippingState* PreviousClippingState = GetPreviousClippingState(InClipRect.GetShouldIntersectParent());

	// Initialize the new clipping state
	FSlateClippingState NewClippingState(InClipRect.GetAlwaysClip() ? EClippingFlags::AlwaysClip : EClippingFlags::None);

	if (PreviousClippingState == nullptr)
	{
		if (InClipRect.IsAxisAligned())
		{
			NewClippingState.ScissorRect = InClipRect;
		}
		else
		{
			NewClippingState.StencilQuads.Add(InClipRect);
		}
	}
	else
	{
		if (PreviousClippingState->GetClippingMethod() == EClippingMethod::Scissor)
		{
			if (InClipRect.IsAxisAligned())
			{
				ensure(PreviousClippingState->ScissorRect.IsSet());
				NewClippingState.ScissorRect = PreviousClippingState->ScissorRect->Intersect(InClipRect);
			}
			else
			{
				NewClippingState.StencilQuads.Add(PreviousClippingState->ScissorRect.GetValue());
				NewClippingState.StencilQuads.Add(InClipRect);
			}
		}
		else
		{
			ensure(PreviousClippingState->StencilQuads.Num() > 0);
			NewClippingState.StencilQuads = PreviousClippingState->StencilQuads;
			NewClippingState.StencilQuads.Add(InClipRect);
		}
	}

	return NewClippingState;
}

int32 FSlateClippingManager::PushClip(const FSlateClippingZone& InClipRect)
{
	return PushClippingState(CreateClippingState(InClipRect));
}

int32 FSlateClippingManager::PushClippingState(const FSlateClippingState& NewClippingState)
{
	int32 NewClippingStateIndex = ClippingStates.AddUnique(NewClippingState);

#if WITH_SLATE_DEBUGGING
	NewClippingState.SetDebuggingStateIndex(NewClippingStateIndex);
#endif

	ClippingStack.Add(NewClippingStateIndex);

	return NewClippingStateIndex;
}

int32 FSlateClippingManager::GetClippingIndex() const
{
	return ClippingStack.Num() > 0 ? ClippingStack.Top() : INDEX_NONE;
}

TOptional<FSlateClippingState> FSlateClippingManager::GetActiveClippingState() const
{
	const int32 CurrentIndex = GetClippingIndex();
	if (CurrentIndex != INDEX_NONE)
	{
		return ClippingStates[CurrentIndex];
	}

	return TOptional<FSlateClippingState>();
}

const TArray< FSlateClippingState >& FSlateClippingManager::GetClippingStates() const
{
	return ClippingStates;
}

void FSlateClippingManager::PopClip()
{
	if (ensure(ClippingStack.Num() != 0))
	{
		ClippingStack.Pop();
	}
	else
	{
		UE_LOGF(LogSlate, Error, "Attempting to pop clipping state below 0.");
	}
}

void FSlateClippingManager::ResetClippingState()
{
	ClippingStates.Reset();
	ClippingStack.Reset();
}

void FSlateClippingManager::PopToStackIndex(int32 Index)
{
	const int32 StartIndexToPop = Index + 1;
	if (ClippingStack.Num() > StartIndexToPop)
	{
		ClippingStack.RemoveAt(StartIndexToPop, ClippingStack.Num() - StartIndexToPop, EAllowShrinking::No);
	}
}

