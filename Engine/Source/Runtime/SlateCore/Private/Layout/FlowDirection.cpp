// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/FlowDirection.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Culture.h"
#include "HAL/IConsoleManager.h"
#include "Misc/FrameValue.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FlowDirection)

EFlowDirection GSlateFlowDirection = EFlowDirection::LeftToRight;

static int32 GSlateEnableLayoutLocalization = 1;
static FAutoConsoleVariableRef CVarSlateEnableLayoutLocalization(
	TEXT("Slate.EnableLayoutLocalization"),
	GSlateEnableLayoutLocalization,
	TEXT("Controls if we enable or disable localized layout, which affects left to right or right to left detection for cultures."),
	ECVF_Default
);

int32 GSlateFlowDirectionShouldFollowCultureByDefault = 0;
static FAutoConsoleVariableRef CVarSlateFlowDirectionShouldFollowCultureByDefault(
	TEXT("Slate.ShouldFollowCultureByDefault"),
	GSlateFlowDirectionShouldFollowCultureByDefault,
	TEXT("Should we initially follow the culture's flow direction at the window level."),
	ECVF_Default
);

EFlowDirection FLayoutLocalization::GetLocalizedLayoutDirection()
{
	if (!GSlateEnableLayoutLocalization)
	{
		return EFlowDirection::LeftToRight;
	}

	// Only do this once per frame max no matter how many times people call the function.
	static TFrameValue<EFlowDirection> FrameCoherentDirection;
	if (!FrameCoherentDirection.IsSet())
	{
		FrameCoherentDirection = FInternationalization::Get().GetCurrentCulture()->IsRightToLeft() ? EFlowDirection::RightToLeft : EFlowDirection::LeftToRight;
	}

	return FrameCoherentDirection.GetValue();
}

