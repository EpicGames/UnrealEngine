// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/Geometry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Geometry)

FGeometry::FGeometry() 
	: Size(0.0f, 0.0f)
	, Scale(1.0f)
	, AbsolutePosition(0.0f, 0.0f)
	, bHasRenderTransform(false)
{
}

FGeometry& FGeometry::operator=(const FGeometry& RHS)
{
	// HACK to allow us to make FGeometry public members immutable to catch misuse.
	if (this != &RHS)
	{
		FMemory::Memcpy(*this, RHS);
	}
	return *this;
}

FString FGeometry::ToString() const
{
	return FString::Printf(TEXT("[Abs=%s, Scale=%.2f, Size=%s]"), *AbsolutePosition.ToString(), Scale, *FVector2f(Size).ToString());
}

