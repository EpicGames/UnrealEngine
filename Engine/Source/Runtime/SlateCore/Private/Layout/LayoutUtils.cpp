// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/LayoutUtils.h"

FSlotProxy::FSlotProxy(const TSharedRef<SWidget>& InWidget)
{
	DesiredSize = InWidget->GetDesiredSize();
	Visibility = InWidget->GetVisibility();
	Widget = InWidget;

	// Much of the calculations are below are from FGeometry::MakeChild, skipping the need for a widget reference later
	RenderTransform = InWidget->GetRenderTransformWithRespectToFlowDirection();
	if (RenderTransform.IsSet())
	{
		RenderTransformPivot = UE::Slate::CastToVector2f(InWidget->GetRenderTransformPivotWithRespectToFlowDirection());
	}
}

bool FSlotProxy::UpdateFromSlot(const TSharedRef<SWidget>& InWidget)
{
	bool bAnyValueChanged = false;

	bAnyValueChanged = bAnyValueChanged || (Visibility != InWidget->GetVisibility());
	Visibility = InWidget->GetVisibility();

	bAnyValueChanged = bAnyValueChanged || (Widget != InWidget);
	Widget = InWidget;

	const TOptional<FSlateRenderTransform> WidgetRenderTransform = InWidget->GetRenderTransformWithRespectToFlowDirection();

	// Much of the calculations are below are from FGeometry::MakeChild, skipping the need for a widget reference later
	bAnyValueChanged = bAnyValueChanged || (RenderTransform != WidgetRenderTransform);
	RenderTransform = WidgetRenderTransform;

	if (RenderTransform.IsSet())
	{
		const FVector2f RenderTransformPivot2f = UE::Slate::CastToVector2f(InWidget->GetRenderTransformPivotWithRespectToFlowDirection());
		bAnyValueChanged = bAnyValueChanged || (!RenderTransformPivot.Equals(RenderTransformPivot2f));
		RenderTransformPivot = RenderTransformPivot2f;
	}

	return bAnyValueChanged;
}

FGeometry FSlotProxy::MakeGeometry(const FGeometry& InParentGeometry, const FVector2f& InChildOffset, const FVector2f& InLocalSize) const
{
	constexpr float ChildScale =  1.0f;

	FSlateLayoutTransform LayoutTransform = FSlateLayoutTransform(ChildScale, TransformPoint(ChildScale, UE::Slate::CastToVector2f(InChildOffset)));

	if (RenderTransform.IsSet())
	{
		return InParentGeometry.MakeChild(InLocalSize, LayoutTransform, RenderTransform.GetValue(), RenderTransformPivot);
	}
	else
	{
		return InParentGeometry.MakeChild(InLocalSize, LayoutTransform);
	}
}

UE::Slate::FDeprecateVector2DResult ComputePopupFitInRect(const FSlateRect& InAnchor, const FSlateRect& PopupRect, const EOrientation& Orientation, const FSlateRect& RectToFit, bool bAllowFlip)
{
	const bool bAdjustmentNeeded = PopupRect.IntersectionWith(RectToFit) != PopupRect;
	if (bAdjustmentNeeded)
	{
		const FVector2f PopupPosition = PopupRect.GetTopLeft2f();
		const FVector2f PopupSize = PopupRect.GetSize2f();

		// In the direction we are opening, see if there is enough room. If there is not, flip the opening direction along the same axis.
		FVector2f NewPosition = FVector2f::ZeroVector;
		if (bAllowFlip)
		{
			if (Orientation == Orient_Horizontal)
			{
				const bool bFitsRight = InAnchor.Right + PopupSize.X < RectToFit.Right;
				const bool bFitsLeft = InAnchor.Left - PopupSize.X >= RectToFit.Left;

				if (bFitsRight || !bFitsLeft)
				{
					// The menu fits to the right of the anchor or it does not fit to the left, display to the right
					NewPosition = FVector2f(InAnchor.Right, InAnchor.Top);
				}
				else
				{
					// The menu does not fit to the right of the anchor but it does fit to the left, display to the left
					NewPosition = FVector2f(InAnchor.Left - PopupSize.X, InAnchor.Top);
				}
			}
			else
			{
				const bool bFitsDown = InAnchor.Bottom + PopupSize.Y < RectToFit.Bottom;
				const bool bFitsUp = InAnchor.Top - PopupSize.Y >= RectToFit.Top;

				if (bFitsDown || !bFitsUp)
				{
					// The menu fits below the anchor or it does not fit above, display below
					NewPosition = FVector2f(InAnchor.Left, InAnchor.Bottom);
				}
				else
				{
					// The menu does not fit below the anchor but it does fit above, display above
					NewPosition = FVector2f(InAnchor.Left, InAnchor.Top - PopupSize.Y);
				}

				if (!bFitsDown && !bFitsUp)
				{
					return ComputePopupFitInRect(InAnchor, PopupRect, EOrientation::Orient_Horizontal, RectToFit);
				}
			}
		}
		else
		{
			NewPosition = FVector2f(InAnchor.Left, InAnchor.Bottom);
		}
		
		// Adjust the position of popup windows so they do not go out of the visible area of the monitor(s)
		// This can happen along the opposite axis that we are opening with
		// Assumes this window has a valid size
		// Adjust any menus that my not fit on the screen where they are opened
		FVector2f StartPos = NewPosition;
		FVector2f EndPos = NewPosition + PopupSize;
		FVector2f Adjust = FVector2f::ZeroVector;
		if (StartPos.X < RectToFit.Left)
		{
			// Window is clipped by the left side of the work area
			Adjust.X = RectToFit.Left - StartPos.X;
		}

		if (StartPos.Y < RectToFit.Top)
		{
			// Window is clipped by the top of the work area
			Adjust.Y = RectToFit.Top - StartPos.Y;
		}

		if (EndPos.X > RectToFit.Right)
		{
			// Window is clipped by the right side of the work area
			Adjust.X = RectToFit.Right - EndPos.X;
		}

		if (EndPos.Y > RectToFit.Bottom)
		{
			// Window is clipped by the bottom of the work area
			Adjust.Y = RectToFit.Bottom - EndPos.Y;
		}

		NewPosition += Adjust;

		return FVector2f(NewPosition);
	}
	else
	{
		return FVector2f(PopupRect.GetTopLeft());
	}
}
