// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/SlateRect.h"

FString FSlateRect::ToString() const
{
	return FString::Printf(TEXT("Left=%3.3f Top=%3.3f Right=%3.3f Bottom=%3.3f"), Left, Top, Right, Bottom);
}

bool FSlateRect::InitFromString(const FString& InSourceString)
{
	// The initialization is only successful if the values can all be parsed from the string
	const bool bSuccessful = FParse::Value(*InSourceString, TEXT("Left="), Left) &&
		FParse::Value(*InSourceString, TEXT("Top="), Top) &&
		FParse::Value(*InSourceString, TEXT("Right="), Right) &&
		FParse::Value(*InSourceString, TEXT("Bottom="), Bottom);

	return bSuccessful;
}
