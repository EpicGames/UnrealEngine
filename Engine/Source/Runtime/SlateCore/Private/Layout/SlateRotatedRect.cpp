// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/SlateRotatedRect.h"
#include "Math/TransformCalculus2D.h"

// !!! WRH 2014/08/25 - this is a brute-force, not efficient implementation, uses a bunch of extra conditionals.
FSlateRect FSlateRotatedRect::ToBoundingRect() const
{
	FVector2f Points[4] = 
	{
		TopLeft,
		TopLeft + ExtentX,
		TopLeft + ExtentY,
		TopLeft + ExtentX + ExtentY
	};
	return FSlateRect(
		FMath::Min(Points[0].X, FMath::Min3(Points[1].X, Points[2].X, Points[3].X)),
		FMath::Min(Points[0].Y, FMath::Min3(Points[1].Y, Points[2].Y, Points[3].Y)),
		FMath::Max(Points[0].X, FMath::Max3(Points[1].X, Points[2].X, Points[3].X)),
		FMath::Max(Points[0].Y, FMath::Max3(Points[1].Y, Points[2].Y, Points[3].Y))
		);
}

bool FSlateRotatedRect::IsUnderLocation(const UE::Slate::FDeprecateVector2DParameter Location) const
{
	const FVector2f Offset = Location - TopLeft;
	const float Det = FVector2f::CrossProduct(ExtentX, ExtentY);

	// Not exhaustively efficient. Could optimize the checks for [0..1] to short circuit faster.
	const float S = -FVector2f::CrossProduct(Offset, ExtentX) / Det;
	if (FMath::IsWithinInclusive(S, 0.0f, 1.0f))
	{
		const float T = FVector2f::CrossProduct(Offset, ExtentY) / Det;
		return FMath::IsWithinInclusive(T, 0.0f, 1.0f);
	}
	return false;
}

FSlateRotatedRect FSlateRotatedRect::MakeRotatedRect(const FSlateRect& ClipRectInLayoutWindowSpace, const FTransform2f& LayoutToRenderTransform)
{
	const FSlateRotatedRect RotatedRect = TransformRect(LayoutToRenderTransform, FSlateRotatedRect(ClipRectInLayoutWindowSpace));

	const FVector2f TopRight = RotatedRect.TopLeft + RotatedRect.ExtentX;
	const FVector2f BottomLeft = RotatedRect.TopLeft + RotatedRect.ExtentY;

	return FSlateRotatedRect(
		RotatedRect.TopLeft,
		TopRight - RotatedRect.TopLeft,
		BottomLeft - RotatedRect.TopLeft);
}

FSlateRotatedRect FSlateRotatedRect::MakeSnappedRotatedRect(const FSlateRect& ClipRectInLayoutWindowSpace, const FTransform2f& LayoutToRenderTransform)
{
	const FSlateRotatedRect RotatedRect = TransformRect(LayoutToRenderTransform, FSlateRotatedRect(ClipRectInLayoutWindowSpace));

	// Pixel snapping is done here by rounding the resulting floats to ints, we do this before
	// calculating the final extents of the clip box otherwise we'll get a smaller clip rect than a visual
	// rect where each point is individually snapped.
	const FVector2f SnappedTopLeft = ( (RotatedRect.TopLeft) ).RoundToVector();
	const FVector2f SnappedTopRight = ( (RotatedRect.TopLeft) + (RotatedRect.ExtentX) ).RoundToVector();
	const FVector2f SnappedBottomLeft = ( (RotatedRect.TopLeft) + (RotatedRect.ExtentY) ).RoundToVector();

	//NOTE: We explicitly do not re-snap the extent x/y, it wouldn't be correct to snap again in distance space
	// even if two points are snapped, their distance wont necessarily be a whole number if those points are not
	// axis aligned.
	return FSlateRotatedRect(
		SnappedTopLeft,
		SnappedTopRight - SnappedTopLeft,
		SnappedBottomLeft - SnappedTopLeft);
}
