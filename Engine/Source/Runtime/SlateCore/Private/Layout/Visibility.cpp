// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/Visibility.h"


/* Static initialization
 *****************************************************************************/

const EVisibility EVisibility::Visible(EVisibility::VIS_Visible);
const EVisibility EVisibility::Collapsed(EVisibility::VIS_Collapsed);
const EVisibility EVisibility::Hidden(EVisibility::VIS_Hidden);
const EVisibility EVisibility::HitTestInvisible(EVisibility::VIS_HitTestInvisible);
const EVisibility EVisibility::SelfHitTestInvisible(EVisibility::VIS_SelfHitTestInvisible);
const EVisibility EVisibility::All(EVisibility::VIS_All);


/* FVisibility interface
 *****************************************************************************/

FString EVisibility::ToString( ) const
{
	static const FString VisibleString("Visible");
	static const FString HiddenString("Hidden");
	static const FString CollapsedString("Collapsed");
	static const FString HitTestInvisibleString("HitTestInvisible");
	static const FString SelfHitTestInvisibleString("SelfHitTestInvisible");

	if (Value == VIS_Visible)
	{
		return VisibleString;
	}

	if (Value == VIS_Collapsed)
	{
		return CollapsedString;
	}

	if (Value == VIS_Hidden)
	{
		return HiddenString;
	}

	if (Value == VIS_HitTestInvisible)
	{
		return HitTestInvisibleString;
	}

	if (Value == VIS_SelfHitTestInvisible)
	{
		return SelfHitTestInvisibleString;
	}

	return FString();
}
