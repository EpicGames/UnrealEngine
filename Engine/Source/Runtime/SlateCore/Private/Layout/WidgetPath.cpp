// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/WidgetPath.h"
#include "SlateGlobals.h"
#include "Types/SlateAttributeMetaData.h"

DECLARE_CYCLE_STAT(TEXT("Weak-To-Strong WidgetPath"), STAT_WeakToStrong_WidgetPath, STATGROUP_Slate);


FWidgetPath::FWidgetPath()
: Widgets( EVisibility::Visible )
, TopLevelWindow()
, VirtualPointerPositions()
{
}

FWidgetPath::FWidgetPath( TSharedPtr<SWindow> InTopLevelWindow, const FArrangedChildren& InWidgetPath )
: Widgets( InWidgetPath )
, TopLevelWindow(InTopLevelWindow)
, VirtualPointerPositions()
{
}

FWidgetPath::FWidgetPath( TArrayView<FWidgetAndPointer> InWidgetsAndPointers )
: Widgets( FArrangedChildren::Hittest2_FromArray(InWidgetsAndPointers) )
, TopLevelWindow( InWidgetsAndPointers.Num() > 0 ? StaticCastSharedRef<SWindow>(InWidgetsAndPointers[0].Widget) : TSharedPtr<SWindow>(nullptr) )
{
	check(InWidgetsAndPointers.Num() == 0 || InWidgetsAndPointers[0].Widget->Advanced_IsWindow());
	VirtualPointerPositions.Reserve(InWidgetsAndPointers.Num());
	for (const FWidgetAndPointer& WidgetAndPointer : InWidgetsAndPointers)
	{
		VirtualPointerPositions.Add(WidgetAndPointer.GetPointerPosition());
	}
}

FWidgetPath FWidgetPath::GetPathDownTo( TSharedRef<const SWidget> MarkerWidget ) const
{
	FArrangedChildren ClippedPath(EVisibility::Visible);
	bool bCopiedMarker = false;
	for( int32 WidgetIndex = 0; !bCopiedMarker && WidgetIndex < Widgets.Num(); ++WidgetIndex )
	{
		ClippedPath.AddWidget( Widgets[WidgetIndex] );
		bCopiedMarker = (Widgets[WidgetIndex].Widget == MarkerWidget);
	}
		
	if ( bCopiedMarker )
	{
		// We found the MarkerWidget and copied the path down to (and including) it.
		return FWidgetPath( TopLevelWindow, ClippedPath );
	}
	else
	{
		// The MarkerWidget was not in the widget path. We failed.
		return FWidgetPath( nullptr, FArrangedChildren(EVisibility::Visible) );		
	}	
}


bool FWidgetPath::ContainsWidget( const SWidget* WidgetToFind ) const
{
	if (WidgetToFind)
	{
		for (int32 WidgetIndex = 0; WidgetIndex < Widgets.Num(); ++WidgetIndex)
		{
			if (&Widgets[WidgetIndex].Widget.Get() == WidgetToFind)
			{
				return true;
			}
		}
	}
		
	return false;
}


TOptional<FArrangedWidget> FWidgetPath::FindArrangedWidget( TSharedRef<const SWidget> WidgetToFind ) const
{
	for(int32 WidgetIndex = 0; WidgetIndex < Widgets.Num(); ++WidgetIndex)
	{
		if ( Widgets[WidgetIndex].Widget == WidgetToFind )
		{
			return Widgets[WidgetIndex];
		}
	}

	return TOptional<FArrangedWidget>();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TOptional<FWidgetAndPointer> FWidgetPath::FindArrangedWidgetAndCursor( TSharedRef<const SWidget> WidgetToFind ) const
{
	const int32 Index = Widgets.IndexOfByPredicate( [&WidgetToFind]( const FArrangedWidget& SomeWidget )
	{
		return SomeWidget.Widget == WidgetToFind;
	} );

	return (Index != INDEX_NONE)
		? FWidgetAndPointer( Widgets[Index], VirtualPointerPositions[Index] )
		: FWidgetAndPointer();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

TSharedRef<SWindow> FWidgetPath::GetWindow() const
{
	check(IsValid());

	TSharedRef<SWindow> FirstWidgetWindow = StaticCastSharedRef<SWindow>(Widgets[0].Widget);
	return FirstWidgetWindow;
}

TSharedRef<SWindow> FWidgetPath::GetDeepestWindow() const
{
	check(IsValid());
	const int32 WindowIndex = Widgets.FindLastByPredicate([](const FArrangedWidget& SomeWidget)
	{
		return SomeWidget.Widget->Advanced_IsWindow();
	});
	check(WindowIndex != INDEX_NONE);
	TSharedRef<SWindow> FirstWidgetWindow = StaticCastSharedRef<SWindow>(Widgets[WindowIndex].Widget);
	return FirstWidgetWindow;
}

bool FWidgetPath::IsValid() const
{
	return Widgets.Num() > 0;
}

	
FString FWidgetPath::ToString() const
{
	FString StringBuffer;
	for( int32 WidgetIndex = Widgets.Num()-1; WidgetIndex >= 0; --WidgetIndex )
	{
		StringBuffer += Widgets[WidgetIndex].ToString();
		StringBuffer += TEXT("\n");
	}
	return StringBuffer;
}


/** Matches any widget that is focusable */
struct FFocusableWidgetMatcher
{
	bool IsMatch( const TSharedRef<const SWidget>& InWidget ) const
	{
		return InWidget->IsEnabled() && InWidget->SupportsKeyboardFocus();
	}
};

/**
 * Move focus either forward on backward in the path level specified by PathLevel.
 * That is, this movement of focus will modify the subtree under Widgets(PathLevel).
 *
 * @param PathLevel					The level in this WidgetPath whose focus to move.
 * @param MoveDirectin				Move focus forward or backward?
 * @param bSearchFromPathWidget		if set false the search for the next will simply start at the beginning or end of the list of children dependant on the direction
 *
 * @return true if the focus moved successfully, false if we were unable to move focus
 */
bool FWidgetPath::MoveFocus(int32 PathLevel, EUINavigation NavigationType, bool bSearchFromPathWidget)
{
	check(NavigationType == EUINavigation::Next || NavigationType == EUINavigation::Previous);

	const int32 MoveDirectionAsInt = (NavigationType == EUINavigation::Next)
		? +1
		: -1;


	if ( PathLevel == Widgets.Num()-1 )
	{
		// We are the currently focused widget because we are at the very bottom of focus path.
		if (NavigationType == EUINavigation::Next)
		{
			// EFocusMoveDirection::Next implies descend, so try to find a focusable descendant.
			return ExtendPathTo( FFocusableWidgetMatcher(), EVisibility::Visible, EWidgetPathSearchPurpose::FocusHandling );
		}
		else
		{
			// EFocusMoveDirection::Previous implies move focus up a level.
			return false;
		}
	}
	else if ( Widgets.Num() > 1 )
	{
		// We are not the last widget in the path.
		// GOAL: Look for a focusable descendant to the left or right of the currently focused path.
	
		// Arrange the children so we can iterate through them regardless of widget type.
		FArrangedChildren ArrangedChildren(EVisibility::Visible);
		const bool bUpdateVisibilityAttributes = true;
		Widgets[PathLevel].Widget->ArrangeChildren( Widgets[PathLevel].Geometry, ArrangedChildren,  bUpdateVisibilityAttributes);

		// Don't continue if there are no children in the widget.
		if (ArrangedChildren.Num() > 0)
		{
			int32 FocusedChildIndex = NavigationType == EUINavigation::Next ? 0 : ArrangedChildren.Num() - 1;
			if (bSearchFromPathWidget)
			{
				// Find the currently focused child among the children.
				FocusedChildIndex = ArrangedChildren.FindItemIndex(Widgets[PathLevel + 1]);
				FocusedChildIndex = (FocusedChildIndex) % ArrangedChildren.Num() + MoveDirectionAsInt;
			}

			// Now actually search for the widget.
			for (; FocusedChildIndex < ArrangedChildren.Num() && FocusedChildIndex >= 0; FocusedChildIndex += MoveDirectionAsInt)
			{
				// Neither disabled widgets nor their children can be focused.
				if (ArrangedChildren[FocusedChildIndex].Widget->IsEnabled())
				{
					// Look for a focusable descendant.
					FArrangedChildren PathToFocusableChild = GeneratePathToWidget(FFocusableWidgetMatcher(), ArrangedChildren[FocusedChildIndex], NavigationType);
					// Either we found a focusable descendant, or an immediate child that is focusable.
					const bool bFoundNextFocusable = (PathToFocusableChild.Num() > 0) || ArrangedChildren[FocusedChildIndex].Widget->SupportsKeyboardFocus();
					if (bFoundNextFocusable)
					{
						// Check if any descendant is disabled.
						int32 DisabledDescendantIndex = PathToFocusableChild.IndexOfByPredicate([](const FArrangedWidget& ArrangedWidget) 
							{ 
								return !ArrangedWidget.Widget->IsEnabled(); 
							});

						if (DisabledDescendantIndex == INDEX_NONE)
						{
							// We found the next focusable widget, so make this path point at the new widget by:
							// First, truncating the FocusPath up to the current level (i.e. PathLevel).
							Widgets.Remove(PathLevel + 1, Widgets.Num() - PathLevel - 1);
							// Second, add the immediate child that is focus or whose descendant is focused.
							Widgets.AddWidget(ArrangedChildren[FocusedChildIndex]);
							// Add path to focused descendants if any.
							Widgets.Append(PathToFocusableChild);
							// We successfully moved focus!
							return true;
						}
					}
				}
			}
		}
	}

	return false;
}

/** Construct a weak widget path from a widget path. Defaults to an invalid path. */
FWeakWidgetPath::FWeakWidgetPath( const FWidgetPath& InWidgetPath )
: Window( InWidgetPath.TopLevelWindow )
{
	for( int32 WidgetIndex = 0; WidgetIndex < InWidgetPath.Widgets.Num(); ++WidgetIndex )
	{
		Widgets.Add( TWeakPtr<SWidget>( InWidgetPath.Widgets[WidgetIndex].Widget ) );
	}
}

/** Make a non-weak WidgetPath out of this WeakWidgetPath. Do this by computing all the relevant geometries and converting the weak pointers to TSharedPtr. */	
FWidgetPath FWeakWidgetPath::ToWidgetPath(EInterruptedPathHandling::Type InterruptedPathHandling, const FPointerEvent* PointerEvent, const EVisibility VisibilityFilter) const
{
	FWidgetPath WidgetPath;
	ToWidgetPath( WidgetPath, InterruptedPathHandling, PointerEvent, VisibilityFilter );
	return WidgetPath;
}

TSharedRef<FWidgetPath> FWeakWidgetPath::ToWidgetPathRef(EInterruptedPathHandling::Type InterruptedPathHandling, const FPointerEvent* PointerEvent, const EVisibility VisibilityFilter) const
{
	TSharedRef<FWidgetPath> WidgetPath = MakeShareable(new FWidgetPath());
	ToWidgetPath(WidgetPath.Get(), InterruptedPathHandling, PointerEvent, VisibilityFilter);
	return WidgetPath;
}

FWeakWidgetPath::EPathResolutionResult::Result FWeakWidgetPath::ToWidgetPath( FWidgetPath& WidgetPath, EInterruptedPathHandling::Type InterruptedPathHandling, const FPointerEvent* PointerEvent, const EVisibility VisibilityFilter) const
{
	SCOPE_CYCLE_COUNTER(STAT_WeakToStrong_WidgetPath);

	TArray< TSharedPtr<SWidget> > WidgetPtrs;

	// Convert the weak pointers into shared pointers because we are about to do something with this path instead of just observe it.
	TSharedPtr<SWindow> TopLevelWindowPtr = Window.Pin();
	int PathSize = 0;
	for (TArray< TWeakPtr<SWidget> >::TConstIterator SomeWeakWidgetPtr(Widgets); SomeWeakWidgetPtr; ++SomeWeakWidgetPtr)
	{
		const int MaxWidgetPath = 1000;
		++PathSize;
		if (ensureMsgf(PathSize < MaxWidgetPath, TEXT("Converting a Widget Path of more that 1000 Widget deep.")))
		{
			WidgetPtrs.Add(SomeWeakWidgetPtr->Pin());
		}
		else
		{
			WidgetPath = FWidgetPath();
			return EPathResolutionResult::Truncated;
		}
	}

	if (GSlateFastWidgetPath)
	{
		TArray<FWidgetAndPointer> PathWithGeometries;

		// The path can get interrupted if some subtree of widgets disappeared, but we still maintain weak references to it.
		bool bPathUninterrupted = false;

		// For each widget in the path compute the geometry. We are able to do this starting with the top-level window because it knows its own geometry.
		if (TopLevelWindowPtr.IsValid())
		{
			bPathUninterrupted = true;

			FGeometry ParentGeometry = TopLevelWindowPtr->GetWindowGeometryInScreen();
			PathWithGeometries.Add(FWidgetAndPointer(FArrangedWidget(TopLevelWindowPtr.ToSharedRef(), ParentGeometry)));

			TOptional<FVirtualPointerPosition> VirtualPointerPos;
			// For every widget in the vertical slice...
			for (int32 WidgetIndex = 0; bPathUninterrupted && WidgetIndex < WidgetPtrs.Num() - 1; ++WidgetIndex)
			{
				TSharedPtr<SWidget> CurWidget = WidgetPtrs[WidgetIndex];

				bool bFoundChild = false;
				if (CurWidget.IsValid())
				{
					FSlateAttributeMetaData::UpdateOnlyVisibilityAttributes(*CurWidget.Get(), FSlateAttributeMetaData::EInvalidationPermission::DelayInvalidation);
					if (EVisibility::DoesVisibilityPassFilter(CurWidget->GetVisibility(), VisibilityFilter))
					{
						TSharedRef<SWidget> CurWidgetRef = CurWidget.ToSharedRef();
						const TSharedPtr<SWidget>& ChildWidgetPtr = WidgetPtrs[WidgetIndex + 1];

						if (ChildWidgetPtr.IsValid() && ChildWidgetPtr->GetParentWidget() == CurWidgetRef)
						{
							if (PointerEvent && !VirtualPointerPos.IsSet())
							{
								FVector2f ScreenPos = PointerEvent->GetScreenSpacePosition();
								FVector2f LastScreenPos = PointerEvent->GetLastScreenSpacePosition();

								VirtualPointerPos = CurWidget->TranslateMouseCoordinateForCustomHitTestChild(*ChildWidgetPtr.Get(), ParentGeometry, FVector2d(ScreenPos), FVector2d(LastScreenPos));
							}

							bFoundChild = true;
							// Remember the widget, the associated geometry, and the pointer position in a transformed space.
							PathWithGeometries.Add(FWidgetAndPointer(FArrangedWidget(ChildWidgetPtr.ToSharedRef(), ChildWidgetPtr->GetCachedGeometry()), VirtualPointerPos));
							// The next child in the vertical slice will be arranged with respect to its parent's geometry.
							ParentGeometry = CurWidgetRef->GetCachedGeometry();
						}
					}
				}

				bPathUninterrupted = bFoundChild;
				if (!bFoundChild && InterruptedPathHandling == EInterruptedPathHandling::ReturnInvalid)
				{
					return EPathResolutionResult::Truncated;
				}
			}
		}

		WidgetPath = FWidgetPath(PathWithGeometries);
		return bPathUninterrupted ? EPathResolutionResult::Live : EPathResolutionResult::Truncated;
	}
	else
	{
		TArray<FWidgetAndPointer> PathWithGeometries;		

		// The path can get interrupted if some subtree of widgets disappeared, but we still maintain weak references to it.
		bool bPathUninterrupted = false;

		// For each widget in the path compute the geometry. We are able to do this starting with the top-level window because it knows its own geometry.
		if (TopLevelWindowPtr.IsValid())
		{
			bPathUninterrupted = true;

			FGeometry ParentGeometry = TopLevelWindowPtr->GetWindowGeometryInScreen();
			PathWithGeometries.Add(FWidgetAndPointer(FArrangedWidget(TopLevelWindowPtr.ToSharedRef(), ParentGeometry)));

			FArrangedChildren ArrangedChildren(VisibilityFilter, true);

			TOptional<FVirtualPointerPosition> VirtualPointerPos;
			// For every widget in the vertical slice...
			for (int32 WidgetIndex = 0; bPathUninterrupted && WidgetIndex < WidgetPtrs.Num() - 1; ++WidgetIndex)
			{
				TSharedPtr<SWidget> CurWidget = WidgetPtrs[WidgetIndex];

				bool bFoundChild = false;
				if (CurWidget.IsValid())
				{
					// Arrange the widget's children to find their geometries.
					ArrangedChildren.Empty();
					const bool bUpdateVisibilityAttributes = true;
					CurWidget->ArrangeChildren(ParentGeometry, ArrangedChildren, bUpdateVisibilityAttributes);

					// Find the next widget in the path among the arranged children.
					for (int32 SearchIndex = 0; !bFoundChild && SearchIndex < ArrangedChildren.Num(); ++SearchIndex)
					{
						FArrangedWidget& ArrangedWidget = ArrangedChildren[SearchIndex];

						if (ArrangedWidget.Widget == WidgetPtrs[WidgetIndex + 1])
						{
							if (PointerEvent && !VirtualPointerPos.IsSet())
							{
								FVector2f ScreenPos = PointerEvent->GetScreenSpacePosition();
								FVector2f LastScreenPos = PointerEvent->GetLastScreenSpacePosition();

								VirtualPointerPos = CurWidget->TranslateMouseCoordinateForCustomHitTestChild(ArrangedWidget.Widget.Get(), ParentGeometry, FVector2d(ScreenPos), FVector2d(LastScreenPos));
							}

							bFoundChild = true;
							// Remember the widget, the associated geometry, and the pointer position in a transformed space.
							PathWithGeometries.Add(FWidgetAndPointer(ArrangedChildren[SearchIndex], VirtualPointerPos));
							// The next child in the vertical slice will be arranged with respect to its parent's geometry.
							ParentGeometry = ArrangedChildren[SearchIndex].Geometry;
						}
					}
				}

				bPathUninterrupted = bFoundChild;
				if (!bFoundChild && InterruptedPathHandling == EInterruptedPathHandling::ReturnInvalid)
				{
					return EPathResolutionResult::Truncated;
				}
			}
		}

		WidgetPath = FWidgetPath(PathWithGeometries);
		return bPathUninterrupted ? EPathResolutionResult::Live : EPathResolutionResult::Truncated;
	}
}

bool FWeakWidgetPath::ContainsWidget(const SWidget* SomeWidget) const
{
	if (SomeWidget)
	{
		for (int32 WidgetIndex = 0; WidgetIndex < Widgets.Num(); ++WidgetIndex)
		{
			if (Widgets[WidgetIndex].Pin().Get() == SomeWidget)
			{
				return true;
			}
		}
	}

	return false;
}

FWidgetPath FWeakWidgetPath::ToNextFocusedPath(EUINavigation NavigationType) const
{
	return ToNextFocusedPath(NavigationType, FNavigationReply::Escape(), FArrangedWidget::GetNullWidget());
}

FWidgetPath FWeakWidgetPath::ToNextFocusedPath(EUINavigation NavigationType, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget) const
{
	check(NavigationType == EUINavigation::Next || NavigationType == EUINavigation::Previous);

	// Make a copy of the focus path. We will mutate it until it meets the necessary requirements.
	FWidgetPath NewFocusPath = this->ToWidgetPath();
	TSharedPtr<SWidget> CurrentlyFocusedWidget = this->Widgets.Last().Pin();

	bool bMovedFocus = false;
	// Attempt to move the focus starting at the leafmost widget and bubbling up to the root (i.e. the window)
	for (int32 FocusNodeIndex=NewFocusPath.Widgets.Num()-1; !bMovedFocus && FocusNodeIndex >= 0; --FocusNodeIndex)
	{
		bMovedFocus = NewFocusPath.MoveFocus(FocusNodeIndex, NavigationType);

		// If we didn't move and we hit our rule widget consider stop and wrap
		if (!bMovedFocus && RuleWidget.Widget == NewFocusPath.Widgets[FocusNodeIndex].Widget)
		{
			// We've reached the stop boundary and not yet moved focus, so don't advance.
			if (NavigationReply.GetBoundaryRule() == EUINavigationRule::Stop)
			{
				break;
			}

			// We've reached the wrap boundary and not yet moved focus so move to the first or last element in BoundaryWidget
			if (NavigationReply.GetBoundaryRule() == EUINavigationRule::Wrap)
			{
				NewFocusPath.MoveFocus(FocusNodeIndex, NavigationType, false);
				break;
			}
		}
	}

	return NewFocusPath;
}
