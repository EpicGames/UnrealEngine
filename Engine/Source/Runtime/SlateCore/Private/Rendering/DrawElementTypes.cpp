// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/DrawElementTypes.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "HAL/IConsoleManager.h"
#include "Types/ReflectionMetadata.h"
#include "Fonts/ShapedTextFwd.h"
#include "Fonts/FontCache.h"
#include "Rendering/SlateObjectReferenceCollector.h"
#include "Debugging/SlateDebugging.h"
#include "Application/SlateApplicationBase.h"

#include <limits>
#include <type_traits>

DECLARE_CYCLE_STAT(TEXT("FSlateDrawElement::Make Time"), STAT_SlateDrawElementMakeTime, STATGROUP_SlateVerbose);
DECLARE_CYCLE_STAT(TEXT("FSlateDrawElement::MakeCustomVerts Time"), STAT_SlateDrawElementMakeCustomVertsTime, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("FSlateDrawElement::Prebatch Time"), STAT_SlateDrawElementPrebatchTime, STATGROUP_Slate);

DEFINE_STAT(STAT_SlateBufferPoolMemory);
DEFINE_STAT(STAT_SlateCachedDrawElementMemory);

static bool bApplyDisabledEffectOnWidgets = true;

static void HandleApplyDisabledEffectOnWidgetsToggled(IConsoleVariable* CVar)
{
	if (FSlateApplicationBase::IsInitialized())
	{
		FSlateApplicationBase::Get().InvalidateAllWidgets(false);
	}
}

static FAutoConsoleVariableRef CVarApplyDisabledEffectOnWidgets(
	TEXT("Slate.ApplyDisabledEffectOnWidgets"),
	bApplyDisabledEffectOnWidgets,
	TEXT("If true, disabled widgets will have alpha multiplied by 0.45 (applies to project content only, not editor)."),
	FConsoleVariableDelegate::CreateStatic(&HandleApplyDisabledEffectOnWidgetsToggled)
);

static bool IsResourceObjectValid(UObject*& InObject)
{
	if (InObject == nullptr)
	{
		return true;
	}

	// Catch GC'd-but-not-yet-collected UObjects via the engine's standard flag/state checks.
	// NOTE: these still dereference InObject (to read flags / internal index), so they do NOT survive
	// a fully freed/unmapped backing page. Such case must be fixed upstream by ensuring the FSlateBrush isn't dangling.
	if (!IsValidChecked(InObject) || InObject->IsUnreachable() || InObject->HasAnyFlags(RF_BeginDestroyed))
	{
		UE_LOGF(LogSlate, Warning, "[%ls] Attempted to access resource for %ls which is pending kill, unreachable or pending destroy - Platform=%ls",
			ANSI_TO_TCHAR(__FUNCTION__), *InObject->GetName(), ANSI_TO_TCHAR(FPlatformProperties::PlatformName()));
		return false;
	}

	return true;
}

static bool ShouldCull(const FSlateWindowElementList& ElementList)
{
	const FSlateClippingManager& ClippingManager = ElementList.GetClippingManager();
	const int32 CurrentIndex = ClippingManager.GetClippingIndex();
	if (CurrentIndex != INDEX_NONE)
	{
		const FSlateClippingState& ClippingState = ClippingManager.GetClippingStates()[CurrentIndex];
		if (ClippingState.GetClippingMethod() == EClippingMethod::Scissor)
		{
			return ClippingState.HasZeroArea();
		}
		else if (ClippingState.GetClippingMethod() == EClippingMethod::Stencil)
		{
			FSlateRect WindowRect = ElementList.GetPaintWindow()->GetClippingRectangleInWindow();
			if (WindowRect.GetArea() > 0)
			{
				for (const FSlateClippingZone& Stencil : ClippingState.StencilQuads)
				{
					bool bOverlapping = false;
					FSlateRect ClippedStencil = Stencil.GetBoundingBox().IntersectionWith(WindowRect, bOverlapping);
					
					if (!bOverlapping || ClippedStencil.GetArea() <= 0)
					{
						return true;
					}
				}
			}
		}

	}

	return false;
}

static bool ShouldCull(const FSlateWindowElementList& ElementList, const FPaintGeometry& PaintGeometry)
{
	const FVector2f LocalSize = UE::Slate::CastToVector2f(PaintGeometry.GetLocalSize());
	const float DrawScale = PaintGeometry.DrawScale;

	const FVector2f PixelSize = (LocalSize * DrawScale);
	if (PixelSize.X <= 0.f || PixelSize.Y <= 0.f)
	{
		return true;
	}

	return ShouldCull(ElementList);
}

static bool ShouldCull(const FSlateWindowElementList& ElementList, const FPaintGeometry& PaintGeometry, const FSlateBrush* InBrush)
{
	if (ShouldCull(ElementList, PaintGeometry))
	{
		return true;
	}

	if (InBrush->GetDrawType() == ESlateBrushDrawType::NoDrawType)
	{
		return true;
	}

	UObject* ResourceObject = InBrush->GetResourceObject();
	if (!IsResourceObjectValid(ResourceObject))
	{
		return true;
	}

	return false;
}


static bool ShouldCull(const FSlateWindowElementList& ElementList, const FPaintGeometry& PaintGeometry, const FLinearColor& InTint)
{
	if (InTint.A < UE_KINDA_SMALL_NUMBER || ShouldCull(ElementList, PaintGeometry))
	{
		return true;
	}

	return false;
}

static bool ShouldCull(const FSlateWindowElementList& ElementList, const FPaintGeometry& PaintGeometry, const FLinearColor& InTint, const FString& InText)
{
	if (InTint.A < UE_KINDA_SMALL_NUMBER || InText.Len() == 0 || ShouldCull(ElementList, PaintGeometry))
	{
		return true;
	}

	return false;
}

static bool ShouldCull(const FSlateWindowElementList& ElementList, const FPaintGeometry& PaintGeometry, const FSlateBrush* InBrush, const FLinearColor& InTint)
{
	bool bIsFullyTransparent = InTint.A < UE_KINDA_SMALL_NUMBER && (InBrush->OutlineSettings.Color.GetSpecifiedColor().A < UE_KINDA_SMALL_NUMBER || InBrush->OutlineSettings.bUseBrushTransparency);
	if (bIsFullyTransparent || ShouldCull(ElementList, PaintGeometry, InBrush))
	{
		return true;
	}

	return false;
}

void FSlateDrawElement::Init(FSlateWindowElementList& ElementList, EElementType InElementType, uint32 InLayer, const FPaintGeometry& PaintGeometry, ESlateDrawEffect InDrawEffects)
{
	if (ElementList.IsInProjectContent() && !bApplyDisabledEffectOnWidgets)
	{
		InDrawEffects &= ~ESlateDrawEffect::DisabledEffect;
	}

	// Apply the pixel snapping effects, if it's unset/inherited we don't do anything.
	switch (ElementList.GetPixelSnappingMethod())
	{
		case EWidgetPixelSnapping::Disabled:
			InDrawEffects |= ESlateDrawEffect::NoPixelSnapping;
			break;
		case EWidgetPixelSnapping::Inherit:
			// NOTE: Do nothing on inherit.
		case EWidgetPixelSnapping::SnapToPixel:
			// NOTE: We don't remove the drawing effects pixel snapping here.  The default behavior of InDrawEffects is always to snap to pixels
			// so all we need to do here is not to snap.  This will allow the continued behavior that if someone does set snap to pixel at a
			// higher level, it won't break people that directly set InDrawEffects |= ESlateDrawEffect::NoPixelSnapping, before drawing their
			// draw element.
		default:
			break;
	}

	RenderTransform = PaintGeometry.GetAccumulatedRenderTransform();
	Position = PaintGeometry.DrawPosition;
	Scale = PaintGeometry.DrawScale;
	LocalSize = FVector2f(PaintGeometry.GetLocalSize());
	ClipStateHandle.SetPreCachedClipIndex(ElementList.GetClippingIndex());

#if UE_SLATE_VERIFY_PIXELSIZE
	const FVector2f PixelSize = FVector2f(LocalSize * Scale);
	ensureMsgf(PixelSize.X >= 0.f && PixelSize.X <= (float)std::numeric_limits<uint16>::max(), TEXT("The size X '%f' is too small or big to fit in the SlateVertex buffer."), PixelSize.X);
	ensureMsgf(PixelSize.Y >= 0.f && PixelSize.Y <= (float)std::numeric_limits<uint16>::max(), TEXT("The size Y '%f' is too small or big to fit in the SlateVertex buffer."), PixelSize.Y);
#endif

	LayerId = InLayer;

	ElementType = InElementType;
	DrawEffects = InDrawEffects;
	
	// Calculate the layout to render transform as this is needed by several calculations downstream.
	const FSlateLayoutTransform InverseLayoutTransform(Inverse(FSlateLayoutTransform(Scale, Position)));

	{
		// This is a workaround because we want to keep track of the various Scenes 
		// in use throughout the UI. We keep a synchronized set with the render thread on the SlateRenderer and 
		// use indices to synchronize between them.
		FSlateRenderer* Renderer = FSlateApplicationBase::Get().GetRenderer();
		checkSlow(Renderer);
		int32 SceneIndexLong = Renderer->GetCurrentSceneIndex();
		ensureMsgf(SceneIndexLong <= std::numeric_limits<int8>::max(), TEXT("The Scene index is saved as an int8 in the DrawElements."));
		SceneIndex = (int8)SceneIndexLong;
	}

	BatchFlags = ESlateBatchDrawFlag::None;
	BatchFlags |= static_cast<ESlateBatchDrawFlag>(static_cast<uint32>(InDrawEffects) & static_cast<uint32>(ESlateDrawEffect::NoBlending | ESlateDrawEffect::PreMultipliedAlpha | ESlateDrawEffect::NoGamma | ESlateDrawEffect::InvertAlpha));

	static_assert(((__underlying_type(ESlateDrawEffect))ESlateDrawEffect::NoBlending) == ((__underlying_type(ESlateBatchDrawFlag))ESlateBatchDrawFlag::NoBlending), "Must keep ESlateBatchDrawFlag and ESlateDrawEffect partial matches");
	static_assert(((__underlying_type(ESlateDrawEffect))ESlateDrawEffect::PreMultipliedAlpha) == ((__underlying_type(ESlateBatchDrawFlag))ESlateBatchDrawFlag::PreMultipliedAlpha), "Must keep ESlateBatchDrawFlag and ESlateDrawEffect partial matches");
	static_assert(((__underlying_type(ESlateDrawEffect))ESlateDrawEffect::NoGamma) == ((__underlying_type(ESlateBatchDrawFlag))ESlateBatchDrawFlag::NoGamma), "Must keep ESlateBatchDrawFlag and ESlateDrawEffect partial matches");
	static_assert(((__underlying_type(ESlateDrawEffect))ESlateDrawEffect::InvertAlpha) == ((__underlying_type(ESlateBatchDrawFlag))ESlateBatchDrawFlag::InvertAlpha), "Must keep ESlateBatchDrawFlag and ESlateDrawEffect partial matches");
	if ((InDrawEffects & ESlateDrawEffect::ReverseGamma) != ESlateDrawEffect::None)
	{
		BatchFlags |= ESlateBatchDrawFlag::ReverseGamma;
	}
}

void FSlateDrawElement::ApplyPositionOffset(UE::Slate::FDeprecateVector2DParameter InOffset)
{
	SetPosition(Position + InOffset);
	RenderTransform = Concatenate(RenderTransform, InOffset);

	// Recompute cached layout to render transform
	const FSlateLayoutTransform InverseLayoutTransform(Inverse(FSlateLayoutTransform(Scale, Position)));
}

void FSlateDrawElement::MakeDebugQuad( FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, FLinearColor Tint)
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList))
	{
		return;
	}

	FSlateBoxElement& Element = ElementList.AddUninitialized<EElementType::ET_DebugQuad>();

	Element.SetTint(Tint);
	Element.Init(ElementList, EElementType::ET_DebugQuad, InLayer, PaintGeometry, ESlateDrawEffect::None);
}

void FSlateDrawElement::MakeGeometryOutline(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, FLinearColor Tint, ESlateDrawEffect InDrawEffects, bool bAntialias, float Thickness)
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	const FVector2f LocalSize = FVector2f(PaintGeometry.GetLocalSize());
	TArray<FVector2f> OutlinePoints =
	{
		FVector2f::ZeroVector,
		FVector2f(LocalSize.X, 0.f),
		FVector2f(LocalSize.X, LocalSize.Y),
		FVector2f(0.f, LocalSize.Y),
		FVector2f::ZeroVector
	};

	MakeLines(ElementList, InLayer, PaintGeometry, MoveTemp(OutlinePoints), InDrawEffects, Tint, bAntialias, Thickness);
}

FSlateDrawElement& FSlateDrawElement::MakeBoxInternal(
	FSlateWindowElementList& ElementList,
	uint32 InLayer,
	const FPaintGeometry& PaintGeometry,
	const FSlateBrush* InBrush,
	ESlateDrawEffect InDrawEffects,
	const FLinearColor& InTint
)
{
	EElementType ElementType = (InBrush->DrawAs == ESlateBrushDrawType::Border) ? EElementType::ET_Border : (InBrush->DrawAs == ESlateBrushDrawType::RoundedBox) ? EElementType::ET_RoundedBox : EElementType::ET_Box;

	// Cast to Rounded Rect to get the internal parameters 
	// New payload type - inherit from BoxPayload 

	switch (ElementType)
	{
		case EElementType::ET_Box:
		{
			FSlateBoxElement& Element = ElementList.AddUninitialized<EElementType::ET_Box>();
			Element.SetTint(InTint);
			Element.SetBrush(InBrush, PaintGeometry.GetLocalSize(), PaintGeometry.DrawScale);
			Element.Init(ElementList, ElementType, InLayer, PaintGeometry, InDrawEffects);
			return Element;
		}
		case EElementType::ET_RoundedBox:
		{
			FSlateRoundedBoxElement& Element = ElementList.AddUninitialized<EElementType::ET_RoundedBox>();
			FVector4f CornerRadii = FVector4f(InBrush->OutlineSettings.CornerRadii);
			if (InBrush->OutlineSettings.RoundingType == ESlateBrushRoundingType::HalfHeightRadius)
			{
				const float UniformRadius = PaintGeometry.GetLocalSize().Y / 2.0f;
				CornerRadii = FVector4f(UniformRadius, UniformRadius, UniformRadius, UniformRadius);
			}
			Element.SetRadius(CornerRadii);

			if (InBrush->OutlineSettings.bUseBrushTransparency)
			{
				FLinearColor Color = InBrush->OutlineSettings.Color.GetSpecifiedColor().CopyWithNewOpacity(InTint.A);
				Element.SetOutline(Color, InBrush->OutlineSettings.Width);
			}
			else
			{
				Element.SetOutline(InBrush->OutlineSettings.Color.GetSpecifiedColor(), InBrush->OutlineSettings.Width);
			}
			Element.SetTint(InTint);
			Element.SetBrush(InBrush, PaintGeometry.GetLocalSize(), PaintGeometry.DrawScale);
			Element.Init(ElementList, ElementType, InLayer, PaintGeometry, InDrawEffects);
			return Element;
		}
		case EElementType::ET_Border:
		{
			FSlateBoxElement& Element = ElementList.AddUninitialized<EElementType::ET_Border>();
			Element.SetTint(InTint);
			Element.SetBrush(InBrush, PaintGeometry.GetLocalSize(), PaintGeometry.DrawScale);
			Element.Init(ElementList, ElementType, InLayer, PaintGeometry, InDrawEffects);
			return Element;

		}
		default:
		{
			checkf(0, TEXT("Invalid box element"));
			FSlateBoxElement& Element = ElementList.AddUninitialized<EElementType::ET_Box>();
			Element.SetTint(InTint);
			Element.SetBrush(InBrush, PaintGeometry.GetLocalSize(), PaintGeometry.DrawScale);
			Element.Init(ElementList, ElementType, InLayer, PaintGeometry, InDrawEffects);
			return Element;
		}
	}
}

void FSlateDrawElement::MakeBox(
	FSlateWindowElementList& ElementList,
	uint32 InLayer, 
	const FPaintGeometry& PaintGeometry, 
	const FSlateBrush* InBrush,
	ESlateDrawEffect InDrawEffects, 
	const FLinearColor& InTint)
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList, PaintGeometry, InBrush, InTint))
	{
		return;
	}

	MakeBoxInternal(ElementList, InLayer, PaintGeometry, InBrush, InDrawEffects, InTint);
}

void FSlateDrawElement::MakeRotatedBox(
	FSlateWindowElementList& ElementList,
	uint32 InLayer,
	const FPaintGeometry& PaintGeometry,
	const FSlateBrush* InBrush,
	ESlateDrawEffect InDrawEffects,
	float Angle2D,
	UE::Slate::FDeprecateOptionalVector2DParameter InRotationPoint,
	ERotationSpace RotationSpace,
	const FLinearColor& InTint)
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList, PaintGeometry, InBrush, InTint))
	{
		return;
	}

	FSlateDrawElement& DrawElement = MakeBoxInternal(ElementList, InLayer, PaintGeometry, InBrush, InDrawEffects, InTint);
	
	if (Angle2D != 0.0f)
	{
		const FVector2f RotationPoint = GetRotationPoint(PaintGeometry, InRotationPoint, RotationSpace);
		const FSlateRenderTransform RotationTransform = Concatenate(Inverse(RotationPoint), FQuat2f(Angle2D), RotationPoint);
		DrawElement.SetRenderTransform(Concatenate(RotationTransform, DrawElement.GetRenderTransform()));
	}
}

#if !UE_BUILD_SHIPPING
namespace UE::Slate::Private
{
	bool GLogPaintedText = false;
	FAutoConsoleVariableRef LogPaintedTextRef(
		TEXT("Slate.LogPaintedText"),
		GLogPaintedText,
		TEXT("If true, all text that is visible to the user will be logged when it is painted. This will log the full text to be painted, not the truncated or clipped version based on UI constraints.")
	);
}
#endif

void FSlateDrawElement::MakeText( FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FString& InText, const int32 InStartIndex, const int32 InEndIndex, const FSlateFontInfo& InFontInfo, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint )
{
	const int32 SourceTextLen = InText.Len();
	const int32 StartIndex = FMath::Min<int32>(InStartIndex, SourceTextLen);
	const int32 EndIndex = FMath::Min<int32>(InEndIndex, SourceTextLen);
	const int32 TextLength = (EndIndex > StartIndex) ? EndIndex - StartIndex : 0;
	if (TextLength > 0)
	{
		MakeText(ElementList, InLayer, PaintGeometry, InText.Mid(StartIndex, TextLength), InFontInfo, InDrawEffects, InTint);
	}
}

void FSlateDrawElement::MakeText( FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FString& InText, const FSlateFontInfo& InFontInfo, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint )
{
	SCOPE_CYCLE_COUNTER(STAT_SlateDrawElementMakeTime)
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList, PaintGeometry, InTint, InText))
	{
		return;
	}

#if !UE_BUILD_SHIPPING
	if (UE::Slate::Private::GLogPaintedText)
	{
		UE_LOGF(LogSlate, Log, "MakeText: '%ls'.", *InText);
	}
#endif
	FSlateTextElement& Element = ElementList.AddUninitialized<EElementType::ET_Text>();

	Element.SetTint(InTint);
	Element.SetText(InText, InFontInfo);

	Element.Init(ElementList, EElementType::ET_Text, InLayer, PaintGeometry, InDrawEffects);
}

namespace SlateDrawElement
{
#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	const FName MaterialInterfaceClassName = "MaterialInterface";
	void CheckInvalidUMaterial(const UObject* InMaterialResource, const TCHAR* MaterialMessage)
	{
		if (InMaterialResource && GSlateCheckUObjectRenderResources)
		{
			bool bIsValidLowLevel = InMaterialResource->IsValidLowLevelFast(false);
			if (!bIsValidLowLevel || !IsValid(InMaterialResource) || InMaterialResource->GetClass()->GetFName() == MaterialInterfaceClassName)
			{
				UE_LOGF(LogSlate, Error, "Material '%ls' is not valid. PendingKill:'%d'. ValidLowLevelFast:'%d'. InvalidClass:'%d'"
					, MaterialMessage
					, (bIsValidLowLevel ? !IsValid(InMaterialResource) : false)
					, bIsValidLowLevel
					, (bIsValidLowLevel ? InMaterialResource->GetClass()->GetFName() == MaterialInterfaceClassName : false));

				if (bIsValidLowLevel)
				{
					UE_LOGF(LogSlate, Error, "Material name: '%ls'", *InMaterialResource->GetFullName());
				}

				const TCHAR* Message = TEXT("We detected an invalid resource in FSlateDrawElement. Check the log for more detail.");
				if (GSlateCheckUObjectRenderResourcesShouldLogFatal)
				{
					UE_LOGF(LogSlate, Fatal, "%ls", Message);
				}
				else
				{
					ensureAlwaysMsgf(false, TEXT("%s"), Message);
				}
			}
		}
	}
#endif
}

void FSlateDrawElement::MakeShapedText(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FShapedGlyphSequenceRef& InShapedGlyphSequence, ESlateDrawEffect InDrawEffects, const FLinearColor& BaseTint, const FLinearColor& OutlineTint, FTextOverflowArgs TextOverflowArgs)
{
	SCOPE_CYCLE_COUNTER(STAT_SlateDrawElementMakeTime)
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (InShapedGlyphSequence->GetGlyphsToRender().Num() == 0)
	{
		return;
	}

	if (ShouldCull(ElementList, PaintGeometry))
	{
		return;
	}

	// Don't do anything if there the font would be completely transparent 
	if ((BaseTint.A < UE_KINDA_SMALL_NUMBER && InShapedGlyphSequence->GetFontOutlineSettings().OutlineSize == 0) ||
		(BaseTint.A < UE_KINDA_SMALL_NUMBER && OutlineTint.A < UE_KINDA_SMALL_NUMBER))
	{
		return;
	}

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	SlateDrawElement::CheckInvalidUMaterial(InShapedGlyphSequence->GetFontMaterial(), TEXT("Font Material"));
	SlateDrawElement::CheckInvalidUMaterial(InShapedGlyphSequence->GetFontOutlineSettings().OutlineMaterial, TEXT("Outline Material"));
#endif

	FSlateShapedTextElement& Element = ElementList.AddUninitialized<EElementType::ET_ShapedText>();

	Element.SetTint(BaseTint);
	Element.SetShapedText(InShapedGlyphSequence, OutlineTint);
	Element.SetOverflowArgs(TextOverflowArgs);

	Element.Init(ElementList, EElementType::ET_ShapedText, InLayer, PaintGeometry, InDrawEffects);
}

void FSlateDrawElement::MakeRotatedShapedText(
	FSlateWindowElementList& ElementList,
	const uint32 InLayer,
	const FPaintGeometry& PaintGeometry,
	const FShapedGlyphSequenceRef& InShapedGlyphSequence,
	ESlateDrawEffect InDrawEffects,
	const FLinearColor& BaseTint,
	const FLinearColor& OutlineTint,
	const float Angle2D,
	const UE::Slate::FDeprecateOptionalVector2DParameter& InRotationPoint,
	const ERotationSpace RotationSpace,
	const FTextOverflowArgs& TextOverflowArgs)
{
	SCOPE_CYCLE_COUNTER(STAT_SlateDrawElementMakeTime)
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();
	
	if (InShapedGlyphSequence->GetGlyphsToRender().Num() == 0)
	{
		return;
	}
	
	if (ShouldCull(ElementList, PaintGeometry))
	{
		return;
	}
	
	// Don't do anything if the font would be completely transparent 
	if ((BaseTint.A == 0 && InShapedGlyphSequence->GetFontOutlineSettings().OutlineSize == 0) || 
		(BaseTint.A == 0 && OutlineTint.A == 0))
	{
		return;
	}
	
#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	SlateDrawElement::CheckInvalidUMaterial(InShapedGlyphSequence->GetFontMaterial(), TEXT("Font Material"));
	SlateDrawElement::CheckInvalidUMaterial(InShapedGlyphSequence->GetFontOutlineSettings().OutlineMaterial, TEXT("Outline Material"));
#endif
	
	FSlateShapedTextElement& Element = ElementList.AddUninitialized<EElementType::ET_ShapedText>();
	
	Element.SetTint(BaseTint);
	Element.SetShapedText(InShapedGlyphSequence, OutlineTint);
	Element.SetOverflowArgs(TextOverflowArgs);
	
	Element.Init(ElementList, EElementType::ET_ShapedText, InLayer, PaintGeometry, InDrawEffects);
	
	if (Angle2D != 0.0f)
	{
		const FVector2f RotationPoint = GetRotationPoint(PaintGeometry, InRotationPoint, RotationSpace);
		const FSlateRenderTransform RotationTransform = Concatenate(Inverse(RotationPoint), FQuat2f(Angle2D), RotationPoint);
		Element.SetRenderTransform(Concatenate(RotationTransform, Element.GetRenderTransform()));
	}
}

void FSlateDrawElement::MakeGradient( FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, TArray<FSlateGradientStop> InGradientStops, EOrientation InGradientType, ESlateDrawEffect InDrawEffects, FVector4f CornerRadius)
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList, PaintGeometry))
	{
		return;
	}

	FSlateGradientElement& Element = ElementList.AddUninitialized<EElementType::ET_Gradient>();

	Element.SetGradient(MoveTemp(InGradientStops), InGradientType, CornerRadius);

	Element.Init(ElementList, EElementType::ET_Gradient, InLayer, PaintGeometry, InDrawEffects);
}

void FSlateDrawElement::MakeSpline(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const UE::Slate::FDeprecateVector2DParameter InStart, const UE::Slate::FDeprecateVector2DParameter InStartDir, const UE::Slate::FDeprecateVector2DParameter InEnd, const UE::Slate::FDeprecateVector2DParameter InEndDir, float InThickness, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint)
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList))
	{
		return;
	}
	FSlateSplineElement& Element = ElementList.AddUninitialized<EElementType::ET_Spline>();

	Element.SetHermiteSpline(InStart, InStartDir, InEnd, InEndDir, InThickness, InTint);

	Element.Init(ElementList, EElementType::ET_Spline, InLayer, PaintGeometry, InDrawEffects);
}

void FSlateDrawElement::MakeCubicBezierSpline(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const UE::Slate::FDeprecateVector2DParameter P0, const UE::Slate::FDeprecateVector2DParameter P1, const UE::Slate::FDeprecateVector2DParameter P2, const UE::Slate::FDeprecateVector2DParameter P3, float InThickness, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint)
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList))
	{
		return;
	}
	FSlateSplineElement& Element = ElementList.AddUninitialized<EElementType::ET_Spline>();

	Element.SetCubicBezier(P0, P1, P2, P3, InThickness, InTint);

	Element.Init(ElementList, EElementType::ET_Spline, InLayer, PaintGeometry, InDrawEffects);
}

void FSlateDrawElement::MakeDrawSpaceSpline(FSlateWindowElementList& ElementList, uint32 InLayer, const UE::Slate::FDeprecateVector2DParameter InStart, const UE::Slate::FDeprecateVector2DParameter InStartDir, const UE::Slate::FDeprecateVector2DParameter InEnd, const UE::Slate::FDeprecateVector2DParameter InEndDir, float InThickness, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint)
{
	MakeSpline(ElementList, InLayer, FPaintGeometry(), InStart, InStartDir, InEnd, InEndDir, InThickness, InDrawEffects, InTint);
}

#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS
void FSlateDrawElement::MakeLines(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const TArray<FVector2d>& Points, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint, bool bAntialias, float Thickness)
{
	if (ShouldCull(ElementList) || Points.Num() < 2)
	{
		return;
	}

	TArray<FVector2f> NewVector;
	NewVector.Reserve(Points.Num());
	for (FVector2d Vect : Points)
	{
		NewVector.Add(UE::Slate::CastToVector2f(Vect));
	}
	MakeLines(ElementList, InLayer, PaintGeometry, MoveTemp(NewVector), InDrawEffects, InTint, bAntialias, Thickness);
}
#endif // UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS

void FSlateDrawElement::MakeLines(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, TArray<FVector2f> Points, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint, bool bAntialias, float Thickness)
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList) || Points.Num() < 2)
	{
		return;
	}

	FSlateLineElement& Element = ElementList.AddUninitialized<EElementType::ET_Line>();

	Element.SetTint(InTint);
	Element.SetThickness(Thickness);
	Element.SetLines(MoveTemp(Points), bAntialias);

	ESlateDrawEffect DrawEffects = InDrawEffects;
	if (bAntialias)
	{
		// If the line is to be anti-aliased, we cannot reliably snap
		// the generated vertices
		DrawEffects |= ESlateDrawEffect::NoPixelSnapping;
	}

	Element.Init(ElementList, EElementType::ET_Line, InLayer, PaintGeometry, DrawEffects);
}

void FSlateDrawElement::MakeDashedLines(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, TArray<FVector2f>&& Points, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint, float Thickness, float DashLengthPx, float DashScreenOffset)
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList) || Points.Num() < 2)
	{
		return;
	}

	FSlateLineElement& Element = ElementList.AddUninitialized<EElementType::ET_Line>();

	Element.SetTint(InTint);
	Element.SetThickness(Thickness);
	Element.SetLines(MoveTemp(Points), true);

	Element.DashLength = DashLengthPx;
	Element.DashOffset = DashScreenOffset;

	ESlateDrawEffect DrawEffects = InDrawEffects | ESlateDrawEffect::NoPixelSnapping;

	Element.Init(ElementList, EElementType::ET_Line, InLayer, PaintGeometry, DrawEffects);
}

#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS
void FSlateDrawElement::MakeLines(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const TArray<FVector2d>& Points, const TArray<FLinearColor>& PointColors, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint, bool bAntialias, float Thickness)
{
	if (ShouldCull(ElementList) || Points.Num() < 2)
	{
		return;
	}

	TArray<FVector2f> NewVector;
	NewVector.Reserve(Points.Num());
	for (FVector2d Vect : Points)
	{
		NewVector.Add(UE::Slate::CastToVector2f(Vect));
	}
	MakeLines(ElementList, InLayer, PaintGeometry, MoveTemp(NewVector), PointColors, InDrawEffects, InTint, bAntialias, Thickness);
}
#endif

void FSlateDrawElement::MakeLines( FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, TArray<FVector2f> Points, TArray<FLinearColor> PointColors, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint, bool bAntialias, float Thickness )
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList) || Points.Num() < 2)
	{
		return;
	}

	FSlateLineElement& Element = ElementList.AddUninitialized<EElementType::ET_Line>();

	Element.SetTint(InTint);
	Element.SetThickness(Thickness);
	Element.SetLines(MoveTemp(Points), bAntialias, MoveTemp(PointColors));

	Element.Init(ElementList, EElementType::ET_Line, InLayer, PaintGeometry, InDrawEffects);
}

void FSlateDrawElement::MakeDashedLines(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, TArray<FVector2f>&& Points, TArray<FLinearColor>&& PointColors, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint, float Thickness, float DashLengthPx, float DashScreenOffset)
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList) || Points.Num() < 2)
	{
		return;
	}

	FSlateLineElement& Element = ElementList.AddUninitialized<EElementType::ET_Line>();

	Element.SetTint(InTint);
	Element.SetThickness(Thickness);
	Element.SetLines(MoveTemp(Points), true, MoveTemp(PointColors));

	Element.DashLength = DashLengthPx;
	Element.DashOffset = DashScreenOffset;

	ESlateDrawEffect DrawEffects = InDrawEffects | ESlateDrawEffect::NoPixelSnapping;

	Element.Init(ElementList, EElementType::ET_Line, InLayer, PaintGeometry, DrawEffects);
}

void FSlateDrawElement::MakeViewport( FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, TSharedPtr<const ISlateViewport> Viewport, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint )
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList))
	{
		return;
	}

	FSlateViewportElement& Element = ElementList.AddUninitialized<EElementType::ET_Viewport>();

	Element.SetViewport(Viewport, InTint);
	check(Element.RenderTargetResource == nullptr || !Element.RenderTargetResource->Debug_IsDestroyed());

	Element.Init(ElementList, EElementType::ET_Viewport, InLayer, PaintGeometry, InDrawEffects);

	if (Viewport->GetViewportDynamicRange() == ESlateViewportDynamicRange::HDR)
	{
		EnumAddFlags(Element.BatchFlags, ESlateBatchDrawFlag::HDR);
	}
}


void FSlateDrawElement::MakeCustom( FSlateWindowElementList& ElementList, uint32 InLayer, TSharedPtr<ICustomSlateElement, ESPMode::ThreadSafe> CustomDrawer )
{
	if (ShouldCull(ElementList))
	{
		return;
	}

	FSlateCustomDrawerElement& Element = ElementList.AddUninitialized<EElementType::ET_Custom>();

	Element.SetCustomDrawer(CustomDrawer);
	
	Element.Init(ElementList, EElementType::ET_Custom, InLayer, FPaintGeometry(), ESlateDrawEffect::None);
	Element.RenderTransform = FSlateRenderTransform();
}


void FSlateDrawElement::MakeCustomVerts(FSlateWindowElementList& ElementList, uint32 InLayer, const FSlateResourceHandle& InRenderResourceHandle, const TArray<FSlateVertex>& InVerts, const TArray<SlateIndex>& InIndexes, ISlateUpdatableInstanceBuffer* InInstanceData, uint32 InInstanceOffset, uint32 InNumInstances, ESlateDrawEffect InDrawEffects, ESlateBatchDrawFlag InBatchFlags)
{
	SCOPE_CYCLE_COUNTER(STAT_SlateDrawElementMakeCustomVertsTime);
	
	if (ShouldCull(ElementList))
	{
		return;
	}

	FSlateCustomVertsElement& Element = ElementList.AddUninitialized<EElementType::ET_CustomVerts>();

	const FSlateShaderResourceProxy* RenderingProxy = InRenderResourceHandle.GetResourceProxy();
	ISlateUpdatableInstanceBufferRenderProxy* RenderProxy = InInstanceData ? InInstanceData->GetRenderProxy() : nullptr;
	Element.SetCustomVerts(RenderingProxy, InVerts, InIndexes, RenderProxy, InInstanceOffset, InNumInstances);

	Element.Init(ElementList, EElementType::ET_CustomVerts, InLayer, FPaintGeometry(), InDrawEffects);
	Element.AddBatchFlags(InBatchFlags);
	Element.RenderTransform = FSlateRenderTransform();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FSlateDrawElement::MakePostProcessPass(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FVector4f& Params, int32 DownsampleAmount, FVector4f CornerRadius)
{
	FSlateDrawElement::MakePostProcessBlur(ElementList, InLayer, PaintGeometry, Params, DownsampleAmount, CornerRadius);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FSlateDrawElement::MakePostProcessBlur(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FVector4f& Params, int32 DownsampleAmount, FVector4f CornerRadius)
{
	PaintGeometry.CommitTransformsIfUsingLegacyConstructor();

	if (ShouldCull(ElementList))
	{
		return;
	}

	FSlatePostProcessElement& Element = ElementList.AddUninitialized<EElementType::ET_PostProcessPass>();

	Element.DownsampleAmount = DownsampleAmount;
	Element.PostProcessData = Params;
	Element.CornerRadius = CornerRadius;

	Element.Init(ElementList, EElementType::ET_PostProcessPass, InLayer, PaintGeometry, ESlateDrawEffect::None);
}

FSlateDrawElement::FSlateDrawElement()
	: bIsCached(false)
{

}

FVector2f FSlateDrawElement::GetRotationPoint(const FPaintGeometry& PaintGeometry, const TOptional<FVector2f>& UserRotationPoint, ERotationSpace RotationSpace)
{
	FVector2f RotationPoint(0, 0);

	const FVector2f LocalSize = UE::Slate::CastToVector2f(PaintGeometry.GetLocalSize());

	switch (RotationSpace)
	{
		case RelativeToElement:
		{
			// If the user did not specify a rotation point, we rotate about the center of the element
			RotationPoint = UserRotationPoint.Get(LocalSize * 0.5f);
		}
		break;
		case RelativeToWorld:
		{
			// its in world space, must convert the point to local space.
			RotationPoint = TransformPoint(Inverse(PaintGeometry.GetAccumulatedRenderTransform()), UserRotationPoint.Get(FVector2f::ZeroVector));
		}
		break;
	default:
		check(0);
		break;
	}

	return RotationPoint;
}