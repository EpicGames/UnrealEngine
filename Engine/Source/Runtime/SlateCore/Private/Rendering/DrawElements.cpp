// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/DrawElements.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "HAL/IConsoleManager.h"
#include "Types/ReflectionMetadata.h"
#include "Fonts/ShapedTextFwd.h"
#include "Fonts/FontCache.h"
#include "Rendering/DrawElementPayloads.h"
#include "Rendering/SlateObjectReferenceCollector.h"
#include "Debugging/SlateDebugging.h"
#include "Application/SlateApplicationBase.h"

#include <limits>
#include <type_traits>
#include <utility>

/** 
 * Helper struct to determine if an element has implemented AddReferencedObject 
 * 
 * For an explantion of how this detection: https://en.cppreference.com/w/cpp/types/void_t
 */
template <typename T>
struct TIsReferencedObjects
{
	template <typename U, typename = void>
	struct IsReferencedObjectsHelper : std::false_type
	{
	};

	template <typename U>
	struct IsReferencedObjectsHelper<U, std::void_t<decltype(std::declval<U>().AddReferencedObjects(std::declval<FReferenceCollector&>()))>> : std::true_type
	{
	};

	static constexpr bool Value = IsReferencedObjectsHelper<T>::value;
};

static_assert(!TIsReferencedObjects<FSlateBoxElement>::Value, "FSlateBoxElement should have not AddReferencedObjects.");
static_assert(TIsReferencedObjects<FSlateTextElement>::Value, "FSlateTextElement should have AddReferencedObjects.");
static_assert(TIsReferencedObjects<FSlateShapedTextElement>::Value, "FSlateShapedTextElement should have AddReferencedObjects.");

FSlateWindowElementList::FSlateWindowElementList(const TSharedPtr<SWindow>& InPaintWindow)
	: WeakPaintWindow(InPaintWindow)
	, RawPaintWindow(InPaintWindow.Get())
	, MemManager()
#if STATS
	, MemManagerAllocatedMemory(0)
#endif
	, RenderTargetWindow(nullptr)
	, bIsProjectContent(false)
	, bNeedsDeferredResolve(false)
	, ResolveToDeferredIndex()
	, WindowSize(FVector2f(0.0f, 0.0f))
{
	if (InPaintWindow.IsValid())
	{
		WindowSize = UE::Slate::CastToVector2f(InPaintWindow->GetSizeInScreen());
	}
}

FSlateWindowElementList::~FSlateWindowElementList()
{
	FinishMergeRenderBatches();
}

void FSlateWindowElementList::SetIsInGameLayer(bool bInGameLayer)
{
	bIsProjectContent = bInGameLayer;
}

bool FSlateWindowElementList::GetIsInGameLayer()
{
	return bIsProjectContent;
}

void FSlateWindowElementList::SetIsInProjectContent(bool bIsInProjectContent)
{
	bIsProjectContent = bIsInProjectContent;
}

void FSlateWindowElementList::StartMergeRenderBatches()
{
	if (!BatchData.GetRenderBatches().IsEmpty() || !BatchDataHDR.GetRenderBatches().IsEmpty())
	{
		MergeBatchDataTask = UE::Tasks::Launch(TEXT("Slate::MergeRenderBatches"), [this]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Slate::MergeRenderBatches);
			BatchData.MergeRenderBatches();
			BatchDataHDR.MergeRenderBatches();
		});
	}
}

void FSlateWindowElementList::FinishMergeRenderBatches()
{
	MergeBatchDataTask.Wait();
	MergeBatchDataTask = {};
}

FSlateWindowElementList::FDeferredPaint::FDeferredPaint( const TSharedRef<const SWidget>& InWidgetToPaint, const FPaintArgs& InArgs, const FGeometry InAllottedGeometry, const FWidgetStyle& InWidgetStyle, bool InParentEnabled )
	: WidgetToPaintPtr( InWidgetToPaint )
	, Args( InArgs )
	, AllottedGeometry( InAllottedGeometry )
	, WidgetStyle( InWidgetStyle )
	, bParentEnabled( InParentEnabled )
{
	const_cast<FPaintArgs&>(Args).SetDeferredPaint(true);

#if WITH_SLATE_DEBUGGING
	// We need to perform this update here, because otherwise we'll warn that this widget
	// was not painted along the fast path, which, it will be, but later because it's deferred,
	// but we need to go ahead and update the painted frame to match the current one, so
	// that we don't think this widget was forgotten.
	const_cast<SWidget&>(InWidgetToPaint.Get()).Debug_UpdateLastPaintFrame();
#endif
}

FSlateWindowElementList::FDeferredPaint::FDeferredPaint(const FDeferredPaint& Copy, const FPaintArgs& InArgs)
	: WidgetToPaintPtr(Copy.WidgetToPaintPtr)
	, Args(InArgs)
	, AllottedGeometry(Copy.AllottedGeometry)
	, WidgetStyle(Copy.WidgetStyle)
	, bParentEnabled(Copy.bParentEnabled)
{
	const_cast<FPaintArgs&>(Args).SetDeferredPaint(true);
}

int32 FSlateWindowElementList::FDeferredPaint::ExecutePaint(int32 LayerId, FSlateWindowElementList& OutDrawElements, const FSlateRect& MyCullingRect) const
{
	TSharedPtr<const SWidget> WidgetToPaint = WidgetToPaintPtr.Pin();
	if ( WidgetToPaint.IsValid() )
	{
		return WidgetToPaint->Paint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, WidgetStyle, bParentEnabled );
	}

	return LayerId;
}

FSlateWindowElementList::FDeferredPaint FSlateWindowElementList::FDeferredPaint::Copy(const FPaintArgs& InArgs)
{
	return FDeferredPaint(*this, InArgs);
}


void FSlateWindowElementList::QueueDeferredPainting( const FDeferredPaint& InDeferredPaint )
{
	DeferredPaintList.Add(MakeShared<FDeferredPaint>(InDeferredPaint));
}

int32 FSlateWindowElementList::PaintDeferred(int32 LayerId, const FSlateRect& MyCullingRect)
{
	bNeedsDeferredResolve = false;

	int32 ResolveIndex = ResolveToDeferredIndex.Pop(EAllowShrinking::No);

	for ( int32 i = ResolveIndex; i < DeferredPaintList.Num(); ++i )
	{
		LayerId = DeferredPaintList[i]->ExecutePaint(LayerId, *this, MyCullingRect);
	}

	for ( int32 i = DeferredPaintList.Num() - 1; i >= ResolveIndex; --i )
	{
		DeferredPaintList.RemoveAt(i, EAllowShrinking::No);
	}

	return LayerId;
}

void FSlateWindowElementList::BeginDeferredGroup()
{
	ResolveToDeferredIndex.Add(DeferredPaintList.Num());
}

void FSlateWindowElementList::EndDeferredGroup()
{
	bNeedsDeferredResolve = true;
}

void FSlateWindowElementList::PushPaintingWidget(const SWidget& CurrentWidget, int32 StartingLayerId, FSlateCachedElementsHandle& CurrentCacheHandle)
{
	FSlateCachedElementData* CurrentCachedElementData = GetCurrentCachedElementData();
	if (CurrentCachedElementData)
	{
		// When a widget is pushed reset its draw elements.  They are being recached or possibly going away
		if (CurrentCacheHandle.IsValid())
		{
#if WITH_SLATE_DEBUGGING
			check(CurrentCacheHandle.IsOwnedByWidget(&CurrentWidget));
#endif
			CurrentCacheHandle.ClearCachedElements();
		}

		WidgetDrawStack.Emplace(CurrentCacheHandle, CurrentWidget.IsVolatileIndirectly() || CurrentWidget.IsVolatile(), &CurrentWidget);
	}
}

FSlateCachedElementsHandle FSlateWindowElementList::PopPaintingWidget(const SWidget& CurrentWidget)
{
	FSlateCachedElementData* CurrentCachedElementData = GetCurrentCachedElementData();
	if (CurrentCachedElementData)
	{
#if WITH_SLATE_DEBUGGING
		check(WidgetDrawStack.Top().Widget == &CurrentWidget);
#endif

		return WidgetDrawStack.Pop(EAllowShrinking::No).CacheHandle;
	}

	return FSlateCachedElementsHandle::Invalid;
}

void FSlateWindowElementList::PushCachedElementData(FSlateCachedElementData& CachedElementData)
{
	check(&CachedElementData); 
	const int32 Index = CachedElementDataList.AddUnique(&CachedElementData);
	CachedElementDataListStack.Push(Index);
}

void FSlateWindowElementList::PopCachedElementData()
{
	if (!CachedElementDataListStack.IsEmpty())
	{
		CachedElementDataListStack.Pop();
	}
}

FSlateDrawElement& FSlateWindowElementList::AddUninitializedLookup(EElementType InElementType)
{
	// Determine what type of element to add
	switch (InElementType)
	{
	case EElementType::ET_Box:
	{
		return AddUninitialized<EElementType::ET_Box>();
	}
	case EElementType::ET_RoundedBox:
	{
		return AddUninitialized<EElementType::ET_RoundedBox>();
	}
	case EElementType::ET_Border:
	{
		return AddUninitialized<EElementType::ET_Border>();
	}
	case EElementType::ET_Text:
	{
		return AddUninitialized<EElementType::ET_Text>();
	}
	case EElementType::ET_ShapedText:
	{
		return AddUninitialized<EElementType::ET_ShapedText>();
	}
	case EElementType::ET_Line:
	{
		return AddUninitialized<EElementType::ET_Line>();
	}
	case EElementType::ET_DebugQuad:
	{
		return AddUninitialized<EElementType::ET_DebugQuad>();
	}
	case EElementType::ET_Spline:
	{
		return AddUninitialized<EElementType::ET_Spline>();
	}
	case EElementType::ET_Gradient:
	{
		return AddUninitialized<EElementType::ET_Gradient>();
	}
	case EElementType::ET_Viewport:
	{
		return AddUninitialized<EElementType::ET_Viewport>();
	}
	case EElementType::ET_Custom:
	{
		return AddUninitialized<EElementType::ET_Custom>();
	}
	case EElementType::ET_CustomVerts:
	{
		return AddUninitialized<EElementType::ET_CustomVerts>();
	}
	case EElementType::ET_PostProcessPass:
	{
		return AddUninitialized<EElementType::ET_PostProcessPass>();
	}
	default:
		return AddUninitialized<EElementType::ET_NonMapped>();
	}
}

int32 FSlateWindowElementList::PushClip(const FSlateClippingZone& InClipZone)
{
	const int32 NewClipIndex = ClippingManager.PushClip(InClipZone);

	return NewClipIndex;
}

int32 FSlateWindowElementList::GetClippingIndex() const
{
	return ClippingManager.GetClippingIndex();
}

TOptional<FSlateClippingState> FSlateWindowElementList::GetClippingState() const
{
	return ClippingManager.GetActiveClippingState();
}

void FSlateWindowElementList::PopClip()
{
	ClippingManager.PopClip();
}

void FSlateWindowElementList::PopClipToStackIndex(int32 Index)
{
	ClippingManager.PopToStackIndex(Index);
}

int32 FSlateWindowElementList::PushPixelSnappingMethod(EWidgetPixelSnapping InPixelSnappingMethod)
{
	PixelSnappingMethodStack.Add(InPixelSnappingMethod);
	return PixelSnappingMethodStack.Num();
}

void FSlateWindowElementList::PopPixelSnappingMethod()
{
	if (ensure(PixelSnappingMethodStack.Num() > 0))
	{
		PixelSnappingMethodStack.Pop();
	}
}

EWidgetPixelSnapping FSlateWindowElementList::GetPixelSnappingMethod() const
{
	return (PixelSnappingMethodStack.Num() > 0) ? PixelSnappingMethodStack.Top() : EWidgetPixelSnapping::Inherit;
}

void FSlateWindowElementList::SetRenderTargetWindow(SWindow* InRenderTargetWindow)
{
	check(IsThreadSafeForSlateRendering());
	RenderTargetWindow = InRenderTargetWindow;
}

DECLARE_MEMORY_STAT(TEXT("FSlateWindowElementList MemManager"), STAT_FSlateWindowElementListMemManager, STATGROUP_SlateMemory);
DECLARE_DWORD_COUNTER_STAT(TEXT("FSlateWindowElementList MemManager Count"), STAT_FSlateWindowElementListMemManagerCount, STATGROUP_SlateMemory);

void FSlateWindowElementList::ResetElementList()
{
	QUICK_SCOPE_CYCLE_COUNTER(Slate_ResetElementList);

	DeferredPaintList.Reset();

	FinishMergeRenderBatches();
	BatchData.ResetData();
	BatchDataHDR.ResetData();

	ClippingManager.ResetClippingState();

	auto ResetElementContainer = [](auto& Container)
	{
		Container.Reset();
	};

	VisitTupleElements(ResetElementContainer, UncachedDrawElements);

#if STATS
	const int32 DeltaMemory = MemManager.GetByteCount() - MemManagerAllocatedMemory;
	INC_DWORD_STAT(STAT_FSlateWindowElementListMemManagerCount);
	INC_MEMORY_STAT_BY(STAT_FSlateWindowElementListMemManager, DeltaMemory);

	MemManagerAllocatedMemory = MemManager.GetByteCount();
#endif

	MemManager.Flush();
	
	CachedElementDataList.Empty();
	CachedElementDataListStack.Empty();

	check(WidgetDrawStack.Num() == 0);
	check(ResolveToDeferredIndex.Num() == 0);

	RenderTargetWindow = nullptr;
}

void FSlateWindowElementList::AddReferencedObjects(FReferenceCollector& Collector)
{
	auto AddReferencedElementContainer = [&](auto& Container)
	{
		// Container should just be some FSlateDrawElementArray<T>
		using FSlateElementType = typename std::remove_reference<decltype(Container)>::type::ElementType;

		if constexpr (TIsReferencedObjects<FSlateElementType>::Value)
		{
			for (FSlateElementType& Element : Container)
			{
				Element.AddReferencedObjects(Collector);
			}
		}
	};

	VisitTupleElements(AddReferencedElementContainer, UncachedDrawElements);
}

const FSlateClippingState* FSlateCachedElementData::GetClipStateFromParent(const FSlateClippingManager& ParentClipManager)
{
	const int32 ClippingIndex = ParentClipManager.GetClippingIndex();

	if(ClippingIndex != INDEX_NONE)
	{
		return &ParentClipManager.GetClippingStates()[ClippingIndex];
	}
	else
	{
		return nullptr;
	}
}

void FSlateCachedElementData::ValidateWidgetOwner(TSharedPtr<FSlateCachedElementList> List, const SWidget* CurrentWidget)
{
#if WITH_SLATE_DEBUGGING
	checkSlow(List->OwningWidget == CurrentWidget);
	checkSlow(CurrentWidget->GetParentWidget().IsValid());
#endif
}

void FSlateCachedElementData::Empty()
{
	if (CachedElementLists.Num())
	{
		UE_LOGF(LogSlate, Verbose, "Resetting cached element data.  Num: %d", CachedElementLists.Num());
	}

#if WITH_SLATE_DEBUGGING
	for (TSharedPtr<FSlateCachedElementList>& CachedElementList : CachedElementLists)
	{
		ensure(CachedElementList.IsUnique());
	}
#endif

	CachedElementLists.Empty();
	CachedBatches.Empty();
	CachedClipStates.Empty();
	ListsWithNewData.Empty();
}

FSlateCachedElementsHandle FSlateCachedElementData::AddCache(const SWidget* Widget)
{
#if WITH_SLATE_DEBUGGING
	for (TSharedPtr<FSlateCachedElementList>& CachedElementList : CachedElementLists)
	{
		ensure(CachedElementList->OwningWidget != Widget);
	}
#endif

	TSharedRef<FSlateCachedElementList> NewList = MakeShared<FSlateCachedElementList>(this, Widget);
	NewList->Initialize();

	CachedElementLists.Add(NewList);

	return FSlateCachedElementsHandle(NewList);
}

FSlateRenderBatch& FSlateCachedElementData::AddCachedRenderBatch(FSlateRenderBatch&& NewBatch, int32& OutIndex)
{
	// Check perf against add.  AddAtLowest makes it generally re-add elements at the same index it just removed which is nicer on the cache
	int32 LowestFreedIndex = 0;
	OutIndex = CachedBatches.EmplaceAtLowestFreeIndex(LowestFreedIndex, NewBatch);
	return CachedBatches[OutIndex];
}

void FSlateCachedElementData::RemoveCachedRenderBatches(const TArray<int32>& CachedRenderBatchIndices)
{
	for (int32 Index : CachedRenderBatchIndices)
	{
		CachedBatches.RemoveAt(Index);
	}
}

FSlateCachedClipState& FSlateCachedElementData::FindOrAddCachedClipState(const FSlateClippingState* RefClipState)
{
	for (auto& CachedState : CachedClipStates)
	{
		if (*CachedState.ClippingState == *RefClipState)
		{
			return CachedState;
		}
	}

	return CachedClipStates.Emplace_GetRef(FSlateCachedClipState(*RefClipState));
}

void FSlateCachedElementData::CleanupUnusedClipStates()
{
	for(int32 CachedStateIdx = 0; CachedStateIdx < CachedClipStates.Num();)
	{
		const FSlateCachedClipState& CachedState = CachedClipStates[CachedStateIdx];
		if (CachedState.ClippingState.IsUnique())
		{
			CachedClipStates.RemoveAtSwap(CachedStateIdx);
		}
		else
		{
			++CachedStateIdx;
		}
	}
}

void FSlateCachedElementData::RemoveList(FSlateCachedElementsHandle& CacheHandle)
{
	TSharedPtr<FSlateCachedElementList> CachedList = CacheHandle.Ptr.Pin();
	CachedElementLists.RemoveSingleSwap(CachedList);
	ListsWithNewData.RemoveSingleSwap(CachedList.Get());

	CachedList->ClearCachedElements();
}

void FSlateCachedElementData::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (TSharedPtr<FSlateCachedElementList>& CachedElementList : CachedElementLists)
	{
		CachedElementList->AddReferencedObjects(Collector);
	}
}

FSlateCachedElementList::~FSlateCachedElementList()
{
	DestroyCachedData();
}

void FSlateCachedElementList::ClearCachedElements()
{
	// Destroy vertex data in a thread safe way
	DestroyCachedData();

	CachedRenderingData = new FSlateCachedFastPathRenderingData;
}

FSlateRenderBatch& FSlateCachedElementList::AddRenderBatch(int32 InLayer, const FShaderParams& InShaderParams, const FSlateShaderResource* InResource, ESlateDrawPrimitive InPrimitiveType, ESlateShader InShaderType, ESlateDrawEffect InDrawEffects, ESlateBatchDrawFlag InDrawFlags, int8 SceneIndex)
{
	FSlateRenderBatch NewRenderBatch(InLayer, InShaderParams, InResource, InPrimitiveType, InShaderType, InDrawEffects, InDrawFlags, SceneIndex, &CachedRenderingData->Vertices, &CachedRenderingData->Indices, CachedRenderingData->Vertices.Num(), CachedRenderingData->Indices.Num());
	int32 RenderBatchIndex = INDEX_NONE;
	FSlateRenderBatch& AddedBatchRef = ParentData->AddCachedRenderBatch(MoveTemp(NewRenderBatch), RenderBatchIndex);
	
	check(RenderBatchIndex != INDEX_NONE);

	CachedRenderBatchIndices.Add(RenderBatchIndex);

	return AddedBatchRef;
}

void FSlateCachedElementList::AddCachedClipState(FSlateCachedClipState& ClipStateToCache)
{
	CachedRenderingData->CachedClipStates.Add(ClipStateToCache);
}

void FSlateCachedElementList::AddReferencedObjects(FReferenceCollector& Collector)
{
	auto AddReferencedElementContainer = [&](auto& Container)
	{
		// Container should just be some FSlateDrawElementArray<T>
		using FSlateElementType = typename std::remove_reference<decltype(Container)>::type::ElementType;

		if constexpr (TIsReferencedObjects<FSlateElementType>::Value)
		{
			for (FSlateElementType& Element : Container)
			{
				Element.AddReferencedObjects(Collector);
			}
		}
	};

	VisitTupleElements(AddReferencedElementContainer, DrawElements);
}

bool FSlateCachedElementList::IsEmpty()
{
	bool bElementsFound = false;

	auto CheckElementsExist = [&](auto& Container)
	{
		if (!bElementsFound)
		{
			bElementsFound = Container.Num() > 0;
		}
	};

	VisitTupleElements(CheckElementsExist, DrawElements);

	return !bElementsFound;
}

int32 FSlateCachedElementList::NumElements()
{
	int32 NumElements = 0;

	auto CountNumElements = [&](auto& Container)
	{
		NumElements += Container.Num();
	};

	VisitTupleElements(CountNumElements, DrawElements);

	return NumElements;
}

void FSlateCachedElementList::DestroyCachedData()
{
	// Clear out any cached draw elements
	auto ResetElementContainer = [](auto& Container)
	{
		Container.Reset();
	};

	VisitTupleElements(ResetElementContainer, DrawElements);

	// Clear out any cached render batches
	if (CachedRenderBatchIndices.Num())
	{
		ParentData->RemoveCachedRenderBatches(CachedRenderBatchIndices);
		CachedRenderBatchIndices.Reset();
	}

	// Destroy any cached rendering data we own.
	if (CachedRenderingData)
	{
		if (FSlateApplicationBase::IsInitialized())
		{
			if (FSlateRenderer* SlateRenderer = FSlateApplicationBase::Get().GetRenderer())
			{
				SlateRenderer->DestroyCachedFastPathRenderingData(CachedRenderingData);
			}
		}
		else
		{
			delete CachedRenderingData;
		}

		CachedRenderingData = nullptr;
	}
}

FSlateCachedElementsHandle FSlateCachedElementsHandle::Invalid;

void FSlateCachedElementsHandle::ClearCachedElements()
{
	if (TSharedPtr<FSlateCachedElementList> List = Ptr.Pin())
	{
		List->ClearCachedElements();
	}
}

void FSlateCachedElementsHandle::RemoveFromCache()
{
	if (TSharedPtr<FSlateCachedElementList> List = Ptr.Pin())
	{
		List->GetOwningData()->RemoveList(*this);
		ensure(List.IsUnique());
	}

	check(!Ptr.IsValid());
}

bool FSlateCachedElementsHandle::IsOwnedByWidget(const SWidget* Widget) const
{
	if (const TSharedPtr<FSlateCachedElementList> List = Ptr.Pin())
	{
		return List->OwningWidget == Widget;
	}

	return false;
}

bool FSlateCachedElementsHandle::HasCachedElements() const
{
	if (const TSharedPtr<FSlateCachedElementList> List = Ptr.Pin())
	{
		bool bElementsFound = false;

		auto CheckElementsExist = [&](auto& Container)
		{
			if (!bElementsFound)
			{
				bElementsFound = Container.Num() > 0;
			}
		};

		VisitTupleElements(CheckElementsExist, List->DrawElements);

		return bElementsFound;
	}

	return false;
}