// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/ElementBatcher.h"
#include "Fonts/SlateFontInfo.h"
#include "Fonts/SlateTextShaper.h"
#include "Fonts/FontCache.h"
#include "Fonts/FontCacheFreeType.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingPolicy.h"
#include "Rendering/ShaderResourceManager.h"
#include "Widgets/SWindow.h"
#include "HAL/IConsoleManager.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Debugging/SlateDebugging.h"
#include "Widgets/SWidgetUtils.h"
#include "Rendering/DrawElementPayloads.h"

#define ENABLE_SDF_DEBUG_VISUAL (!UE_BUILD_SHIPPING)

CSV_DECLARE_CATEGORY_MODULE_EXTERN(SLATECORE_API, Slate);

DECLARE_CYCLE_STAT(TEXT("Slate RT: Create Batches"), STAT_SlateRTCreateBatches, STATGROUP_Slate);

DEFINE_STAT(STAT_SlateAddElements);

DEFINE_STAT(STAT_SlateElements);
DEFINE_STAT(STAT_SlateElements_Box);
DEFINE_STAT(STAT_SlateElements_Border);
DEFINE_STAT(STAT_SlateElements_Text);
DEFINE_STAT(STAT_SlateElements_ShapedText);
DEFINE_STAT(STAT_SlateElements_Line);
DEFINE_STAT(STAT_SlateElements_Other);
DEFINE_STAT(STAT_SlateInvalidation_RecachedElements);

int32 GSlateFeathering = 0;

#if ENABLE_SDF_DEBUG_VISUAL
enum class ESdfTextDebugVisual
{
	Disabled = 0,
	Raw = 1,
	Color = 2,
	Boxes = 3
};

static int32 SdfTextDebugVisual = static_cast<int32>(ESdfTextDebugVisual::Disabled);

FAutoConsoleVariableRef CVarSlateSdfTextDebugVisual(
	TEXT("UI.SlateSDFText.DebugVisual"),
	SdfTextDebugVisual,
	TEXT("Slate SDF text debug visuals - 0: Disabled, 1: Raw SDF, 2: Text color, 3: Colored boxes."));

static const FColor DebugVisualBitmapTextColor(0x80, 0x80, 0x80); // Gray
static const FColor DebugVisualMsdfTextColor(0xff, 0x00, 0xff); // Magenta
static const FColor DebugVisualSdfTextColor(0x00, 0xff, 0xff); // Cyan
static const FColor DebugVisualApproximateSdfTextColor(0xff, 0x80, 0x00); // Orange
#endif // ENABLE_SDF_DEBUG_VISUAL

FSlateElementBatch::FSlateElementBatch(const FSlateShaderResource* InShaderResource, const FShaderParams& InShaderParams, ESlateShader ShaderType, ESlateDrawPrimitive PrimitiveType, ESlateDrawEffect InDrawEffects, ESlateBatchDrawFlag InBatchFlags, const FSlateDrawElement& InDrawElement, int32 InstanceCount, uint32 InstanceOffset, ISlateUpdatableInstanceBuffer* InstanceData)
	: BatchKey(InShaderParams, ShaderType, PrimitiveType, InDrawEffects, InBatchFlags, InDrawElement.GetClippingHandle(), InstanceCount, InstanceOffset, InstanceData, InDrawElement.GetSceneIndex())
	, ShaderResource(InShaderResource)
	, NumElementsInBatch(0)
	, VertexArrayIndex(INDEX_NONE)
	, IndexArrayIndex(INDEX_NONE)
{
}

FSlateElementBatch::FSlateElementBatch(TWeakPtr<ICustomSlateElement, ESPMode::ThreadSafe> InCustomDrawer, const FSlateDrawElement& InDrawElement)
	: BatchKey(InCustomDrawer, InDrawElement.GetClippingHandle())
	, ShaderResource(nullptr)
	, NumElementsInBatch(0)
	, VertexArrayIndex(INDEX_NONE)
	, IndexArrayIndex(INDEX_NONE)
{
}

FSlateBatchData::FSlateBatchData()
	: FirstRenderBatchIndex(INDEX_NONE)
	, NumLayers(0)
	, NumBatches(0)
	, bIsStencilBufferRequired(false)
	, bRequiresVirtualTextureFeedback(false)
{}

FSlateBatchData::~FSlateBatchData() = default;

void FSlateBatchData::ResetData()
{
	RenderBatches.Reset();
	UncachedSourceBatchIndices.Reset();
	UncachedSourceBatchVertices.Reset();
	FinalIndexData.Reset();
	FinalVertexData.Reset();

	FirstRenderBatchIndex = INDEX_NONE;

	NumBatches = 0;
	NumLayers = 0;

	bIsStencilBufferRequired = false;
	bRequiresVirtualTextureFeedback = false;
}

#define MAX_VERT_ARRAY_RECYCLE (200)
#define MAX_INDEX_ARRAY_RECYCLE (500)

bool FSlateBatchData::IsStencilClippingRequired() const
{
	return bIsStencilBufferRequired;
}

bool FSlateBatchData::IsVirtualTextureFeedbackRequired() const
{
	return bRequiresVirtualTextureFeedback;
}

FSlateRenderBatch& FSlateBatchData::AddRenderBatch(int32 InLayer, const FShaderParams& InShaderParams, const FSlateShaderResource* InResource, ESlateDrawPrimitive InPrimitiveType, ESlateShader InShaderType, ESlateDrawEffect InDrawEffects, ESlateBatchDrawFlag InDrawFlags, int8 SceneIndex)
{
	return RenderBatches.Emplace_GetRef(InLayer, InShaderParams, InResource, InPrimitiveType, InShaderType, InDrawEffects, InDrawFlags, SceneIndex, &UncachedSourceBatchVertices, &UncachedSourceBatchIndices, UncachedSourceBatchVertices.Num(), UncachedSourceBatchIndices.Num());
}

void FSlateBatchData::FillBuffersFromNewBatch(FSlateRenderBatch& Batch, FSlateVertexArray& FinalVertices, FSlateIndexArray& FinalIndices)
{
	if(Batch.HasVertexData())
	{
		const int32 SourceVertexOffset = Batch.VertexOffset;
		const int32 SourceIndexOffset = Batch.IndexOffset;

		// At the start of a new batch, just direct copy the verts
		// todo: May need to change this to use absolute indices
		Batch.VertexOffset = FinalVertices.Num();
		Batch.IndexOffset = FinalIndices.Num();
		
		FinalVertices.Append(&(*Batch.SourceVertices)[SourceVertexOffset], Batch.NumVertices);
		FinalIndices.Append(&(*Batch.SourceIndices)[SourceIndexOffset], Batch.NumIndices);
	}
}

void FSlateBatchData::CombineBatches(FSlateRenderBatch& FirstBatch, FSlateRenderBatch& SecondBatch, FSlateVertexArray& FinalVertices, FSlateIndexArray& FinalIndices)
{
	check(!SecondBatch.bIsMerged);
	if (FirstBatch.HasVertexData() || SecondBatch.HasVertexData())
	{
		// when merging verts we have to offset the indices in the second batch based on the first batches existing number of verts
		const int32 BatchOffset = FirstBatch.NumVertices;

		if (SecondBatch.HasVertexData())
		{
			// Final vertices is assumed to have the first batch already in it
			FirstBatch.NumVertices += SecondBatch.NumVertices;
			FirstBatch.NumIndices += SecondBatch.NumIndices;

			FinalVertices.Append(&(*SecondBatch.SourceVertices)[SecondBatch.VertexOffset], SecondBatch.NumVertices);

			FinalIndices.Reserve(FinalIndices.Num() + SecondBatch.NumIndices);

			// Get source indices at the source index offset and shift each index by the batches current offset
			for (int32 i = 0; i < SecondBatch.NumIndices; ++i)
			{
				const int32 FinalIndex = (*SecondBatch.SourceIndices)[i + SecondBatch.IndexOffset] + BatchOffset;
				FinalIndices.Add(FinalIndex);
			}
		}
	}

	SecondBatch.bIsMerged = true;
}


void FSlateBatchData::MergeRenderBatches()
{
	SCOPED_NAMED_EVENT_TEXT("Slate::MergeRenderBatches", FColor::Magenta);

	if(RenderBatches.Num())
	{
		TArray<TPair<int32, int32>, TInlineAllocator<100, FConcurrentLinearArrayAllocator>> BatchIndices;

		{
			SCOPED_NAMED_EVENT_TEXT("Slate::SortRenderBatches", FColor::Magenta);

			// Sort an index array instead of the render batches since they are large and not trivially relocatable 
			BatchIndices.AddUninitialized(RenderBatches.Num());
			for (int32 Index = 0; Index < RenderBatches.Num(); ++Index)
			{
				BatchIndices[Index].Key = Index;
				BatchIndices[Index].Value = RenderBatches[Index].GetLayer();
			}

			// Stable sort because order in the same layer should be preserved
			BatchIndices.StableSort
			(
				[](const TPair<int32, int32>& A, const TPair<int32, int32>& B)
				{
					return A.Value < B.Value;
				}
			);
		}

		NumBatches = 0;
		NumLayers = 0;

#if STATS
		int32 CurLayerId = INDEX_NONE;
		int32 PrevLayerId = INDEX_NONE;
#endif

		FirstRenderBatchIndex = BatchIndices[0].Key;

		FSlateRenderBatch* PrevBatch = nullptr;
		for (int32 BatchIndex = 0; BatchIndex < BatchIndices.Num(); ++BatchIndex)
		{
			const TPair<int32, int32>& BatchIndexPair = BatchIndices[BatchIndex];

			FSlateRenderBatch& CurBatch = RenderBatches[BatchIndexPair.Key];


			if (CurBatch.bIsMerged || !CurBatch.IsValidForRendering())
			{
				// skip already merged batches or batches with invalid data (e.g text with pure whitespace)
				continue;
			}

#if STATS
			CurLayerId = CurBatch.GetLayer();
			if (PrevLayerId != CurLayerId)
			{
				++NumLayers;
			}
			CurLayerId = PrevLayerId;
#endif

			if (PrevBatch != nullptr)
			{
				PrevBatch->NextBatchIndex = BatchIndexPair.Key;
			}

			++NumBatches;

			FillBuffersFromNewBatch(CurBatch, FinalVertexData, FinalIndexData);

			if (CurBatch.ClippingState)
			{
				bIsStencilBufferRequired |= CurBatch.ClippingState->GetClippingMethod() == EClippingMethod::Stencil;
			}

			if (const FSlateShaderResource* ShaderResource = CurBatch.GetShaderResource())
			{
				bRequiresVirtualTextureFeedback |= ShaderResource->RequiresVirtualTextureFeedback();
			}

#if 1  // Do batching at all?

			if (CurBatch.bIsMergable)
			{
				for (int32 TestIndex = BatchIndex + 1; TestIndex < BatchIndices.Num(); ++TestIndex)
				{
					const TPair<int32, int32>& NextBatchIndexPair = BatchIndices[TestIndex];
					FSlateRenderBatch& TestBatch = RenderBatches[NextBatchIndexPair.Key];
					if (TestBatch.GetLayer() != CurBatch.GetLayer())
					{
						// none of the batches will be compatible since we encountered an incompatible layer
						break;
					}
					else if (!TestBatch.bIsMerged && CurBatch.IsBatchableWith(TestBatch))
					{
						CombineBatches(CurBatch, TestBatch, FinalVertexData, FinalIndexData);

						check(TestBatch.NextBatchIndex == INDEX_NONE);

					}
				}
			}
#endif
			PrevBatch = &CurBatch;
		}

		MaxNumFinalVertices = FMath::Max(MaxNumFinalVertices, FinalVertexData.Num());
		MaxNumFinalIndices = FMath::Max(MaxNumFinalIndices, FinalIndexData.Num());
	}
}

FSlateElementBatcher::FSlateElementBatcher( TSharedRef<FSlateRenderingPolicy> InRenderingPolicy )
	: BatchData( nullptr )
	, BatchDataHDR(nullptr)
	, CurrentCachedElementList(nullptr)
	, PrecachedClippingStates(nullptr)
	, RenderingPolicy( &InRenderingPolicy.Get() )
	, NumPostProcessPasses(0)
	, PixelCenterOffset( InRenderingPolicy->GetPixelCenterOffset() )
	, bSRGBVertexColor( !InRenderingPolicy->IsVertexColorInLinearSpace() )
	, bRequiresVsync(false)
	, bCompositeHDRViewports(false)
	, UsedSlatePostBuffers(ESlatePostRT::None)
	, ResourceUpdatingPostBuffers(ESlatePostRT::None)
{
}

FSlateElementBatcher::~FSlateElementBatcher()
{
}

void FSlateElementBatcher::AddElements(FSlateWindowElementList& WindowElementList)
{
	SCOPED_NAMED_EVENT_TEXT("Slate::AddElements", FColor::Magenta);

	SCOPE_CYCLE_COUNTER(STAT_SlateAddElements);

#if STATS
	ElementStat_Other = 0;
	ElementStat_Boxes = 0;
	ElementStat_Borders = 0;
	ElementStat_Text = 0;
	ElementStat_ShapedText = 0;
	ElementStat_Line = 0;
	ElementStat_RecachedElements = 0;
#endif

	BatchData = &WindowElementList.GetBatchData();
	BatchDataHDR = &WindowElementList.GetBatchDataHDR();
	check(BatchData->GetRenderBatches().Num() == 0);
	check(BatchDataHDR->GetRenderBatches().Num() == 0);


	FVector2f ViewportSize = UE::Slate::CastToVector2f(WindowElementList.GetPaintWindow()->GetViewportSize());

	PrecachedClippingStates = &WindowElementList.ClippingManager.GetClippingStates();

	AddElementsInternal(WindowElementList.GetUncachedDrawElements(), ViewportSize);

	const TArrayView<FSlateCachedElementData* const> CachedElementDataList = WindowElementList.GetCachedElementDataList();


	if(CachedElementDataList.Num())
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddCachedElements", FColor::Magenta);

		for (FSlateCachedElementData* CachedElementData : CachedElementDataList)
		{
			if (CachedElementData)
			{
				AddCachedElements(*CachedElementData, ViewportSize);
			}
		}
	}

	WindowElementList.StartMergeRenderBatches();

	// Done with the element list
	BatchData = nullptr;
	BatchDataHDR = nullptr;
	PrecachedClippingStates = nullptr;

#if STATS
	const int32 ElementStat_All =
		ElementStat_Boxes +
		ElementStat_Borders +
		ElementStat_Text +
		ElementStat_ShapedText +
		ElementStat_Line +
		ElementStat_Other;

	INC_DWORD_STAT_BY(STAT_SlateElements, ElementStat_All);
	INC_DWORD_STAT_BY(STAT_SlateElements_Box, ElementStat_Boxes);
	INC_DWORD_STAT_BY(STAT_SlateElements_Border, ElementStat_Borders);
	INC_DWORD_STAT_BY(STAT_SlateElements_Text, ElementStat_Text);
	INC_DWORD_STAT_BY(STAT_SlateElements_ShapedText, ElementStat_ShapedText);
	INC_DWORD_STAT_BY(STAT_SlateElements_Line, ElementStat_Line);
	INC_DWORD_STAT_BY(STAT_SlateElements_Other, ElementStat_Other);
	INC_DWORD_STAT_BY(STAT_SlateInvalidation_RecachedElements, ElementStat_RecachedElements);
#endif
}

void FSlateElementBatcher::AddElementsInternal(const FSlateDrawElementMap& DrawElements, FVector2f ViewportSize)
{
	const FSlateDrawElementArray<FSlateBoxElement>& BoxElements = DrawElements.Get<(uint8)EElementType::ET_Box>();
	if (BoxElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddBoxElements", FColor::Magenta);
		STAT(ElementStat_Boxes += BoxElements.Num());
		AddBoxElements(BoxElements);
	}

	const FSlateDrawElementArray<FSlateRoundedBoxElement>& RoundedBoxElements = DrawElements.Get<(uint8)EElementType::ET_RoundedBox>();
	if (RoundedBoxElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddRoundedBoxElements", FColor::Magenta);
		STAT(ElementStat_Boxes += RoundedBoxElements.Num());
		AddBoxElements(RoundedBoxElements);
	}

	const FSlateDrawElementArray<FSlateBoxElement>& BorderElements = DrawElements.Get<(uint8)EElementType::ET_Border>();
	if (BorderElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddBorderElement", FColor::Magenta);
		STAT(ElementStat_Borders += BorderElements.Num());
		for (const FSlateBoxElement& DrawElement : BorderElements)
		{
			DrawElement.IsPixelSnapped() ? AddBorderElement<ESlateVertexRounding::Enabled>(DrawElement) : AddBorderElement<ESlateVertexRounding::Disabled>(DrawElement);
		}
	}

	const FSlateDrawElementArray<FSlateTextElement>& TextElements = DrawElements.Get<(uint8)EElementType::ET_Text>();
	if (TextElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddTextElement", FColor::Magenta);
		STAT(ElementStat_Text += TextElements.Num());
		for (const FSlateTextElement& DrawElement : TextElements)
		{
			DrawElement.IsPixelSnapped() ? AddTextElement<ESlateVertexRounding::Enabled>(DrawElement) : AddTextElement<ESlateVertexRounding::Disabled>(DrawElement);
		}
	}

	const FSlateDrawElementArray<FSlateShapedTextElement>& ShapedTextElements = DrawElements.Get<(uint8)EElementType::ET_ShapedText>();
	if (ShapedTextElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddShapedTextElement", FColor::Magenta);
		// ElementStat_ShapedText/Sdf incremented in AddShapedTextElement
		for (const FSlateShapedTextElement& DrawElement : ShapedTextElements)
		{
			DrawElement.IsPixelSnapped() ? AddShapedTextElement<ESlateVertexRounding::Enabled>(DrawElement) : AddShapedTextElement<ESlateVertexRounding::Disabled>(DrawElement);
		}
	}

	const FSlateDrawElementArray<FSlateLineElement>& LineElements = DrawElements.Get<(uint8)EElementType::ET_Line>();
	if (LineElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddLineElements", FColor::Magenta);
		STAT(ElementStat_Line += LineElements.Num());
		AddLineElements(LineElements);
	}

	const FSlateDrawElementArray<FSlateBoxElement>& DebugQuadElements = DrawElements.Get<(uint8)EElementType::ET_DebugQuad>();
	if (DebugQuadElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddDebugQuadElement", FColor::Magenta);
		STAT(ElementStat_Other += DebugQuadElements.Num());
		for (const FSlateBoxElement& DrawElement : DebugQuadElements)
		{
			DrawElement.IsPixelSnapped() ? AddDebugQuadElement<ESlateVertexRounding::Enabled>(DrawElement) : AddDebugQuadElement<ESlateVertexRounding::Disabled>(DrawElement);
		}
	}

	const FSlateDrawElementArray<FSlateSplineElement>& SplineElements = DrawElements.Get<(uint8)EElementType::ET_Spline>();
	if (SplineElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddSplineElement", FColor::Magenta);

		// Note that we ignore pixel snapping here; see implementation for more info.
		STAT(ElementStat_Other += SplineElements.Num());
		for (const FSlateSplineElement& DrawElement : SplineElements)
		{
			AddSplineElement(DrawElement);
		}
	}

	const FSlateDrawElementArray<FSlateGradientElement>& GradientElements = DrawElements.Get<(uint8)EElementType::ET_Gradient>();
	if (GradientElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddGradientElement", FColor::Magenta);
		STAT(ElementStat_Other += GradientElements.Num());
		for (const FSlateGradientElement& DrawElement : GradientElements)
		{
			DrawElement.IsPixelSnapped() ? AddGradientElement<ESlateVertexRounding::Enabled>(DrawElement) : AddGradientElement<ESlateVertexRounding::Disabled>(DrawElement);
		}
	}

	const FSlateDrawElementArray<FSlateViewportElement>& ViewportElements = DrawElements.Get<(uint8)EElementType::ET_Viewport>();
	if (ViewportElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddViewportElement", FColor::Magenta);
		STAT(ElementStat_Other += ViewportElements.Num());
		for (const FSlateViewportElement& DrawElement : ViewportElements)
		{
			DrawElement.IsPixelSnapped() ? AddViewportElement<ESlateVertexRounding::Enabled>(DrawElement) : AddViewportElement<ESlateVertexRounding::Disabled>(DrawElement);
		}
	}

	const FSlateDrawElementArray<FSlateCustomDrawerElement>& CustomElements = DrawElements.Get<(uint8)EElementType::ET_Custom>();
	if (CustomElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddCustomElement", FColor::Magenta);
		STAT(ElementStat_Other += CustomElements.Num());
		for (const FSlateCustomDrawerElement& DrawElement : CustomElements)
		{
			AddCustomElement(DrawElement);
		}
	}

	const FSlateDrawElementArray<FSlateCustomVertsElement>& CustomVertElements = DrawElements.Get<(uint8)EElementType::ET_CustomVerts>();
	if (CustomVertElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddCustomVertsElement", FColor::Magenta);
		STAT(ElementStat_Other += CustomVertElements.Num());
		for (const FSlateCustomVertsElement& DrawElement : CustomVertElements)
		{
			AddCustomVerts(DrawElement);
		}
	}

	const FSlateDrawElementArray<FSlatePostProcessElement>& PostProcessElements = DrawElements.Get<(uint8)EElementType::ET_PostProcessPass>();
	if (PostProcessElements.Num() > 0)
	{
		SCOPED_NAMED_EVENT_TEXT("Slate::AddPostProcessElement", FColor::Magenta);
		STAT(ElementStat_Other += PostProcessElements.Num());
		for (const FSlatePostProcessElement& DrawElement : PostProcessElements)
		{
			AddPostProcessPass(DrawElement, ViewportSize);
		}
	}

	{
		const FSlateDrawElementArray<FSlateDrawElement>& Container = DrawElements.Get<(uint8)EElementType::ET_NonMapped>();
		ensure(Container.IsEmpty());
	}
}

void FSlateElementBatcher::AddCachedElements(FSlateCachedElementData& CachedElementData, FVector2f ViewportSize)
{
	SCOPED_NAMED_EVENT_TEXT("Slate::AddCachedElements", FColor::Magenta);

#if SLATE_CSV_TRACKER
	FCsvProfiler::RecordCustomStat("Paint/CacheListsWithNewData", CSV_CATEGORY_INDEX(Slate), CachedElementData.ListsWithNewData.Num(), ECsvCustomStatOp::Set);
	int32 RecachedDrawElements = 0;
	int32 RecachedEmptyDrawLists = 0;
#endif

	for (FSlateCachedElementList* List : CachedElementData.ListsWithNewData)
	{
		int32 NumElements = List->NumElements();
		if (NumElements > 0)
		{
			STAT(ElementStat_RecachedElements += NumElements);

#if SLATE_CSV_TRACKER
			RecachedDrawElements += NumElements;
#endif

			CurrentCachedElementList = List;
			{
				SCOPE_CYCLE_SWIDGET(List->OwningWidget);
				AddElementsInternal(List->DrawElements, ViewportSize);
			}
			CurrentCachedElementList = nullptr;
		}
#if SLATE_CSV_TRACKER
		else
		{
			RecachedEmptyDrawLists++;
		}
#endif
	}
	CachedElementData.ListsWithNewData.Empty();

	// Add the existing and new cached batches.
	const TSparseArray<FSlateRenderBatch>& CachedBatches = CachedElementData.GetCachedBatches();
	TArray<FSlateRenderBatch>& RenderBatchesSDR = BatchData->RenderBatches;
	TArray<FSlateRenderBatch>& RenderBatchesHDR = BatchDataHDR->RenderBatches;

	ESlatePostRT CachedPostBuffers = ESlatePostRT::None;
	auto AddBatch = [&](const FSlateRenderBatch& CachedBatch)
	{
		if (EnumHasAnyFlags(CachedBatch.GetDrawFlags(), ESlateBatchDrawFlag::HDR))
		{
			RenderBatchesHDR.Add(CachedBatch);
		}
		else
		{
			RenderBatchesSDR.Add(CachedBatch);
		}
		
		CachedPostBuffers |= CachedBatch.CachedUsedPostBuffers;
	};

	RenderBatchesSDR.Reserve(RenderBatchesSDR.Num() + CachedBatches.Num());
	RenderBatchesHDR.Reserve(RenderBatchesHDR.Num() + CachedBatches.Num());

	for (const FSlateRenderBatch& CachedBatch : CachedBatches)
	{
		AddBatch(CachedBatch);
	}

	UsedSlatePostBuffers |= CachedPostBuffers;

	CachedElementData.CleanupUnusedClipStates();

#if SLATE_CSV_TRACKER
	FCsvProfiler::RecordCustomStat("Paint/RecachedElements", CSV_CATEGORY_INDEX(Slate), RecachedDrawElements, ECsvCustomStatOp::Accumulate);
	FCsvProfiler::RecordCustomStat("Paint/RecachedEmptyDrawLists", CSV_CATEGORY_INDEX(Slate), RecachedEmptyDrawLists, ECsvCustomStatOp::Accumulate);
#endif
}

template<typename ElementType, typename ElementAdder, typename ElementBatchParamCreator, typename ElementBatchReserver>
void FSlateElementBatcher::GenerateIndexedVertexBatches(
	const FSlateDrawElementArray<ElementType>& DrawElements
	, ElementAdder&& InElementAdder
	, ElementBatchParamCreator&& InElementBatchParamCreator
	, ElementBatchReserver&& InElementBatchReserver)
{
	for (int32 Index = 0; Index < DrawElements.Num();)
	{
		// Index is set to end of last processed batch at end of loop, may not be valid
		if (!DrawElements.IsValidIndex(Index))
		{
			return;
		}

		// Determine batch range
		FSlateRenderBatchParams NewBatchParams;
		InElementBatchParamCreator(DrawElements[Index], NewBatchParams);

		int32 BatchIndexEnd = Index;
		while (DrawElements.IsValidIndex(++BatchIndexEnd))
		{
			const ElementType& NextDrawElement = DrawElements[BatchIndexEnd];
			FSlateRenderBatchParams NextBatchParams;
			InElementBatchParamCreator(NextDrawElement, NextBatchParams);

			if (!NextBatchParams.IsBatchableWith(NewBatchParams))
			{
				break;
			}
		}

		// Process valid range, we always create the batch - even if it may not have any elements later on (Ex: 1 point lines).
		FSlateRenderBatch& RenderBatch = CreateRenderBatch(
			NewBatchParams.Layer
			, NewBatchParams.ShaderParams
			, NewBatchParams.Resource
			, NewBatchParams.PrimitiveType
			, NewBatchParams.ShaderType
			, NewBatchParams.DrawEffects
			, NewBatchParams.DrawFlags
			, NewBatchParams.SceneIndex
			, NewBatchParams.ClippingState);
		
		InElementBatchReserver(RenderBatch, Index, BatchIndexEnd);

		for (int32 BatchIndex = Index; BatchIndex < BatchIndexEnd; BatchIndex++)
		{
			InElementAdder(RenderBatch, DrawElements[BatchIndex]);
		}

		Index = BatchIndexEnd;
	}
}

template<ESlateVertexRounding Rounding>
void FSlateElementBatcher::AddDebugQuadElement(const FSlateBoxElement& DrawElement)
{
	const FColor Tint = PackVertexColor(DrawElement.GetTint());
	const FSlateRenderTransform& RenderTransform = DrawElement.GetRenderTransform();
	const FVector2f LocalSize = DrawElement.GetLocalSize();
	const ESlateDrawEffect InDrawEffects = DrawElement.GetDrawEffects();
	const int32 Layer = DrawElement.GetLayer();

	FSlateRenderBatch& RenderBatch = CreateRenderBatch(Layer, FShaderParams(), nullptr, ESlateDrawPrimitive::TriangleList, ESlateShader::Default, InDrawEffects, ESlateBatchDrawFlag::None, DrawElement);
	
	// Determine the four corners of the quad
	FVector2f TopLeft = FVector2f::ZeroVector;
	FVector2f TopRight = FVector2f(LocalSize.X, 0);
	FVector2f BotLeft = FVector2f(0.f, LocalSize.Y);
	FVector2f BotRight = FVector2f(LocalSize.X, LocalSize.Y);

	// Add four vertices to the list of verts to be added to the vertex buffer
	RenderBatch.AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(TopLeft), FVector2f(0.0f,0.0f),  Tint));
	RenderBatch.AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(TopRight), FVector2f(1.0f,0.0f), Tint));
	RenderBatch.AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(BotLeft), FVector2f(0.0f,1.0f),  Tint));
	RenderBatch.AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(BotRight), FVector2f(1.0f,1.0f), Tint));

	// Add 6 indices to the vertex buffer.  (2 tri's per quad, 3 indices per tri)
	RenderBatch.AddIndex(0);
	RenderBatch.AddIndex(1);
	RenderBatch.AddIndex(2);

	RenderBatch.AddIndex(2);
	RenderBatch.AddIndex(1);
	RenderBatch.AddIndex(3);
}

FORCEINLINE void IndexQuad(FSlateRenderBatch& RenderBatch, int32 TopLeft, int32 TopRight, int32 BottomRight, int32 BottomLeft)
{
	RenderBatch.AddIndex(TopLeft);
	RenderBatch.AddIndex(TopRight);
	RenderBatch.AddIndex(BottomRight);

	RenderBatch.AddIndex(BottomRight);
	RenderBatch.AddIndex(BottomLeft);
	RenderBatch.AddIndex(TopLeft);
}

template<typename ElementType>
void FSlateElementBatcher::AddBoxElements(const FSlateDrawElementArray<ElementType>& DrawElements)
{
	auto AddBoxElementInternal = [&](FSlateRenderBatch& RenderBatch, const ElementType& DrawElement)
	{
		const ESlateVertexRounding Rounding = DrawElement.IsPixelSnapped() ? ESlateVertexRounding::Enabled : ESlateVertexRounding::Disabled;

		const FColor Tint = PackVertexColor(DrawElement.GetTint());
		const FSlateRenderTransform& ElementRenderTransform = DrawElement.GetRenderTransform();
		const FSlateRenderTransform RenderTransform = DrawElement.GetRenderTransform();// GetBoxRenderTransform(DrawElement);
		const FVector2f LocalSize = DrawElement.GetLocalSize();

		const ESlateDrawEffect InDrawEffects = DrawElement.GetDrawEffects();
		const int32 Layer = DrawElement.GetLayer();

		const float DrawScale = DrawElement.GetScale();

		// Do pixel snapping
		FVector2f TopLeft(0.f, 0.f);
		FVector2f BotRight(LocalSize);

		uint32 TextureWidth = 1;
		uint32 TextureHeight = 1;

		// Get the default start and end UV.  If the texture is atlased this value will be a subset of this
		FVector2f StartUV = FVector2f(0.0f, 0.0f);
		FVector2f EndUV = FVector2f(1.0f, 1.0f);
		FVector2f SizeUV;

		FVector2f HalfTexel;

		const FSlateShaderResourceProxy* ResourceProxy = DrawElement.GetResourceProxy();
		FSlateShaderResource* Resource = nullptr;
		if (ResourceProxy)
		{
			// The actual texture for rendering.  If the texture is atlased this is the atlas
			Resource = ResourceProxy->Resource;
		}
		if(Resource != nullptr && !Resource->Debug_IsDestroyed())
		{
			// The width and height of the texture (non-atlased size)
			TextureWidth = ResourceProxy->ActualSize.X != 0 ? ResourceProxy->ActualSize.X : 1;
			TextureHeight = ResourceProxy->ActualSize.Y != 0 ? ResourceProxy->ActualSize.Y : 1;

			// Texel offset
			HalfTexel = FVector2f(PixelCenterOffset / TextureWidth, PixelCenterOffset / TextureHeight);

			const FBox2f& BrushUV = DrawElement.GetBrushUVRegion();
			//In case brush has valid UV region - use it instead of proxy UV
			if (BrushUV.bIsValid)
			{
				SizeUV = BrushUV.GetSize();
				StartUV = BrushUV.Min + HalfTexel;
				EndUV = StartUV + SizeUV;
			}
			else
			{
				SizeUV = FVector2f(ResourceProxy->SizeUV);
				StartUV = FVector2f(ResourceProxy->StartUV) + HalfTexel;
				EndUV = StartUV + FVector2f(ResourceProxy->SizeUV);
			}
		}
		else
		{
			// no texture
			SizeUV = FVector2f(1.0f, 1.0f);
			HalfTexel = FVector2f(PixelCenterOffset, PixelCenterOffset);
		}


		const ESlateBrushTileType::Type TilingRule = DrawElement.GetBrushTiling();
		const bool bTileHorizontal = (TilingRule == ESlateBrushTileType::Both || TilingRule == ESlateBrushTileType::Horizontal);
		const bool bTileVertical = (TilingRule == ESlateBrushTileType::Both || TilingRule == ESlateBrushTileType::Vertical);

		const ESlateBrushMirrorType::Type MirroringRule = DrawElement.GetBrushMirroring();
		const bool bMirrorHorizontal = (MirroringRule == ESlateBrushMirrorType::Both || MirroringRule == ESlateBrushMirrorType::Horizontal);
		const bool bMirrorVertical = (MirroringRule == ESlateBrushMirrorType::Both || MirroringRule == ESlateBrushMirrorType::Vertical);

		// Pass the tiling information as a flag so we can pick the correct texture addressing mode
		ESlateBatchDrawFlag DrawFlags = DrawElement.GetBatchFlags();
		DrawFlags |= ((bTileHorizontal ? ESlateBatchDrawFlag::TileU : ESlateBatchDrawFlag::None) | (bTileVertical ? ESlateBatchDrawFlag::TileV : ESlateBatchDrawFlag::None));

		// Add Shader Parameters for extra RoundedBox parameters
		ESlateShader ShaderType = ESlateShader::Default;
		FShaderParams ShaderParams;
		FColor SecondaryColor = FColor::Black;
		if constexpr (std::is_same<ElementType, FSlateRoundedBoxElement>::value)
		{
			ShaderType = ESlateShader::RoundedBox;

			ShaderParams.PixelParams = FVector4f(0, DrawElement.GetOutlineWeight(), LocalSize.X, LocalSize.Y);//RadiusWeight;
			ShaderParams.PixelParams2 = DrawElement.GetRadius();

			SecondaryColor = PackVertexColor(DrawElement.OutlineColor);
		}

		float HorizontalTiling = bTileHorizontal ? LocalSize.X / TextureWidth : 1.0f;
		float VerticalTiling = bTileVertical ? LocalSize.Y / TextureHeight : 1.0f;

		const FVector2f Tiling(HorizontalTiling, VerticalTiling);

		// The start index of these vertices in the index buffer
		const uint32 IndexStart = RenderBatch.GetNumVertices();

		const FMargin& Margin = DrawElement.GetBrushMargin();

		const FVector2f TopRight = FVector2f(BotRight.X, TopLeft.Y);
		const FVector2f BotLeft = FVector2f(TopLeft.X, BotRight.Y);

		const FColor FeatherColor(0, 0, 0, 0);

		if (DrawElement.GetBrushDrawType() != ESlateBrushDrawType::Image &&
			(Margin.Left != 0.0f || Margin.Top != 0.0f || Margin.Right != 0.0f || Margin.Bottom != 0.0f))
		{
			// Create 9 quads for the box element based on the following diagram
			//     ___LeftMargin    ___RightMargin
			//    /                /
			//  +--+-------------+--+
			//  |  |c1           |c2| ___TopMargin
			//  +--o-------------o--+
			//  |  |             |  |
			//  |  |c3           |c4|
			//  +--o-------------o--+
			//  |  |             |  | ___BottomMargin
			//  +--+-------------+--+


			// Determine the texture coordinates for each quad
			// These are not scaled.
			float LeftMarginU = (Margin.Left > 0.0f)
				? StartUV.X + Margin.Left * SizeUV.X + HalfTexel.X
				: StartUV.X;
			float TopMarginV = (Margin.Top > 0.0f)
				? StartUV.Y + Margin.Top * SizeUV.Y + HalfTexel.Y
				: StartUV.Y;
			float RightMarginU = (Margin.Right > 0.0f)
				? EndUV.X - Margin.Right * SizeUV.X + HalfTexel.X
				: EndUV.X;
			float BottomMarginV = (Margin.Bottom > 0.0f)
				? EndUV.Y - Margin.Bottom * SizeUV.Y + HalfTexel.Y
				: EndUV.Y;

			if (bMirrorHorizontal || bMirrorVertical)
			{
				const FVector2f UVMin = StartUV;
				const FVector2f UVMax = EndUV;

				if (bMirrorHorizontal)
				{
					StartUV.X = UVMax.X - (StartUV.X - UVMin.X);
					EndUV.X = UVMax.X - (EndUV.X - UVMin.X);
					LeftMarginU = UVMax.X - (LeftMarginU - UVMin.X);
					RightMarginU = UVMax.X - (RightMarginU - UVMin.X);
				}
				if (bMirrorVertical)
				{
					StartUV.Y = UVMax.Y - (StartUV.Y - UVMin.Y);
					EndUV.Y = UVMax.Y - (EndUV.Y - UVMin.Y);
					TopMarginV = UVMax.Y - (TopMarginV - UVMin.Y);
					BottomMarginV = UVMax.Y - (BottomMarginV - UVMin.Y);
				}
			}

			// Determine the margins for each quad

			float LeftMarginX = TextureWidth * Margin.Left;
			float TopMarginY = TextureHeight * Margin.Top;
			float RightMarginX = LocalSize.X - TextureWidth * Margin.Right;
			float BottomMarginY = LocalSize.Y - TextureHeight * Margin.Bottom;

			// If the margins are overlapping the margins are too big or the button is too small
			// so clamp margins to half of the box size
			if (RightMarginX < LeftMarginX)
			{
				LeftMarginX = LocalSize.X / 2;
				RightMarginX = LeftMarginX;
			}

			if (BottomMarginY < TopMarginY)
			{
				TopMarginY = LocalSize.Y / 2;
				BottomMarginY = TopMarginY;
			}

			FVector2f Position = TopLeft;
			FVector2f EndPos = BotRight;

			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position.X, Position.Y), LocalSize, DrawScale, FVector4f(FVector2f(StartUV), Tiling), Tint, SecondaryColor, Rounding)); //0
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position.X, TopMarginY), LocalSize, DrawScale, FVector4f(FVector2f(StartUV.X, TopMarginV), Tiling), Tint, SecondaryColor, Rounding)); //1
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(LeftMarginX, Position.Y), LocalSize, DrawScale, FVector4f(FVector2f(LeftMarginU, StartUV.Y), Tiling), Tint, SecondaryColor, Rounding)); //2
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(LeftMarginX, TopMarginY), LocalSize, DrawScale, FVector4f(FVector2f(LeftMarginU, TopMarginV), Tiling), Tint, SecondaryColor, Rounding)); //3
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(RightMarginX, Position.Y), LocalSize, DrawScale, FVector4f(FVector2f(RightMarginU, StartUV.Y), Tiling), Tint, SecondaryColor, Rounding)); //4
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(RightMarginX, TopMarginY), LocalSize, DrawScale, FVector4f(FVector2f(RightMarginU, TopMarginV), Tiling), Tint, SecondaryColor, Rounding)); //5
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(EndPos.X, Position.Y), LocalSize, DrawScale, FVector4f(FVector2f(EndUV.X, StartUV.Y), Tiling), Tint, SecondaryColor, Rounding)); //6
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(EndPos.X, TopMarginY), LocalSize, DrawScale, FVector4f(FVector2f(EndUV.X, TopMarginV), Tiling), Tint, SecondaryColor, Rounding)); //7

			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position.X, BottomMarginY), LocalSize, DrawScale, FVector4f(FVector2f(StartUV.X, BottomMarginV), Tiling), Tint, SecondaryColor, Rounding)); //8
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(LeftMarginX, BottomMarginY), LocalSize, DrawScale, FVector4f(FVector2f(LeftMarginU, BottomMarginV), Tiling), Tint, SecondaryColor, Rounding)); //9
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(RightMarginX, BottomMarginY), LocalSize, DrawScale, FVector4f(FVector2f(RightMarginU, BottomMarginV), Tiling), Tint, SecondaryColor, Rounding)); //10
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(EndPos.X, BottomMarginY), LocalSize, DrawScale, FVector4f(FVector2f(EndUV.X, BottomMarginV), Tiling), Tint, SecondaryColor, Rounding)); //11
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position.X, EndPos.Y), LocalSize, DrawScale, FVector4f(FVector2f(StartUV.X, EndUV.Y), Tiling), Tint, SecondaryColor, Rounding)); //12
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(LeftMarginX, EndPos.Y), LocalSize, DrawScale, FVector4f(FVector2f(LeftMarginU, EndUV.Y), Tiling), Tint, SecondaryColor, Rounding)); //13
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(RightMarginX, EndPos.Y), LocalSize, DrawScale, FVector4f(FVector2f(RightMarginU, EndUV.Y), Tiling), Tint, SecondaryColor, Rounding)); //14
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(EndPos.X, EndPos.Y), LocalSize, DrawScale, FVector4f(FVector2f(EndUV), Tiling), Tint, SecondaryColor, Rounding)); //15

			RenderBatch.EmplaceIndex(IndexStart + 0);
			RenderBatch.EmplaceIndex(IndexStart + 1);
			RenderBatch.EmplaceIndex(IndexStart + 2);
			RenderBatch.EmplaceIndex(IndexStart + 2);
			RenderBatch.EmplaceIndex(IndexStart + 1);
			RenderBatch.EmplaceIndex(IndexStart + 3);

			RenderBatch.EmplaceIndex(IndexStart + 2);
			RenderBatch.EmplaceIndex(IndexStart + 3);
			RenderBatch.EmplaceIndex(IndexStart + 4);
			RenderBatch.EmplaceIndex(IndexStart + 4);
			RenderBatch.EmplaceIndex(IndexStart + 3);
			RenderBatch.EmplaceIndex(IndexStart + 5);

			RenderBatch.EmplaceIndex(IndexStart + 4);
			RenderBatch.EmplaceIndex(IndexStart + 5);
			RenderBatch.EmplaceIndex(IndexStart + 6);
			RenderBatch.EmplaceIndex(IndexStart + 6);
			RenderBatch.EmplaceIndex(IndexStart + 5);
			RenderBatch.EmplaceIndex(IndexStart + 7);

			// Middle
			RenderBatch.EmplaceIndex(IndexStart + 1);
			RenderBatch.EmplaceIndex(IndexStart + 8);
			RenderBatch.EmplaceIndex(IndexStart + 3);
			RenderBatch.EmplaceIndex(IndexStart + 3);
			RenderBatch.EmplaceIndex(IndexStart + 8);
			RenderBatch.EmplaceIndex(IndexStart + 9);

			RenderBatch.EmplaceIndex(IndexStart + 3);
			RenderBatch.EmplaceIndex(IndexStart + 9);
			RenderBatch.EmplaceIndex(IndexStart + 5);
			RenderBatch.EmplaceIndex(IndexStart + 5);
			RenderBatch.EmplaceIndex(IndexStart + 9);
			RenderBatch.EmplaceIndex(IndexStart + 10);

			RenderBatch.EmplaceIndex(IndexStart + 5);
			RenderBatch.EmplaceIndex(IndexStart + 10);
			RenderBatch.EmplaceIndex(IndexStart + 7);
			RenderBatch.EmplaceIndex(IndexStart + 7);
			RenderBatch.EmplaceIndex(IndexStart + 10);
			RenderBatch.EmplaceIndex(IndexStart + 11);

			// Bottom
			RenderBatch.EmplaceIndex(IndexStart + 8);
			RenderBatch.EmplaceIndex(IndexStart + 12);
			RenderBatch.EmplaceIndex(IndexStart + 9);
			RenderBatch.EmplaceIndex(IndexStart + 9);
			RenderBatch.EmplaceIndex(IndexStart + 12);
			RenderBatch.EmplaceIndex(IndexStart + 13);

			RenderBatch.EmplaceIndex(IndexStart + 9);
			RenderBatch.EmplaceIndex(IndexStart + 13);
			RenderBatch.EmplaceIndex(IndexStart + 10);
			RenderBatch.EmplaceIndex(IndexStart + 10);
			RenderBatch.EmplaceIndex(IndexStart + 13);
			RenderBatch.EmplaceIndex(IndexStart + 14);

			RenderBatch.EmplaceIndex(IndexStart + 10);
			RenderBatch.EmplaceIndex(IndexStart + 14);
			RenderBatch.EmplaceIndex(IndexStart + 11);
			RenderBatch.EmplaceIndex(IndexStart + 11);
			RenderBatch.EmplaceIndex(IndexStart + 14);
			RenderBatch.EmplaceIndex(IndexStart + 15);

			if (GSlateFeathering && Rounding == ESlateVertexRounding::Disabled)
			{
				const int32 FeatherStart = RenderBatch.GetNumVertices();

				// Top
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position.X, Position.Y) + FVector2f(-1.f, -1.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(StartUV), Tiling), FeatherColor, SecondaryColor, Rounding)); //0
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(LeftMarginX, Position.Y) + FVector2f(0.f, -1.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(LeftMarginU, StartUV.Y), Tiling), FeatherColor, SecondaryColor, Rounding)); //1
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(RightMarginX, Position.Y) + FVector2f(0.f, -1.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(RightMarginU, StartUV.Y), Tiling), FeatherColor, SecondaryColor, Rounding)); //2
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(EndPos.X, Position.Y) + FVector2f(1.f, -1.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(EndUV.X, StartUV.Y), Tiling), FeatherColor, SecondaryColor, Rounding)); //3

				// Left
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position.X, TopMarginY) + FVector2f(-1.f, 0.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(StartUV.X, TopMarginV), Tiling), FeatherColor, SecondaryColor, Rounding)); //4
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position.X, BottomMarginY) + FVector2f(-1.f, 0.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(StartUV.X, BottomMarginV), Tiling), FeatherColor, SecondaryColor, Rounding)); //5

				// Right
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(EndPos.X, TopMarginY) + FVector2f(1.f, 0.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(EndUV.X, TopMarginV), Tiling), FeatherColor, SecondaryColor, Rounding)); //6
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(EndPos.X, BottomMarginY) + FVector2f(1.f, 0.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(EndUV.X, BottomMarginV), Tiling), FeatherColor, SecondaryColor, Rounding)); //7

				// Bottom
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position.X, EndPos.Y) + FVector2f(-1.f, 1.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(StartUV.X, EndUV.Y), Tiling), FeatherColor, SecondaryColor, Rounding)); //8
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(LeftMarginX, EndPos.Y) + FVector2f(0.f, 1.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(LeftMarginU, EndUV.Y), Tiling), FeatherColor, SecondaryColor, Rounding)); //9
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(RightMarginX, EndPos.Y) + FVector2f(0.f, 1.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(RightMarginU, EndUV.Y), Tiling), FeatherColor, SecondaryColor, Rounding)); //10
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(EndPos.X, EndPos.Y) + FVector2f(1.f, 1.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(EndUV), Tiling), FeatherColor, SecondaryColor, Rounding)); //11

				auto IndexQuadArray = [](FSlateRenderBatch& RenderBatch, int32 TopLeft, int32 TopRight, int32 BottomRight, int32 BottomLeft)
				{
					RenderBatch.EmplaceIndex(TopLeft);
					RenderBatch.EmplaceIndex(TopRight);
					RenderBatch.EmplaceIndex(BottomRight);

					RenderBatch.EmplaceIndex(BottomRight);
					RenderBatch.EmplaceIndex(BottomLeft);
					RenderBatch.EmplaceIndex(TopLeft);
				};

				// Top Left
				IndexQuadArray(RenderBatch, FeatherStart + 0, FeatherStart + 1, IndexStart + 2, IndexStart + 0);
				// Top Middle
				IndexQuadArray(RenderBatch, FeatherStart + 1, FeatherStart + 2, IndexStart + 4, IndexStart + 2);
				// Top Right
				IndexQuadArray(RenderBatch, FeatherStart + 2, FeatherStart + 3, IndexStart + 6, IndexStart + 4);

				//-----------------------------------------------------------

				// Left Top
				IndexQuadArray(RenderBatch, FeatherStart + 0, IndexStart + 0, IndexStart + 1, FeatherStart + 4);
				// Left Middle
				IndexQuadArray(RenderBatch, FeatherStart + 4, IndexStart + 1, IndexStart + 8, FeatherStart + 5);
				// Left Bottom
				IndexQuadArray(RenderBatch, FeatherStart + 5, IndexStart + 8, IndexStart + 12, FeatherStart + 8);

				//-----------------------------------------------------------

				// Right Top
				IndexQuadArray(RenderBatch, IndexStart + 6, FeatherStart + 3, FeatherStart + 6, IndexStart + 7);
				// Right Middle
				IndexQuadArray(RenderBatch, IndexStart + 7, FeatherStart + 6, FeatherStart + 7, IndexStart + 11);
				// Right Bottom
				IndexQuadArray(RenderBatch, IndexStart + 11, FeatherStart + 7, FeatherStart + 11, IndexStart + 15);

				//-----------------------------------------------------------

				// Bottom Left
				IndexQuadArray(RenderBatch, IndexStart + 12, IndexStart + 13, FeatherStart + 9, FeatherStart + 8);
				// Bottom Middle
				IndexQuadArray(RenderBatch, IndexStart + 13, IndexStart + 14, FeatherStart + 10, FeatherStart + 9);
				// Bottom Right
				IndexQuadArray(RenderBatch, IndexStart + 14, IndexStart + 15, FeatherStart + 11, FeatherStart + 10);
			}
		}
		else
		{
			if (bMirrorHorizontal || bMirrorVertical)
			{
				const FVector2f UVMin = StartUV;
				const FVector2f UVMax = EndUV;

				if (bMirrorHorizontal)
				{
					StartUV.X = UVMax.X - (StartUV.X - UVMin.X);
					EndUV.X = UVMax.X - (EndUV.X - UVMin.X);
				}
				if (bMirrorVertical)
				{
					StartUV.Y = UVMax.Y - (StartUV.Y - UVMin.Y);
					EndUV.Y = UVMax.Y - (EndUV.Y - UVMin.Y);
				}
			}

			// Add four vertices to the list of verts to be added to the vertex buffer
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, TopLeft, LocalSize, DrawScale, FVector4f(FVector2f(StartUV), Tiling), Tint, SecondaryColor, Rounding));
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, TopRight, LocalSize, DrawScale, FVector4f(FVector2f(EndUV.X, StartUV.Y), Tiling), Tint, SecondaryColor, Rounding));
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, BotLeft, LocalSize, DrawScale, FVector4f(FVector2f(StartUV.X, EndUV.Y), Tiling), Tint, SecondaryColor, Rounding));
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, BotRight, LocalSize, DrawScale, FVector4f(FVector2f(EndUV), Tiling), Tint, SecondaryColor, Rounding));
	
			RenderBatch.EmplaceIndex(IndexStart + 0);
			RenderBatch.EmplaceIndex(IndexStart + 1);
			RenderBatch.EmplaceIndex(IndexStart + 2);

			RenderBatch.EmplaceIndex(IndexStart + 2);
			RenderBatch.EmplaceIndex(IndexStart + 1);
			RenderBatch.EmplaceIndex(IndexStart + 3);

			if (GSlateFeathering && Rounding == ESlateVertexRounding::Disabled)
			{
				const int32 FeatherStart = RenderBatch.GetNumVertices();

				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, TopLeft + FVector2f(-1.f, -1.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(StartUV), Tiling), FeatherColor, SecondaryColor, Rounding));
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, TopRight + FVector2f(1.f, -1.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(EndUV.X, StartUV.Y), Tiling), FeatherColor, SecondaryColor, Rounding));
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, BotLeft + FVector2f(-1.f, 1.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(StartUV.X, EndUV.Y), Tiling), FeatherColor, SecondaryColor, Rounding));
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, BotRight + FVector2f(1.f, 1.f) / DrawScale, LocalSize, DrawScale, FVector4f(FVector2f(EndUV), Tiling), FeatherColor, SecondaryColor, Rounding));

				const int32 TopLeftIndex = IndexStart + 0;
				const int32 TopRightIndex = IndexStart + 1;
				const int32 BottomLeftIndex = IndexStart + 2;
				const int32 BottomRightIndex = IndexStart + 3;

				// Top-Top
				RenderBatch.EmplaceIndex(FeatherStart + 0);
				RenderBatch.EmplaceIndex(FeatherStart + 1);
				RenderBatch.EmplaceIndex(TopRightIndex);

				// Top-Bottom
				RenderBatch.EmplaceIndex(FeatherStart + 0);
				RenderBatch.EmplaceIndex(TopRightIndex);
				RenderBatch.EmplaceIndex(TopLeftIndex);

				// Left-Top
				RenderBatch.EmplaceIndex(FeatherStart + 0);
				RenderBatch.EmplaceIndex(BottomLeftIndex);
				RenderBatch.EmplaceIndex(FeatherStart + 2);

				// Left-Bottom
				RenderBatch.EmplaceIndex(FeatherStart + 0);
				RenderBatch.EmplaceIndex(TopLeftIndex);
				RenderBatch.EmplaceIndex(BottomLeftIndex);

				// Right-Top
				RenderBatch.EmplaceIndex(TopRightIndex);
				RenderBatch.EmplaceIndex(FeatherStart + 1);
				RenderBatch.EmplaceIndex(FeatherStart + 3);

				// Right-Bottom
				RenderBatch.EmplaceIndex(TopRightIndex);
				RenderBatch.EmplaceIndex(FeatherStart + 3);
				RenderBatch.EmplaceIndex(BottomRightIndex);

				// Bottom-Top
				RenderBatch.EmplaceIndex(BottomLeftIndex);
				RenderBatch.EmplaceIndex(BottomRightIndex);
				RenderBatch.EmplaceIndex(FeatherStart + 3);

				// Bottom-Bottom
				RenderBatch.EmplaceIndex(FeatherStart + 3);
				RenderBatch.EmplaceIndex(FeatherStart + 2);
				RenderBatch.EmplaceIndex(BottomLeftIndex);
			}
		}
	};
	
	auto GenerateBoxElementBatchParams = [&](const ElementType& InDrawElement, FSlateRenderBatchParams& OutBatchParameters)
	{
		const FVector2f LocalSize = InDrawElement.GetLocalSize();
		const ESlateDrawEffect DrawEffects = InDrawElement.GetDrawEffects();

		// Shader type and params
		ESlateShader ShaderType = ESlateShader::Default;
		if constexpr (std::is_same<ElementType, FSlateRoundedBoxElement>::value)
		{
			ShaderType = ESlateShader::RoundedBox;
			
			OutBatchParameters.ShaderParams.PixelParams = FVector4f(0, InDrawElement.GetOutlineWeight(), LocalSize.X, LocalSize.Y);//RadiusWeight;
			OutBatchParameters.ShaderParams.PixelParams2 = InDrawElement.GetRadius();
		}

		// Shader Resource
		const FSlateShaderResourceProxy* ResourceProxy = InDrawElement.GetResourceProxy();
		FSlateShaderResource* Resource = nullptr;
		if (ResourceProxy)
		{
			// The actual texture for rendering.  If the texture is atlased this is the atlas
			Resource = ResourceProxy->Resource;
		}

		// Draw Flags
		const ESlateBrushTileType::Type TilingRule = InDrawElement.GetBrushTiling();
		const bool bTileHorizontal = (TilingRule == ESlateBrushTileType::Both || TilingRule == ESlateBrushTileType::Horizontal);
		const bool bTileVertical = (TilingRule == ESlateBrushTileType::Both || TilingRule == ESlateBrushTileType::Vertical);

		const ESlateBrushMirrorType::Type MirroringRule = InDrawElement.GetBrushMirroring();
		const bool bMirrorHorizontal = (MirroringRule == ESlateBrushMirrorType::Both || MirroringRule == ESlateBrushMirrorType::Horizontal);
		const bool bMirrorVertical = (MirroringRule == ESlateBrushMirrorType::Both || MirroringRule == ESlateBrushMirrorType::Vertical);

		// Pass the tiling information as a flag so we can pick the correct texture addressing mode
		ESlateBatchDrawFlag DrawFlags = InDrawElement.GetBatchFlags();
		DrawFlags |= ((bTileHorizontal ? ESlateBatchDrawFlag::TileU : ESlateBatchDrawFlag::None) | (bTileVertical ? ESlateBatchDrawFlag::TileV : ESlateBatchDrawFlag::None));

		OutBatchParameters.Layer = InDrawElement.GetLayer();
		OutBatchParameters.Resource = Resource;
		OutBatchParameters.PrimitiveType = ESlateDrawPrimitive::TriangleList;
		OutBatchParameters.ShaderType = ShaderType;
		OutBatchParameters.DrawEffects = DrawEffects;
		OutBatchParameters.DrawFlags = DrawFlags;
		OutBatchParameters.SceneIndex = InDrawElement.GetSceneIndex();
		OutBatchParameters.ClippingState = ResolveClippingState(InDrawElement);
	};

	auto ReserveBoxElementBatch = [&](FSlateRenderBatch& RenderBatch, uint32 InBatchStart, uint32 InBatchEnd)
	{
		uint32 NumVertexes = 0;
		uint32 NumIndices = 0;

		// For source of magic numbers, see internal method above
		for (uint32 Index = InBatchStart; Index < InBatchEnd; Index++)
		{
			const ElementType& DrawElement = DrawElements[Index];
			const ESlateVertexRounding Rounding = DrawElement.IsPixelSnapped() ? ESlateVertexRounding::Enabled : ESlateVertexRounding::Disabled;
			const FMargin& Margin = DrawElement.GetBrushMargin();

			if (DrawElement.GetBrushDrawType() != ESlateBrushDrawType::Image &&
				(Margin.Left != 0.0f || Margin.Top != 0.0f || Margin.Right != 0.0f || Margin.Bottom != 0.0f))
			{
				NumVertexes += 8 + 8;
				NumIndices += 6 * 3 * 3;

				if (GSlateFeathering && Rounding == ESlateVertexRounding::Disabled)
				{
					NumVertexes += 4 + 2 + 2 + 4;
					NumIndices += 6 * 3 * 4;
				}
			}
			else
			{
				NumVertexes += 4;
				NumIndices += 6;

				if (GSlateFeathering && Rounding == ESlateVertexRounding::Disabled)
				{
					NumVertexes += 4;
					NumIndices += 3 * 8;
				}
			}
		}

		RenderBatch.SourceVertices->Reserve(RenderBatch.SourceVertices->Num() + NumVertexes);
		RenderBatch.SourceIndices->Reserve(RenderBatch.SourceIndices->Num() + NumVertexes);
	};

	GenerateIndexedVertexBatches(DrawElements, AddBoxElementInternal, GenerateBoxElementBatchParams, ReserveBoxElementBatch);
}

namespace SlateElementBatcher
{
#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	const FName MaterialInterfaceClassName = "MaterialInterface";

	void CheckUObject(const FSlateTextElement& InDrawElement, const UObject* InFontMaterial)
	{
		if (InFontMaterial && GSlateCheckUObjectRenderResources)
		{
			bool bIsValidLowLevel = InFontMaterial->IsValidLowLevelFast(false);
			if (!bIsValidLowLevel || !IsValid(InFontMaterial) || InFontMaterial->GetClass()->GetFName() == MaterialInterfaceClassName)
			{
				UE_LOGF(LogSlate, Error, "We are rendering a string with an invalid font. The string is: '%ls'"
					, InDrawElement.GetText());
				// We expect to log more info in the SlateMaterialResource.
				//In case we crash before that, we also log some info here.
				UE_LOGF(LogSlate, Error, "Material is not valid. PendingKill:'%d'. ValidLowLevelFast:'%d'. InvalidClass:'%d'"
					, (bIsValidLowLevel ? !IsValid(InFontMaterial) : false)
					, bIsValidLowLevel
					, (bIsValidLowLevel ? InFontMaterial->GetClass()->GetFName() == MaterialInterfaceClassName : false));
			}
		}
	}
#endif
}

template<ESlateVertexRounding Rounding>
void FSlateElementBatcher::AddTextElement(const FSlateTextElement& DrawElement)
{
	FColor BaseTint = PackVertexColor(DrawElement.GetTint());

	const FFontOutlineSettings& OutlineSettings = DrawElement.GetFontInfo().OutlineSettings;

	int32 Len = DrawElement.GetTextLength();
	ensure(Len > 0);

	ESlateDrawEffect InDrawEffects = DrawElement.GetDrawEffects();

	const int32 Layer = DrawElement.GetLayer();

	// extract the layout transform from the draw element
	FSlateLayoutTransform LayoutTransform(DrawElement.GetScale(), FVector2f(DrawElement.GetPosition()));

	// We don't just scale up fonts, we draw them in local space pre-scaled so we don't get scaling artifacts.
	// So we need to pull the layout scale out of the layout and render transform so we can apply them
	// in local space with pre-scaled fonts.
	const float FontScale = LayoutTransform.GetScale();
	FSlateLayoutTransform InverseLayoutTransform = Inverse(Concatenate(Inverse(FontScale), LayoutTransform));
	const FSlateRenderTransform RenderTransform = Concatenate(Inverse(FontScale), DrawElement.GetRenderTransform());

	FSlateFontCache& FontCache = *RenderingPolicy->GetFontCache();
	FSlateShaderResourceManager& ResourceManager = *RenderingPolicy->GetResourceManager();

	auto BuildFontGeometry = [&](const FFontOutlineSettings& InOutlineSettings, const FColor& InTint, const UObject* FontMaterial, int32 InLayer, float InOutlineHorizontalOffset)
	{
		FCharacterList& CharacterList = FontCache.GetCharacterList(DrawElement.GetFontInfo(), FontScale, InOutlineSettings);

		float MaxHeight = CharacterList.GetMaxHeight();

		if (MaxHeight == 0)
		{
			// If the max text height is 0, we'll create NaN's further in the code, so avoid drawing text if this happens.
			return;
		}

		uint32 FontTextureIndex = 0;
		FSlateShaderResource* FontAtlasTexture = nullptr;
		FSlateShaderResource* FontShaderResource = nullptr;
		FColor FontTint = InTint;

		FSlateRenderBatch* RenderBatch = nullptr;
		FSlateVertexArray* BatchVertices = nullptr;
		FSlateIndexArray* BatchIndices = nullptr;

		uint32 VertexOffset = 0;
		uint32 IndexOffset = 0;

		float InvTextureSizeX = 0;
		float InvTextureSizeY = 0;

		float LineX = 0;

		FCharacterEntry PreviousCharEntry;

		int32 Kerning = 0;

		FVector2f TopLeft(0.f, 0.f);

		const float PosX = TopLeft.X;
		float PosY = TopLeft.Y;

		LineX = PosX;

		const bool bIsFontMaterial = FontMaterial != nullptr;
		const bool bEnableOutline = InOutlineSettings.OutlineSize > 0;

		uint32 NumChars = Len;

		// The number of glyphs can technically be unbound, so pick some number that if smaller we use the preallocated index array
		const int32 MaxPreAllocatedGlyphIndicies = 4096 + 1024;
		const bool bUseStaticIndicies = NumChars < MaxPreAllocatedGlyphIndicies;

		uint32 NumLines = 1;
		for( uint32 CharIndex = 0; CharIndex < NumChars; ++CharIndex )
		{
			const TCHAR CurrentChar = DrawElement.GetText()[ CharIndex ];

			ensure(CurrentChar != '\0');

			const bool IsNewline = (CurrentChar == '\n');

			if (IsNewline)
			{
				// Move down: we are drawing the next line.
				PosY += MaxHeight;
				// Carriage return 
				LineX = PosX;

				++NumLines;

			}
			else
			{
				const FCharacterEntry& Entry = CharacterList.GetCharacter(CurrentChar, DrawElement.GetFontInfo().FontFallback);

				if( Entry.Valid && (FontAtlasTexture == nullptr || Entry.TextureIndex != FontTextureIndex) )
				{
					// Font has a new texture for this glyph. Refresh the batch we use and the index we are currently using
					FontTextureIndex = Entry.TextureIndex;

					ISlateFontTexture* SlateFontTexture = FontCache.GetFontTexture(FontTextureIndex);
					check(SlateFontTexture);

					FontAtlasTexture = SlateFontTexture->GetSlateTexture();
					check(FontAtlasTexture);

					FontShaderResource = ResourceManager.GetFontShaderResource( FontTextureIndex, FontAtlasTexture, DrawElement.GetFontInfo().FontMaterial );
					check(FontShaderResource);

					const ESlateFontAtlasContentType ContentType = SlateFontTexture->GetContentType();
					FontTint = ContentType == ESlateFontAtlasContentType::Color ? FColor::White : InTint;

					ESlateShader ShaderType = ESlateShader::Default;
					switch (ContentType)
					{
						case ESlateFontAtlasContentType::Alpha:
							ShaderType = ESlateShader::GrayscaleFont;
							break;
						case ESlateFontAtlasContentType::Color:
							ShaderType = ESlateShader::ColorFont;
							break;
						case ESlateFontAtlasContentType::Msdf:
							// SDF / MSDF not supported here because we don't have SDF attributes available
							checkNoEntry();
							// Still use SDF shader to prevent raw MSDF texture from showing
							ShaderType = !bEnableOutline || InOutlineSettings.bMiteredCorners ? ESlateShader::MsdfFont : ESlateShader::SdfFont;
							break;
						default:
							checkNoEntry();
							// Default to Color
							ShaderType = ESlateShader::ColorFont;
							break;
					}
					check(ShaderType != ESlateShader::Default);

					RenderBatch = &CreateRenderBatch(InLayer, FShaderParams(), FontShaderResource, ESlateDrawPrimitive::TriangleList, ShaderType, InDrawEffects, ESlateBatchDrawFlag::None, DrawElement);

					// Reserve memory for the glyphs.  This isn't perfect as the text could contain spaces and we might not render the rest of the text in this batch but its better than resizing constantly
					const int32 GlyphsLeft = NumChars - CharIndex;
					RenderBatch->ReserveVertices(GlyphsLeft * 4);

					if (bUseStaticIndicies)
					{
						static FSlateIndexArray DefaultTextIndiciesArray = [MaxPreAllocatedGlyphIndicies = MaxPreAllocatedGlyphIndicies]()
						{
							FSlateIndexArray DefaultTextIndicies = {};
							DefaultTextIndicies.Reserve(MaxPreAllocatedGlyphIndicies * 6);

							for (int32 Glyph = 0; Glyph < MaxPreAllocatedGlyphIndicies; ++Glyph)
							{
								const uint32 IndexStart = Glyph * 4;

								DefaultTextIndicies.Add(IndexStart + 0);
								DefaultTextIndicies.Add(IndexStart + 1);
								DefaultTextIndicies.Add(IndexStart + 2);
								DefaultTextIndicies.Add(IndexStart + 1);
								DefaultTextIndicies.Add(IndexStart + 3);
								DefaultTextIndicies.Add(IndexStart + 2);
							}

							return DefaultTextIndicies;
						}();

						// NumIndicies can be dynamic based on newline chars, so increment it later as we iterate glyphs
						RenderBatch->SourceIndices = &DefaultTextIndiciesArray;
						RenderBatch->NumIndices = 0;
						RenderBatch->IndexOffset = 0;
					}
					else
					{
						RenderBatch->ReserveIndices(GlyphsLeft * 6);
					}

					InvTextureSizeX = 1.0f / FontAtlasTexture->GetWidth();
					InvTextureSizeY = 1.0f / FontAtlasTexture->GetHeight();
				}

				const bool bIsWhitespace = !Entry.Valid || (bEnableOutline && !Entry.SupportsOutline) || FChar::IsWhitespace(CurrentChar);

				if( !bIsWhitespace && PreviousCharEntry.Valid )
				{
					Kerning = CharacterList.GetKerning( PreviousCharEntry, Entry );
				}
				else
				{
					Kerning = 0;
				}

				LineX += Kerning;
				PreviousCharEntry = Entry;

				if( !bIsWhitespace )
				{
					const float InvBitmapRenderScale = 1.0f / Entry.BitmapRenderScale;

					const float X = LineX + Entry.HorizontalOffset+InOutlineHorizontalOffset;
					// Note PosX,PosY is the upper left corner of the bounding box representing the string.  This computes the Y position of the baseline where text will sit

					const float Y = PosY - Entry.VerticalOffset + ((MaxHeight + Entry.GlobalDescender) * InvBitmapRenderScale);
					const float U = Entry.StartU * InvTextureSizeX;
					const float V = Entry.StartV * InvTextureSizeY;
					const float SizeX = Entry.USize * Entry.BitmapRenderScale;
					const float SizeY = Entry.VSize * Entry.BitmapRenderScale;
					const float SizeU = Entry.USize * InvTextureSizeX;
					const float SizeV = Entry.VSize * InvTextureSizeY;

					{
						FVector2f UpperLeft(X, Y);
						FVector2f UpperRight(X + SizeX, Y);
						FVector2f LowerLeft(X, Y + SizeY);
						FVector2f LowerRight(X + SizeX, Y + SizeY);

						// The start index of these vertices in the index buffer
						const uint32 IndexStart = RenderBatch->GetNumVertices();

						float Ut = 0.0f, Vt = 0.0f, UtMax = 0.0f, VtMax = 0.0f;
						if( bIsFontMaterial )
						{
							float DistAlpha = (float)CharIndex/NumChars;
							float DistAlphaNext = (float)(CharIndex+1)/NumChars;

							// This creates a set of UV's that goes from 0-1, from left to right of the string in U and 0-1 baseline to baseline top to bottom in V
							Ut = FMath::Lerp(0.0f, 1.0f, DistAlpha);
							Vt = FMath::Lerp(0.0f, 1.0f, UpperLeft.Y / (MaxHeight*NumLines));

							UtMax = FMath::Lerp(0.0f, 1.0f, DistAlphaNext);
							VtMax = FMath::Lerp(0.0f, 1.0f, LowerLeft.Y / (MaxHeight*NumLines));
						}

						// Add four vertices to the list of verts to be added to the vertex buffer
						RenderBatch->AddVertex(FSlateVertex::Make<Rounding>( RenderTransform, FVector2f(UpperLeft),					FVector4f(U,V,Ut,Vt),						FVector2f(0.0f,0.0f), FontTint ));
						RenderBatch->AddVertex(FSlateVertex::Make<Rounding>( RenderTransform, FVector2f(LowerRight.X,UpperLeft.Y),	FVector4f(U+SizeU, V, UtMax,Vt),			FVector2f(1.0f,0.0f), FontTint ));
						RenderBatch->AddVertex(FSlateVertex::Make<Rounding>( RenderTransform, FVector2f(UpperLeft.X,LowerRight.Y),	FVector4f(U, V+SizeV, Ut,VtMax),			FVector2f(0.0f,1.0f), FontTint ));
						RenderBatch->AddVertex(FSlateVertex::Make<Rounding>( RenderTransform, FVector2f(LowerRight),				FVector4f(U+SizeU, V+SizeV, UtMax,VtMax),	FVector2f(1.0f,1.0f), FontTint ));

						if (bUseStaticIndicies)
						{
							RenderBatch->NumIndices += 6;
						}
						else
						{
							RenderBatch->AddIndex(IndexStart + 0);
							RenderBatch->AddIndex(IndexStart + 1);
							RenderBatch->AddIndex(IndexStart + 2);
							RenderBatch->AddIndex(IndexStart + 1);
							RenderBatch->AddIndex(IndexStart + 3);
							RenderBatch->AddIndex(IndexStart + 2);
						}
					}
				}

				LineX += Entry.XAdvance;
			}
		}
	};

	const UObject* BaseFontMaterial = DrawElement.GetFontInfo().FontMaterial;
	const UObject* OutlineFontMaterial = OutlineSettings.OutlineMaterial;

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	SlateElementBatcher::CheckUObject(DrawElement, BaseFontMaterial);
	SlateElementBatcher::CheckUObject(DrawElement, OutlineFontMaterial);
#endif

	const bool bOutlineFont = OutlineSettings.OutlineSize > 0;
	if (bOutlineFont)
	{
		//The fill area was measured without an outline so it must be shifted by the scaled outline size
		const float HorizontalOffset = FMath::RoundToFloat((float)OutlineSettings.OutlineSize * FontScale);

		// Build geometry for the outline
		BuildFontGeometry(OutlineSettings, PackVertexColor(OutlineSettings.OutlineColor), OutlineFontMaterial, Layer, HorizontalOffset);
		// Build geometry for the base font which is always rendered on top of the outline
		BuildFontGeometry(FFontOutlineSettings::NoOutline, BaseTint, BaseFontMaterial, Layer + 1, HorizontalOffset);
	}
	else
	{
		// No outline, draw normally
		BuildFontGeometry(FFontOutlineSettings::NoOutline, BaseTint, BaseFontMaterial, Layer, 0.f);
	}
}

/**
 * Determines if the 2x2 matrix represents a rotation that will keep an axis-aligned rect
 * axis-aligned (i.e. a rotation of 90-degree increments). This allows both "proper rotations"
 * (those without a reflection) and "improper rotations" (rotations combined with a reflection
 * over a single axis).
 */
static bool IsAxisAlignedRotation(const FMatrix2x2& Matrix)
{
	constexpr float Tolerance = KINDA_SMALL_NUMBER;

	float A, B, C, D;
	Matrix.GetMatrix(A, B, C, D);

	// The 90- and 270-degree rotation matrices have zeroes on the main diagonal e.g.
	// [0 n]
	// [n 0] with n = 1 or -1
	if (FMath::IsNearlyZero(A, Tolerance) && FMath::IsNearlyZero(D, Tolerance))
	{
		return FMath::IsNearlyEqual(1.0f, FMath::Abs(B), Tolerance) && FMath::IsNearlyEqual(1.0f, FMath::Abs(C), Tolerance);
	}

	// The 0- and 180-degree rotation matrices have zeroes on the secondary diagonal e.g.
	// [n 0]
	// [0 n] with n = 1 or -1
	if (FMath::IsNearlyZero(B, Tolerance) && FMath::IsNearlyZero(C, Tolerance))
	{
		return FMath::IsNearlyEqual(1.0f, FMath::Abs(A), Tolerance) && FMath::IsNearlyEqual(1.0f, FMath::Abs(D), Tolerance);
	}

	return false;
}

template<ESlateVertexRounding Rounding>
void FSlateElementBatcher::AddShapedTextElement( const FSlateShapedTextElement& DrawElement )
{
	const FShapedGlyphSequence* ShapedGlyphSequence = DrawElement.GetShapedGlyphSequence().Get();

	const FShapedGlyphSequence* OverflowGlyphSequence = DrawElement.OverflowArgs.OverflowTextPtr.Get();

	checkSlow(ShapedGlyphSequence);

	const FFontOutlineSettings& OutlineSettings = ShapedGlyphSequence->GetFontOutlineSettings();

	ensure(ShapedGlyphSequence->GetGlyphsToRender().Num() > 0);

	const FColor BaseTint = PackVertexColor(DrawElement.GetTint());

	FSlateFontCache& FontCache = *RenderingPolicy->GetFontCache();


	const int16 TextBaseline = ShapedGlyphSequence->GetTextBaseline();
	const uint16 MaxHeight = ShapedGlyphSequence->GetMaxTextHeight();

	FShapedTextBuildContext BuildContext;

	BuildContext.DrawElement = &DrawElement;
	BuildContext.FontCache = &FontCache;
	BuildContext.ShapedGlyphSequence = ShapedGlyphSequence;
	BuildContext.OverflowGlyphSequence = OverflowGlyphSequence;
	BuildContext.TextBaseline = TextBaseline;
	BuildContext.MaxHeight = MaxHeight;

	if (MaxHeight == 0)
	{
		// If the max text height is 0, we'll create NaN's further in the code, so avoid drawing text if this happens.
		return;
	}

	const int32 Layer = DrawElement.GetLayer();

	// extract the layout transform from the draw element
	FSlateLayoutTransform LayoutTransform(DrawElement.GetScale(), FVector2f(DrawElement.GetPosition()));

	// We don't just scale up fonts, we draw them in local space pre-scaled so we don't get scaling artifacts.
	// So we need to pull the layout scale out of the layout and render transform so we can apply them
	// in local space with pre-scaled fonts.
	const float FontScale = LayoutTransform.GetScale();

	const FSlateRenderTransform RenderTransform = Concatenate(Inverse(FontScale), DrawElement.GetRenderTransform());
	BuildContext.RenderTransform = &RenderTransform;

	auto BuildFontGeometry = [&](const FFontOutlineSettings& InOutlineSettings, const FColor& InTint, const UObject* FontMaterial, int32 InLayer, float InHorizontalOffset, int32 MaxGlyphCountToRender = -1) -> int32
	{
		FVector2f TopLeft(0, 0);

		const float PosX = TopLeft.X+InHorizontalOffset;
		float PosY = TopLeft.Y;

		BuildContext.FontMaterial = FontMaterial;
	
		BuildContext.OutlineSettings = &InOutlineSettings;
		BuildContext.StartLineX = PosX;
		BuildContext.StartLineY = PosY;
		BuildContext.LayerId = InLayer;
		BuildContext.FontTint = InTint;

		// Optimize by culling
		// Todo: this doesn't work with cached clipping
		BuildContext.bEnableCulling = false;
		BuildContext.bForceEllipsis = DrawElement.OverflowArgs.bIsLastVisibleBlock && DrawElement.OverflowArgs.bIsNextBlockClipped;
		BuildContext.OverflowDirection = DrawElement.OverflowArgs.OverflowDirection;
		BuildContext.OverflowPolicy = DrawElement.OverflowArgs.OverflowPolicy;
		BuildContext.MaxGlyphCountToRender = MaxGlyphCountToRender;

		if (ShapedGlyphSequence->GetGlyphsToRender().Num() > 200 || (OverflowGlyphSequence && BuildContext.OverflowDirection != ETextOverflowDirection::NoOverflow))
		{
			const FSlateClippingState* ClippingState = ResolveClippingState(DrawElement);

			if (ClippingState && ClippingState->ScissorRect.IsSet() && ClippingState->ScissorRect->IsAxisAligned() && IsAxisAlignedRotation(RenderTransform.GetMatrix()))
			{
				// Non-render transformed box or rotation is axis-aligned at 90-degree increments
				const FSlateRect ScissorRectBox = ClippingState->ScissorRect->GetBoundingBox();

				// We know that this will be axis-aligned because the scissor rect is axis-aligned and we already
				// checked that the render transform is axis-aligned as well
				const FSlateRect LocalClipBoundingBox = TransformRect(RenderTransform.Inverse(), ScissorRectBox);
				BuildContext.LocalClipBoundingBoxLeft = LocalClipBoundingBox.Left;
				BuildContext.LocalClipBoundingBoxRight = LocalClipBoundingBox.Right;

				// In checks below, ignore floating-point differences caused by transforming and untransforming the clip rect
				const bool NeedLeftEllipsis = FMath::FloorToInt(BuildContext.LocalClipBoundingBoxLeft) > 0 && BuildContext.OverflowDirection == ETextOverflowDirection::RightToLeft;
				const bool NeedRightEllipsis = ShapedGlyphSequence->GetMeasuredWidth() > FMath::CeilToInt(BuildContext.LocalClipBoundingBoxRight) && BuildContext.OverflowDirection == ETextOverflowDirection::LeftToRight;
				const bool NeedMiddleEllipsis = BuildContext.OverflowPolicy == ETextOverflowPolicy::MiddleEllipsis && (FMath::FloorToInt(BuildContext.LocalClipBoundingBoxLeft) > 0 || ShapedGlyphSequence->GetMeasuredWidth() > FMath::CeilToInt(BuildContext.LocalClipBoundingBoxRight));
				if (!NeedLeftEllipsis && !NeedRightEllipsis && !NeedMiddleEllipsis && !DrawElement.OverflowArgs.bIsNextBlockClipped)
				{
					BuildContext.OverflowDirection = ETextOverflowDirection::NoOverflow;
				}
				else if(!OverflowGlyphSequence)
				{
					BuildContext.bEnableCulling = true;
				}
			}
			else
			{
				// Overflow not supported on non-identity transforms (except for 90-degree rotations)
				BuildContext.OverflowDirection = ETextOverflowDirection::NoOverflow;
			}
		}

		return BuildShapedTextSequence<Rounding>(BuildContext);
	};

	STAT(ElementStat_ShapedText++);

	const UObject* BaseFontMaterial = ShapedGlyphSequence->GetFontMaterial();
	const bool bOutlineFont = OutlineSettings.OutlineSize > 0;
	if (bOutlineFont)
	{
		//The fill area was measured without an outline so it must be shifted by the scaled outline size.
		//The bounding box of the text is bigger than a standard one, due to the outline (we add the scaled outline size * 2), 
		//so the horizontal offset ensure that the outline won't bleed outside of the box and will be correctly positioned inside the box.
		const float HorizontalOffset = FMath::RoundToFloat((float)OutlineSettings.OutlineSize * FontScale);
		const UObject* OutlineFontMaterial = OutlineSettings.OutlineMaterial;

		// Build geometry for the outline
		const int32 GlyphsRendered = BuildFontGeometry(OutlineSettings, PackVertexColor(DrawElement.GetOutlineTint()), OutlineFontMaterial, Layer, HorizontalOffset);
		// Build geometry for the base font which is always rendered on top of the outline 
		BuildFontGeometry(FFontOutlineSettings::NoOutline, BaseTint, BaseFontMaterial, Layer+1, HorizontalOffset, GlyphsRendered);
	}
	else
	{
		// No outline
		BuildFontGeometry(FFontOutlineSettings::NoOutline, BaseTint, BaseFontMaterial, Layer, 0.f);
	}
}

template<ESlateVertexRounding Rounding>
void FSlateElementBatcher::AddGradientElement( const FSlateGradientElement& DrawElement )
{
	const FSlateRenderTransform& RenderTransform = DrawElement.GetRenderTransform();
	const FVector2f LocalSize = DrawElement.GetLocalSize();
	const ESlateDrawEffect InDrawEffects = DrawElement.GetDrawEffects();
	const int32 Layer = DrawElement.GetLayer();
	const float DrawScale = DrawElement.GetScale();

	// There must be at least one gradient stop
	check(DrawElement.GradientStops.Num() > 0 );

	FShaderParams ShaderParams;

	ESlateShader ShaderType = ESlateShader::Default;
	if (DrawElement.CornerRadius != FVector4f(0.f))
	{
		ShaderType = ESlateShader::RoundedBox;
		ShaderParams.PixelParams = FVector4f(0.0f, 0.0f, LocalSize.X, LocalSize.Y);
		ShaderParams.PixelParams2 = DrawElement.CornerRadius;
	}

	FSlateRenderBatch& RenderBatch = 
		CreateRenderBatch( 
			Layer,
			ShaderParams,
			nullptr,
			ESlateDrawPrimitive::TriangleList,
			ShaderType,
			InDrawEffects,
			DrawElement.GetBatchFlags(),
			DrawElement);

	// Determine the four corners of the quad containing the gradient
	FVector2f TopLeft = FVector2f::ZeroVector;
	FVector2f TopRight = FVector2f(LocalSize.X, 0);
	FVector2f BotLeft = FVector2f(0, LocalSize.Y);
	FVector2f BotRight = FVector2f(LocalSize.X, LocalSize.Y);

	// Copy the gradient stops.. We may need to add more
	TArray<FSlateGradientStop> GradientStops = DrawElement.GradientStops;

	const FSlateGradientStop& FirstStop = DrawElement.GradientStops[0];
	const FSlateGradientStop& LastStop = DrawElement.GradientStops[DrawElement.GradientStops.Num() - 1 ];
		
	// Determine if the first and last stops are not at the start and end of the quad
	// If they are not add a gradient stop with the same color as the first and/or last stop
	if( DrawElement.GradientType == Orient_Vertical )
	{
		if( 0.0f < FirstStop.Position.X )
		{
			// The first stop is after the left side of the quad.  Add a stop at the left side of the quad using the same color as the first stop
			GradientStops.Insert( FSlateGradientStop( FVector2f(0.0f, 0.0f), FirstStop.Color ), 0 );
		}

		if( LocalSize.X > LastStop.Position.X )
		{
			// The last stop is before the right side of the quad.  Add a stop at the right side of the quad using the same color as the last stop
			GradientStops.Add( FSlateGradientStop( LocalSize, LastStop.Color ) ); 
		}
	}
	else
	{
		if( 0.0f < FirstStop.Position.Y )
		{
			// The first stop is after the top side of the quad.  Add a stop at the top side of the quad using the same color as the first stop
			GradientStops.Insert( FSlateGradientStop( FVector2f(0.0f, 0.0f), FirstStop.Color ), 0 );
		}

		if( LocalSize.Y > LastStop.Position.Y )
		{
			// The last stop is before the bottom side of the quad.  Add a stop at the bottom side of the quad using the same color as the last stop
			GradientStops.Add( FSlateGradientStop( LocalSize, LastStop.Color ) ); 
		}
	}

	// Add a pair of vertices for each gradient stop. Connecting them to the previous stop if necessary
	// Assumes gradient stops are sorted by position left to right or top to bottom
	for( int32 StopIndex = 0; StopIndex < GradientStops.Num(); ++StopIndex )
	{
		const uint32 IndexStart = RenderBatch.GetNumVertices();

		const FSlateGradientStop& CurStop = GradientStops[StopIndex];

		// The start vertex at this stop
		FVector2f StartPt;
		// The end vertex at this stop
		FVector2f EndPt;

		FVector2f StartUV;
		FVector2f EndUV;

		if( DrawElement.GradientType == Orient_Vertical )
		{
			// Gradient stop is vertical so gradients to left to right
			StartPt = TopLeft;
			EndPt = BotLeft;
			// Gradient stops are interpreted in local space.
			StartPt.X += CurStop.Position.X;
			EndPt.X += CurStop.Position.X;

			StartUV.X = StartPt.X / TopRight.X;
			StartUV.Y = 0.f;

			EndUV.X = EndPt.X / TopRight.X;
			EndUV.Y = 1.f;
		}
		else
		{

			// Gradient stop is horizontal so gradients to top to bottom
			StartPt = TopLeft;
			EndPt = TopRight;
			// Gradient stops are interpreted in local space.
			StartPt.Y += CurStop.Position.Y;
			EndPt.Y += CurStop.Position.Y;

			StartUV.X = 0.f;
			StartUV.Y = StartPt.Y / BotLeft.Y;

			EndUV.X = 1.f;
			EndUV.Y = StartPt.Y / BotLeft.Y;
		}

		if( StopIndex == 0 )
		{
			// First stop does not have a full quad yet so do not create indices
			RenderBatch.AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, StartPt, LocalSize, DrawScale, FVector4f(StartUV.X, StartUV.Y, 0.f, 0.f), PackVertexColor(CurStop.Color), FColor::Transparent));
			RenderBatch.AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, EndPt, LocalSize, DrawScale, FVector4f(EndUV.X, EndUV.Y, 0.f, 0.f), PackVertexColor(CurStop.Color), FColor::Transparent));
		}
		else
		{
			// All stops after the first have indices and generate quads
			RenderBatch.AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, StartPt, LocalSize, DrawScale, FVector4f(StartUV.X, StartUV.Y, 0.f, 0.f), PackVertexColor(CurStop.Color), FColor::Transparent));
			RenderBatch.AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, EndPt, LocalSize, DrawScale, FVector4f(EndUV.X, EndUV.Y, 0.f, 0.f), PackVertexColor(CurStop.Color), FColor::Transparent));

			// Connect the indices to the previous vertices
			RenderBatch.AddIndex(IndexStart - 2);
			RenderBatch.AddIndex(IndexStart - 1);
			RenderBatch.AddIndex(IndexStart + 0);

			RenderBatch.AddIndex(IndexStart + 0);
			RenderBatch.AddIndex(IndexStart - 1);
			RenderBatch.AddIndex(IndexStart + 1);
		}
	}
}


/** Utility class for building a strip of triangles for a spline. */
struct FSplineBuilder
{
	FSplineBuilder(FSlateRenderBatch& InRenderBatch, const FVector2f StartPoint, float HalfThickness, const FSlateRenderTransform& InRenderTransform, const FColor& InColor)
		: RenderBatch(InRenderBatch)
		, RenderTransform(InRenderTransform)
		, LastPointAdded()
		, LastNormal(FVector2f::ZeroVector)
		, HalfLineThickness(HalfThickness)
		, NumPointsAdded(1)
		, SingleColor(InColor)
	{
		LastPointAdded[0] = LastPointAdded[1] = StartPoint;
	}

	
	void BuildBezierGeometry_WithColorGradient(const TArray<FSlateGradientStop>& GradientStops, int32 GradientStopIndex, const FVector2f P0, const FVector2f P1, const FVector2f P2, const FVector2f P3, const FSlateElementBatcher& InBatcher)
	{
		const int32 NumGradientStops = GradientStops.Num();
		const float SubdivisionPoint = 1.0f / (NumGradientStops - GradientStopIndex);
		
		if (GradientStopIndex < NumGradientStops - 1)
		{
			FVector2f TwoCurves[7];
			deCasteljauSplit_WithColorGradient(P0, P1, P2, P3, TwoCurves, SubdivisionPoint);
			Subdivide_WithColorGradient(GradientStops[GradientStopIndex - 1].Color, GradientStops[GradientStopIndex].Color, InBatcher, TwoCurves[0], TwoCurves[1], TwoCurves[2], TwoCurves[3], *this, 1.0f);
			BuildBezierGeometry_WithColorGradient(GradientStops, GradientStopIndex + 1, TwoCurves[3], TwoCurves[4], TwoCurves[5], TwoCurves[6], InBatcher);
		}
		else
		{
			// We have reached the last gradient stop, so we can finish this spline.
			Subdivide_WithColorGradient(GradientStops[GradientStopIndex - 1].Color, GradientStops[GradientStopIndex].Color, InBatcher, P0, P1, P2, P3, *this, 1.0f);
			Finish(P3, InBatcher.PackVertexColor(GradientStops[GradientStopIndex].Color));
		}	
		
	}

	void BuildBezierGeometry(const FVector2f P0, const FVector2f P1, const FVector2f P2, const FVector2f P3)
	{
		Subdivide(P0, P1, P2, P3, *this, 1.0f);
		Finish(P3, SingleColor);
	}
	
private:
	void AppendPoint(const FVector2f NewPoint, const FColor& InColor)
	{
		// We only add vertexes for the previous line segment.
		// This is because we want to average the previous and new normals
		// In order to prevent overlapping line segments on the spline.
		// These occur especially when curvature is high.

		const FVector2f NewNormal = FVector2f(LastPointAdded[0].Y - NewPoint.Y, NewPoint.X - LastPointAdded[0].X).GetSafeNormal();

		if (NumPointsAdded == 2)
		{
			// Once we have two points, we have a normal, so we can generate the first bit of geometry.
			const FVector2f LastUp = LastNormal*HalfLineThickness;

			RenderBatch.AddVertex(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, FVector2f(LastPointAdded[1] + LastUp), FVector2f(1.0f, 0.0f), FVector2f::ZeroVector, InColor));
			RenderBatch.AddVertex(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, FVector2f(LastPointAdded[1] - LastUp), FVector2f(-1.0f, 0.0f), FVector2f::ZeroVector, InColor));
		}

		if (NumPointsAdded >= 2)
		{
			const FVector2f AveragedUp = (0.5f*(NewNormal + LastNormal)).GetSafeNormal()*HalfLineThickness;

			RenderBatch.AddVertex(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, FVector2f(LastPointAdded[0] + AveragedUp), FVector2f(1.0f, 0.0f), FVector2f::ZeroVector, InColor));
			RenderBatch.AddVertex(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, FVector2f(LastPointAdded[0] - AveragedUp), FVector2f(-1.0f, 0.0f), FVector2f::ZeroVector, InColor));

			const int32 NumVerts = RenderBatch.GetNumVertices();

			// Counterclockwise winding on triangles
			RenderBatch.AddIndex(NumVerts - 3);
			RenderBatch.AddIndex(NumVerts - 4);
			RenderBatch.AddIndex(NumVerts - 2);

			RenderBatch.AddIndex(NumVerts - 3);
			RenderBatch.AddIndex(NumVerts - 2);
			RenderBatch.AddIndex(NumVerts - 1);
		}

		LastPointAdded[1] = LastPointAdded[0];
		LastPointAdded[0] = NewPoint;
		LastNormal = NewNormal;

		++NumPointsAdded;
	}

	void Finish(const FVector2f LastPoint, const FColor& InColor)
	{
		if (NumPointsAdded < 3)
		{
			// Line builder needs at least two line segments (3 points) to
			// complete building its geometry.
			// This will only happen in the case when we have a straight line.
			AppendPoint(LastPoint, InColor);
		}
		else
		{
			// We have added the last point, but the line builder only builds
			// geometry for the previous line segment. Build geometry for the
			// last line segment.
			const FVector2f LastUp = LastNormal*HalfLineThickness;

			RenderBatch.AddVertex(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, FVector2f(LastPointAdded[0] + LastUp), FVector2f(1.0f, 0.0f), FVector2f::ZeroVector, InColor));
			RenderBatch.AddVertex(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, FVector2f(LastPointAdded[0] - LastUp), FVector2f(-1.0f, 0.0f), FVector2f::ZeroVector, InColor));

			const int32 NumVerts = RenderBatch.GetNumVertices();

			// Counterclockwise winding on triangles
			RenderBatch.AddIndex(NumVerts - 3);
			RenderBatch.AddIndex(NumVerts - 4);
			RenderBatch.AddIndex(NumVerts - 2);

			RenderBatch.AddIndex(NumVerts - 3);
			RenderBatch.AddIndex(NumVerts - 2);
			RenderBatch.AddIndex(NumVerts - 1);
		}
	}

	/**
	* Based on comp.graphics.algorithms: Adaptive Subdivision of Bezier Curves.
	*
	*   P1 + - - - - + P2
	*     /           \
	* P0 *             * P3
	*
	* In a perfectly flat curve P1 is the midpoint of (P0, P2) and P2 is the midpoint of (P1,P3).
	* Computing the deviation of points P1 and P2 from the midpoints of P0,P2 and P1,P3 provides
	* a simple and reliable measure of flatness.
	*
	* P1Deviation = (P0 + P2)/2 - P1
	* P2Deviation = (P1 + P3)/2 - P2
	*
	* Eliminate divides: same expression but gets us twice the allowable error
	* P1Deviation*2 = P0 + P2 - 2*P1
	* P2Deviation*2 = P1 + P3 - 2*P2
	*
	* Use manhattan distance: 2*Deviation = |P1Deviation.x| + |P1Deviation.y| + |P2Deviation.x| + |P2Deviation.y|
	*
	*/
	static float ComputeCurviness(const FVector2f P0, const FVector2f P1, const FVector2f P2, const FVector2f P3)
	{
		FVector2f TwoP1Deviations = P0 + P2 - 2 * P1;
		FVector2f TwoP2Deviations = P1 + P3 - 2 * P2;
		float TwoDeviations = FMath::Abs(TwoP1Deviations.X) + FMath::Abs(TwoP1Deviations.Y) + FMath::Abs(TwoP2Deviations.X) + FMath::Abs(TwoP2Deviations.Y);
		return TwoDeviations;
	}


	/**
	* deCasteljau subdivision of Bezier Curves based on reading of Gernot Hoffmann's Bezier Curves.
	*
	*       P1 + - - - - + P2                P1 +
	*         /           \                    / \
	*     P0 *             * P3            P0 *   \   * P3
	*                                              \ /
	*                                               + P2
	*
	*
	* Split the curve defined by P0,P1,P2,P3 into two new curves L0..L3 and R0..R3 that define the same shape.
	*
	* Points L0 and R3 are P0 and P3.
	* First find points L1, M, R2  as the midpoints of (P0,P1), (P1,P2), (P2,P3).
	* Find two more points: L2, R1 defined by midpoints of (L1,M) and (M,R2) respectively.
	* The final points L3 and R0 are both the midpoint of (L2,R1)
	*
	*/
	static void deCasteljauSplit(const FVector2f P0, const FVector2f P1, const FVector2f P2, const FVector2f P3, FVector2f OutCurveParams[7])
	{
		FVector2f L1 = (P0 + P1) * 0.5f;
		FVector2f M = (P1 + P2) * 0.5f;
		FVector2f R2 = (P2 + P3) * 0.5f;

		FVector2f L2 = (L1 + M) * 0.5f;
		FVector2f R1 = (M + R2) * 0.5f;

		FVector2f L3R0 = (L2 + R1) * 0.5f;

		OutCurveParams[0] = P0;
		OutCurveParams[1] = L1;
		OutCurveParams[2] = L2;
		OutCurveParams[3] = L3R0;
		OutCurveParams[4] = R1;
		OutCurveParams[5] = R2;
		OutCurveParams[6] = P3;
	}

	/** More general form of the deCasteljauSplit splits the curve into two parts at a point between 0 and 1 along the curve's length. */
	static void deCasteljauSplit_WithColorGradient(const FVector2f P0, const FVector2f P1, const FVector2f P2, const FVector2f P3, FVector2f OutCurveParams[7], float SplitPoint = 0.5f)
	{
		FVector2f L1 = FMath::Lerp(P0,P1,SplitPoint);
		FVector2f M = FMath::Lerp(P1, P2, SplitPoint);
		FVector2f R2 = FMath::Lerp(P2, P3, SplitPoint);

		FVector2f L2 = FMath::Lerp(L1, M, SplitPoint);
		FVector2f R1 = FMath::Lerp(M, R2, SplitPoint);

		FVector2f L3R0 = FMath::Lerp(L2,R1,SplitPoint);

		OutCurveParams[0] = P0;
		OutCurveParams[1] = L1;
		OutCurveParams[2] = L2;
		OutCurveParams[3] = L3R0;
		OutCurveParams[4] = R1;
		OutCurveParams[5] = R2;
		OutCurveParams[6] = P3;
	}

	static void Subdivide(const FVector2f P0, const FVector2f P1, const FVector2f P2, const FVector2f P3, FSplineBuilder& SplineBuilder, float MaxBiasTimesTwo = 2.0f)
	{
		const float Curviness = ComputeCurviness(P0, P1, P2, P3);
		if (Curviness > MaxBiasTimesTwo)
		{
			// Split the Bezier into two curves.
			FVector2f TwoCurves[7];
			deCasteljauSplit(P0, P1, P2, P3, TwoCurves);
			// Subdivide left, then right
			Subdivide(TwoCurves[0], TwoCurves[1], TwoCurves[2], TwoCurves[3], SplineBuilder, MaxBiasTimesTwo);
			Subdivide(TwoCurves[3], TwoCurves[4], TwoCurves[5], TwoCurves[6], SplineBuilder, MaxBiasTimesTwo);
		}
		else
		{
			SplineBuilder.AppendPoint(P3, SplineBuilder.SingleColor);
		}
	}

	static void Subdivide_WithColorGradient(const FLinearColor& StartColor, const FLinearColor& EndColor, const FSlateElementBatcher& InBatcher, const FVector2f P0, const FVector2f P1, const FVector2f P2, const FVector2f P3, FSplineBuilder& SplineBuilder, float MaxBiasTimesTwo = 2.0f)
	{
		const float Curviness = ComputeCurviness(P0, P1, P2, P3);
		if (Curviness > MaxBiasTimesTwo)
		{
			// Split the Bezier into two curves.
			FVector2f TwoCurves[7];
			deCasteljauSplit(P0, P1, P2, P3, TwoCurves);
			const FLinearColor MidpointColor = FLinearColor::LerpUsingHSV(StartColor, EndColor, 0.5f);
			// Subdivide left, then right
			Subdivide_WithColorGradient(StartColor, MidpointColor, InBatcher, TwoCurves[0], TwoCurves[1], TwoCurves[2], TwoCurves[3], SplineBuilder, MaxBiasTimesTwo);
			Subdivide_WithColorGradient(MidpointColor, EndColor, InBatcher, TwoCurves[3], TwoCurves[4], TwoCurves[5], TwoCurves[6], SplineBuilder, MaxBiasTimesTwo);
		}
		else
		{
			SplineBuilder.AppendPoint(P3, InBatcher.PackVertexColor(EndColor));
		}
	}
	
private:
	FSlateRenderBatch& RenderBatch;
	const FSlateRenderTransform& RenderTransform;
	FVector2f LastPointAdded[2];
	FVector2f LastNormal;
	float HalfLineThickness;
	int32 NumPointsAdded;
	FColor SingleColor;

};


void FSlateElementBatcher::AddSplineElement(const FSlateSplineElement& DrawElement)
{
	// WHY NO PIXEL SNAPPING?
	//
	// Pixel snapping with splines does not make sense.
	// If any of the segments are snapped to pixels, the line will
	// not appear continuous. It is possible to snap the first and
	// last points to pixels, but unclear what that means given
	// a floating point line width.

	const FSlateRenderTransform& RenderTransform = DrawElement.GetRenderTransform();
	const ESlateDrawEffect InDrawEffects = DrawElement.GetDrawEffects();
	const int32 Layer = DrawElement.GetLayer();

	// Filter size to use for anti-aliasing.
	// Increasing this value will increase the fuzziness of line edges.
	const float FilterRadius = 1.0f;

	const float HalfThickness = 0.5f * FMath::Max(0.0f, DrawElement.GetThickness());

	// The amount we increase each side of the line to generate enough pixels
	const float FilteredHalfThickness = HalfThickness + FilterRadius;

	const float InsideFilterU = (HalfThickness - FilterRadius) / FilteredHalfThickness;
	const FVector4f ShaderParams = FVector4f(InsideFilterU, 0.0f, 0.0f, 0.0f);
	FSlateRenderBatch& RenderBatch = CreateRenderBatch(Layer, FShaderParams::MakePixelShaderParams(ShaderParams), nullptr, ESlateDrawPrimitive::TriangleList, ESlateShader::LineSegment, InDrawEffects, ESlateBatchDrawFlag::None, DrawElement);

	// Thickness is given in screenspace, convert it to local space for geometry build
	const float LocalHalfThickness = FilteredHalfThickness / DrawElement.GetScale();

	const FColor SplineColor = (DrawElement.GradientStops.Num()==1) ? PackVertexColor(DrawElement.GradientStops[0].Color) : PackVertexColor(DrawElement.GetTint());

	FSplineBuilder SplineBuilder(
		RenderBatch,
		DrawElement.P0,
		LocalHalfThickness,
		RenderTransform,
		SplineColor
	);

	if (/*const bool bNoGradient = */DrawElement.GradientStops.Num() <= 1)
	{
		// Normal scenario where there is no color gradient.
		SplineBuilder.BuildBezierGeometry(DrawElement.P0, DrawElement.P1, DrawElement.P2, DrawElement.P3);
	}
	else
	{
		// Deprecated scenario _WithColorGradient
		SplineBuilder.BuildBezierGeometry_WithColorGradient(DrawElement.GradientStops, 1, DrawElement.P0, DrawElement.P1, DrawElement.P2, DrawElement.P3, *this);
	}
}

/** Equivalent to V.GetRotated(90.0f) */
static FVector2f GetRotated90(const FVector2f V)
{
	return FVector2f(-V.Y, V.X);
}


/** Utility class that builds triangle strips for antialiased lines */
struct FLineBuilder
{
	/** Constructor
	 *
	 * @param ElementScale		Element layout scale, used to convert screenspace thickness/radius.
	 * @param HalfThickness		Half thickness of the lines in screenspace pixels.
	 * @param FilterRadius		Antialiasing filter radius in screenspace pixels.
	 * @param AngleCosineLimit	Miter Angle Limit after being passed through AngleCosine.
	 */
	FLineBuilder(FSlateRenderBatch& InRenderBatch, const FSlateRenderTransform& InRenderTransform, float ElementScale, float HalfThickness, float FilterRadius, float MiterAngleLimit, float InDashLength, float InDashOffset) :
		RenderBatch(InRenderBatch),
		RenderTransform(InRenderTransform),
		LocalHalfThickness((HalfThickness + FilterRadius) / ElementScale),
		LocalFilterRadius(FilterRadius / ElementScale),
		LocalCapLength((FilterRadius / ElementScale) * 2.0f),
		AngleCosineLimit(FMath::DegreesToRadians((180.0f - MiterAngleLimit) * 0.5f)),
		DashLength(InDashLength),
		DashOffset(InDashOffset)
	{
		if (DashLength < 1.f)
		{
			DashLength = SolidLineDashLength;
		}
	}

	/**
	 * Calculate num elements per line
	 */
	void NumElements(const TArray<FVector2f>& Points, uint32& OutNumVertex, uint32& OutNumIndex)
	{
		const bool bIsDashed = DashLength != SolidLineDashLength;

		FVector2f Position = Points[0];
		FVector2f NextPosition = Points[1];

		FVector2f Direction;
		float Length;
		(NextPosition - Position).ToDirectionAndLength(Direction, Length);
		FVector2f Up = GetRotated90(Direction) * LocalHalfThickness;

		NumStartCap(Length, OutNumVertex, OutNumIndex);

		// Build the intermediate points and their incoming segments
		const int32 LastPointIndex = Points.Num() - 1;
		for (int32 Point = 1; Point < LastPointIndex; ++Point)
		{
			const FVector2f LastDirection = Direction;
			const float LastLength = Length;

			Position = NextPosition;
			NextPosition = Points[Point + 1];

			(NextPosition - Position).ToDirectionAndLength(Direction, Length);
			Up = GetRotated90(Direction) * LocalHalfThickness;

			// Project "up" onto the miter normal to get perpendicular distance to the miter line
			const FVector2f MiterNormal = GetMiterNormal(LastDirection, Direction);
			const float DistanceToMiterLine = FVector2f::DotProduct(Up, MiterNormal);

			// Get the component of Direction perpendicular to the miter line
			const float DirDotMiterNormal = FVector2f::DotProduct(Direction, MiterNormal);

			// Break the strip at zero-length segments, if the miter angle is
			// too tight or the miter would cross either segment's bisector
			const float MinSegmentLength = FMath::Min(LastLength, Length);
			if (MinSegmentLength > SMALL_NUMBER &&
				DirDotMiterNormal >= AngleCosineLimit &&
				(MinSegmentLength * 0.5f * DirDotMiterNormal) >= FMath::Abs(DistanceToMiterLine))
			{
				OutNumVertex += bIsDashed ? 4 : 2;
				OutNumIndex += 6;
			}
			else
			{
				// Can't miter, so end the current strip and start a new one
				NumEndCap(Length, OutNumVertex, OutNumIndex);
				NumStartCap(LastLength, OutNumVertex, OutNumIndex);
			}
		}

		NumEndCap(Length, OutNumVertex, OutNumIndex);
	}

	/**
	 * Build geometry for each line segment between the passed-in points. Segments
	 * are represented by trapezoids with a pair of edges parallel to the segment.
	 * Each trapezoid connects to those for neighbouring segments if it's possible to
	 * miter the join cleanly. Rectangular caps are added to antialias any open ends.
	 *
	 * All quads are built using two triangles. The UV coordinates are based on
	 * topology only, not modified by vertex position. This is why each antialiased
	 * edge is kept parallel to the opposite side. Not doing so would result in the
	 * UVs shearing apart at the diagonal where the two triangles meet.
	 */
	void BuildLineGeometry(const TArray<FVector2f>& Points, const TArray<FColor>& PackedColors, const FColor& PackedTint, ESlateVertexRounding Rounding)
	{
		const bool bIsDashed = DashLength != SolidLineDashLength;

		FColor PointColor = PackedColors.Num() ? PackedColors[0] : PackedTint;
		FVector2f Position = Points[0];
		FVector2f NextPosition = Points[1];

		FVector2f Direction;
		float Length;
		(NextPosition - Position).ToDirectionAndLength(Direction, Length);
		FVector2f Up = GetRotated90(Direction) * LocalHalfThickness;

		// Start the position along the line with the user-provided dash offset.
		//    This allows users to 'anchor' the dashes in screen space based on some external virtual position
		PositionAlongLine = DashOffset;

		// Build the start cap at the first point
		MakeStartCap(Position, Direction, Length, Up, PointColor, Rounding);

		// Increase the position for the next point
		PositionAlongLine += Length;

		// @TODO: Since the vertex - index relationship is not homogenous whenever 
		// a miter angle break occurs we cannot pre-allocate indicies here
		// However if we were to create a new renderbatch on this break then it would be possible

		// Build the intermediate points and their incoming segments
		const int32 LastPointIndex = Points.Num() - 1;
		for (int32 Point = 1; Point < LastPointIndex; ++Point)
		{
			const FVector2f LastDirection = Direction;
			const FVector2f LastPosition = Position;
			const FVector2f LastUp = Up;
			const float LastLength = Length;

			PointColor = (Point < PackedColors.Num()) ? PackedColors[Point] : PackedTint;
			Position = NextPosition;
			NextPosition = Points[Point + 1];

			(NextPosition - Position).ToDirectionAndLength(Direction, Length);
			Up = GetRotated90(Direction) * LocalHalfThickness;

			// Project "up" onto the miter normal to get perpendicular distance to the miter line
			const FVector2f MiterNormal = GetMiterNormal(LastDirection, Direction);
			const float DistanceToMiterLine = FVector2f::DotProduct(Up, MiterNormal);

			// Get the component of Direction perpendicular to the miter line
			const float DirDotMiterNormal = FVector2f::DotProduct(Direction, MiterNormal);

			// Break the strip at zero-length segments, if the miter angle is
			// too tight or the miter would cross either segment's bisector
			const float MinSegmentLength = FMath::Min(LastLength, Length);
			if (MinSegmentLength > SMALL_NUMBER &&
				DirDotMiterNormal >= AngleCosineLimit &&
				(MinSegmentLength * 0.5f * DirDotMiterNormal) >= FMath::Abs(DistanceToMiterLine))
			{
				// Calculate the offset to put the vertices on the miter line while
				// keeping the antialiased edges of each segment's quad parallel
				const float ParallelDistance = DistanceToMiterLine / DirDotMiterNormal;
				const FVector2f MiterUp = Up - (Direction * ParallelDistance);

				const float MiterOffset = FVector2f::DotProduct(Direction, MiterUp);
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position + MiterUp), FVector2f(1.0f, 0.0f), FVector2f(PositionAlongLine-MiterOffset, DashLength), PointColor, {}, Rounding));
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position - MiterUp), FVector2f(-1.0f, 0.0f), FVector2f(PositionAlongLine+MiterOffset, DashLength), PointColor, {}, Rounding));
				AddQuadIndices(RenderBatch);

				if (bIsDashed)
				{
					RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position + MiterUp), FVector2f(1.0f, 0.0f), FVector2f(PositionAlongLine+MiterOffset, DashLength), PointColor, {}, Rounding));
					RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position - MiterUp), FVector2f(-1.0f, 0.0f), FVector2f(PositionAlongLine-MiterOffset, DashLength), PointColor, {}, Rounding));
				}
			}
			else
			{
				// Can't miter, so end the current strip and start a new one
				MakeEndCap(Position, LastDirection, LastLength, LastUp, PointColor, Rounding);
				MakeStartCap(Position, Direction, Length, Up, PointColor, Rounding);
			}

			PositionAlongLine += Length;
		}

		// Build the last point's incoming segment and end cap
		PointColor = (LastPointIndex < PackedColors.Num()) ? PackedColors[LastPointIndex] : PackedTint;
		MakeEndCap(NextPosition, Direction, Length, Up, PointColor, Rounding);
	}

	/**
	* Add a quad using the last four vertices added to the batch
	*
	*  Topology:
	*    1--3
	*    |\ |
	*    | \|
	*    0--2
	*/
	static void AddQuadIndices(FSlateRenderBatch& InRenderBatch)
	{
		const int32 NumVerts = InRenderBatch.GetNumVertices();

		auto IndexQuad = [](FSlateRenderBatch& InRenderBatch, int32 TopLeft, int32 TopRight, int32 BottomRight, int32 BottomLeft)
		{
			InRenderBatch.EmplaceIndex(TopLeft);
			InRenderBatch.EmplaceIndex(TopRight);
			InRenderBatch.EmplaceIndex(BottomRight);

			InRenderBatch.EmplaceIndex(BottomRight);
			InRenderBatch.EmplaceIndex(BottomLeft);
			InRenderBatch.EmplaceIndex(TopLeft);
		};

		IndexQuad(InRenderBatch, NumVerts - 3, NumVerts - 1, NumVerts - 2, NumVerts - 4);
	}

private:
	/**
	 * Calculate num elements in start cap
	 */
	void NumStartCap(float SegmentLength, uint32& OutNumVertex, uint32& OutNumIndex) const
	{
		if (SegmentLength > SMALL_NUMBER)
		{
			OutNumVertex += 4;
			OutNumIndex += 6;
		}
	}

	/**
	 * Calculate num elements in start cap
	 */
	void NumEndCap(float SegmentLength, uint32& OutNumVertex, uint32& OutNumIndex) const
	{
		if (SegmentLength > SMALL_NUMBER)
		{
			OutNumVertex += 4;
			OutNumIndex += 12;
		}
	}

	/**
	 * Create a rectangular cap to antialias the start of a segment
	 */
	void MakeStartCap(
		const FVector2f Position,
		const FVector2f Direction,
		float SegmentLength,
		const FVector2f Up,
		const FColor& Color,
		ESlateVertexRounding Rounding) const
	{
		// Don't build a cap for a zero-length segment
		// Up would be zero-length anyway, so it would be invisible
		if (SegmentLength > SMALL_NUMBER)
		{
			// Extend the segment to cover the filtered pixels at the start
			// Center the cap over Position if possible, but never place
			// vertices on the far side of this segment's midpoint
			const float InwardDistance = FMath::Min(LocalFilterRadius, SegmentLength * 0.5f);
			const float OutwardDistance = (InwardDistance - LocalCapLength);
			const FVector2f CapInward = Direction * InwardDistance;
			const FVector2f CapOutward = Direction * OutwardDistance;

			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position + CapOutward + Up), FVector2f(1.0f, -1.0f), FVector2f(PositionAlongLine + OutwardDistance, DashLength), Color, {}, Rounding));
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position + CapOutward - Up), FVector2f(-1.0f, -1.0f), FVector2f(PositionAlongLine + OutwardDistance, DashLength), Color, {}, Rounding));
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position + CapInward + Up), FVector2f(1.0f, 0.0f), FVector2f(PositionAlongLine + InwardDistance, DashLength), Color, {}, Rounding));
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position + CapInward - Up), FVector2f(-1.0f, 0.0f), FVector2f(PositionAlongLine + InwardDistance, DashLength), Color, {}, Rounding));
			AddQuadIndices(RenderBatch);
		}
	}

	/**
	 * Create a rectangular cap to antialias the end of a segment
	 */
	void MakeEndCap(
		const FVector2f Position,
		const FVector2f Direction,
		float SegmentLength,
		const FVector2f Up,
		const FColor& Color,
		ESlateVertexRounding Rounding) const
	{
		// Don't build a cap for a zero-length segment
		// Up would be zero-length anyway, so it would be invisible
		if (SegmentLength > SMALL_NUMBER)
		{
			// Extend the segment to cover the filtered pixels at the end
			// Center the cap over Position if possible, but never place
			// vertices on the far side of this segment's midpoint
			const float InwardDistance = FMath::Min(LocalFilterRadius, SegmentLength * 0.5f);
			const float OutwardDistance = (InwardDistance - LocalCapLength);
			const FVector2f CapInward = Direction * -InwardDistance;
			const FVector2f CapOutward = Direction * OutwardDistance;

			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position + CapInward + Up), FVector2f(1.0f, 0.0f), FVector2f(PositionAlongLine - InwardDistance, DashLength), Color, {}, Rounding));
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position + CapInward - Up), FVector2f(-1.0f, 0.0f), FVector2f(PositionAlongLine - InwardDistance, DashLength), Color, {}, Rounding));
			AddQuadIndices(RenderBatch);

			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position + CapOutward + Up), FVector2f(1.0f, 1.0f), FVector2f(PositionAlongLine + OutwardDistance, DashLength), Color, {}, Rounding));
			RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(Position + CapOutward - Up), FVector2f(-1.0f, 1.0f), FVector2f(PositionAlongLine + OutwardDistance, DashLength), Color, {}, Rounding));
			AddQuadIndices(RenderBatch);
		}
	}

	/**
	* Returns the normal to the miter line given the direction of two segments.
	* The miter line between two segments bisects the angle between them.
	*
	* Inputs must be normalized vectors pointing from the start to the end of
	* two consecutive segments.
	*/
	static FVector2f GetMiterNormal(const FVector2f InboundSegmentDir, const FVector2f OutboundSegmentDir)
	{
		const FVector2f DirSum = InboundSegmentDir + OutboundSegmentDir;
		const float SizeSquared = DirSum.SizeSquared();

		if (SizeSquared > SMALL_NUMBER)
		{
			return DirSum * FMath::InvSqrt(SizeSquared);
		}

		// Inputs are antiparallel, so any perpendicular vector suffices
		return GetRotated90(InboundSegmentDir);
	}

	// A dashlength of -10 forces results in a computation of 1.f for the entire line in the pixel shader
	// by feeding it an 'inside-out' trangle wave that modulates between +10 and +30
	static constexpr float SolidLineDashLength = -10.f;

	FSlateRenderBatch& RenderBatch;
	const FSlateRenderTransform& RenderTransform;

	const float LocalHalfThickness;
	const float LocalFilterRadius;
	const float LocalCapLength;
	const float AngleCosineLimit;
	float DashLength = SolidLineDashLength; // Default to no dash
	float DashOffset = 0.f;
	float PositionAlongLine = 0.f;
};


void FSlateElementBatcher::AddLineElements( const FSlateDrawElementArray<FSlateLineElement>& DrawElements )
{
	auto AddLineElementInternal = [&](FSlateRenderBatch& RenderBatch, const FSlateLineElement& DrawElement)
	{
		const ESlateVertexRounding Rounding = DrawElement.IsPixelSnapped() ? ESlateVertexRounding::Enabled : ESlateVertexRounding::Disabled;
		const FSlateRenderTransform& RenderTransform = DrawElement.GetRenderTransform();
		const ESlateDrawEffect DrawEffects = DrawElement.GetDrawEffects();
		const int32 Layer = DrawElement.GetLayer();

		const TArray<FVector2f>& Points = DrawElement.GetPoints();
		const TArray<FLinearColor>& PointColors = DrawElement.GetPointColors();

		const int32 NumPoints = Points.Num();
		if (NumPoints < 2)
		{
			return;
		}

		// Pack element tint and per-point colors
		const FColor PackedTint = PackVertexColor(DrawElement.GetTint());
		TArray<FColor> PackedColors;
		PackedColors.Reserve(PointColors.Num());
		for (const FLinearColor& PointColor : PointColors)
		{
			PackedColors.Add(PackVertexColor(PointColor * DrawElement.GetTint()));
		}

		if (DrawElement.IsAntialiased())
		{
			// Filter size to use for anti-aliasing.
			// Increasing this value will increase the fuzziness of line edges.
			const float FilterRadius = 1.0f;

			// Corners sharper than this will be split instead of mitered
			// A split can also be forced by repeating points in the input
			const float MiterAngleLimit = 90.0f - KINDA_SMALL_NUMBER;

			const float HalfThickness = 0.5f * FMath::Max(0.0f, DrawElement.GetThickness());

			FLineBuilder LineBuilder(
				RenderBatch,
				RenderTransform,
				DrawElement.GetScale(),
				HalfThickness,
				FilterRadius,
				MiterAngleLimit,
				DrawElement.DashLength,
				DrawElement.DashOffset);

			LineBuilder.BuildLineGeometry(Points, PackedColors, PackedTint, Rounding);
		}
		else
		{
			if (DrawElement.GetThickness() == 1.0f)
			{
				const FColor StartColor = PackedColors.Num() ? PackedColors[0] : PackedTint;
				RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, Points[0], FVector2f::ZeroVector, StartColor, {}, Rounding));

				for (int32 Point = 1; Point < NumPoints; ++Point)
				{
					const FColor PointColor = PackedColors.Num() ? PackedColors[Point] : PackedTint;
					RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, Points[Point], FVector2f::ZeroVector, PointColor, {}, Rounding));

					const uint32 IndexStart = RenderBatch.GetNumVertices();
					RenderBatch.EmplaceIndex(IndexStart - 2);
					RenderBatch.EmplaceIndex(IndexStart - 1);
				}

			}
			else
			{

				// Thickness is given in screenspace, convert it to local space for geometry build
				const float LocalHalfThickness = (DrawElement.GetThickness() * 0.5f) / DrawElement.GetScale();

				for (int32 Point = 0; Point < NumPoints - 1; ++Point)
				{
					const FVector2f StartPos = Points[Point];
					const FVector2f EndPos = Points[Point + 1];

					const FColor StartColor = PackedColors.Num() ? PackedColors[Point] : PackedTint;
					const FColor EndColor = PackedColors.Num() ? PackedColors[Point + 1] : PackedTint;

					const FVector2f SegmentDir = (EndPos - StartPos).GetSafeNormal();
					const FVector2f Up = GetRotated90(SegmentDir) * LocalHalfThickness;

					RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(StartPos + Up), FVector2f::ZeroVector, StartColor, {}, Rounding));
					RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(StartPos - Up), FVector2f::ZeroVector, StartColor, {}, Rounding));
					RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(EndPos + Up), FVector2f::ZeroVector, EndColor, {}, Rounding));
					RenderBatch.EmplaceVertex(FSlateVertex::Make(RenderTransform, FVector2f(EndPos - Up), FVector2f::ZeroVector, EndColor, {}, Rounding));

					FLineBuilder::AddQuadIndices(RenderBatch);
				}
			}
		}
	};
	
	auto GenerateLineElementBatchParams = [&](const FSlateLineElement& InDrawElement, FSlateRenderBatchParams& OutBatchParameters)
	{
		const ESlateDrawEffect DrawEffects = InDrawElement.GetDrawEffects();

		// Shader type and params
		ESlateShader ShaderType = ESlateShader::Default;
		if (InDrawElement.IsAntialiased())
		{
			// Filter size to use for anti-aliasing.
			// Increasing this value will increase the fuzziness of line edges.
			const float FilterRadius = 1.0f;

			const float HalfThickness = 0.5f * FMath::Max(0.0f, InDrawElement.GetThickness());

			// The amount we increase each side of the line to generate enough pixels
			const float FilteredHalfThickness = HalfThickness + FilterRadius;

			const float InsideFilterU = (HalfThickness - FilterRadius) / FilteredHalfThickness;
			const FVector4f PixelShaderParams = FVector4f(InsideFilterU, 0.0f, 0.0f, 0.0f);
			OutBatchParameters.ShaderParams = FShaderParams::MakePixelShaderParams(PixelShaderParams);
			ShaderType = ESlateShader::LineSegment;
		}

		// Primitive type
		ESlateDrawPrimitive PrimitiveType = ESlateDrawPrimitive::LineList;

		if (InDrawElement.IsAntialiased())
		{
			PrimitiveType = ESlateDrawPrimitive::TriangleList;
		}
		else
		{
			if (InDrawElement.GetThickness() == 1.0f)
			{
				PrimitiveType = ESlateDrawPrimitive::LineList;
			}
			else
			{
				PrimitiveType = ESlateDrawPrimitive::TriangleList;
			}
		}

		OutBatchParameters.Layer = InDrawElement.GetLayer();
		OutBatchParameters.Resource = nullptr;
		OutBatchParameters.PrimitiveType = PrimitiveType;
		OutBatchParameters.ShaderType = ShaderType;
		OutBatchParameters.DrawEffects = DrawEffects;
		OutBatchParameters.DrawFlags = ESlateBatchDrawFlag::None;
		OutBatchParameters.SceneIndex = InDrawElement.GetSceneIndex();
		OutBatchParameters.ClippingState = ResolveClippingState(InDrawElement);
	};

	auto ReserveLineElementBatch = [&](FSlateRenderBatch& RenderBatch, uint32 InBatchStart, uint32 InBatchEnd)
	{
		uint32 NumVertexes = 0;
		uint32 NumIndices = 0;

		// For source of magic numbers, see internal method above
		for (uint32 Index = InBatchStart; Index < InBatchEnd; Index++)
		{
			const FSlateLineElement& DrawElement = DrawElements[Index];
			const ESlateVertexRounding Rounding = DrawElement.IsPixelSnapped() ? ESlateVertexRounding::Enabled : ESlateVertexRounding::Disabled;
			const FSlateRenderTransform& RenderTransform = DrawElement.GetRenderTransform();
			const ESlateDrawEffect DrawEffects = DrawElement.GetDrawEffects();
			const int32 Layer = DrawElement.GetLayer();

			const TArray<FVector2f>& Points = DrawElement.GetPoints();

			const int32 NumPoints = Points.Num();
			if (NumPoints < 2)
			{
				return;
			}

			if (DrawElement.IsAntialiased())
			{
				const float FilterRadius = 1.0f;

				const float MiterAngleLimit = 90.0f - KINDA_SMALL_NUMBER;

				const float HalfThickness = 0.5f * FMath::Max(0.0f, DrawElement.GetThickness());

				FLineBuilder LineBuilder(
					RenderBatch,
					RenderTransform,
					DrawElement.GetScale(),
					HalfThickness,
					FilterRadius,
					MiterAngleLimit,
					DrawElement.DashLength,
					DrawElement.DashOffset);

				LineBuilder.NumElements(Points, NumVertexes, NumIndices);
			}
			else
			{
				if (DrawElement.GetThickness() == 1.0f)
				{
					NumVertexes += NumPoints;
					NumIndices += (NumPoints - 1) * 2;

				}
				else
				{
					NumVertexes += (NumPoints - 1) * 4;
					NumIndices += (NumPoints - 1) * 6;
				}
			}
		}

		RenderBatch.SourceVertices->Reserve(RenderBatch.SourceVertices->Num() + NumVertexes);
		RenderBatch.SourceIndices->Reserve(RenderBatch.SourceIndices->Num() + NumIndices);
	};

	GenerateIndexedVertexBatches(DrawElements, AddLineElementInternal, GenerateLineElementBatchParams, ReserveLineElementBatch);
}

template<ESlateVertexRounding Rounding>
void FSlateElementBatcher::AddViewportElement( const FSlateViewportElement& DrawElement )
{
	const FSlateRenderTransform& RenderTransform = DrawElement.GetRenderTransform();
	const FVector2f LocalSize = DrawElement.GetLocalSize();
	ESlateDrawEffect InDrawEffects = DrawElement.GetDrawEffects();
	const int32 Layer = DrawElement.GetLayer();

	const FColor FinalColor = PackVertexColor(DrawElement.GetTint());

	ESlateBatchDrawFlag DrawFlags = DrawElement.GetBatchFlags();

	FSlateShaderResource* ViewportResource = DrawElement.RenderTargetResource;
	ESlateShader ShaderType = ESlateShader::Default;

	if (ViewportResource != nullptr && ViewportResource->Debug_IsDestroyed())
	{
		UE_LOGF(LogSlate, Error, "Adding a viewport element with an already destroyed viewport resource.");
		return;
	}

	if(DrawElement.bViewportTextureAlphaOnly )
	{
		// This is a slight hack, but the grayscale font shader is the same as the general shader except it reads alpha only textures and doesn't support tiling
		ShaderType = ESlateShader::GrayscaleFont;
	}

	bool bIsHDRViewport = EnumHasAnyFlags(DrawFlags, ESlateBatchDrawFlag::HDR);
	bool bUseBatchDataHDR = (bCompositeHDRViewports && bIsHDRViewport);

	FSlateBatchData* UsedBatchData = bUseBatchDataHDR ? BatchDataHDR : BatchData;
	FSlateRenderBatch& RenderBatch = CreateRenderBatch(UsedBatchData, Layer, FShaderParams(), ViewportResource, ESlateDrawPrimitive::TriangleList, ShaderType, InDrawEffects, DrawFlags, DrawElement);

	// Tag this batch as requiring vsync if the viewport requires it.
	if( ViewportResource != nullptr && !DrawElement.bAllowViewportScaling )
	{
		bRequiresVsync |= DrawElement.bRequiresVSync;
	}

	// Do pixel snapping
	FVector2f TopLeft(0.f, 0.f);
	FVector2f BotRight(LocalSize);

	// If the viewport disallows scaling, force size to current texture size.
	if (ViewportResource != nullptr && !DrawElement.bAllowViewportScaling)
	{
		const float ElementScale = DrawElement.GetScale();
		BotRight = FVector2f(ViewportResource->GetWidth() / ElementScale, ViewportResource->GetHeight() / ElementScale);
	}

	FVector2f TopRight = FVector2f(BotRight.X, TopLeft.Y);
	FVector2f BotLeft = FVector2f(TopLeft.X, BotRight.Y);

	// The start index of these vertices in the index buffer
	const uint32 IndexStart = 0;

	// Add four vertices to the list of verts to be added to the vertex buffer
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, TopLeft,	FVector2f(0.0f,0.0f),	FinalColor ) );
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, TopRight,	FVector2f(1.0f,0.0f),	FinalColor ) );
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, BotLeft,	FVector2f(0.0f,1.0f),	FinalColor ) );
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, BotRight,	FVector2f(1.0f,1.0f),	FinalColor ) );

	// Add 6 indices to the vertex buffer.  (2 tri's per quad, 3 indices per tri)
	RenderBatch.AddIndex( IndexStart + 0 );
	RenderBatch.AddIndex( IndexStart + 1 );
	RenderBatch.AddIndex( IndexStart + 2 );

	RenderBatch.AddIndex( IndexStart + 2 );
	RenderBatch.AddIndex( IndexStart + 1 );
	RenderBatch.AddIndex( IndexStart + 3 );

	if (bUseBatchDataHDR)
	{
		// used to poke a hole in the slate tree: in case HDR is enabled, we need to compose the hdr scene with the SDR ui based on UI alpha
		// The problem is that in editor mode, there's already a few quads already drawn below the viewport that we actually don't want. If we had an easy way to split
		// in-game UI from tool UI, we could have prevented this with colorwritemask, but it doesn't seem to be the case. Instead, we force to draw
		// an alpha=0 quad to allow the scene RT to be composited properly. AFAICT, game UI is rendered on top of it
		DrawFlags = ESlateBatchDrawFlag::NoBlending;
		const FColor TransparentBlackColor(0, 0, 0, 0);
		FSlateRenderBatch& RenderBatch2 = CreateRenderBatch(Layer, FShaderParams(), nullptr, ESlateDrawPrimitive::TriangleList, ShaderType, InDrawEffects, DrawFlags, DrawElement);
		RenderBatch2.AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, TopLeft, FVector2f(0.0f, 0.0f), TransparentBlackColor));
		RenderBatch2.AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, TopRight, FVector2f(1.0f, 0.0f), TransparentBlackColor));
		RenderBatch2.AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, BotLeft, FVector2f(0.0f, 1.0f), TransparentBlackColor));
		RenderBatch2.AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, BotRight, FVector2f(1.0f, 1.0f), TransparentBlackColor));

		// Add 6 indices to the vertex buffer.  (2 tri's per quad, 3 indices per tri)
		RenderBatch2.AddIndex(IndexStart + 0);
		RenderBatch2.AddIndex(IndexStart + 1);
		RenderBatch2.AddIndex(IndexStart + 2);

		RenderBatch2.AddIndex(IndexStart + 2);
		RenderBatch2.AddIndex(IndexStart + 1);
		RenderBatch2.AddIndex(IndexStart + 3);
	}

}

template<ESlateVertexRounding Rounding>
void FSlateElementBatcher::AddBorderElement( const FSlateBoxElement& DrawElement )
{
	const FSlateRenderTransform& RenderTransform = DrawElement.GetRenderTransform();
	const FVector2f LocalSize = DrawElement.GetLocalSize();
	const ESlateDrawEffect InDrawEffects = DrawElement.GetDrawEffects();

	const int32 Layer = DrawElement.GetLayer();

	const float DrawScale = DrawElement.GetScale();

	uint32 TextureWidth = 1;
	uint32 TextureHeight = 1;

	// Currently borders are not atlased because they are tiled.  So we just assume the texture proxy holds the actual texture
	const FSlateShaderResourceProxy* ResourceProxy = DrawElement.GetResourceProxy();
	FSlateShaderResource* Resource = ResourceProxy ? ResourceProxy->Resource : nullptr;
	if( Resource )
	{
		TextureWidth = Resource->GetWidth();
		TextureHeight = Resource->GetHeight();
	}
	FVector2f TextureSizeLocalSpace = TransformVector(DrawElement.GetInverseLayoutTransform(), FVector2f((float)TextureWidth, (float)TextureHeight));
 
	// Texel offset
	const FVector2f HalfTexel( PixelCenterOffset/TextureWidth, PixelCenterOffset/TextureHeight );

	const FVector2f StartUV = HalfTexel;
	const FVector2f EndUV = FVector2f( 1.0f, 1.0f ) + HalfTexel;

	FMargin Margin = DrawElement.GetBrushMargin();

	// Do pixel snapping
	FVector2f TopLeft(0.f, 0.f);
	FVector2f BotRight(LocalSize);

	// Account for negative sizes
	bool bIsFlippedX = TopLeft.X > BotRight.X;
	bool bIsFlippedY = TopLeft.Y > BotRight.Y;
	Margin.Left = bIsFlippedX ? -Margin.Left : Margin.Left;
	Margin.Top = bIsFlippedY ? -Margin.Top : Margin.Top;
	Margin.Right = bIsFlippedX ? -Margin.Right : Margin.Right;
	Margin.Bottom = bIsFlippedY ? -Margin.Bottom : Margin.Bottom;

	// Determine the margins for each quad
	FVector2f TopLeftMargin(TextureSizeLocalSpace * FVector2f(Margin.Left, Margin.Top));
	FVector2f BotRightMargin(LocalSize - TextureSizeLocalSpace * FVector2f(Margin.Right, Margin.Bottom));

	float LeftMarginX = TopLeftMargin.X;
	float TopMarginY = TopLeftMargin.Y;
	float RightMarginX = BotRightMargin.X;
	float BottomMarginY = BotRightMargin.Y;

	// If the margins are overlapping the margins are too big or the button is too small
	// so clamp margins to half of the box size
	if (FMath::Abs(RightMarginX) < FMath::Abs(LeftMarginX))
	{
		LeftMarginX = LocalSize.X / 2;
		RightMarginX = LeftMarginX;
	}

	if (FMath::Abs(BottomMarginY) < FMath::Abs(TopMarginY))
	{
		TopMarginY = LocalSize.Y / 2;
		BottomMarginY = TopMarginY;
	}

	// Determine the texture coordinates for each quad
	float LeftMarginU = FMath::Abs(Margin.Left);
	float TopMarginV = FMath::Abs(Margin.Top);
	float RightMarginU = 1.0f - FMath::Abs(Margin.Right);
	float BottomMarginV = 1.0f - FMath::Abs(Margin.Bottom);

	LeftMarginU += HalfTexel.X;
	TopMarginV += HalfTexel.Y;
	BottomMarginV += HalfTexel.Y;
	RightMarginU += HalfTexel.X;

	// Determine the amount of tiling needed for the texture in this element.  The formula is number of pixels covered by the tiling portion of the texture / the number number of texels corresponding to the tiled portion of the texture.
	float TopTiling = 1.0f;
	float LeftTiling = 1.0f;
	float Denom = TextureSizeLocalSpace.X * (1.0f - Margin.GetTotalSpaceAlong<Orient_Horizontal>());
	if (!FMath::IsNearlyZero(Denom))
	{
		TopTiling = (RightMarginX - LeftMarginX) / Denom;
	}
	Denom = TextureSizeLocalSpace.Y * (1.0f - Margin.GetTotalSpaceAlong<Orient_Vertical>());
	if (!FMath::IsNearlyZero(Denom))
	{
		LeftTiling = (BottomMarginY - TopMarginY) / Denom;
	}
	
	FShaderParams ShaderParams = FShaderParams::MakePixelShaderParams(FVector4f(LeftMarginU,RightMarginU,TopMarginV,BottomMarginV) );

	// The tint color applies to all brushes and is passed per vertex
	const FColor Tint = PackVertexColor(DrawElement.GetTint());

	// Pass the tiling information as a flag so we can pick the correct texture addressing mode
	ESlateBatchDrawFlag DrawFlags = (ESlateBatchDrawFlag::TileU|ESlateBatchDrawFlag::TileV);

	FSlateRenderBatch& RenderBatch = CreateRenderBatch( Layer, ShaderParams, Resource, ESlateDrawPrimitive::TriangleList, ESlateShader::Border, InDrawEffects, DrawFlags, DrawElement);

	// Ensure tiling of at least 1.  
	TopTiling = TopTiling >= 1.0f ? TopTiling : 1.0f;
	LeftTiling = LeftTiling >= 1.0f ? LeftTiling : 1.0f;
	float RightTiling = LeftTiling;
	float BottomTiling = TopTiling;

	FVector2f Position = TopLeft;
	FVector2f EndPos = BotRight;

	// The start index of these vertices in the index buffer
	const uint32 IndexStart = RenderBatch.GetNumVertices();

	// Zero for second UV indicates no tiling and to just pass the UV though (for the corner sections)
	FVector2f Zero(0.f,0.f);

	// Add all the vertices needed for this element.  Vertices are duplicated so that we can have some sections with no tiling and some with tiling.
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( Position ),					LocalSize, DrawScale, FVector4f( StartUV.X, StartUV.Y, 0.0f, 0.0f),				Tint ) ); //0
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( Position.X, TopMarginY ),		LocalSize, DrawScale, FVector4f( StartUV.X, TopMarginV, 0.0f, 0.0f),				Tint ) ); //1
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( LeftMarginX, Position.Y ),		LocalSize, DrawScale, FVector4f( LeftMarginU, StartUV.Y, 0.0f, 0.0f),			Tint ) ); //2
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( LeftMarginX, TopMarginY ),		LocalSize, DrawScale, FVector4f( LeftMarginU, TopMarginV, 0.0f, 0.0f),			Tint ) ); //3

	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( LeftMarginX, Position.Y ),		LocalSize, DrawScale, FVector4f( StartUV.X, StartUV.Y, TopTiling, 0.0f),			Tint ) ); //4
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( LeftMarginX, TopMarginY ),		LocalSize, DrawScale, FVector4f( StartUV.X, TopMarginV, TopTiling, 0.0f),		Tint ) ); //5
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( RightMarginX, Position.Y ),	LocalSize, DrawScale, FVector4f( EndUV.X, StartUV.Y, TopTiling, 0.0f),			Tint ) ); //6
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( RightMarginX, TopMarginY ),	LocalSize, DrawScale, FVector4f( EndUV.X, TopMarginV, TopTiling, 0.0f),			Tint ) ); //7

	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( RightMarginX, Position.Y ),	LocalSize, DrawScale, FVector4f( RightMarginU, StartUV.Y, 0.0f, 0.0f),			Tint ) ); //8
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( RightMarginX, TopMarginY ),	LocalSize, DrawScale, FVector4f( RightMarginU, TopMarginV, 0.0f, 0.0f),			Tint ) ); //9
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( EndPos.X, Position.Y ),		LocalSize, DrawScale, FVector4f( EndUV.X, StartUV.Y, 0.0f, 0.0f),				Tint ) ); //10
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( EndPos.X, TopMarginY ),		LocalSize, DrawScale, FVector4f( EndUV.X, TopMarginV, 0.0f, 0.0f),				Tint ) ); //11

	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( Position.X, TopMarginY ),		LocalSize, DrawScale, FVector4f( StartUV.X, StartUV.Y, 0.0f, LeftTiling),		Tint ) ); //12
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( Position.X, BottomMarginY ),	LocalSize, DrawScale, FVector4f( StartUV.X, EndUV.Y, 0.0f, LeftTiling),			Tint ) ); //13
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( LeftMarginX, TopMarginY ),		LocalSize, DrawScale, FVector4f( LeftMarginU, StartUV.Y, 0.0f, LeftTiling),		Tint ) ); //14
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( LeftMarginX, BottomMarginY ),	LocalSize, DrawScale, FVector4f( LeftMarginU, EndUV.Y, 0.0f, LeftTiling),		Tint ) ); //15

	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( RightMarginX, TopMarginY ),	LocalSize, DrawScale, FVector4f( RightMarginU, StartUV.Y, 0.0f, RightTiling),	Tint ) ); //16
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( RightMarginX, BottomMarginY ), LocalSize, DrawScale, FVector4f( RightMarginU, EndUV.Y, 0.0f, RightTiling),		Tint ) ); //17
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( EndPos.X, TopMarginY ),		LocalSize, DrawScale, FVector4f( EndUV.X, StartUV.Y, 0.0f, RightTiling),			Tint ) ); //18
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( EndPos.X, BottomMarginY ),		LocalSize, DrawScale, FVector4f( EndUV.X, EndUV.Y, 0.0f, RightTiling),			Tint ) ); //19

	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( Position.X, BottomMarginY ),	LocalSize, DrawScale, FVector4f( StartUV.X, BottomMarginV, 0.0f, 0.0f),			Tint ) ); //20
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( Position.X, EndPos.Y ),		LocalSize, DrawScale, FVector4f( StartUV.X, EndUV.Y, 0.0f, 0.0f),				Tint ) ); //21
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( LeftMarginX, BottomMarginY ),	LocalSize, DrawScale, FVector4f( LeftMarginU, BottomMarginV, 0.0f, 0.0f),		Tint ) ); //22
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( LeftMarginX, EndPos.Y ),		LocalSize, DrawScale, FVector4f( LeftMarginU, EndUV.Y, 0.0f, 0.0f),				Tint ) ); //23

	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( LeftMarginX, BottomMarginY ),	LocalSize, DrawScale, FVector4f( StartUV.X, BottomMarginV, BottomTiling, 0.0f),	Tint ) ); //24
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( LeftMarginX, EndPos.Y ),		LocalSize, DrawScale, FVector4f( StartUV.X, EndUV.Y, BottomTiling, 0.0f),		Tint ) ); //25
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( RightMarginX,BottomMarginY ),	LocalSize, DrawScale, FVector4f( EndUV.X, BottomMarginV, BottomTiling, 0.0f),	Tint ) ); //26
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( RightMarginX, EndPos.Y ),		LocalSize, DrawScale, FVector4f( EndUV.X, EndUV.Y, BottomTiling, 0.0f),			Tint ) ); //27

	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( RightMarginX, BottomMarginY ), LocalSize, DrawScale, FVector4f( RightMarginU, BottomMarginV, 0.0f, 0.0f),		Tint ) ); //29
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( RightMarginX, EndPos.Y ),		LocalSize, DrawScale, FVector4f( RightMarginU, EndUV.Y, 0.0f, 0.0f),				Tint ) ); //30
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( EndPos.X, BottomMarginY ),		LocalSize, DrawScale, FVector4f( EndUV.X, BottomMarginV, 0.0f, 0.0f),			Tint ) ); //31
	RenderBatch.AddVertex( FSlateVertex::Make<Rounding>( RenderTransform, FVector2f( EndPos.X, EndPos.Y ),			LocalSize, DrawScale, FVector4f( EndUV.X, EndUV.Y, 0.0f, 0.0f),					Tint ) ); //32

	static FSlateIndexArray DefaultBorderIndiciesArray = []()
	{
		const int32 IndexStart = 0;
		FSlateIndexArray DefaultBorderIndicies = {};
		DefaultBorderIndicies.Reserve(6 * (3 + 2 + 3));

		// Top
		DefaultBorderIndicies.Add(IndexStart + 0);
		DefaultBorderIndicies.Add(IndexStart + 1);
		DefaultBorderIndicies.Add(IndexStart + 2);
		DefaultBorderIndicies.Add(IndexStart + 2);
		DefaultBorderIndicies.Add(IndexStart + 1);
		DefaultBorderIndicies.Add(IndexStart + 3);

		DefaultBorderIndicies.Add(IndexStart + 4);
		DefaultBorderIndicies.Add(IndexStart + 5);
		DefaultBorderIndicies.Add(IndexStart + 6);
		DefaultBorderIndicies.Add(IndexStart + 6);
		DefaultBorderIndicies.Add(IndexStart + 5);
		DefaultBorderIndicies.Add(IndexStart + 7);

		DefaultBorderIndicies.Add(IndexStart + 8);
		DefaultBorderIndicies.Add(IndexStart + 9);
		DefaultBorderIndicies.Add(IndexStart + 10);
		DefaultBorderIndicies.Add(IndexStart + 10);
		DefaultBorderIndicies.Add(IndexStart + 9);
		DefaultBorderIndicies.Add(IndexStart + 11);

		// Middle
		DefaultBorderIndicies.Add(IndexStart + 12);
		DefaultBorderIndicies.Add(IndexStart + 13);
		DefaultBorderIndicies.Add(IndexStart + 14);
		DefaultBorderIndicies.Add(IndexStart + 14);
		DefaultBorderIndicies.Add(IndexStart + 13);
		DefaultBorderIndicies.Add(IndexStart + 15);

		DefaultBorderIndicies.Add(IndexStart + 16);
		DefaultBorderIndicies.Add(IndexStart + 17);
		DefaultBorderIndicies.Add(IndexStart + 18);
		DefaultBorderIndicies.Add(IndexStart + 18);
		DefaultBorderIndicies.Add(IndexStart + 17);
		DefaultBorderIndicies.Add(IndexStart + 19);

		// Bottom
		DefaultBorderIndicies.Add(IndexStart + 20);
		DefaultBorderIndicies.Add(IndexStart + 21);
		DefaultBorderIndicies.Add(IndexStart + 22);
		DefaultBorderIndicies.Add(IndexStart + 22);
		DefaultBorderIndicies.Add(IndexStart + 21);
		DefaultBorderIndicies.Add(IndexStart + 23);

		DefaultBorderIndicies.Add(IndexStart + 24);
		DefaultBorderIndicies.Add(IndexStart + 25);
		DefaultBorderIndicies.Add(IndexStart + 26);
		DefaultBorderIndicies.Add(IndexStart + 26);
		DefaultBorderIndicies.Add(IndexStart + 25);
		DefaultBorderIndicies.Add(IndexStart + 27);

		DefaultBorderIndicies.Add(IndexStart + 28);
		DefaultBorderIndicies.Add(IndexStart + 29);
		DefaultBorderIndicies.Add(IndexStart + 30);
		DefaultBorderIndicies.Add(IndexStart + 30);
		DefaultBorderIndicies.Add(IndexStart + 29);
		DefaultBorderIndicies.Add(IndexStart + 31);

		return DefaultBorderIndicies;
	}();
	RenderBatch.SourceIndices = &DefaultBorderIndiciesArray;
	RenderBatch.NumIndices = DefaultBorderIndiciesArray.Num();
	RenderBatch.IndexOffset = 0;
}

void FSlateElementBatcher::AddCustomElement( const FSlateCustomDrawerElement& DrawElement )
{
	const int32 Layer = DrawElement.GetLayer();

	if (TSharedPtr<ICustomSlateElement> CustomDrawerPinned = DrawElement.CustomDrawer.Pin())
	{
		FSlateRenderBatch& RenderBatch = CreateRenderBatch(Layer, FShaderParams(), nullptr, ESlateDrawPrimitive::None, ESlateShader::Default, ESlateDrawEffect::None, ESlateBatchDrawFlag::None, DrawElement);
		RenderBatch.CustomDrawer = CustomDrawerPinned.Get();
		RenderBatch.bIsMergable = false;
		RenderBatch.CustomDrawer->PostCustomElementAdded(*this);
	}
}

void FSlateElementBatcher::AddCustomVerts(const FSlateCustomVertsElement& DrawElement)
{
	const int32 Layer = DrawElement.GetLayer();

	if (DrawElement.Vertices.Num() > 0)
	{
		FSlateRenderBatch& RenderBatch = CreateRenderBatch(
			Layer, 
			FShaderParams(), 
			DrawElement.ResourceProxy != nullptr ? DrawElement.ResourceProxy->Resource : nullptr,
			ESlateDrawPrimitive::TriangleList,
			ESlateShader::Custom, 
			DrawElement.GetDrawEffects(), 
			DrawElement.GetBatchFlags(),
			DrawElement);

		RenderBatch.bIsMergable = false;
		RenderBatch.InstanceCount = DrawElement.NumInstances;
		RenderBatch.InstanceOffset = DrawElement.InstanceOffset;
		RenderBatch.InstanceData = DrawElement.InstanceData;

		RenderBatch.AddVertices(DrawElement.Vertices);
		RenderBatch.AddIndices(DrawElement.Indices);

	}
}

void FSlateElementBatcher::AddPostProcessPass(const FSlatePostProcessElement& DrawElement, FVector2f WindowSize)
{
	++NumPostProcessPasses;

	const FSlateRenderTransform& RenderTransform = DrawElement.GetRenderTransform();
	const FVector2f LocalSize = DrawElement.GetLocalSize();

	//@todo doesn't work with rotated or skewed objects yet
	const FVector2f Position = DrawElement.GetPosition();

	const int32 Layer = DrawElement.GetLayer();

	// Determine the four corners of the quad
	FVector2f TopLeft = FVector2f::ZeroVector;
	FVector2f TopRight = FVector2f(LocalSize.X, 0.f);
	FVector2f BotLeft = FVector2f(0.f, LocalSize.Y);
	FVector2f BotRight = FVector2f(LocalSize.X, LocalSize.Y);


	// Offset by half a texel if the platform requires it for pixel perfect sampling
	FVector2f HalfTexel = FVector2f(PixelCenterOffset / WindowSize.X, PixelCenterOffset / WindowSize.Y);

	FVector2f WorldTopLeft = TransformPoint(RenderTransform, TopLeft).RoundToVector();
	FVector2f WorldBotRight = TransformPoint(RenderTransform, BotRight).RoundToVector();

	FVector2f SizeUV = (WorldBotRight - WorldTopLeft) / WindowSize;

	// These could be negative with rotation or negative scales.  This is not supported yet
	if(SizeUV.X > 0 && SizeUV.Y > 0)
	{
		FShaderParams Params = FShaderParams::MakePixelShaderParams(
			FVector4f(WorldTopLeft, WorldBotRight),
			FVector4f(DrawElement.PostProcessData.X, DrawElement.PostProcessData.Y, (float)DrawElement.DownsampleAmount, 0.f),
			FVector4f(DrawElement.CornerRadius));

		CreateRenderBatch(Layer, Params, nullptr, ESlateDrawPrimitive::TriangleList, ESlateShader::PostProcess, ESlateDrawEffect::None, ESlateBatchDrawFlag::None, DrawElement);
	}
}

FSlateRenderBatch& FSlateElementBatcher::CreateRenderBatch(
	FSlateBatchData* SlateBatchData
	, int32 Layer
	, const FShaderParams& ShaderParams
	, const FSlateShaderResource* InResource
	, ESlateDrawPrimitive PrimitiveType
	, ESlateShader ShaderType
	, ESlateDrawEffect DrawEffects
	, ESlateBatchDrawFlag DrawFlags
	, int8 SceneIndex
	, const FSlateClippingState* ClippingState)
{
	FSlateRenderBatch& NewBatch = CurrentCachedElementList
		? CurrentCachedElementList->AddRenderBatch(Layer, ShaderParams, InResource, PrimitiveType, ShaderType, DrawEffects, DrawFlags, SceneIndex)
		: SlateBatchData->AddRenderBatch(Layer, ShaderParams, InResource, PrimitiveType, ShaderType, DrawEffects, DrawFlags, SceneIndex);

	NewBatch.ClippingState = ClippingState;

	if (InResource)
	{
		NewBatch.CachedUsedPostBuffers = InResource->GetUsedSlatePostBuffers();
		UsedSlatePostBuffers |= NewBatch.CachedUsedPostBuffers;
	}

	return NewBatch;
}

FSlateRenderBatch& FSlateElementBatcher::CreateRenderBatch(
	FSlateBatchData* SlateBatchData,
	int32 Layer, 
	const FShaderParams& ShaderParams,
	const FSlateShaderResource* InResource,
	ESlateDrawPrimitive PrimitiveType,
	ESlateShader ShaderType,
	ESlateDrawEffect DrawEffects,
	ESlateBatchDrawFlag DrawFlags,
	const FSlateDrawElement& DrawElement)
{
	FSlateRenderBatch& NewBatch = CurrentCachedElementList
		? CurrentCachedElementList->AddRenderBatch(Layer, ShaderParams, InResource, PrimitiveType, ShaderType, DrawEffects, DrawFlags, DrawElement.GetSceneIndex())
		: SlateBatchData->AddRenderBatch(Layer, ShaderParams, InResource, PrimitiveType, ShaderType, DrawEffects, DrawFlags, DrawElement.GetSceneIndex());

	NewBatch.ClippingState = ResolveClippingState(DrawElement);

	if (InResource)
	{
		NewBatch.CachedUsedPostBuffers = InResource->GetUsedSlatePostBuffers();
		UsedSlatePostBuffers |= NewBatch.CachedUsedPostBuffers;
	}

	return NewBatch;
}

const FSlateClippingState* FSlateElementBatcher::ResolveClippingState(const FSlateDrawElement& DrawElement) const
{
	const FClipStateHandle& ClipHandle = DrawElement.GetClippingHandle();
	// Do cached first
	if (ClipHandle.GetCachedClipState())
	{
		// We should be working with cached elements if we have a cached clip state
		check(CurrentCachedElementList);
		return ClipHandle.GetCachedClipState();
	}
	else if (PrecachedClippingStates->IsValidIndex(ClipHandle.GetPrecachedClipIndex()))
	{
		// Store the clipping state so we can use it later for rendering.
		return &(*PrecachedClippingStates)[ClipHandle.GetPrecachedClipIndex()];
	}

	return nullptr;
}

void FSlateElementBatcher::CalculateMiddleEllipsisSkipIndexAndOffset(const FShapedTextBuildContext& InContext, ETextOverflowDirection InOverflowDirection, FMiddleEllipsisOverflowData& OutMiddleEllipsisData)
{
	const FShapedGlyphSequence* GlyphSequenceToRender = InContext.ShapedGlyphSequence;
	const TArray<FShapedGlyphEntry>& GlyphsToRender = GlyphSequenceToRender->GetGlyphsToRender();
	const int32 NumGlyphs = GlyphsToRender.Num();

	float OverflowGlyphSequenceWidth = InContext.OverflowGlyphSequence ? InContext.OverflowGlyphSequence->GetMeasuredWidth() : 0.f;
	float OverflowWidth = (GlyphSequenceToRender->GetMeasuredWidth() + OverflowGlyphSequenceWidth) - (InContext.LocalClipBoundingBoxRight - InContext.LocalClipBoundingBoxLeft);
	int32 MiddleUp = GlyphsToRender.Num() >> 1;
	int32 MiddleDown = MiddleUp;

	while (OverflowWidth > 0.f && (MiddleDown >= 0 || MiddleUp < GlyphsToRender.Num()))
	{
		if (MiddleUp == MiddleDown)
		{
			OverflowWidth -= GlyphsToRender[MiddleUp].XAdvance;
			OutMiddleEllipsisData.SkipIndexStart = OutMiddleEllipsisData.SkipIndexEnd = MiddleUp;
			MiddleUp++;
			MiddleDown--;
		}
		else
		{
			if (GlyphsToRender.IsValidIndex(MiddleDown))
			{
				OverflowWidth -= GlyphsToRender[MiddleDown].XAdvance;
				OutMiddleEllipsisData.SkipIndexStart = MiddleDown;
			}

			if (OverflowWidth <= 0)
			{
				break;
			}

			if (GlyphsToRender.IsValidIndex(MiddleUp))
			{
				OverflowWidth -= GlyphsToRender[MiddleUp].XAdvance;
				OutMiddleEllipsisData.SkipIndexEnd = MiddleUp;
			}
			MiddleUp++;
			MiddleDown--;
		}
	}

	// if the OverflowDirection is RightToLeft we need to calculate the offset of all clipped Character/Whitespaces to add to the LineX of the visible characters to align them correctly
	if (InOverflowDirection == ETextOverflowDirection::RightToLeft && OutMiddleEllipsisData.SkipIndexStart != INDEX_NONE && OutMiddleEllipsisData.SkipIndexEnd != INDEX_NONE)
	{
		int32 ClippedIndexWidth = 0;
		for (int32 GlyphToSkipIndex = OutMiddleEllipsisData.SkipIndexStart; GlyphToSkipIndex <= OutMiddleEllipsisData.SkipIndexEnd; ++GlyphToSkipIndex)
		{
			const FShapedGlyphEntry& GlyphToSkip = GlyphsToRender[GlyphToSkipIndex];
			ClippedIndexWidth += GlyphToSkip.XAdvance;
		}

		// Get the previous and the next index of the skipped ones
		int32 PreviousSkippedIndex = OutMiddleEllipsisData.SkipIndexStart - 1;
		int32 NextSkippedIndex = OutMiddleEllipsisData.SkipIndexEnd + 1;

		// True if the first clipped index is not less than 0 and the previous index are whitespaces/Non-visible Glyphs
		bool bHaveWhiteSpaceBeforeAndIsValidIndex = GlyphsToRender.IsValidIndex(PreviousSkippedIndex);
		// True if the last clipped index is not the last index of the GlyphsToRender and the next index(s) are whitespaces/Non-visible Glyphs
		bool bHaveWhiteSpaceAfterAndIsValidIndex = GlyphsToRender.IsValidIndex(NextSkippedIndex);

		// Calculate the X offset to add to each character to align them correctly at the right
		while (bHaveWhiteSpaceBeforeAndIsValidIndex || bHaveWhiteSpaceAfterAndIsValidIndex)
		{
			if (bHaveWhiteSpaceBeforeAndIsValidIndex)
			{
				if (!GlyphsToRender[PreviousSkippedIndex].bIsVisible)
				{
					ClippedIndexWidth += GlyphsToRender[PreviousSkippedIndex].XAdvance;
				}
				else
				{
					bHaveWhiteSpaceBeforeAndIsValidIndex = false;
				}
				PreviousSkippedIndex--;
				bHaveWhiteSpaceBeforeAndIsValidIndex &= GlyphsToRender.IsValidIndex(PreviousSkippedIndex);
			}

			if (bHaveWhiteSpaceAfterAndIsValidIndex)
			{
				if (!GlyphsToRender[NextSkippedIndex].bIsVisible)
				{
					ClippedIndexWidth += GlyphsToRender[NextSkippedIndex].XAdvance;
				}
				else
				{
					bHaveWhiteSpaceAfterAndIsValidIndex = false;
				}
				NextSkippedIndex++;
				bHaveWhiteSpaceAfterAndIsValidIndex &= GlyphsToRender.IsValidIndex(NextSkippedIndex);
			}
		}

		// BoundingBoxRight Edge - (GlyphSequence Width - ClippedGlyphs + EllipsisWidth)
		const float FinalRightToLeftClippedOffset = InContext.LocalClipBoundingBoxRight - (GlyphSequenceToRender->GetMeasuredWidth() - ClippedIndexWidth + OverflowGlyphSequenceWidth);

		// Start at the Offset needed to align the text to the right for the MiddleEllipsis
		OutMiddleEllipsisData.LineX += FinalRightToLeftClippedOffset;
	}

	int32 SkippedIndexNum = OutMiddleEllipsisData.SkipIndexEnd - OutMiddleEllipsisData.SkipIndexStart + 1;

	// If we are skipping all the Glyphs or all but 1, assign already the ellipsis X and Y since it may happen that the left side is all clipped, and they are not calculated correctly
	if (SkippedIndexNum == NumGlyphs || SkippedIndexNum == NumGlyphs - 1)
	{
		OutMiddleEllipsisData.EllipsisLineX = OutMiddleEllipsisData.LineX;
		OutMiddleEllipsisData.EllipsisLineY = OutMiddleEllipsisData.LineY;
	}

	// If the only visible Glyph is on the right of the ellipsis add the Ellipsis Width to the LineX to not paint the Glyph on top of the ellipsis
	if (OutMiddleEllipsisData.SkipIndexStart == 0 && OutMiddleEllipsisData.SkipIndexEnd + 1 == NumGlyphs - 1)
	{
		OutMiddleEllipsisData.LineX += OverflowGlyphSequenceWidth;
	}
}

template<ESlateVertexRounding Rounding>
int32 FSlateElementBatcher::BuildShapedTextSequence(const FShapedTextBuildContext& Context)
{
	const FShapedGlyphSequence* GlyphSequenceToRender = Context.ShapedGlyphSequence;

	FSlateShaderResourceManager& ResourceManager = *RenderingPolicy->GetResourceManager();

	float InvTextureSizeX = 0;
	float InvTextureSizeY = 0;

	FSlateRenderBatch* RenderBatch = nullptr;

	int32 FontTextureIndex = -1;
	FSlateShaderResource* FontAtlasTexture = nullptr;
	FSlateShaderResource* FontShaderResource = nullptr;

	float LineX = Context.StartLineX;
	float LineY = Context.StartLineY;

	FSlateRenderTransform RenderTransform = *Context.RenderTransform;

	FColor FontTint = Context.FontTint;
	FColor Tint = FColor::White;

	const UObject* FontMaterial = Context.FontMaterial;
	ETextOverflowDirection OverflowDirection = Context.OverflowDirection;
	ETextOverflowPolicy OverflowPolicy = Context.OverflowPolicy;

	float EllipsisLineX = 0;
	float EllipsisLineY = 0;
	bool bNeedEllipsis = false;
	bool bNeedSpaceForEllipsis = false;
	const bool bNonZeroSkew = !FMath::IsNearlyEqual(GlyphSequenceToRender->GetFontSkew(), 0.f);
	bool bPrevSdfGlyph = false;
	float SdfEmSpread = 0;
	float SdfPixelSpread = 0;
	float SdfBias = 0;
	const bool bMaterialIsStencil = GlyphSequenceToRender->IsMaterialStencil();
	// For left to right overflow direction - Sum of total whitespace we're currently advancing through. Once a non-whitespace glyph is detected this will return to 0
	// For right to left this value is unused. We just skip all leading whitespace
	float PreviousWhitespaceAdvance = 0;

	// The number of glyphs can technically be unbound, so pick some number that if smaller we use the preallocated index array
	const int32 MaxPreAllocatedGlyphIndicies = 4096 + 1024;
	const bool bUseStaticIndicies = GlyphSequenceToRender->GetGlyphsToRender().Num() < MaxPreAllocatedGlyphIndicies;

	const bool bOutlineFont = Context.OutlineSettings->OutlineSize > 0;
	const int32 NumGlyphs = GlyphSequenceToRender->GetGlyphsToRender().Num();
	const TArray<FShapedGlyphEntry>& GlyphsToRender = GlyphSequenceToRender->GetGlyphsToRender();
	int32 GlyphsRendered = 0;

	// Middle ellipsis Section
	const bool bWillBeClipped = GlyphSequenceToRender->GetMeasuredWidth() > (Context.LocalClipBoundingBoxRight - Context.LocalClipBoundingBoxLeft) && OverflowDirection != ETextOverflowDirection::NoOverflow;
	int32 SkipIndexStart = INDEX_NONE;
	int32 SkipIndexEnd = INDEX_NONE;

	// True by default, will be checked only after the middle skip and will become false as soon as we find a Visible character after the ellipsis, used to skip whitespaces after the ellipsis
	bool bIsWhitespaceAfterMiddleSkip = true;

#if ENABLE_SDF_DEBUG_VISUAL
	TArray<TPair<FMargin, FColor> > DebugVisualBoxes;

	switch (SdfTextDebugVisual)
	{
		case static_cast<int32>(ESdfTextDebugVisual::Disabled):
		default:
			// No effect
			break;
		case static_cast<int32>(ESdfTextDebugVisual::Raw):
		case static_cast<int32>(ESdfTextDebugVisual::Color):
			FontTint = FColor::White;
			FontMaterial = nullptr;
			break;
	}
#endif // ENABLE_SDF_DEBUG_VISUAL

	// Calculate the Glyph Indexes to skip to fit the ellipsis and add eventually the offset to the LineX to align the text correctly for the RightToLeft direction
	if (OverflowPolicy == ETextOverflowPolicy::MiddleEllipsis && bWillBeClipped)
	{
		FMiddleEllipsisOverflowData OutMiddleEllipsisData;
		OutMiddleEllipsisData.LineX = LineX;
		OutMiddleEllipsisData.LineY = LineY;

		CalculateMiddleEllipsisSkipIndexAndOffset(Context, OverflowDirection, OutMiddleEllipsisData);

		// Assign the new data
		SkipIndexStart = OutMiddleEllipsisData.SkipIndexStart;
		SkipIndexEnd = OutMiddleEllipsisData.SkipIndexEnd;
		LineX = OutMiddleEllipsisData.LineX;
		LineY = OutMiddleEllipsisData.LineY;
		EllipsisLineX = OutMiddleEllipsisData.EllipsisLineX;
		EllipsisLineY = OutMiddleEllipsisData.EllipsisLineY;
	}

	for (int32 GlyphIndex = 0; GlyphIndex < NumGlyphs; ++GlyphIndex)
	{
		// Skip the index if its in between the Glyph to skip
		if (GlyphIndex >= SkipIndexStart && GlyphIndex <= SkipIndexEnd)
		{
			continue;
		}

		const FShapedGlyphEntry& GlyphToRender = GlyphsToRender[GlyphIndex];

		const float BitmapRenderScale = GlyphToRender.GetBitmapRenderScale();
		const float InvBitmapRenderScale = 1.0f / BitmapRenderScale;
		const bool bIsSdfGlyph = GlyphToRender.FontFaceData->RasterizationMode != EFontRasterizationMode::Bitmap;

		float X = 0;
		float SizeX = 0;
		float Y = 0;
		float U = 0;
		float V = 0;
		float SizeY = 0;
		float SizeU = 0;
		float SizeV = 0;

		bool bIsVisible = GlyphToRender.bIsVisible;
		bool bCanRenderGlyph = bIsVisible;

		FVector2f SpriteSize(0.f, 0.f);
		FVector2f SpriteOffset(0.f, 0.f);
		FVector2f QuadMeshSize(0.f, 0.f);
		FVector2f QuadMeshOffsets(0.f, 0.f);

		if (bCanRenderGlyph)
		{
			// Get Sizing and atlas info
			int8 NextAtlasDataTextureIndex = -1;
			float NextSdfEmSpread = 0;
			float NextSdfPixelSpread = 0;
			float NextSdfBias = 0;

			if (bIsSdfGlyph)
			{
				const FSdfGlyphFontAtlasData SdfGlyphAtlasData = Context.FontCache->GetSdfGlyphFontAtlasData(GlyphToRender, *Context.OutlineSettings);
				bCanRenderGlyph = (SdfGlyphAtlasData.Valid && SdfGlyphAtlasData.bSupportsSdf);
				if (bCanRenderGlyph)
				{
					float EmOutlineSize = 0.f;
#if WITH_FREETYPE
					if (bOutlineFont)
					{
						const float TargetPpem = static_cast<float>(FreeTypeUtils::ComputeFontPixelSize(GlyphToRender.FontFaceData->FontSize, GlyphToRender.FontFaceData->FontScale));
						EmOutlineSize = FMath::RoundToFloat(Context.OutlineSettings->OutlineSize * GlyphToRender.FontFaceData->FontScale)/TargetPpem;
					}
#endif // WITH_FREETYPE
					NextAtlasDataTextureIndex = SdfGlyphAtlasData.TextureIndex;
					QuadMeshOffsets = FVector2f(SdfGlyphAtlasData.Metrics.BearingX, SdfGlyphAtlasData.Metrics.BearingY);
					QuadMeshSize = FVector2f(SdfGlyphAtlasData.Metrics.Width, SdfGlyphAtlasData.Metrics.Height);
					// We are cutting off one half of a pixel's width from each side to avoid interpolation with neighbors.
					// SdfGlyphAtlasData.Metrics values assume this operation, so QuadMeshSize already accounts for this
					SpriteSize = FVector2f(SdfGlyphAtlasData.USize-1, SdfGlyphAtlasData.VSize-1);
					SpriteOffset = FVector2f(SdfGlyphAtlasData.StartU+.5f, SdfGlyphAtlasData.StartV+.5f);
					NextSdfEmSpread = SdfGlyphAtlasData.EmInnerSpread + SdfGlyphAtlasData.EmOuterSpread;
					NextSdfPixelSpread = NextSdfEmSpread * float(GlyphToRender.FontFaceData->SdfPpem);
					// Value representing zero distance
					NextSdfBias = (SdfGlyphAtlasData.EmOuterSpread-EmOutlineSize)/(SdfGlyphAtlasData.EmInnerSpread+SdfGlyphAtlasData.EmOuterSpread);
				}
			}
			else
			{
				const FShapedGlyphFontAtlasData GlyphAtlasData = Context.FontCache->GetShapedGlyphFontAtlasData(GlyphToRender, *Context.OutlineSettings);
				bCanRenderGlyph = (GlyphAtlasData.Valid && (!bOutlineFont || GlyphAtlasData.SupportsOutline));
				if (bCanRenderGlyph)
				{
					NextAtlasDataTextureIndex = GlyphAtlasData.TextureIndex;
					QuadMeshOffsets = FVector2f(GlyphAtlasData.HorizontalOffset, GlyphAtlasData.VerticalOffset);
					QuadMeshSize = SpriteSize = FVector2f(GlyphAtlasData.USize, GlyphAtlasData.VSize);
					SpriteOffset = FVector2f(GlyphAtlasData.StartU, GlyphAtlasData.StartV);
				}
			}

			if (bCanRenderGlyph)
			{
				// If we can render a glyph after the middle skip it means that there are no more whitespaces to skip
				if (GlyphIndex > SkipIndexEnd)
				{
					bIsWhitespaceAfterMiddleSkip = false;
				}

				// Note PosX,PosY is the upper left corner of the bounding box representing the string.  This computes the Y position of the baseline where text will sit
				X = LineX + QuadMeshOffsets.X + (float)GlyphToRender.XOffset;
				Y = LineY - QuadMeshOffsets.Y + (float)GlyphToRender.YOffset + ((Context.MaxHeight + Context.TextBaseline) * InvBitmapRenderScale);

				if (Context.bEnableCulling)
				{
					if (X + QuadMeshSize.X < Context.LocalClipBoundingBoxLeft)
					{
						LineX += GlyphToRender.XAdvance;
						LineY += GlyphToRender.YAdvance;
						continue;
					}
					else if (X > Context.LocalClipBoundingBoxRight)
					{
						break;
					}
				}

				// Initiate new render batch for first glyph and whenever texture / shader / shader parameters change between previous and current glyph
				check(NextAtlasDataTextureIndex >= 0);
				if (FontAtlasTexture == nullptr || NextAtlasDataTextureIndex != FontTextureIndex || bIsSdfGlyph != bPrevSdfGlyph || (bIsSdfGlyph && (NextSdfEmSpread != SdfEmSpread || NextSdfPixelSpread != SdfPixelSpread || NextSdfBias != SdfBias)))
				{
					// Font has a new texture for this glyph or shader parameters changed. Refresh the batch we use and the index we are currently using
					FontTextureIndex = NextAtlasDataTextureIndex;

					ISlateFontTexture* SlateFontTexture = Context.FontCache->GetFontTexture(FontTextureIndex);
					check(SlateFontTexture);

					FontAtlasTexture = SlateFontTexture->GetSlateTexture();
					check(FontAtlasTexture);

					FontShaderResource = ResourceManager.GetFontShaderResource(FontTextureIndex, FontAtlasTexture, FontMaterial);
					check(FontShaderResource);

					InvTextureSizeX = 1.0f / FontAtlasTexture->GetWidth();
					InvTextureSizeY = 1.0f / FontAtlasTexture->GetHeight();

					const ESlateFontAtlasContentType ContentType = SlateFontTexture->GetContentType();

					ESlateShader ShaderType = ESlateShader::Default;
					switch (ContentType)
					{
						case ESlateFontAtlasContentType::Alpha:
							ShaderType = bIsSdfGlyph ? ESlateShader::SdfFont : ESlateShader::GrayscaleFont;
							break;
						case ESlateFontAtlasContentType::Color:
							check(!bIsSdfGlyph);
							ShaderType = ESlateShader::ColorFont;
							break;
						case ESlateFontAtlasContentType::Msdf:
							check(bIsSdfGlyph);
							ShaderType = !bOutlineFont || Context.OutlineSettings->bMiteredCorners ? ESlateShader::MsdfFont : ESlateShader::SdfFont;
							break;
						default:
							checkNoEntry();
							// Default to Color
							ShaderType = ESlateShader::ColorFont;
							break;
					}
					check(ShaderType != ESlateShader::Default);

#if ENABLE_SDF_DEBUG_VISUAL
					switch (SdfTextDebugVisual)
					{
						case static_cast<int32>(ESdfTextDebugVisual::Disabled):
						default:
							break;
						case static_cast<int32>(ESdfTextDebugVisual::Raw):
							if (bIsSdfGlyph)
							{
								ShaderType = ContentType == ESlateFontAtlasContentType::Alpha ? ESlateShader::GrayscaleFont : ESlateShader::ColorFont;
							}
							break;
						case static_cast<int32>(ESdfTextDebugVisual::Color):
							switch (GlyphToRender.FontFaceData->RasterizationMode)
							{
								case EFontRasterizationMode::Bitmap:
								default:
									FontTint = DebugVisualBitmapTextColor;
									break;
								case EFontRasterizationMode::Msdf:
									FontTint = DebugVisualMsdfTextColor;
									break;
								case EFontRasterizationMode::Sdf:
									FontTint = DebugVisualSdfTextColor;
									break;
								case EFontRasterizationMode::SdfApproximation:
									FontTint = DebugVisualApproximateSdfTextColor;
									break;
							}
							break;
					}
#endif // ENABLE_SDF_DEBUG_VISUAL

					Tint = ContentType == ESlateFontAtlasContentType::Color ? FColor::White : FontTint;

					FShaderParams ShaderParams;
					if (bIsSdfGlyph)
					{
						SdfEmSpread = NextSdfEmSpread;
						SdfPixelSpread = NextSdfPixelSpread;
						SdfBias = NextSdfBias;
						// Note - it would be much better to pass the SDF shader params as per-vertex attributes instead to avoid having to switch batches too often
						ShaderParams = FShaderParams::MakePixelShaderParams(FVector4f(
							// Half of horizontal, vertical spread in texture coordinate units
							.5f*InvTextureSizeX*SdfPixelSpread,
							.5f*InvTextureSizeY*SdfPixelSpread,
							// Signed distance sample bias, the color value (between 0 to 1) representing zero distance
							SdfBias,
							// Signed distance spread in em
							SdfEmSpread
						), FVector4f(
							// This parameter needs to be 0 for alpha texture single-channel SDF, 1 for BGRA/RGBA MTSDF
							ContentType == ESlateFontAtlasContentType::Msdf ? 1.f : 0.f,
							bMaterialIsStencil ? 1.f : 0.f,
							0.f, 0.f
						));
					}
					else
					{
						ShaderParams = FShaderParams::MakePixelShaderParams(FVector4f(), FVector4f(0.f, bMaterialIsStencil ? 1.f : 0.f, 0.f, 0.f));
					}

					RenderBatch = &CreateRenderBatch(Context.LayerId,
						ShaderParams,
						FontShaderResource,
						ESlateDrawPrimitive::TriangleList,
						ShaderType,
						Context.DrawElement->GetDrawEffects(),
						ESlateBatchDrawFlag::None,
						*Context.DrawElement);

					// Reserve memory for the glyphs.  This isn't perfect as the text could contain spaces and we might not render the rest of the text in this batch but its better than resizing constantly
					const int32 GlyphsLeft = NumGlyphs - GlyphIndex;
					RenderBatch->ReserveVertices(GlyphsLeft * 4);

					if (bUseStaticIndicies)
					{
						static FSlateIndexArray DefaultShapedTextIndiciesArray = [MaxPreAllocatedGlyphIndicies = MaxPreAllocatedGlyphIndicies]()
						{
							FSlateIndexArray DefaultShapedTextIndicies = {};
							DefaultShapedTextIndicies.Reserve(MaxPreAllocatedGlyphIndicies * 6);

							for (int32 Glyph = 0; Glyph < MaxPreAllocatedGlyphIndicies; ++Glyph)
							{
								const uint32 IndexStart = Glyph * 4;

								DefaultShapedTextIndicies.Add(IndexStart + 0);
								DefaultShapedTextIndicies.Add(IndexStart + 1);
								DefaultShapedTextIndicies.Add(IndexStart + 2);
								DefaultShapedTextIndicies.Add(IndexStart + 1);
								DefaultShapedTextIndicies.Add(IndexStart + 3);
								DefaultShapedTextIndicies.Add(IndexStart + 2);
							}

							return DefaultShapedTextIndicies;
						}();

						// NumIndicies can be dynamic based on newline & overflow / ellipsis, so increment it later as we iterate glyphs
						RenderBatch->SourceIndices = &DefaultShapedTextIndiciesArray;
						RenderBatch->NumIndices = 0; 
						RenderBatch->IndexOffset = 0;
					}
					else
					{
						RenderBatch->ReserveIndices(GlyphsLeft * 6);
					}
					bPrevSdfGlyph = bIsSdfGlyph;
				}
				U = SpriteOffset.X * InvTextureSizeX;
				V = SpriteOffset.Y * InvTextureSizeY;
				SizeU = SpriteSize.X * InvTextureSizeX;
				SizeV = SpriteSize.Y * InvTextureSizeY;
				SizeX = QuadMeshSize.X * BitmapRenderScale;
				SizeY = QuadMeshSize.Y * BitmapRenderScale;
			}
		}
		else
		{
			X = LineX;
			SizeX = GlyphToRender.XAdvance;
		}

		// Overflow Detection
		// First figure out the size of the glyph. If the glyph contains multiple characters we have to measure all of them and if clipped, omit them all. This is common in complex languages with lots of diacritics
		float OverflowTestWidth = SizeX;
		bool bMiddleEllipsisIsNextGlyphFirstSkipped = false;
		int32 NextGlyphIndex = GlyphIndex + 1;
		if (OverflowPolicy == ETextOverflowPolicy::MiddleEllipsis && bWillBeClipped && (NextGlyphIndex >= SkipIndexStart && NextGlyphIndex <= SkipIndexEnd))
		{
			bMiddleEllipsisIsNextGlyphFirstSkipped = true;
			EllipsisLineX = LineX + GlyphToRender.XAdvance;
			EllipsisLineY = LineY;
		}
		else if (OverflowDirection != ETextOverflowDirection::NoOverflow && (GlyphToRender.NumGraphemeClustersInGlyph > 1 || GlyphToRender.NumCharactersInGlyph > 1))
		{
			const int32 StartIndex = GlyphIndex;
			int32 EndIndex = GlyphIndex;
			int32 NextIndex = GlyphIndex + 1;
			const int32 SourceIndex = GlyphToRender.SourceIndex;
			while (NextIndex < NumGlyphs && GlyphsToRender[NextIndex].SourceIndex == SourceIndex)
			{
				++EndIndex;
				++NextIndex;
			}
			if (StartIndex < EndIndex)
			{
				OverflowTestWidth = GlyphSequenceToRender->GetMeasuredWidth(StartIndex, EndIndex).Get(SizeX);
			}
		}

		// Left to right overflow - If the current pen position + the ellipsis cannot fit, we have reached the end of the possible area for drawing this text. 
		if (OverflowDirection == ETextOverflowDirection::LeftToRight)
		{
			// MiddleEllipsis is handled in a different way above
			if (OverflowPolicy != ETextOverflowPolicy::MiddleEllipsis)
			{
				// If we are on the last glyph don't bother checking if the ellipsis can fit. If the last glyph can fit there is no need for ellipsis
				float OverflowSequenceNeededSize = GlyphIndex < NumGlyphs - 1 ? Context.OverflowGlyphSequence->GetMeasuredWidth() : 0;
				if(X + OverflowTestWidth + OverflowSequenceNeededSize >= Context.LocalClipBoundingBoxRight || (Context.MaxGlyphCountToRender >= 0 && GlyphIndex >= Context.MaxGlyphCountToRender))
				{
					bNeedEllipsis = true;
					// We subtract out any whitespace advance. This avoids the ellipsis from ever floating out in the middle of a block of whitespace.
					// e.g without this something like "The quick brown		fox jumps over the lazy dog" could be clipped to "The quick brown	..." but we want it to be "The quick brown..."
					EllipsisLineX = LineX - PreviousWhitespaceAdvance;
					EllipsisLineY = LineY;
					// No characters to render after the ellipsis on the right side
					break;
				}
				GlyphsRendered++;
			}
		}
		else if (OverflowDirection == ETextOverflowDirection::RightToLeft)
		{
			// MiddleEllipsis is handled in a different way above
			if (OverflowPolicy != ETextOverflowPolicy::MiddleEllipsis)
			{
				bool bClipped = false;
				// Right to left overflow
				if (X < Context.LocalClipBoundingBoxLeft || (Context.MaxGlyphCountToRender >= 0 && GlyphIndex < (NumGlyphs - Context.MaxGlyphCountToRender)))
				{
					// This glyph is in the clipped region or is not visible so just advance. It cannot be shown
					bClipped = true;
					bNeedSpaceForEllipsis = true;
				}
				if (bNeedSpaceForEllipsis || !GlyphToRender.bIsVisible)
				{
					bClipped = true;

					// Can the ellipsis fit in the free spot by skipping the previous glyph(s)
					const float EllipsisWidth = Context.OverflowGlyphSequence->GetMeasuredWidth();
					const float AvailableX = X + GlyphToRender.XAdvance - Context.LocalClipBoundingBoxLeft;
					if (AvailableX >= EllipsisWidth)
					{
						// The available area can fit the ellipsis. Mark that we need an ellipsis and stop checking for overflow. The rest of the text can be built normally
						bNeedSpaceForEllipsis = false;
					}

					//Always try to put the ellipsis, whether it fits or not: it's better to have an ellipsis a bit clipped than no feedback at all.
					bNeedEllipsis = true;
					EllipsisLineX = (LineX + GlyphToRender.XAdvance - EllipsisWidth);
					EllipsisLineY = LineY;
				}
				else
				{
					OverflowDirection = ETextOverflowDirection::NoOverflow;
					GlyphsRendered++;
				}

				// If we just clipped a glyph omit all characters in said glyph. Otherwise floating diacritics would be visible above the ellipsis. This is common in complex languages.
				if (bClipped && GlyphToRender.NumCharactersInGlyph > 1)
				{
					GlyphIndex += GlyphToRender.NumCharactersInGlyph - 1;
					LineX += GlyphToRender.XAdvance;
					continue;
				}

				bCanRenderGlyph = !bClipped;
			}
		}
		else
		{
			GlyphsRendered++;
		}

		if (bCanRenderGlyph && RenderBatch)
		{
			FVector2f UpperLeft(X, Y);
			FVector2f UpperRight(X + SizeX, Y);
			FVector2f LowerLeft(X, Y + SizeY);
			FVector2f LowerRight(X + SizeX, Y + SizeY);

			// The start index of these vertices in the index buffer
			const uint32 IndexStart = RenderBatch->GetNumVertices();

			float Ut = 0.0f, Vt = 0.0f, UtMax = 0.0f, VtMax = 0.0f;
			if (FontMaterial)
			{
				float DistAlpha = (float)GlyphIndex / NumGlyphs;
				float DistAlphaNext = (float)(GlyphIndex + 1) / NumGlyphs;

				// This creates a set of UV's that goes from 0-1, from left to right of the string in U and 0-1 baseline to baseline top to bottom in V
				Ut = FMath::Lerp(0.0f, 1.0f, DistAlpha);
				Vt = FMath::Lerp(0.0f, 1.0f, UpperLeft.Y / Context.MaxHeight);

				UtMax = FMath::Lerp(0.0f, 1.0f, DistAlphaNext);
				VtMax = FMath::Lerp(0.0f, 1.0f, LowerLeft.Y / Context.MaxHeight);
			}

			if (bNonZeroSkew && bIsSdfGlyph)
			{
				FTransform2f ShearTransform(FShear2f(GlyphSequenceToRender->GetFontSkew(), 0.f));
				FVector2f DeltaTopLeft(0, QuadMeshOffsets.Y);
				FVector2f DeltaBottomLeft(0, QuadMeshOffsets.Y - QuadMeshSize.Y);
				DeltaTopLeft = ShearTransform.TransformVector(DeltaTopLeft);
				DeltaBottomLeft = ShearTransform.TransformVector(DeltaBottomLeft);
				UpperLeft.X += DeltaTopLeft.X;		UpperRight.X += DeltaTopLeft.X;
				LowerLeft.X += DeltaBottomLeft.X;	LowerRight.X += DeltaBottomLeft.X;
			}
			// Add four vertices to the list of verts to be added to the vertex buffer
			if (bIsSdfGlyph)
			{
				// Vertex rounding has no benefit for distance field rendering but makes positioning worse, so disable it
				RenderBatch->AddVertex(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, FVector2f(UpperLeft), FVector4f(U, V, Ut, Vt), FVector2f(0.0f, 0.0f), Tint));
				RenderBatch->AddVertex(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, FVector2f(UpperRight), FVector4f(U + SizeU, V, UtMax, Vt), FVector2f(1.0f, 0.0f), Tint));
				RenderBatch->AddVertex(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, FVector2f(LowerLeft), FVector4f(U, V + SizeV, Ut, VtMax), FVector2f(0.0f, 1.0f), Tint));
				RenderBatch->AddVertex(FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, FVector2f(LowerRight), FVector4f(U + SizeU, V + SizeV, UtMax, VtMax), FVector2f(1.0f, 1.0f), Tint));
			}
			else
			{
				RenderBatch->AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(UpperLeft), FVector4f(U, V, Ut, Vt), FVector2f(0.0f, 0.0f), Tint));
				RenderBatch->AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(UpperRight), FVector4f(U + SizeU, V, UtMax, Vt), FVector2f(1.0f, 0.0f), Tint));
				RenderBatch->AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(LowerLeft), FVector4f(U, V + SizeV, Ut, VtMax), FVector2f(0.0f, 1.0f), Tint));
				RenderBatch->AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(LowerRight), FVector4f(U + SizeU, V + SizeV, UtMax, VtMax), FVector2f(1.0f, 1.0f), Tint));
			}

			if (bUseStaticIndicies)
			{
				RenderBatch->NumIndices += 6;
			}
			else
			{
				RenderBatch->AddIndex(IndexStart + 0);
				RenderBatch->AddIndex(IndexStart + 1);
				RenderBatch->AddIndex(IndexStart + 2);
				RenderBatch->AddIndex(IndexStart + 1);
				RenderBatch->AddIndex(IndexStart + 3);
				RenderBatch->AddIndex(IndexStart + 2);
			}

			// Reset whitespace advance to 0, this is a visible character
			PreviousWhitespaceAdvance = 0;

#if ENABLE_SDF_DEBUG_VISUAL
			if (SdfTextDebugVisual == static_cast<int32>(ESdfTextDebugVisual::Boxes))
			{
				FColor BoxColor;
				switch (GlyphToRender.FontFaceData->RasterizationMode)
				{
					case EFontRasterizationMode::Bitmap:
					default:
						BoxColor = DebugVisualBitmapTextColor;
						BoxColor.A = 0x60;
						break;
					case EFontRasterizationMode::Msdf:
						BoxColor = DebugVisualMsdfTextColor;
						break;
					case EFontRasterizationMode::Sdf:
						BoxColor = DebugVisualSdfTextColor;
						break;
					case EFontRasterizationMode::SdfApproximation:
						BoxColor = DebugVisualApproximateSdfTextColor;
						break;
				}
				DebugVisualBoxes.Add(TPair<FMargin, FColor>(FMargin(UpperLeft.X, UpperLeft.Y, LowerRight.X, LowerRight.Y), BoxColor));
			}
#endif // ENABLE_SDF_DEBUG_VISUAL
		}
		else if (!bIsVisible)
		{
			// How much whitespace we are currently walking through
			PreviousWhitespaceAdvance += GlyphToRender.XAdvance;

			if (OverflowPolicy == ETextOverflowPolicy::MiddleEllipsis && bMiddleEllipsisIsNextGlyphFirstSkipped)
			{
				EllipsisLineX -= PreviousWhitespaceAdvance;
				LineX -= PreviousWhitespaceAdvance;
			}
		}

		if (OverflowPolicy == ETextOverflowPolicy::MiddleEllipsis)
		{
			// If the next Glyph is clipped add to the LineX the width of the Ellipsis
			if (bMiddleEllipsisIsNextGlyphFirstSkipped)
			{
				LineX += Context.OverflowGlyphSequence->GetMeasuredWidth();
			}

			// If after the MiddleEllipsis there are still non-visible Glyphs, do not add their XAdvance to the LineX
			LineX += (GlyphIndex > SkipIndexEnd && bIsWhitespaceAfterMiddleSkip) ? 0 : GlyphToRender.XAdvance;
		}
		else
		{
			LineX += GlyphToRender.XAdvance;
		}

		LineY += GlyphToRender.YAdvance;
	}

	if (!bNeedEllipsis && Context.bForceEllipsis)
	{
		bNeedEllipsis = true;
		EllipsisLineX = LineX - PreviousWhitespaceAdvance; //It's ok to always substract whitespace advance (LTR or RTL overflow direction), as the value will always be 0 for RTL overflow direction.
		EllipsisLineY = LineY;
	}
	else if (OverflowPolicy == ETextOverflowPolicy::MiddleEllipsis)
	{
		bNeedEllipsis = bWillBeClipped;
	}

	if (bNeedEllipsis)
	{
		// Ellipsis can fit, place it at the current lineX
		FShapedTextBuildContext EllipsisContext = Context;
		EllipsisContext.bForceEllipsis = false;
		EllipsisContext.ShapedGlyphSequence = Context.OverflowGlyphSequence;
		EllipsisContext.OverflowGlyphSequence = nullptr;
		EllipsisContext.bEnableCulling = false;
		EllipsisContext.OverflowDirection = ETextOverflowDirection::NoOverflow;
		EllipsisContext.OverflowPolicy = ETextOverflowPolicy::Clip;
		EllipsisContext.StartLineX = EllipsisLineX;
		EllipsisContext.StartLineY = EllipsisLineY;

		BuildShapedTextSequence<Rounding>(EllipsisContext);
	}
	
#if ENABLE_SDF_DEBUG_VISUAL
	if (SdfTextDebugVisual == static_cast<int32>(ESdfTextDebugVisual::Boxes) && !DebugVisualBoxes.IsEmpty())
	{
		RenderBatch = &CreateRenderBatch(Context.LayerId,
			FShaderParams(),
			nullptr,
			ESlateDrawPrimitive::LineList,
			ESlateShader::Default,
			Context.DrawElement->GetDrawEffects(),
			ESlateBatchDrawFlag::None,
			*Context.DrawElement);
		RenderBatch->ReserveVertices(6*DebugVisualBoxes.Num());
		RenderBatch->ReserveIndices(8*DebugVisualBoxes.Num());
		for (const TPair<FMargin, FColor>& Box : DebugVisualBoxes)
		{
			FColor TransparentColor = Box.Value;
			TransparentColor.A = 0;
			RenderBatch->AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(Box.Key.Left, .5f*(Box.Key.Top + Box.Key.Bottom)), FVector4f(), FVector2f(0.0f, 0.0f), TransparentColor));
			RenderBatch->AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(Box.Key.Left, Box.Key.Bottom), FVector4f(), FVector2f(0.0f, 0.0f), Box.Value));
			RenderBatch->AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(Box.Key.Right, Box.Key.Bottom), FVector4f(), FVector2f(0.0f, 0.0f), TransparentColor));
			RenderBatch->AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(Box.Key.Right, .5f*(Box.Key.Top + Box.Key.Bottom)), FVector4f(), FVector2f(0.0f, 0.0f), TransparentColor));
			RenderBatch->AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(Box.Key.Right, Box.Key.Top), FVector4f(), FVector2f(0.0f, 0.0f), Box.Value));
			RenderBatch->AddVertex(FSlateVertex::Make<Rounding>(RenderTransform, FVector2f(Box.Key.Left, Box.Key.Top), FVector4f(), FVector2f(0.0f, 0.0f), TransparentColor));
		}
		for (int32 I = 0; I < DebugVisualBoxes.Num(); ++I)
		{
			RenderBatch->AddIndex(static_cast<SlateIndex>(6*I+0));
			RenderBatch->AddIndex(static_cast<SlateIndex>(6*I+1));
			RenderBatch->AddIndex(static_cast<SlateIndex>(6*I+1));
			RenderBatch->AddIndex(static_cast<SlateIndex>(6*I+2));
			RenderBatch->AddIndex(static_cast<SlateIndex>(6*I+3));
			RenderBatch->AddIndex(static_cast<SlateIndex>(6*I+4));
			RenderBatch->AddIndex(static_cast<SlateIndex>(6*I+4));
			RenderBatch->AddIndex(static_cast<SlateIndex>(6*I+5));
		}
	}
#endif // ENABLE_SDF_DEBUG_VISUAL

	return GlyphsRendered;
}

void FSlateElementBatcher::ResetBatches()
{
	bRequiresVsync = false;
	bCompositeHDRViewports = false;
	UsedSlatePostBuffers = ESlatePostRT::None;
	ResourceUpdatingPostBuffers = ESlatePostRT::None;
	NumPostProcessPasses = 0;
}
