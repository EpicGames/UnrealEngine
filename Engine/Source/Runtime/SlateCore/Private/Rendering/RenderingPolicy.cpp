// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/RenderingPolicy.h"
#include "Rendering/SlateRenderer.h"

/* FSlateRenderingPolicy interface
 *****************************************************************************/

TSharedRef<class FSlateFontCache> FSlateRenderingPolicy::GetFontCache() const
{
	return FontServices->GetFontCache();
}

TSharedRef<class FSlateFontServices> FSlateRenderingPolicy::GetFontServices() const
{
	return FontServices;
}
