// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/ShaderResourceManager.h"

DECLARE_CYCLE_STAT(TEXT("GetResourceHandle Time"), STAT_SlateGetResourceHandle, STATGROUP_Slate);

FSlateResourceHandle::FSlateResourceHandle()
{
}

bool FSlateResourceHandle::IsValid() const
{
	return Data.IsValid() && Data->Proxy;
}

const FSlateShaderResourceProxy* FSlateResourceHandle::GetResourceProxy() const
{
	return Data.IsValid() ? Data->Proxy : nullptr;
}

FSlateResourceHandle::FSlateResourceHandle(const TSharedPtr<FSlateSharedHandleData>& InData)
	: Data(InData)
{
}

FSlateShaderResourceManager::FSlateShaderResourceManager() = default;

FSlateShaderResourceManager::~FSlateShaderResourceManager()
{
	ClearTextureMap();
}

FSlateResourceHandle FSlateShaderResourceManager::GetResourceHandle(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale)
{
	SCOPE_CYCLE_COUNTER(STAT_SlateGetResourceHandle);

	FSlateShaderResourceProxy* Proxy = GetShaderResource(Brush, LocalSize, DrawScale);

	const FSlateResourceHandle& ExistingHandle = Brush.ResourceHandle;

	// validates we rasterized the svg at the correct size
	//check(Brush.GetImageType() != ESlateBrushImageType::Vector || Proxy->ActualSize == (LocalSize * DrawScale).IntPoint());

	if(Proxy != ExistingHandle.GetResourceProxy())
	{
		FSlateResourceHandle NewHandle;
		if (Proxy)
		{
			if (!Proxy->HandleData.IsValid())
			{
				Proxy->HandleData = MakeShareable(new FSlateSharedHandleData(Proxy));
			}

			NewHandle.Data = Proxy->HandleData;
		}

		return NewHandle;
	}
	else
	{
		return ExistingHandle;
	}
}

FSlateResourceHandle FSlateShaderResourceManager::GetResourceHandle(const FSlateBrush& Brush)
{
	return GetResourceHandle(Brush, FVector2f::ZeroVector, 1.0f);
}
