// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SlateDrawBuffer.h"
#include "Rendering/DrawElements.h"
#include "Application/SlateApplicationBase.h"
#include "Widgets/SWindow.h"
#include "Input/HittestGrid.h"


/* FSlateDrawBuffer interface
 *****************************************************************************/

FSlateDrawBuffer::FSlateDrawBuffer()
	: bIsLocked(false)
	, bIsLockedBySlateThread(false)
	, ResourceVersion(0)
{ }

FSlateDrawBuffer::~FSlateDrawBuffer() = default;

FSlateWindowElementList& FSlateDrawBuffer::AddWindowElementList(TSharedRef<SWindow> ForWindow)
{
	ensureMsgf(IsLocked(), TEXT("The SlateDrawBuffer should be lock before modifying it."));
	FScopeLock ScopeLock(&GCLock);

	for ( int32 WindowIndex = 0; WindowIndex < WindowElementListsPool.Num(); ++WindowIndex )
	{
		TSharedRef<FSlateWindowElementList> ExistingElementList = WindowElementListsPool[WindowIndex];

		if (ExistingElementList->GetPaintWindow() == &ForWindow.Get())
		{
			WindowElementLists.Add(ExistingElementList);
			WindowElementListsPool.RemoveAtSwap(WindowIndex);

			ensureMsgf(ExistingElementList->GetBatchData().GetNumFinalBatches() == 0, TEXT("The Buffer should have been clear when it was locked."));
			ExistingElementList->ResetElementList();

			return *ExistingElementList;
		}
	}

	TSharedRef<FSlateWindowElementList> WindowElements = MakeShared<FSlateWindowElementList>(ForWindow);
	WindowElements->ResetElementList();
	WindowElementLists.Add(WindowElements);

	return *WindowElements;
}

void FSlateDrawBuffer::RemoveUnusedWindowElement(const TArray<SWindow*>& AllWindows)
{
	ensureMsgf(IsLocked(), TEXT("The SlateDrawBuffer should be lock before modifying it."));
	FScopeLock ScopeLock(&GCLock);

	// Remove any window elements that are no longer valid.
	for (int32 WindowIndex = 0; WindowIndex < WindowElementLists.Num(); ++WindowIndex)
	{
		SWindow* CandidateWindow = WindowElementLists[WindowIndex]->GetPaintWindow();
		if (!CandidateWindow || !AllWindows.Contains(CandidateWindow))
		{
			WindowElementLists[WindowIndex]->ResetElementList();
			WindowElementLists.RemoveAtSwap(WindowIndex);
			--WindowIndex;
		}
	}
}

bool FSlateDrawBuffer::Lock()
{
	bool ExpectedValue = false;
	bool bIsLock = bIsLocked.compare_exchange_strong(ExpectedValue, true);
	if (bIsLock)
	{
		bIsLockedBySlateThread = IsInSlateThread();
	}
	return bIsLock;
}

void FSlateDrawBuffer::Unlock(const UE::Tasks::FTask& PrerequisiteTask)
{
	// Launch an inline task that will unlock the buffer when the prerequisite tasks have completed.
	UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]
	{
		ensureMsgf(IsLocked(), TEXT("The SlateDrawBuffer should be lock before modifying it."));
		FScopeLock ScopeLock(&GCLock);

		// Rendering doesn't need the batch data anymore
		for (TSharedRef<FSlateWindowElementList>& ExistingElementList : WindowElementLists)
		{
			ExistingElementList->ResetElementList();
		}

		bIsLocked = false;

	}, PrerequisiteTask, UE::Tasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::Inline);
}

void FSlateDrawBuffer::AddReferencedObjects(FReferenceCollector& Collector)
{
	// Locked buffers are the only ones that are currently referencing objects.
	//If unlocked, the element list is not in-use and contains "to-be-cleared" data.
	if(bIsLocked && !bIsLockedBySlateThread)
	{
		FScopeLock ScopeLock(&GCLock);
		for (TSharedRef<FSlateWindowElementList>& ElementList : WindowElementLists)
		{
			ElementList->AddReferencedObjects(Collector);
		}
	}
}

void FSlateDrawBuffer::ClearBuffer()
{
	ensureMsgf(IsLocked(), TEXT("The SlateDrawBuffer should be lock before modifying it."));
	FScopeLock ScopeLock(&GCLock);

	// Remove any window elements that are no longer valid.
	for (int32 WindowIndex = 0; WindowIndex < WindowElementListsPool.Num(); ++WindowIndex)
	{
		if (WindowElementListsPool[WindowIndex]->GetPaintWindow() == nullptr)
		{
			WindowElementListsPool.RemoveAtSwap(WindowIndex);
			--WindowIndex;
		}
	}

	// Move all the window elements back into the pool.
	for (TSharedRef<FSlateWindowElementList> ExistingList : WindowElementLists)
	{
		if (ExistingList->GetPaintWindow() != nullptr)
		{
			WindowElementListsPool.Add(ExistingList);
		}
	}

	WindowElementLists.Reset();
}


void FSlateDrawBuffer::UpdateResourceVersion(uint32 NewResourceVersion)
{
	ensureMsgf(IsLocked(), TEXT("The SlateDrawBuffer should be lock before modifying it."));

	if (IsInGameThread() && NewResourceVersion != ResourceVersion)
	{
		WindowElementListsPool.Empty();
		ResourceVersion = NewResourceVersion;
	}
}
