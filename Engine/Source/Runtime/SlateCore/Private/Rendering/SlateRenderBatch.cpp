// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SlateRenderBatch.h"
#include "Rendering/DrawElements.h"
#include "Rendering/DrawElementPayloads.h"
#include "Rendering/ElementBatcher.h"
#include "Textures/SlateShaderResource.h"

FSlateRenderBatch::FSlateRenderBatch(
	int32 InLayer,
	const FShaderParams& InShaderParams,
	const FSlateShaderResource* InResource,
	ESlateDrawPrimitive InPrimitiveType,
	ESlateShader InShaderType,
	ESlateDrawEffect InDrawEffects,
	ESlateBatchDrawFlag InDrawFlags,
	int8 InSceneIndex,
	FSlateVertexArray* InSourceVertexArray,
	FSlateIndexArray* InSourceIndexArray,
	int32 InVertexOffset,
	int32 InIndexOffset) 
	: ShaderParams(InShaderParams)
	, DynamicOffset(EForceInit::ForceInitToZero)
	, ClippingState(nullptr)
	, ShaderResource(InResource)
	, InstanceData(nullptr)
	, SourceVertices(InSourceVertexArray)
	, SourceIndices(InSourceIndexArray)
	, CustomDrawer(nullptr)
	, LayerId(InLayer)
	, VertexOffset(InVertexOffset)
	, IndexOffset(InIndexOffset)
	, NumVertices(0)
	, NumIndices(0)
	, NextBatchIndex(INDEX_NONE)
	, InstanceCount(0)
	, InstanceOffset(0)
	, DrawFlags(InDrawFlags)
	, ShaderType(InShaderType)
	, DrawPrimitiveType(InPrimitiveType)
	, DrawEffects(InDrawEffects)
	, CachedUsedPostBuffers(ESlatePostRT::None)
	, SceneIndex(InSceneIndex)
	, bIsMergable(true)
	, bIsMerged(false)
{
	check(ShaderResource == nullptr || !ShaderResource->Debug_IsDestroyed());
}
