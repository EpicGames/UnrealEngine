// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SlateRenderer.h"
#include "Textures/TextureAtlas.h"
#include "Fonts/FontCache.h"
#include "Fonts/FontMeasure.h"
#include "Widgets/SWindow.h"

DEFINE_STAT(STAT_SlatePreFullBufferTime);

/* FSlateFontCacheProvider interface
 *****************************************************************************/

FSlateFontServices::FSlateFontServices(TSharedRef<class FSlateFontCache> InGameThreadFontCache, TSharedRef<class FSlateFontCache> InRenderThreadFontCache)
	: GameThreadFontCache(InGameThreadFontCache)
	, RenderThreadFontCache(InRenderThreadFontCache)
	, GameThreadFontMeasure(FSlateFontMeasure::Create(GameThreadFontCache))
	, RenderThreadFontMeasure((GameThreadFontCache == RenderThreadFontCache) ? GameThreadFontMeasure : FSlateFontMeasure::Create(RenderThreadFontCache))
{
	UE_LOGF(LogSlate, Log, "SlateFontServices - WITH_FREETYPE: %d, WITH_HARFBUZZ: %d", WITH_FREETYPE, WITH_HARFBUZZ);
	
	GameThreadFontCache->OnReleaseResources().AddRaw(this, &FSlateFontServices::HandleFontCacheReleaseResources);

	if (GameThreadFontCache != RenderThreadFontCache)
	{
		RenderThreadFontCache->OnReleaseResources().AddRaw(this, &FSlateFontServices::HandleFontCacheReleaseResources);
	}
}


FSlateFontServices::~FSlateFontServices()
{
	GameThreadFontCache->OnReleaseResources().RemoveAll(this);

	if (GameThreadFontCache != RenderThreadFontCache)
	{
		RenderThreadFontCache->OnReleaseResources().RemoveAll(this);
	}
}


TSharedRef<FSlateFontCache> FSlateFontServices::GetFontCache() const
{
	const ESlateTextureAtlasThreadId AtlasThreadId = GetCurrentSlateTextureAtlasThreadId();
	check(AtlasThreadId != ESlateTextureAtlasThreadId::Unknown);

	if (AtlasThreadId == ESlateTextureAtlasThreadId::Game)
	{
		return GameThreadFontCache;
	}
	else
	{
		return RenderThreadFontCache;
	}
}


TSharedRef<class FSlateFontMeasure> FSlateFontServices::GetFontMeasureService() const
{
	const ESlateTextureAtlasThreadId AtlasThreadId = GetCurrentSlateTextureAtlasThreadId();
	check(AtlasThreadId != ESlateTextureAtlasThreadId::Unknown);

	if (AtlasThreadId == ESlateTextureAtlasThreadId::Game)
	{
		return GameThreadFontMeasure;
	}
	else
	{
		return RenderThreadFontMeasure;
	}
}


void FSlateFontServices::FlushFontCache(const FString& FlushReason)
{
	const ESlateTextureAtlasThreadId AtlasThreadId = GetCurrentSlateTextureAtlasThreadId();
	check(AtlasThreadId != ESlateTextureAtlasThreadId::Unknown);

	if (AtlasThreadId == ESlateTextureAtlasThreadId::Game)
	{
		return FlushGameThreadFontCache(FlushReason);
	}
	else
	{
		return FlushRenderThreadFontCache(FlushReason);
	}
}


void FSlateFontServices::FlushGameThreadFontCache(const FString& FlushReason)
{
	GameThreadFontCache->RequestFlushCache(FlushReason);
	GameThreadFontMeasure->FlushCache();
}


void FSlateFontServices::FlushRenderThreadFontCache(const FString& FlushReason)
{
	RenderThreadFontCache->RequestFlushCache(FlushReason);
	RenderThreadFontMeasure->FlushCache();
}


void FSlateFontServices::ReleaseResources()
{
	GameThreadFontCache->ReleaseResources();

	if (GameThreadFontCache != RenderThreadFontCache)
	{
		RenderThreadFontCache->ReleaseResources();
	}
}


FOnReleaseFontResources& FSlateFontServices::OnReleaseResources()
{
	return OnReleaseResourcesDelegate;
}


void FSlateFontServices::HandleFontCacheReleaseResources(const FSlateFontCache& InFontCache)
{
	OnReleaseResourcesDelegate.Broadcast(InFontCache);
}


/* FSlateRenderer interface
 *****************************************************************************/

FSlateRenderer::FSlateRenderer(const TSharedRef<FSlateFontServices>& InSlateFontServices)
	: SlateFontServices(InSlateFontServices)
{
	SlateFontServices->OnReleaseResources().AddRaw(this, &FSlateRenderer::HandleFontCacheReleaseResources);
}


FSlateRenderer::~FSlateRenderer()
{
	if (SlateFontServices)
	{
		SlateFontServices->OnReleaseResources().RemoveAll(this);
	}
}


void FSlateRenderer::HandleFontCacheReleaseResources(const class FSlateFontCache& InFontCache)
{
	FlushCommands();
}


bool FSlateRenderer::IsViewportFullscreen( const SWindow& Window ) const
{
	checkSlow( IsThreadSafeForSlateRendering() );

	bool bFullscreen = false;

	if (FPlatformProperties::SupportsWindowedMode())
	{
		if( GIsEditor)
		{
			bFullscreen = false;
		}
		else
		{
			bFullscreen = Window.GetNativeWindow()->IsFullscreenSupported() && Window.GetWindowMode() == EWindowMode::Fullscreen;

#if PLATFORM_WINDOWS
			// When we are in fullscreen mode but the user alt-tabs out we need to temporarily drop out of fullscreen while the window has lost focus, otherwise DXGI will eventually
			// forcibly throw us out of fullscreen mode with device loss and crash as typical result. By returning false here we'll trigger a mode switch to windowed when the user
			// alt-tabs, and back to fullscreen again once the window comes back in focus, through the regular path. DXGI will never need to intervene and everyone is happy.
			bFullscreen = bFullscreen && Window.GetNativeWindow()->IsForegroundWindow();
#endif
		}
	}
	else
	{
		bFullscreen = true;
	}

	return bFullscreen;
}


ISlateAtlasProvider* FSlateRenderer::GetTextureAtlasProvider()
{
	return nullptr;
}


ISlateAtlasProvider* FSlateRenderer::GetFontAtlasProvider()
{
	return &SlateFontServices->GetGameThreadFontCache().Get();
}

void FSlateRenderer::DestroyCachedFastPathRenderingData(struct FSlateCachedFastPathRenderingData* FastPathRenderingData)
{
	check(FastPathRenderingData);
	delete FastPathRenderingData;
}

void FSlateRenderer::DestroyCachedFastPathElementData(struct FSlateCachedElementData* ElementData)
{
	check(ElementData);
	delete ElementData;
}

/* Global functions
 *****************************************************************************/

bool IsThreadSafeForSlateRendering()
{
	return ( ( GSlateLoadingThreadId != 0 ) || IsInGameThread() );
}

bool DoesThreadOwnSlateRendering()
{
	if ( IsInGameThread() )
	{
		return GSlateLoadingThreadId == 0;
	}
	else
	{
		return FPlatformTLS::GetCurrentThreadId() == GSlateLoadingThreadId;
	}
}
