// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SlateSVGRasterizer.h"

#if !UE_SERVER
THIRD_PARTY_INCLUDES_START
#include <nanosvg.h>
#include <nanosvgrast.h>
THIRD_PARTY_INCLUDES_END
#endif

#include "Misc/FileHelper.h"
#include "Containers/StringConv.h"
#include "SlateGlobals.h"

#if !UE_SERVER
namespace SVGConstants
{
	// The DPI is only used for unit conversion
	static const float DPI = 96.0f;
	// Bytes per pixel (RGBA)
	static const int32 BPP = 4;
}
#endif

TArray<uint8> FSlateSVGRasterizer::RasterizeSVGFromFile(const FString& Filename, FIntPoint PixelSize)
{
	TArray<uint8> PixelData;

	FString SVGString;
	if (FFileHelper::LoadFileToString(SVGString, *Filename))
	{
#if !UE_SERVER
		NSVGimage* Image = nsvgParse(TCHAR_TO_ANSI(*SVGString), "px", SVGConstants::DPI);

		if (Image)
		{
			PixelData.AddUninitialized(PixelSize.X * PixelSize.Y * SVGConstants::BPP);

			NSVGrasterizer* Rasterizer = nsvgCreateRasterizer();

			const float SVGScaleX = (float)PixelSize.X / Image->width;
			const float SVGScaleY = (float)PixelSize.Y / Image->height;

			const int32 Stride = PixelSize.X * SVGConstants::BPP;
			nsvgRasterizeFull(Rasterizer, Image, 0, 0, SVGScaleX, SVGScaleY, PixelData.GetData(), PixelSize.X, PixelSize.Y, Stride);

			nsvgDeleteRasterizer(Rasterizer);

			nsvgDelete(Image);
		}
#else
		UE_LOGF(LogSlate, Warning, "Unable to rasterize '%ls'. Nanosvg library not available in server build.", *Filename);
#endif
	}
	else
	{
		UE_LOGF(LogSlate, Warning, "Unable to rasterize '%ls'. File could not be found", *Filename);
	}

	return PixelData;
}

