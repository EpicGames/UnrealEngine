// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rendering/SlateVectorGraphicsCache.h"
#include "Rendering/ShaderResourceManager.h"
#include "Rendering/SlateSVGRasterizer.h"
#include "Textures/TextureAtlas.h"
#include "Types/SlateVector2.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Async/ParallelFor.h"
#include "Application/SlateApplicationBase.h"

static FAtlasFlushParams SVGAtlasFlushParams;
FAutoConsoleVariableRef CVarMaxSVGAtlasPagesBeforeFlush(
	TEXT("Slate.MaxSVGAtlasPagesBeforeFlush"),
	SVGAtlasFlushParams.InitialMaxAtlasPagesBeforeFlushRequest,
	TEXT("The number of atlas textures created and used before we flush the cache if a texture atlas is full"));

FAutoConsoleVariableRef CVarMaxSVGNonAtlasPagesBeforeFlush(
	TEXT("Slate.MaxSVGNonAtlasTexturesBeforeFlush"),
	SVGAtlasFlushParams.InitialMaxNonAtlasPagesBeforeFlushRequest,
	TEXT("The number of large textures initially."));

FAutoConsoleVariableRef CVarGrowSVGAtlasFrameWindow(
	TEXT("Slate.GrowSVGAtlasFrameWindow"),
	SVGAtlasFlushParams.GrowAtlasFrameWindow,
	TEXT("The number of frames within the atlas will resize rather than flush."));

FAutoConsoleVariableRef CVarGrowSVGNonAtlasFrameWindow(
	TEXT("Slate.GrowSVGNonAtlasFrameWindow"),
	SVGAtlasFlushParams.GrowNonAtlasFrameWindow,
	TEXT("The number of frames within the large pool will resize rather than flush."));

FSlateVectorGraphicsCache::FSlateVectorGraphicsCache(TSharedPtr<ISlateTextureAtlasFactory> InAtlasFactory)
	: FSlateFlushableAtlasCache(&SVGAtlasFlushParams)
	, AtlasFactory(InAtlasFactory)
	, bFlushRequested(false)
{
}

//FSlateShaderResourceProxy* FSlateVectorGraphicsCache::GetShaderResource(const FSlateBrush& Brush, FVector2d LocalSize, float DrawScale)
//{
//	return GetShaderResource(Brush, UE::Slate::CastToVector2f(LocalSize), DrawScale);
//}

FSlateShaderResourceProxy* FSlateVectorGraphicsCache::GetShaderResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale)
{
	FSlateShaderResourceProxy* Proxy = nullptr;

	static double TotalTime = 0;
	if (Brush.GetImageType() == ESlateBrushImageType::Vector)
	{
		const FVectorCacheKey CacheKey(Brush.GetResourceName(), LocalSize, DrawScale);
		TUniquePtr<FSlateShaderResourceProxy>* ProxyPtr = ResourceMap.Find(CacheKey);

		if (ProxyPtr)
		{
			Proxy = ProxyPtr->Get();
		}
		else
		{
			if(LocalSize.X > 0 && LocalSize.Y > 0)
			{
				PendingRequests.Emplace(Brush.GetResourceName(), LocalSize, DrawScale);

				TUniquePtr<FSlateShaderResourceProxy> NewProxy = MakeUnique<FSlateShaderResourceProxy>();

				Proxy = NewProxy.Get();

				ResourceMap.Add(PendingRequests.Last().Key, MoveTemp(NewProxy));
			}
		}

	}

	return Proxy;
}

void FSlateVectorGraphicsCache::UpdateCache()
{
	double Time = 0;
	if(PendingRequests.Num() > 0)
	{
		const int32 AtlasStride = 4;
		const int32 AtlasSize = 1024;
		const uint8 Padding = 1;

		FScopedDurationTimer Logger(Time);
		ParallelFor(PendingRequests.Num(), [this](int32 Index)
			{
				FRasterRequest& CurrentRequest = PendingRequests[Index];
				CurrentRequest.PixelData = FSlateSVGRasterizer::RasterizeSVGFromFile(CurrentRequest.Key.BrushName.ToString(), CurrentRequest.Key.PixelSize);

			});

		for (const FRasterRequest& Request : PendingRequests)
		{
			const TArray<uint8>& PixelData = Request.PixelData;
			FIntPoint PixelSize = Request.Key.PixelSize;

			if (PixelData.Num())
			{
				if (PixelSize.X > AtlasSize || PixelSize.Y > AtlasSize)
				{
					TUniquePtr<FSlateShaderResource> NewResource = AtlasFactory->CreateNonAtlasedTexture(PixelSize.X, PixelSize.Y, PixelData);

					// create proxy, put it in map
					TUniquePtr<FSlateShaderResourceProxy>& NewProxy = ResourceMap.FindChecked(Request.Key);
					NewProxy->Resource = NewResource.Get();

					NewProxy->StartUV = FVector2f(0, 0);
					NewProxy->SizeUV = FVector2f(1, 1);
					NewProxy->ActualSize = FIntPoint(PixelSize.X, PixelSize.Y);

					NonAtlasedTextures.Add(MoveTemp(NewResource));
				}
				else
				{
					const FAtlasedTextureSlot* NewSlot = nullptr;
					FSlateTextureAtlas* FoundAtlas = nullptr;
					for (TUniquePtr<FSlateTextureAtlas>& Atlas : Atlases)
					{
						NewSlot = Atlas->AddTexture(PixelSize.X, PixelSize.Y, PixelData);
						if (NewSlot)
						{
							FoundAtlas = Atlas.Get();
						}
					}

					if (!NewSlot)
					{
						const bool bUpdatesAfterInitialization = true;
						TUniquePtr<FSlateTextureAtlas> NewAtlas = AtlasFactory->CreateTextureAtlas(AtlasSize, AtlasStride, ESlateTextureAtlasPaddingStyle::DilateBorder, bUpdatesAfterInitialization);
						NewSlot = NewAtlas->AddTexture(PixelSize.X, PixelSize.Y, PixelData);

						FoundAtlas = NewAtlas.Get();

						Atlases.Add(MoveTemp(NewAtlas));

						UpdateFlushCounters(0, Atlases.Num(), 0, 0);
					}


					if (NewSlot)
					{
						// create proxy, put it in map
						TUniquePtr<FSlateShaderResourceProxy>& NewProxy = ResourceMap.FindChecked(Request.Key);

						NewProxy->Resource = FoundAtlas->GetAtlasTexture();
						// Compute the sub-uvs for the location of this texture in the atlas, accounting for padding
						NewProxy->StartUV = FVector2f((float)(NewSlot->X + Padding) / FoundAtlas->GetWidth(), (float)(NewSlot->Y + Padding) / FoundAtlas->GetHeight());
						NewProxy->SizeUV = FVector2f((float)(NewSlot->Width - Padding * 2) / FoundAtlas->GetWidth(), (float)(NewSlot->Height - Padding * 2) / FoundAtlas->GetHeight());
						NewProxy->ActualSize = FIntPoint(PixelSize.X, PixelSize.Y);

#if WITH_ATLAS_DEBUGGING
						AtlasDebugData.Add(NewSlot, Request.Key.BrushName);
#endif
					}
				}
				UpdateFlushCounters(0, Atlases.Num(), 0, NonAtlasedTextures.Num());
			}
		}
	}

	if (PendingRequests.Num() > 0)
	{
		UE_LOGF(LogSlate, Verbose, "SVG raster took: %fms", Time * 1000.f);
	}
	PendingRequests.Empty();


	for (TUniquePtr<FSlateTextureAtlas>& Atlas : Atlases)
	{
		Atlas->ConditionalUpdateTexture();
	}
}

SLATECORE_API void FSlateVectorGraphicsCache::ConditionalFlushCache()
{
	if (bFlushRequested)
	{
		FlushCache();
		bFlushRequested = false;
	}
}

void FSlateVectorGraphicsCache::RequestFlushCache(const FString& Reason)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_LOGF(LogSlate, Log, "Vector cache flush requested. Reason: %ls", *Reason);
#else
	UE_LOGF(LogSlate, Warning, "Vector cache flush requested. Reason: %ls", *Reason);
#endif

	bFlushRequested = true;
}

void FSlateVectorGraphicsCache::FlushCache()
{
	// Ensure all invalidation panels are cleared of cached widgets
	FSlateApplicationBase::Get().InvalidateAllWidgets(false);

	const bool bWaitForRelease = true;
	ReleaseResources(bWaitForRelease);

	DeleteResources();
}

void FSlateVectorGraphicsCache::ReleaseResources(bool bWaitForRelease /* = false */)
{
	AtlasFactory->ReleaseTextureAtlases(Atlases, NonAtlasedTextures, bWaitForRelease);
}

void FSlateVectorGraphicsCache::DeleteResources()
{
#if WITH_ATLAS_DEBUGGING
	AtlasDebugData.Empty();
#endif
	Atlases.Empty();
	NonAtlasedTextures.Empty();
	ResourceMap.Empty();
}

