// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "SlateGlobals.h"

#include "HAL/IConsoleManager.h"
#include "Styling/SlateWidgetStyle.h"
#include "Types/SlateConstants.h"

#if WITH_SLATE_DEBUGGING
#include "Containers/StringFwd.h"
#include "Misc/OutputDeviceRedirector.h"
#endif

/** How much to scroll for each click of the mouse wheel (in Slate Screen Units). */
TAutoConsoleVariable<float> GlobalScrollAmount(
	TEXT("Slate.GlobalScrollAmount"),
	32.0f,
	TEXT("How much to scroll for each click of the mouse wheel (in Slate Screen Units)."));

/** How much to scroll for each click of the mouse wheel (in Slate Screen Units). */
TAutoConsoleVariable<float> GlobalGamepadScrollMultiplier(
	TEXT("Slate.GlobalGamepadScrollMultiplier"),
	1.0f,
	TEXT("How much to multiply scroll amount when using gamepad."));


float GSlateContrast = 1;

FAutoConsoleVariableRef CVarSlateContrast(
	TEXT("Slate.Contrast"),
	GSlateContrast,
	TEXT("The amount of contrast to apply to the UI (default 1).")
);

// When async lazily loading fonts, when we finish we bump the generation version to
// tell the text layout engine that we need a new pass now that new glyphs will actually
// be available now to measure and render.
int32 GSlateLayoutGeneration = 0;

// Enable fast widget paths outside the editor by default.  Only reason we don't enable them everywhere
// is that the editor is more complex than a game, and there are likely a larger swath of edge cases.
bool GSlateFastWidgetPath = false;

FAutoConsoleVariableRef CVarSlateFastWidgetPath(
	TEXT("Slate.EnableFastWidgetPath"),
	GSlateFastWidgetPath,
	TEXT("Whether or not we enable fast widget pathing.  This mode relies on parent pointers to work correctly.")
);


bool GSlateEnableGlobalInvalidation = false;
static FAutoConsoleVariableRef CVarSlateNewUpdateMethod(
	TEXT("Slate.EnableGlobalInvalidation"), 
	GSlateEnableGlobalInvalidation, 
	TEXT("")
);

bool GSlateIsOnFastUpdatePath = false;
bool GSlateIsOnFastProcessInvalidation = false;
bool GSlateIsInInvalidationSlowPath = false;

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
bool GSlateCheckUObjectRenderResources = true;
static FAutoConsoleVariableRef CVarSlateCheckUObjectRenderResources(
	TEXT("Slate.CheckUObjectRenderResources"),
	GSlateCheckUObjectRenderResources,
	TEXT("")
);

bool GSlateCheckUObjectRenderResourcesShouldLogFatal = false;
#endif

#if SLATE_CHECK_UOBJECT_SHAPED_GLYPH_SEQUENCE
bool GSlateCheckUObjectShapedGlyphSequence = true;
static FAutoConsoleVariableRef CVarSlateCheckUObjectShapedGlyphSequence(
	TEXT("Slate.CheckUObjectShapedGlyphSequence"),
	GSlateCheckUObjectShapedGlyphSequence,
	TEXT("")
);
#endif

#if WITH_SLATE_DEBUGGING
FAutoConsoleVariable CVarInvalidationDebugging(
	TEXT("Slate.InvalidationDebugging"),
	false,
	TEXT("Deprecated - Use SlateDebugger.Invalidate.Enable"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		TStringBuilder<64> Builder;
		Builder << TEXT("SlateDebugger.Invalidate.Enable ")
				<< Variable->GetBool();
		IConsoleManager::Get().ProcessUserConsoleInput(Builder.GetData(), *GLog, nullptr);
	}));

bool GSlateHitTestGridDebugging = false;
/** True if we should allow widgets to be cached in the UI at all. */
FAutoConsoleVariableRef CVarHitTestGridDebugging(
	TEXT("Slate.HitTestGridDebugging"),
	GSlateHitTestGridDebugging,
	TEXT("Whether to show a visualization of everything in the hit teest grid"));
#endif

#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
bool GSlateNavigationDebugging = false;
FAutoConsoleVariableRef CVarNavigationDebugging(
	TEXT("Slate.NavigationDebugging"),
	GSlateNavigationDebugging,
	TEXT("Whether to show a visualization of navigation from current widget"));
#endif

bool GSlateUseSharedBreakIterator = false;
static FAutoConsoleVariableRef CVarUseSharedBreakIterator(
	TEXT("Slate.UseSharedBreakIterator"),
	GSlateUseSharedBreakIterator,
	TEXT("If true, reuse cached break iterators to reduce memory overhead")
);


FSlateWidgetStyle::FSlateWidgetStyle() = default;
FSlateWidgetStyle::~FSlateWidgetStyle() = default;

