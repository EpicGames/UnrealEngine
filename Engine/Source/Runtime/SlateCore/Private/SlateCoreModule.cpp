// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlateCoreModule.h"
#include "Debugging/ConsoleSlateDebugger.h"
#include "Debugging/ConsoleSlateDebuggerInvalidate.h"
#include "Debugging/ConsoleSlateDebuggerInvalidationRoot.h"
#include "Debugging/ConsoleSlateDebuggerPaint.h"
#include "Debugging/ConsoleSlateDebuggerPrepass.h"
#include "Debugging/ConsoleSlateDebuggerUpdate.h"
#include "Debugging/ConsoleSlateDebuggerBreak.h"
#include "Modules/ModuleManager.h"
#include "SlateGlobals.h"
#include "Types/SlateStructs.h"

DEFINE_LOG_CATEGORY(LogSlate);
DEFINE_LOG_CATEGORY(LogSlateStyles);

const float FOptionalSize::Unspecified = -1.0f;

namespace
{
	UE::FSlateCoreModule* SlateCoreModuleInstance;
}

namespace UE
{
	FSlateCoreModule& FSlateCoreModule::Get()
	{
		check(SlateCoreModuleInstance);
		return *SlateCoreModuleInstance;
	}

	FSlateCoreModule::FSlateCoreModule()
	{
#if WITH_SLATE_DEBUGGING
		SlateDebuggerEvent = MakeUnique<FConsoleSlateDebugger>();
		SlateDebuggerInvalidate = MakeUnique<FConsoleSlateDebuggerInvalidate>();
		SlateDebuggerInvalidationRoot = MakeUnique<FConsoleSlateDebuggerInvalidationRoot>();
		SlateDebuggerPaint = MakeUnique<FConsoleSlateDebuggerPaint>();
		SlateDebuggerPaint->LoadConfig();
		SlateDebuggerPrepass = MakeUnique<FConsoleSlateDebuggerPrepass>();
		SlateDebuggerPrepass->LoadConfig();
		SlateDebuggerUpdate = MakeUnique<FConsoleSlateDebuggerUpdate>();
		SlateDebuggerBreak = MakeUnique<FConsoleSlateDebuggerBreak>();
#endif
	}

	void FSlateCoreModule::StartupModule()
	{
		SlateCoreModuleInstance = this;
	}

	void FSlateCoreModule::ShutdownModule()
	{
		SlateCoreModuleInstance = nullptr;
	}

#if WITH_SLATE_DEBUGGING
	FConsoleSlateDebuggerPaint& FSlateCoreModule::GetPaintDebugger() const
	{
		return *SlateDebuggerPaint;
	}

	FConsoleSlateDebuggerPrepass& FSlateCoreModule::GetPrepassDebugger() const
	{
		return *SlateDebuggerPrepass;
	}

	FConsoleSlateDebuggerUpdate& FSlateCoreModule::GetUpdateDebugger() const
	{
		return *SlateDebuggerUpdate;
	}
#endif
}


IMPLEMENT_MODULE(UE::FSlateCoreModule, SlateCore);
