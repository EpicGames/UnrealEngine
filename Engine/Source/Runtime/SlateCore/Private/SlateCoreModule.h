// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Debugging/SlateDebugging.h"
#include "Modules/ModuleInterface.h"

#if WITH_SLATE_DEBUGGING
class FConsoleSlateDebugger;
class FConsoleSlateDebuggerInvalidate;
class FConsoleSlateDebuggerInvalidationRoot;
class FConsoleSlateDebuggerPaint;
class FConsoleSlateDebuggerPrepass;
class FConsoleSlateDebuggerUpdate;
class FConsoleSlateDebuggerBreak;
#endif

namespace UE
{
	class FSlateCoreModule : public IModuleInterface
	{
	public:
		static FSlateCoreModule& Get();

		FSlateCoreModule();
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

#if WITH_SLATE_DEBUGGING
		FConsoleSlateDebuggerPaint& GetPaintDebugger() const;
		FConsoleSlateDebuggerPrepass& GetPrepassDebugger() const;
		FConsoleSlateDebuggerUpdate& GetUpdateDebugger() const;

	private:
		TUniquePtr<FConsoleSlateDebugger> SlateDebuggerEvent;
		TUniquePtr<FConsoleSlateDebuggerInvalidate> SlateDebuggerInvalidate;
		TUniquePtr<FConsoleSlateDebuggerInvalidationRoot> SlateDebuggerInvalidationRoot;
		TUniquePtr<FConsoleSlateDebuggerPaint> SlateDebuggerPaint;
		TUniquePtr<FConsoleSlateDebuggerPrepass> SlateDebuggerPrepass;
		TUniquePtr<FConsoleSlateDebuggerUpdate> SlateDebuggerUpdate;
		TUniquePtr<FConsoleSlateDebuggerBreak> SlateDebuggerBreak;
#endif
	};
}
