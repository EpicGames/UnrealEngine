// Copyright Epic Games, Inc. All Rights Reserved.

#include "SlotBase.h"
#include "Layout/Children.h"
#include "Widgets/SWidget.h"
#include "Widgets/SNullWidget.h"

FSlotBase::FSlotBase()
	: Owner(nullptr)
	, Widget(SNullWidget::NullWidget)
{

}

FSlotBase::FSlotBase(const FChildren& Children)
	: Owner(&Children)
	, Widget(SNullWidget::NullWidget)
{

}

FSlotBase::FSlotBase( const TSharedRef<SWidget>& InWidget )
	: Owner(nullptr)
	, Widget(InWidget)
{

}

SWidget* FSlotBase::GetOwnerWidget() const
{
	return GetOwner() ? &(GetOwner()->GetOwner()) : nullptr;
}

void FSlotBase::SetOwner(const FChildren& InChildren)
{
	if (Owner != &InChildren)
	{
		if (ensureMsgf(Owner == nullptr, TEXT("Slots should not be reassigned to different parents.")))
		{
			Owner = &InChildren;
			AfterContentOrOwnerAssigned();
		}
	}
}

const TSharedPtr<SWidget> FSlotBase::DetachWidget()
{
	if (Widget != SNullWidget::NullWidget)
	{
		Widget->ConditionallyDetatchParentWidget(GetOwnerWidget());

		// Invalidate Prepass?

		const TSharedRef<SWidget> MyExWidget = Widget;
		Widget = SNullWidget::NullWidget;	
		return MyExWidget;
	}
	else
	{
		// Nothing to detach!
		return TSharedPtr<SWidget>();
	}
}

void FSlotBase::Invalidate(EInvalidateWidgetReason InvalidateReason)
{
	// If a slot invalidates it needs to invalidate the parent of widget of its content.
	if (SWidget* OwnerWidget = GetOwnerWidget())
	{
		OwnerWidget->Invalidate(InvalidateReason);
	}
}

void FSlotBase::DetatchParentFromContent()
{
	if (Widget != SNullWidget::NullWidget)
	{
		Widget->ConditionallyDetatchParentWidget(GetOwnerWidget());
	}
}

void FSlotBase::AfterContentOrOwnerAssigned()
{
	if (SWidget* OwnerWidget = GetOwnerWidget())
	{
		if (Widget != SNullWidget::NullWidget)
		{
			// TODO NDarnell I want to enable this, but too many places in the codebase
			// have made assumptions about being able to freely reparent widgets, while they're
			// still connected to an existing hierarchy.
			//ensure(!Widget->IsParentValid());
			Widget->AssignParentWidget(OwnerWidget->AsShared());
		}
	}
}

FSlotBase::~FSlotBase()
{
	DetatchParentFromContent();
}
