// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/NullSlateSoundDevice.h"

void FNullSlateSoundDevice::PlaySound(const FSlateSound&, int32) const
{
}

float FNullSlateSoundDevice::GetSoundDuration(const FSlateSound& Sound) const
{
	return 0.0f;
}

FNullSlateSoundDevice::~FNullSlateSoundDevice()
{
}
