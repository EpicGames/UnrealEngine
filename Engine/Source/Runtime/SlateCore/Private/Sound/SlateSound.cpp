// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sound/SlateSound.h"
#include "UObject/PropertyTag.h"
#include "SlateGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateSound)


/* FSlateSound interface
 *****************************************************************************/

FSlateSound::FSlateSound()
	: ResourceObject(nullptr)
{
}


FSlateSound::FSlateSound(const FSlateSound&) = default;

FSlateSound& FSlateSound::operator=(const FSlateSound&) = default;

UObject* FSlateSound::GetResourceObject( ) const
{
	// We might still be holding a legacy resource name, in which case we need to test that first and load it if required
	if (LegacyResourceName_DEPRECATED != NAME_None)
	{
		// Do we still have a valid reference in our weak-ptr?
		UObject* LegacyResourceObject = LegacyResourceObject_DEPRECATED.Get();

		if (!LegacyResourceObject)
		{
			if (!IsInGameThread())
			{
				UE_LOGF(LogSlate, Warning, "Can't find/load sound %ls because Slate is being updated in another thread! (loading screen?)", *LegacyResourceName_DEPRECATED.ToString());
			}
			else
			{
				// We can't check the object type against USoundBase as we don't have access to it here
				// The user is required to cast the result of FSlateSound::GetResourceObject so we should be fine
				LegacyResourceObject = StaticFindObject(UObject::StaticClass(), nullptr, *LegacyResourceName_DEPRECATED.ToString());
				if (!ResourceObject)
				{
					LegacyResourceObject = StaticLoadObject(UObject::StaticClass(), nullptr, *LegacyResourceName_DEPRECATED.ToString());
				}

				// Cache this in the weak-ptr to try and avoid having to load it all the time
				LegacyResourceObject_DEPRECATED = LegacyResourceObject;
			}
		}

		return LegacyResourceObject;
	}

	return ResourceObject;
}


bool FSlateSound::SerializeFromMismatchedTag(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot)
{
	// Sounds in Slate used to be stored as FName properties, so allow them to be upgraded in-place
	if (Tag.Type == NAME_NameProperty)
	{
		FName SoundName;
		Slot << SoundName;
		*this = FromName_DEPRECATED(SoundName);

		return true;
	}

	return false;
}


void FSlateSound::StripLegacyData_DEPRECATED( )
{
	LegacyResourceName_DEPRECATED = NAME_None;
	LegacyResourceObject_DEPRECATED.Reset();
}



/* FSlateSound static functions
 *****************************************************************************/

FSlateSound FSlateSound::FromName_DEPRECATED( const FName& SoundName )
{
	FSlateSound Sound;

	// Just set the name, the sound will get loaded the first time it's required
	Sound.LegacyResourceName_DEPRECATED = SoundName;

#if WITH_EDITOR
	if (GIsEditor)
	{
		// If we're in the editor, we need to load the sound immediately so that the ResourceObject is valid for editing
		Sound.ResourceObject = Sound.GetResourceObject();
	}
#endif

	return Sound;
}

