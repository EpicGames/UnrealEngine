// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/AppStyle.h"

#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"

FName FAppStyle::AppStyleName = "CoreStyle";

const ISlateStyle& FAppStyle::Get()
{
	if (const ISlateStyle* AppStyle = FSlateStyleRegistry::FindSlateStyle(AppStyleName))
	{
		return *AppStyle;
	}

	return FCoreStyle::GetCoreStyle();
}

void FAppStyle::SetAppStyleSetName(const FName& InName)
{
	// warn style not found ?

	AppStyleName = InName;
}

const FName FAppStyle::GetAppStyleSetName()
{
	return AppStyleName;
}

void FAppStyle::SetAppStyleSet(const ISlateStyle& InStyle)
{
	AppStyleName = InStyle.GetStyleSetName();
}

