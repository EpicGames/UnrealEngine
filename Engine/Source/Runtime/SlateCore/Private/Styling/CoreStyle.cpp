// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/CoreStyle.h"
#include "SlateGlobals.h"
#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Fonts/LegacySlateFontInfoCache.h"
#include "Misc/CommandLine.h"

#define RootToContentDir Style->RootToContentDir


/* Static initialization
 *****************************************************************************/

TSharedPtr< ISlateStyle > FCoreStyle::Instance = nullptr;

namespace CoreStyleConstants
{
	// Note, these sizes are in Slate Units.
	// Slate Units do NOT have to map to pixels.
	const UE::Slate::FDeprecateVector2DResult Icon5x16(5.0f, 16.0f);
	const UE::Slate::FDeprecateVector2DResult Icon6x8(6.0f, 8.0f);
	const UE::Slate::FDeprecateVector2DResult Icon8x4(8.0f, 4.0f);
	const UE::Slate::FDeprecateVector2DResult Icon16x4(16.0f, 4.0f);
	const UE::Slate::FDeprecateVector2DResult Icon8x8(8.0f, 8.0f);
	const UE::Slate::FDeprecateVector2DResult Icon4x4(4.0f, 4.0f);
	const UE::Slate::FDeprecateVector2DResult Icon10x10(10.0f, 10.0f);
	const UE::Slate::FDeprecateVector2DResult Icon12x12(12.0f, 12.0f);
	const UE::Slate::FDeprecateVector2DResult Icon12x16(12.0f, 16.0f);
	const UE::Slate::FDeprecateVector2DResult Icon14x14(14.0f, 14.0f);
	const UE::Slate::FDeprecateVector2DResult Icon16x16(16.0f, 16.0f);
	const UE::Slate::FDeprecateVector2DResult Icon18x18(18.0f, 18.0f);
	const UE::Slate::FDeprecateVector2DResult Icon20x20(20.0f, 20.0f);
	const UE::Slate::FDeprecateVector2DResult Icon22x22(22.0f, 22.0f);
	const UE::Slate::FDeprecateVector2DResult Icon24x24(24.0f, 24.0f);
	const UE::Slate::FDeprecateVector2DResult Icon25x25(25.0f, 25.0f);
	const UE::Slate::FDeprecateVector2DResult Icon26x26(26.0f, 26.0f);
	const UE::Slate::FDeprecateVector2DResult Icon32x32(32.0f, 32.0f);
	const UE::Slate::FDeprecateVector2DResult Icon40x40(40.0f, 40.0f);
	const UE::Slate::FDeprecateVector2DResult Icon64x64(64.0f, 64.0f);
	const UE::Slate::FDeprecateVector2DResult Icon36x24(36.0f, 24.0f);
	const UE::Slate::FDeprecateVector2DResult Icon128x128(128.0f, 128.0f);

	// Common Margins
	const FMargin DefaultMargins(8.f, 4.f);
	// Buttons already have a built in (4., 2.) padding - adding to that a little
	const FMargin ButtonMargins(12.f, 1.5f, 12.f, 1.5f);

	const FMargin PressedButtonMargins(12.f, 2.5f, 12.f, 0.5f);
	const FMargin ToggleButtonMargins(16.0f, 2.0f);
	const FMargin ComboButtonMargin(8.f, 1.f, 8.f, 1.f);
	const FMargin PressedComboButtonMargin(8.f, 2.f, 8.f, 0.f);

	const float InputFocusRadius = 4.f;
	const float InputFocusThickness = 1.0f;
}

/* FSlateCoreStyle helper class
 *****************************************************************************/

class FSlateCoreStyle
	: public FSlateStyleSet
{
public:
	FSlateCoreStyle(const FName& InStyleSetName)
		: FSlateStyleSet(InStyleSetName)

		// These are the colors that are updated by the user style customizations
		, DefaultForeground_LinearRef(MakeShareable(new FLinearColor(0.72f, 0.72f, 0.72f, 1.f)))
		, InvertedForeground_LinearRef(MakeShareable(new FLinearColor(0.0f, 0.0f, 0.0f)))
		, SelectorColor_LinearRef(MakeShareable(new FLinearColor(0.701f, 0.225f, 0.003f)))
		, SelectionColor_LinearRef(MakeShareable(new FLinearColor(0.728f, 0.364f, 0.003f)))
		, SelectionColor_Inactive_LinearRef(MakeShareable(new FLinearColor(0.25f, 0.25f, 0.25f)))
		, SelectionColor_Pressed_LinearRef(MakeShareable(new FLinearColor(0.701f, 0.225f, 0.003f)))
		, HighlightColor_LinearRef(MakeShareable(new FLinearColor(0.068f, 0.068f, 0.068f)))
	{
	}

	static void SetColor( const TSharedRef<FLinearColor>& Source, const FLinearColor& Value )
	{
		Source->R = Value.R;
		Source->G = Value.G;
		Source->B = Value.B;
		Source->A = Value.A;
	}

	// These are the colors that are updated by the user style customizations
	const TSharedRef<FLinearColor> DefaultForeground_LinearRef;
	const TSharedRef<FLinearColor> InvertedForeground_LinearRef;
	const TSharedRef<FLinearColor> SelectorColor_LinearRef;
	const TSharedRef<FLinearColor> SelectionColor_LinearRef;
	const TSharedRef<FLinearColor> SelectionColor_Inactive_LinearRef;
	const TSharedRef<FLinearColor> SelectionColor_Pressed_LinearRef;
	const TSharedRef<FLinearColor> HighlightColor_LinearRef;
};


/* FSlateCoreStyle static functions
 *****************************************************************************/

TSharedRef<const FCompositeFont> FCoreStyle::GetDefaultFont()
{
	// FCoreStyle::GetDefaultFont is an alias so that the default font from FLegacySlateFontInfoCache can be acccessed outside of SlateCore (FLegacySlateFontInfoCache is private)
	return FLegacySlateFontInfoCache::Get().GetDefaultFont();
}

TSharedRef<const FCompositeFont> FCoreStyle::GetIconFont()
{
#if WITH_EDITOR
	return FLegacySlateFontInfoCache::Get().GetCompositeFont(TEXT("Editor/Slate/Fonts/FontAwesome.ttf"), EFontHinting::Default).ToSharedRef();
#else
	return GetDefaultFont();
#endif
}

FSlateFontInfo FCoreStyle::GetDefaultFontStyle(const FName InTypefaceFontName, const float InSize, const FFontOutlineSettings& InOutlineSettings)
{
	return FSlateFontInfo(GetDefaultFont(), InSize, InTypefaceFontName, InOutlineSettings);
}

FSlateFontInfo FCoreStyle::GetRegularIconFontStyle(const float InSize, const FFontOutlineSettings& InOutlineSettings)
{
	return FSlateFontInfo(GetIconFont(), InSize, "Regular", InOutlineSettings);
}

void FCoreStyle::ResetToDefault( )
{
	SetStyle(FCoreStyle::Create());
}


void FCoreStyle::SetSelectorColor( const FLinearColor& NewColor )
{
	TSharedPtr<FSlateCoreStyle> Style = StaticCastSharedPtr<FSlateCoreStyle>(Instance);
	check(Style.IsValid());

	FSlateCoreStyle::SetColor(Style->SelectorColor_LinearRef, NewColor);
}


void FCoreStyle::SetSelectionColor( const FLinearColor& NewColor )
{
	TSharedPtr<FSlateCoreStyle> Style = StaticCastSharedPtr<FSlateCoreStyle>(Instance);
	check(Style.IsValid());

	FSlateCoreStyle::SetColor(Style->SelectionColor_LinearRef, NewColor);
}


void FCoreStyle::SetInactiveSelectionColor( const FLinearColor& NewColor )
{
	TSharedPtr<FSlateCoreStyle> Style = StaticCastSharedPtr<FSlateCoreStyle>(Instance);
	check(Style.IsValid());

	FSlateCoreStyle::SetColor(Style->SelectionColor_Inactive_LinearRef, NewColor);
}


void FCoreStyle::SetPressedSelectionColor( const FLinearColor& NewColor )
{
	TSharedPtr<FSlateCoreStyle> Style = StaticCastSharedPtr<FSlateCoreStyle>(Instance);
	check(Style.IsValid());

	FSlateCoreStyle::SetColor(Style->SelectionColor_Pressed_LinearRef, NewColor);
}

void FCoreStyle::SetFocusBrush(FSlateBrush* NewBrush)
{
	TSharedRef<FSlateCoreStyle> Style = StaticCastSharedRef<FSlateCoreStyle>(Instance.ToSharedRef());
	FSlateStyleRegistry::UnRegisterSlateStyle(Style.Get());

	Style->Set("FocusRectangle", NewBrush);

	FSlateStyleRegistry::RegisterSlateStyle(Style.Get());
}


UE_DISABLE_OPTIMIZATION_SHIP

// We no longer use this style. it is kept around for reference only
#define ALLOW_LEGACY_CORE_STYLE 0

TSharedRef<ISlateStyle> FCoreStyle::Create( const FName& InStyleSetName )
{
	TSharedRef<FSlateCoreStyle> Style = MakeShareable(new FSlateCoreStyle(InStyleSetName));
	Style->SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FString CanaryPath = RootToContentDir(TEXT("Checkerboard"), TEXT(".png"));

	if (!FPaths::FileExists(CanaryPath))
	{
		// Checkerboard is the default brush so we check for that. No slate fonts are required as those will fall back properly
		UE_LOGF(LogSlate, Warning, "FCoreStyle assets not detected, skipping FCoreStyle initialization");

		return Style;
	}

#if ALLOW_LEGACY_CORE_STYLE

	// Note, these sizes are in Slate Units.
	// Slate Units do NOT have to map to pixels.
	const FVector2D Icon5x16(5.0f, 16.0f);
	const FVector2D Icon8x4(8.0f, 4.0f);
	const FVector2D Icon16x4(16.0f, 4.0f);
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon10x10(10.0f, 10.0f);
	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon12x16(12.0f, 16.0f);
	const FVector2D Icon14x14(14.0f, 14.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon22x22(22.0f, 22.0f);
	const FVector2D Icon24x24(24.0f, 24.0f);
	const FVector2D Icon25x25(25.0f, 25.0f);
	const FVector2D Icon32x32(32.0f, 32.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);
	const FVector2D Icon36x24(36.0f, 24.0f);
	const FVector2D Icon128x128(128.0f, 128.0f);

	// These are the Slate colors which reference the dynamic colors in FSlateCoreStyle; these are the colors to put into the style
	const FSlateColor DefaultForeground( Style->DefaultForeground_LinearRef );
	const FSlateColor InvertedForeground( Style->InvertedForeground_LinearRef );
	const FSlateColor SelectorColor( Style->SelectorColor_LinearRef );
	const FSlateColor SelectionColor( Style->SelectionColor_LinearRef );
	const FSlateColor SelectionColor_Inactive( Style->SelectionColor_Inactive_LinearRef );
	const FSlateColor SelectionColor_Pressed( Style->SelectionColor_Pressed_LinearRef );
	const FSlateColor HighlightColor( Style->HighlightColor_LinearRef );

	Style->Set("AppIcon", new IMAGE_BRUSH("Icons/DefaultAppIcon", Icon24x24));
	Style->Set("AppIcon.Small", new IMAGE_BRUSH("Icons/DefaultAppIcon", Icon24x24));

	Style->Set("AppIconPadding", FMargin(4, 4, 0, 0));
	Style->Set("AppIconPadding.Small", FMargin(4, 4, 0, 0));

	Style->Set("NormalFont", DEFAULT_FONT("Regular", RegularTextSize));

	Style->Set("SmallFont", DEFAULT_FONT("Regular", SmallTextSize));

	FSlateBrush* DefaultTextUnderlineBrush = new IMAGE_BRUSH("Old/White", Icon8x8, FLinearColor::White, ESlateBrushTileType::Both);
	Style->Set("DefaultTextUnderline", DefaultTextUnderlineBrush);

	// Normal Text
	const FTextBlockStyle NormalText = FTextBlockStyle()
		.SetFont(DEFAULT_FONT("Regular", RegularTextSize))
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black)
		.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
		.SetHighlightShape(BOX_BRUSH("Common/TextBlockHighlightShape", FMargin(3.f /8.f)));

	const FTextBlockStyle NormalUnderlinedText = FTextBlockStyle(NormalText)
		.SetUnderlineBrush(*DefaultTextUnderlineBrush);

	// Monospaced Text
	const FTextBlockStyle MonospacedText = FTextBlockStyle()
		.SetFont(DEFAULT_FONT("Mono", 10))
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2D::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black)
		.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
		.SetHighlightShape(BOX_BRUSH("Common/TextBlockHighlightShape", FMargin(3.f/8.f))
		);

	const FTextBlockStyle MonospacedUnderlinedText = FTextBlockStyle(MonospacedText)
		.SetUnderlineBrush(*DefaultTextUnderlineBrush);

	Style->Set("MonospacedText", MonospacedText);
	Style->Set("MonospacedUnderlinedText", MonospacedUnderlinedText);

	// Small Text
	const FTextBlockStyle SmallText = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", SmallTextSize));

	const FTextBlockStyle SmallUnderlinedText = FTextBlockStyle(SmallText)
		.SetUnderlineBrush(*DefaultTextUnderlineBrush);

	// Embossed Text
	Style->Set("EmbossedText", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 24))
		.SetColorAndOpacity(FLinearColor::Black )
		.SetShadowOffset( FVector2D(0.0f, 1.0f))
		.SetShadowColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.8f, 0.5))
		);

	// Common brushes
	FSlateBrush* GenericWhiteBox = new IMAGE_BRUSH("Old/White", Icon16x16);
	{
		Style->Set("Checkerboard", new IMAGE_BRUSH("Checkerboard", Icon16x16, FLinearColor::White, ESlateBrushTileType::Both));

		Style->Set("GenericWhiteBox", GenericWhiteBox);

		Style->Set("BlackBrush", new FSlateColorBrush(FLinearColor::Black));
		Style->Set("WhiteBrush", new FSlateColorBrush(FLinearColor::White));

		Style->Set("BoxShadow", new BOX_BRUSH( "Common/BoxShadow" , FMargin( 5.0f / 64.0f)));

		Style->Set("FocusRectangle", new BORDER_BRUSH( "Old/DashedBorder", FMargin(6.0f / 32.0f), FLinearColor(1, 1, 1, 0.5)));
	}

	// Important colors
	{
		Style->Set("DefaultForeground", DefaultForeground);
		Style->Set("InvertedForeground", InvertedForeground);

		Style->Set("SelectorColor", SelectorColor);
		Style->Set("SelectionColor", SelectionColor);
		Style->Set("SelectionColor_Inactive", SelectionColor_Inactive);
		Style->Set("SelectionColor_Pressed", SelectionColor_Pressed);
	}

	// Invisible buttons, borders, etc.
	const FButtonStyle NoBorder = FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateNoResource())
		.SetPressed(FSlateNoResource())
		.SetNormalPadding(FMargin(0.0f, 0.0f, 0.0f, 1.0f))
		.SetPressedPadding(FMargin(0.0f, 1.0f, 0.0f, 0.0f));

	// Convenient transparent/invisible elements
	{
		Style->Set("NoBrush", new FSlateNoResource());
		Style->Set("NoBorder", new FSlateNoResource());
		Style->Set("NoBorder.Normal", new FSlateNoResource());
		Style->Set("NoBorder.Hovered", new FSlateNoResource());
		Style->Set("NoBorder.Pressed", new FSlateNoResource());
		Style->Set("NoBorder", NoBorder);
	}

	
	// Demo Recording
	{
		Style->Set("DemoRecording.CursorPing", new IMAGE_BRUSH("Common/CursorPing", FVector2D(31,31)));
	}

	// Error Reporting
	{
		Style->Set("ErrorReporting.Box", new BOX_BRUSH( "Common/TextBlockHighlightShape", FMargin(3.f / 8.f)));
		Style->Set("ErrorReporting.EmptyBox", new BOX_BRUSH( "Common/TextBlockHighlightShape_Empty", FMargin(3.f / 8.f)));
		Style->Set("ErrorReporting.BackgroundColor", FLinearColor(0.35f, 0.0f, 0.0f));
		Style->Set("ErrorReporting.WarningBackgroundColor", FLinearColor(0.828f, 0.364f, 0.003f));
		Style->Set("ErrorReporting.ForegroundColor", FLinearColor::White);
	}

	// Cursor icons
	{
		Style->Set("SoftwareCursor_Grab", new IMAGE_BRUSH( "Icons/cursor_grab", Icon24x24));
		Style->Set("SoftwareCursor_CardinalCross", new IMAGE_BRUSH( "Icons/cursor_cardinal_cross", Icon24x24));
	}

	// Common icons
	{
		Style->Set("TrashCan", new IMAGE_BRUSH( "Icons/TrashCan", FVector2D(64, 64)));
		Style->Set("TrashCan_Small", new IMAGE_BRUSH( "Icons/TrashCan_Small", FVector2D(18, 18)));
	}

	// Common icons
	{
		Style->Set( "Icons.Cross", new IMAGE_BRUSH( "Icons/Cross_12x", Icon12x12 ) );
		Style->Set( "Icons.Denied", new IMAGE_BRUSH( "Icons/denied_16x", Icon16x16 ) );
		Style->Set( "Icons.Error", new IMAGE_BRUSH( "Icons/icon_error_16x", Icon16x16) );
		Style->Set( "Icons.Help", new IMAGE_BRUSH( "Icons/icon_help_16x", Icon16x16) );
		Style->Set( "Icons.Info", new IMAGE_BRUSH( "Icons/icon_info_16x", Icon16x16) );
		Style->Set( "Icons.Warning", new IMAGE_BRUSH( "Icons/icon_warning_16x", Icon16x16) );
		Style->Set( "Icons.Download", new IMAGE_BRUSH( "Icons/icon_Downloads_16x", Icon16x16) );
	}

	// Tool panels
	{
		Style->Set( "ToolPanel.GroupBorder", new BOX_BRUSH( "Common/GroupBorder", FMargin(4.0f/16.0f) ) );
		Style->Set( "Debug.Border", new BOX_BRUSH( "Common/DebugBorder", 4.0f/16.0f) );
	}

	// Popup text
	{
		Style->Set( "PopupText.Background", new BOX_BRUSH( "Old/Menu_Background", FMargin(8.0f/64.0f) ) );
	}

	// Generic command icons
	{
		Style->Set( "GenericCommands.Undo", new IMAGE_BRUSH( "Icons/icon_undo_16px", Icon16x16 ) );
		Style->Set( "GenericCommands.Redo", new IMAGE_BRUSH( "Icons/icon_redo_16px", Icon16x16 ) );

		Style->Set( "GenericCommands.Copy", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Copy_16x", Icon16x16) );
		Style->Set( "GenericCommands.Cut", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Cut_16x", Icon16x16) );
		Style->Set( "GenericCommands.Delete", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Delete_16x", Icon16x16) );
		Style->Set( "GenericCommands.Paste", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Paste_16x", Icon16x16) );
		Style->Set( "GenericCommands.Duplicate", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Duplicate_16x", Icon16x16) );
		Style->Set( "GenericCommands.Rename", new IMAGE_BRUSH( "Icons/Edit/icon_Edit_Rename_16x", Icon16x16 ) );
	}

	// SVerticalBox Drag& Drop icon
	Style->Set("VerticalBoxDragIndicator", new IMAGE_BRUSH("Common/VerticalBoxDragIndicator", FVector2D(6, 45)));
	Style->Set("VerticalBoxDragIndicatorShort", new IMAGE_BRUSH("Common/VerticalBoxDragIndicatorShort", FVector2D(6, 15)));
	// SScrollBar defaults...
	const FScrollBarStyle ScrollBar = FScrollBarStyle()
		.SetVerticalTopSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Vertical", FVector2D(8, 8)))
		.SetVerticalBottomSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Vertical", FVector2D(8, 8)))
		.SetHorizontalTopSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Horizontal", FVector2D(8, 8)))
		.SetHorizontalBottomSlotImage(IMAGE_BRUSH("Common/Scrollbar_Background_Horizontal", FVector2D(8, 8)))
		.SetNormalThumbImage( BOX_BRUSH( "Common/Scrollbar_Thumb", FMargin(4.f/16.f) ) )
		.SetDraggedThumbImage( BOX_BRUSH( "Common/Scrollbar_Thumb", FMargin(4.f/16.f) ) )
		.SetHoveredThumbImage( BOX_BRUSH( "Common/Scrollbar_Thumb", FMargin(4.f/16.f) ) );
	{
		Style->Set( "Scrollbar", ScrollBar );
	}

	// SButton defaults...
	const FButtonStyle Button = FButtonStyle()
		.SetNormal(BOX_BRUSH("Common/Button", FVector2D(32, 32), 8.0f / 32.0f))
		.SetHovered(BOX_BRUSH("Common/Button_Hovered", FVector2D(32, 32), 8.0f / 32.0f))
		.SetPressed(BOX_BRUSH("Common/Button_Pressed", FVector2D(32, 32), 8.0f / 32.0f))
		.SetNormalPadding(FMargin(2, 2, 2, 2))
		.SetPressedPadding(FMargin(2, 3, 2, 1))
		.SetNormalForeground(InvertedForeground)
		.SetPressedForeground(InvertedForeground)
		.SetHoveredForeground(InvertedForeground)
		.SetDisabledForeground(InvertedForeground);
	{
		Style->Set( "Button", Button );

		Style->Set( "InvertedForeground", InvertedForeground );
	}

	// SComboButton and SComboBox defaults...
	{
		FComboButtonStyle ComboButton = FComboButtonStyle()
			.SetButtonStyle(Button)
			.SetDownArrowImage(IMAGE_BRUSH("Common/ComboArrow", Icon8x8))
			.SetMenuBorderBrush(BOX_BRUSH("Old/Menu_Background", FMargin(8.0f/64.0f)))
			.SetMenuBorderPadding(FMargin(0.0f));
		Style->Set( "ComboButton", ComboButton );

		ComboButton.SetMenuBorderPadding(FMargin(1.0));

		FComboBoxStyle ComboBox = FComboBoxStyle()
			.SetComboButtonStyle(ComboButton);
		Style->Set( "ComboBox", ComboBox );
	}

	// SMessageLogListing
	{
		FComboButtonStyle MessageLogListingComboButton = FComboButtonStyle()
			.SetButtonStyle(NoBorder)
			.SetDownArrowImage(IMAGE_BRUSH("Common/ComboArrow", Icon8x8))
			.SetMenuBorderBrush(FSlateNoResource())
			.SetMenuBorderPadding(FMargin(0.0f));
		Style->Set("MessageLogListingComboButton", MessageLogListingComboButton);
	}

	// SEditableComboBox defaults...
	{
		Style->Set( "EditableComboBox.Add", new IMAGE_BRUSH( "Icons/PlusSymbol_12x", Icon12x12 ) );
		Style->Set( "EditableComboBox.Delete", new IMAGE_BRUSH( "Icons/Cross_12x", Icon12x12 ) );
		Style->Set( "EditableComboBox.Rename", new IMAGE_BRUSH( "Icons/ellipsis_12x", Icon12x12 ) );
		Style->Set( "EditableComboBox.Accept", new IMAGE_BRUSH( "Common/Check", Icon16x16 ) );
	}

	// SCheckBox defaults...
	{
		const FCheckBoxStyle BasicCheckBoxStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::CheckBox)
			.SetUncheckedImage( IMAGE_BRUSH( "Common/CheckBox", Icon16x16 ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/CheckBox", Icon16x16 ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Hovered", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/CheckBox_Checked_Hovered", Icon16x16 ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/CheckBox_Checked_Hovered", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Checked", Icon16x16 ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined", Icon16x16 ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon16x16 ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) );
		Style->Set( "Checkbox", BasicCheckBoxStyle );

		/* Set images for various transparent SCheckBox states ... */
		const FCheckBoxStyle BasicTransparentCheckBoxStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage( FSlateNoResource() )
			.SetUncheckedHoveredImage( FSlateNoResource() )
			.SetUncheckedPressedImage( FSlateNoResource() )
			.SetCheckedImage( FSlateNoResource() )
			.SetCheckedHoveredImage( FSlateNoResource() )
			.SetCheckedPressedImage( FSlateNoResource() )
			.SetUndeterminedImage( FSlateNoResource() )
			.SetUndeterminedHoveredImage( FSlateNoResource() )
			.SetUndeterminedPressedImage( FSlateNoResource() );
		Style->Set( "TransparentCheckBox", BasicTransparentCheckBoxStyle );

		/* Default Style for a toggleable button */
		const FCheckBoxStyle ToggleButtonStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage( FSlateNoResource() )
			.SetUncheckedHoveredImage( BOX_BRUSH("Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor ) )
			.SetUncheckedPressedImage( BOX_BRUSH("Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed ) )
			.SetCheckedImage( BOX_BRUSH("Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) )
			.SetCheckedHoveredImage( BOX_BRUSH("Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor ) )
			.SetCheckedPressedImage( BOX_BRUSH("Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) );
		Style->Set( "ToggleButtonCheckbox", ToggleButtonStyle );

		/* Style for a toggleable button that mimics the coloring and look of a Table Row */
		const FCheckBoxStyle ToggleButtonRowStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(FSlateNoResource())
			.SetUncheckedHoveredImage(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			.SetUncheckedPressedImage(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			.SetCheckedImage(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
			.SetCheckedHoveredImage(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
			.SetCheckedPressedImage(BOX_BRUSH("Common/Selector", 4.0f / 16.0f, SelectorColor));
		Style->Set("ToggleButtonRowStyle", ToggleButtonRowStyle);

		/* A radio button is actually just a SCheckBox box with different images */
		/* Set images for various radio button (SCheckBox) states ... */
		const FCheckBoxStyle BasicRadioButtonStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16 ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16, SelectionColor ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor_Pressed ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor_Pressed ) );
		Style->Set( "RadioButton", BasicRadioButtonStyle );
	}

	// SEditableText defaults...
	{
		FSlateBrush* SelectionBackground = new BOX_BRUSH( "Common/EditableTextSelectionBackground",  FMargin(4.f/16.f) );
		FSlateBrush* SelectionTarget = new BOX_BRUSH( "Old/DashedBorder", FMargin(6.0f/32.0f), FLinearColor( 0.0f, 0.0f, 0.0f, 0.75f ) );
		FSlateBrush* CompositionBackground = new BORDER_BRUSH( "Old/HyperlinkDotted",  FMargin(0,0,0,3/16.0f) );

		const FEditableTextStyle NormalEditableTextStyle = FEditableTextStyle()
			.SetBackgroundImageSelected( *SelectionBackground )
			.SetBackgroundImageComposing( *CompositionBackground )
			.SetCaretImage( *GenericWhiteBox );
		Style->Set( "NormalEditableText", NormalEditableTextStyle );

		Style->Set( "EditableText.SelectionBackground", SelectionBackground );
		Style->Set( "EditableText.SelectionTarget", SelectionTarget );
		Style->Set( "EditableText.CompositionBackground", CompositionBackground );
	}

	// SEditableTextBox defaults...
	const FEditableTextBoxStyle NormalEditableTextBoxStyle = FEditableTextBoxStyle()
		.SetBackgroundImageNormal( BOX_BRUSH( "Common/TextBox", FMargin(4.0f/16.0f) ) )
		.SetBackgroundImageHovered( BOX_BRUSH( "Common/TextBox_Hovered", FMargin(4.0f/16.0f) ) )
		.SetBackgroundImageFocused( BOX_BRUSH( "Common/TextBox_Hovered", FMargin(4.0f/16.0f) ) )
		.SetBackgroundImageReadOnly( BOX_BRUSH( "Common/TextBox_ReadOnly", FMargin(4.0f/16.0f) ) )
		.SetScrollBarStyle( ScrollBar );
	{
		Style->Set( "NormalEditableTextBox", NormalEditableTextBoxStyle );
		// "NormalFont".
	}

	const FEditableTextBoxStyle DarkEditableTextBoxStyle = FEditableTextBoxStyle()
		.SetBackgroundImageNormal(BOX_BRUSH("Common/TextBox_Dark", FMargin(4.0f / 16.0f)))
		.SetBackgroundImageHovered(BOX_BRUSH("Common/TextBox_Hovered_Dark", FMargin(4.0f / 16.0f)))
		.SetBackgroundImageFocused(BOX_BRUSH("Common/TextBox_Hovered_Dark", FMargin(4.0f / 16.0f)))
		.SetBackgroundImageReadOnly(BOX_BRUSH("Common/TextBox_ReadOnly", FMargin(4.0f / 16.0f)))
		.SetScrollBarStyle(ScrollBar);
	{
		Style->Set("DarkEditableTextBox", DarkEditableTextBoxStyle);
		// "NormalFont".
	}

	// SSpinBox defaults...
	{
		// "NormalFont".

		// "InvertedForeground".
	}

	// SSuggestionTextBox defaults...
	{
		// "NormalFont".

		// "InvertedForeground".
	}

	// STextBlock defaults...
	{
		Style->Set("NormalText", NormalText);
		Style->Set("NormalUnderlinedText", NormalUnderlinedText);

		Style->Set("SmallText", SmallText);
		Style->Set("SmallUnderlinedText", SmallUnderlinedText);
	}

	// SInlineEditableTextBlock
	{
		FTextBlockStyle InlineEditableTextBlockReadOnly = FTextBlockStyle(NormalText)
			.SetColorAndOpacity( FSlateColor::UseForeground() )
			.SetShadowOffset( FVector2D::ZeroVector )
			.SetShadowColorAndOpacity( FLinearColor::Black );

		FTextBlockStyle InlineEditableTextBlockSmallReadOnly = FTextBlockStyle(InlineEditableTextBlockReadOnly)
			.SetFont(SmallText.Font);

		FEditableTextBoxStyle InlineEditableTextBlockEditable = FEditableTextBoxStyle()
			.SetFont(NormalText.Font)
			.SetBackgroundImageNormal( BOX_BRUSH( "Common/TextBox", FMargin(4.0f/16.0f) ) )
			.SetBackgroundImageHovered( BOX_BRUSH( "Common/TextBox_Hovered", FMargin(4.0f/16.0f) ) )
			.SetBackgroundImageFocused( BOX_BRUSH( "Common/TextBox_Hovered", FMargin(4.0f/16.0f) ) )
			.SetBackgroundImageReadOnly( BOX_BRUSH( "Common/TextBox_ReadOnly", FMargin(4.0f/16.0f) ) )
			.SetScrollBarStyle( ScrollBar );

		FEditableTextBoxStyle InlineEditableTextBlockSmallEditable = FEditableTextBoxStyle(InlineEditableTextBlockEditable)
			.SetFont(SmallText.Font);

		FInlineEditableTextBlockStyle InlineEditableTextBlockStyle = FInlineEditableTextBlockStyle()
			.SetTextStyle(InlineEditableTextBlockReadOnly)
			.SetEditableTextBoxStyle(InlineEditableTextBlockEditable);

		Style->Set( "InlineEditableTextBlockStyle", InlineEditableTextBlockStyle );

		FInlineEditableTextBlockStyle InlineEditableTextBlockSmallStyle = FInlineEditableTextBlockStyle()
			.SetTextStyle(InlineEditableTextBlockSmallReadOnly)
			.SetEditableTextBoxStyle(InlineEditableTextBlockSmallEditable);

		Style->Set("InlineEditableTextBlockSmallStyle", InlineEditableTextBlockSmallStyle);
	}

	// SSuggestionTextBox defaults...
	{
		Style->Set( "SuggestionTextBox.Background",	new BOX_BRUSH( "Old/Menu_Background", FMargin(8.0f/64.0f) ) );
		Style->Set( "SuggestionTextBox.Text", FTextBlockStyle()
			.SetFont( DEFAULT_FONT( "Regular", 9 ) )
			.SetColorAndOpacity( FLinearColor(FColor(0xffaaaaaa)) )
			);
	}

	// SToolTip defaults...
	{
		Style->Set("ToolTip.Font", DEFAULT_FONT("Regular", 8));
		Style->Set("ToolTip.Background", new BOX_BRUSH("Old/ToolTip_Background", FMargin(8.0f / 64.0f)));

		Style->Set("ToolTip.LargerFont", DEFAULT_FONT("Regular", 9));
		Style->Set("ToolTip.BrightBackground", new BOX_BRUSH("Old/ToolTip_BrightBackground", FMargin(8.0f / 64.0f)));
	}

	// SBorder defaults...
	{
		Style->Set( "Border", new BORDER_BRUSH( "Old/Border", 4.0f/16.0f ) );
	}

	// SHyperlink defaults...
	{
		FButtonStyle HyperlinkButton = FButtonStyle()
			.SetNormal(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0,0,0,3/16.0f) ) )
			.SetPressed(FSlateNoResource() )
			.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0,0,0,3/16.0f) ) );

		FHyperlinkStyle Hyperlink = FHyperlinkStyle()
			.SetUnderlineStyle(HyperlinkButton)
			.SetTextStyle(NormalText)
			.SetPadding(FMargin(0.0f));
		Style->Set("Hyperlink", Hyperlink);
	}

	// SProgressBar defaults...
	{
		Style->Set( "ProgressBar", FProgressBarStyle()
			.SetBackgroundImage( BOX_BRUSH( "Common/ProgressBar_Background", FMargin(5.f/12.f) ) )
			.SetFillImage( BOX_BRUSH( "Common/ProgressBar_Fill", FMargin(5.f/12.f), FLinearColor( 1.0f, 0.22f, 0.0f )  ) )
			.SetMarqueeImage( IMAGE_BRUSH( "Common/ProgressBar_Marquee", FVector2D(20,12), FLinearColor::White, ESlateBrushTileType::Horizontal ) )
			);
	}

	// SThrobber, SCircularThrobber defaults...
	{
		Style->Set( "Throbber.Chunk", new IMAGE_BRUSH( "Common/Throbber_Piece", FVector2D(16,16) ) );
		Style->Set( "Throbber.CircleChunk", new IMAGE_BRUSH( "Common/Throbber_Piece", FVector2D(8,8) ) );
	}

	// SExpandableArea defaults...
	{
		Style->Set( "ExpandableArea", FExpandableAreaStyle()
			.SetCollapsedImage( IMAGE_BRUSH( "Common/TreeArrow_Collapsed", Icon10x10, DefaultForeground ) )
			.SetExpandedImage( IMAGE_BRUSH( "Common/TreeArrow_Expanded", Icon10x10, DefaultForeground ) )
			);
		Style->Set( "ExpandableArea.TitleFont", DEFAULT_FONT( "Bold", 8 ) );
		Style->Set( "ExpandableArea.Border", new BOX_BRUSH( "Common/GroupBorder", FMargin(4.0f/16.0f) ) );
	}

	// SSearchBox defaults...
	{
		const FEditableTextBoxStyle SpecialEditableTextBoxStyle = FEditableTextBoxStyle()
			.SetBackgroundImageNormal( BOX_BRUSH( "Common/TextBox_Special", FMargin(8.0f/32.0f) ) )
			.SetBackgroundImageHovered( BOX_BRUSH( "Common/TextBox_Special_Hovered", FMargin(8.0f/32.0f) ) )
			.SetBackgroundImageFocused( BOX_BRUSH( "Common/TextBox_Special_Hovered", FMargin(8.0f/32.0f) ) )
			.SetBackgroundImageReadOnly( BOX_BRUSH( "Common/TextBox_ReadOnly", FMargin(4.0f/16.0f) ) )
			.SetScrollBarStyle( ScrollBar );

		Style->Set( "SearchBox", FSearchBoxStyle()
			.SetTextBoxStyle( SpecialEditableTextBoxStyle )
			.SetUpArrowImage( IMAGE_BRUSH( "Common/UpArrow", Icon8x8 ) )
			.SetDownArrowImage( IMAGE_BRUSH( "Common/DownArrow", Icon8x8 ) )
			.SetGlassImage( IMAGE_BRUSH( "Common/SearchGlass", Icon16x16 ) )
			.SetClearImage( IMAGE_BRUSH( "Common/X", Icon16x16 ) )
			);
	}

	// SSlider and SVolumeControl defaults...
	{
		FSliderStyle SliderStyle = FSliderStyle()
			.SetNormalBarImage(FSlateColorBrush(FColor::White))
			.SetHoveredBarImage(FSlateColorBrush(FColor::White))
			.SetDisabledBarImage(FSlateColorBrush(FLinearColor::Gray))
			.SetNormalThumbImage( IMAGE_BRUSH( "Common/Button", FVector2D(8.0f, 14.0f) ) )
			.SetHoveredThumbImage(IMAGE_BRUSH("Common/Button", FVector2D(8.0f, 14.0f)))
			.SetDisabledThumbImage( IMAGE_BRUSH( "Common/Button_Disabled", FVector2D(8.0f, 14.0f) ) )
			.SetBarThickness(2.0f);
		Style->Set( "Slider", SliderStyle );

		Style->Set( "VolumeControl", FVolumeControlStyle()
			.SetSliderStyle( SliderStyle )
			.SetHighVolumeImage( IMAGE_BRUSH( "Common/VolumeControl_High", Icon16x16 ) )
			.SetMidVolumeImage( IMAGE_BRUSH( "Common/VolumeControl_Mid", Icon16x16 ) )
			.SetLowVolumeImage( IMAGE_BRUSH( "Common/VolumeControl_Low", Icon16x16 ) )
			.SetNoVolumeImage( IMAGE_BRUSH( "Common/VolumeControl_Off", Icon16x16 ) )
			.SetMutedImage( IMAGE_BRUSH( "Common/VolumeControl_Muted", Icon16x16 ) )
			);
	}

	// SSpinBox defaults...
	{
		Style->Set( "SpinBox", FSpinBoxStyle()
			.SetBackgroundBrush( BOX_BRUSH( "Common/Spinbox", FMargin(4.0f/16.0f) ) )
			.SetHoveredBackgroundBrush( BOX_BRUSH( "Common/Spinbox_Hovered", FMargin(4.0f/16.0f) ) )
			.SetActiveFillBrush( BOX_BRUSH( "Common/Spinbox_Fill", FMargin(4.0f/16.0f, 4.0f/16.0f, 8.0f/16.0f, 4.0f/16.0f) ) )
			.SetInactiveFillBrush( BOX_BRUSH( "Common/Spinbox_Fill_Hovered", FMargin(4.0f/16.0f) ) )
			.SetArrowsImage( IMAGE_BRUSH( "Common/SpinArrows", Icon12x12 ) )
			.SetForegroundColor( InvertedForeground )
			);
	}

	// SNumericEntryBox defaults...
	{
		Style->Set( "NumericEntrySpinBox", FSpinBoxStyle()
			.SetBackgroundBrush( FSlateNoResource() )
			.SetHoveredBackgroundBrush( FSlateNoResource() )
			.SetActiveFillBrush( BOX_BRUSH( "Common/Spinbox_Fill_Hovered", FMargin(4.0f/16.0f) ) )
			.SetInactiveFillBrush( BOX_BRUSH( "Common/Spinbox_Fill", FMargin(4.0f/16.0f, 4.0f/16.0f, 8.0f/16.0f, 4.0f/16.0f) ) )
			.SetArrowsImage( IMAGE_BRUSH( "Common/SpinArrows", Icon12x12 ) )
			.SetTextPadding( FMargin(0.0f) )
			.SetForegroundColor( InvertedForeground )
			);

		Style->Set("NumericEntrySpinBox_Dark", FSpinBoxStyle()
			.SetBackgroundBrush(FSlateNoResource())
			.SetHoveredBackgroundBrush(FSlateNoResource())
			.SetActiveFillBrush(BOX_BRUSH("Common/Spinbox_Fill_Hovered_Dark", FMargin(4.0f / 16.0f)))
			.SetInactiveFillBrush(BOX_BRUSH("Common/Spinbox_Fill_Dark", FMargin(4.0f / 16.0f, 4.0f / 16.0f, 8.0f / 16.0f, 4.0f / 16.0f)))
			.SetArrowsImage(IMAGE_BRUSH("Common/SpinArrows", Icon12x12))
			.SetTextPadding(FMargin(0.0f))
			.SetForegroundColor(InvertedForeground)
			);

		Style->Set( "NumericEntrySpinBox.Decorator", new BOX_BRUSH( "Common/TextBoxLabelBorder", FMargin(5.0f/16.0f) ) );

		Style->Set( "NumericEntrySpinBox.NarrowDecorator", new BOX_BRUSH( "Common/TextBoxLabelBorder", FMargin(2.0f/16.0f, 4.0f/16.0f, 2.0f/16.0f, 4.0f/16.0f) ) );
	}

	// SColorPicker defaults...
	{
		Style->Set( "ColorPicker.Border", new BOX_BRUSH( "Common/GroupBorder", FMargin(4.0f/16.0f) ) );
		Style->Set( "ColorPicker.AlphaBackground", new IMAGE_BRUSH( "Common/Checker", Icon16x16, FLinearColor::White, ESlateBrushTileType::Both ) );
		Style->Set( "ColorPicker.EyeDropper", new IMAGE_BRUSH( "Icons/eyedropper_16px", Icon16x16) );
		Style->Set( "ColorPicker.Font", DEFAULT_FONT( "Regular", 10 ) );
		Style->Set( "ColorPicker.Mode", new IMAGE_BRUSH( "Common/ColorPicker_Mode_16x", Icon16x16) );
		Style->Set( "ColorPicker.Separator", new IMAGE_BRUSH( "Common/ColorPicker_Separator", FVector2D(2.0f, 2.0f) ) );
		Style->Set( "ColorPicker.Selector", new IMAGE_BRUSH( "Common/Circle", FVector2D(8, 8) ) );
		Style->Set( "ColorPicker.Slider", FSliderStyle()
			.SetDisabledThumbImage( IMAGE_BRUSH( "Common/ColorPicker_SliderHandle", FVector2D(8.0f, 32.0f)) )
			.SetNormalThumbImage( IMAGE_BRUSH( "Common/ColorPicker_SliderHandle", FVector2D(8.0f, 32.0f)) )
			.SetHoveredThumbImage( IMAGE_BRUSH("Common/ColorPicker_SliderHandle", FVector2D(8.0f, 32.0f)) )
			);
	}

	// SColorSpectrum defaults...
	{
		Style->Set( "ColorSpectrum.Spectrum", new IMAGE_BRUSH( "Common/ColorSpectrum", FVector2D(256, 256) ) );
		Style->Set( "ColorSpectrum.Selector", new IMAGE_BRUSH( "Common/Circle", FVector2D(8, 8) ) );
	}

	// SColorThemes defaults...
	{
		Style->Set( "ColorThemes.DeleteButton", new IMAGE_BRUSH( "Common/X", Icon16x16) );
	}

	// SColorWheel defaults...
	{
		Style->Set( "ColorWheel.HueValueCircle", new IMAGE_BRUSH( "Common/ColorWheel", FVector2D(192, 192) ) );
		Style->Set( "ColorWheel.Selector", new IMAGE_BRUSH( "Common/Circle", FVector2D(8, 8) ) );
	}

	// SColorGradingWheel defaults...
	{
		Style->Set("ColorGradingWheel.HueValueCircle", new IMAGE_BRUSH("Common/ColorGradingWheel", FVector2D(192, 192)));
		Style->Set("ColorGradingWheel.Selector", new IMAGE_BRUSH("Common/Circle", FVector2D(8, 8)));
	}

	// SSplitter
	{
		Style->Set( "Splitter", FSplitterStyle()
			.SetHandleNormalBrush( FSlateNoResource() )
			.SetHandleHighlightBrush( IMAGE_BRUSH( "Common/SplitterHandleHighlight", Icon8x8, FLinearColor::White ) )
			);
	}

	// TableView defaults...
	{
		const FTableRowStyle DefaultTableRowStyle = FTableRowStyle()
			.SetEvenRowBackgroundBrush(FSlateNoResource())
			.SetEvenRowBackgroundHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(1.0f, 1.0f, 1.0f, 0.1f)))
			.SetOddRowBackgroundBrush(FSlateNoResource())
			.SetOddRowBackgroundHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(1.0f, 1.0f, 1.0f, 0.1f)))
			.SetSelectorFocusedBrush(BORDER_BRUSH("Common/Selector", FMargin(4.f / 16.f), SelectorColor))
			.SetActiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
			.SetActiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
			.SetInactiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			.SetInactiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			.SetActiveHighlightedBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, HighlightColor))
			.SetInactiveHighlightedBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FSlateColor(FLinearColor(.1f, .1f, .1f))))

			.SetTextColor(DefaultForeground)
			.SetSelectedTextColor(InvertedForeground)
			.SetDropIndicator_Above(BOX_BRUSH("Common/DropZoneIndicator_Above", FMargin(10.0f / 16.0f, 10.0f / 16.0f, 0, 0), SelectionColor))
			.SetDropIndicator_Onto(BOX_BRUSH("Common/DropZoneIndicator_Onto", FMargin(4.0f / 16.0f), SelectionColor))
			.SetDropIndicator_Below(BOX_BRUSH("Common/DropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0, 0, 10.0f / 16.0f), SelectionColor));
		Style->Set("TableView.Row", DefaultTableRowStyle);

		// Make this the default for the ComboBox rows also
		Style->Set("ComboBox.Row", DefaultTableRowStyle);  

		const FTableRowStyle DarkTableRowStyle = FTableRowStyle(DefaultTableRowStyle)
			.SetEvenRowBackgroundBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(0.0f, 0.0f, 0.0f, 0.1f)))
			.SetOddRowBackgroundBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(0.0f, 0.0f, 0.0f, 0.1f)));
		Style->Set("TableView.DarkRow", DarkTableRowStyle);

		Style->Set( "TreeArrow_Collapsed", new IMAGE_BRUSH( "Common/TreeArrow_Collapsed", Icon10x10, DefaultForeground ) );
		Style->Set( "TreeArrow_Collapsed_Hovered", new IMAGE_BRUSH( "Common/TreeArrow_Collapsed_Hovered", Icon10x10, DefaultForeground ) );
		Style->Set( "TreeArrow_Expanded", new IMAGE_BRUSH( "Common/TreeArrow_Expanded", Icon10x10, DefaultForeground ) );
		Style->Set( "TreeArrow_Expanded_Hovered", new IMAGE_BRUSH( "Common/TreeArrow_Expanded_Hovered", Icon10x10, DefaultForeground ) );

		const FTableColumnHeaderStyle TableColumnHeaderStyle = FTableColumnHeaderStyle()
			.SetSortPrimaryAscendingImage(IMAGE_BRUSH("Common/SortUpArrow", Icon8x4))
			.SetSortPrimaryDescendingImage(IMAGE_BRUSH("Common/SortDownArrow", Icon8x4))
			.SetSortSecondaryAscendingImage(IMAGE_BRUSH("Common/SortUpArrows", Icon16x4))
			.SetSortSecondaryDescendingImage(IMAGE_BRUSH("Common/SortDownArrows", Icon16x4))
			.SetNormalBrush( BOX_BRUSH( "Common/ColumnHeader", 4.f/32.f ) )
			.SetHoveredBrush( BOX_BRUSH( "Common/ColumnHeader_Hovered", 4.f/32.f ) )
			.SetMenuDropdownImage( IMAGE_BRUSH( "Common/ColumnHeader_Arrow", Icon8x8 ) )
			.SetMenuDropdownNormalBorderBrush( BOX_BRUSH( "Common/ColumnHeaderMenuButton_Normal", 4.f/32.f ) )
			.SetMenuDropdownHoveredBorderBrush( BOX_BRUSH( "Common/ColumnHeaderMenuButton_Hovered", 4.f/32.f ) );
		Style->Set( "TableView.Header.Column", TableColumnHeaderStyle );

		const FTableColumnHeaderStyle TableLastColumnHeaderStyle = FTableColumnHeaderStyle()
			.SetSortPrimaryAscendingImage(IMAGE_BRUSH("Common/SortUpArrow", Icon8x4))
			.SetSortPrimaryDescendingImage(IMAGE_BRUSH("Common/SortDownArrow", Icon8x4))
			.SetSortSecondaryAscendingImage(IMAGE_BRUSH("Common/SortUpArrows", Icon16x4))
			.SetSortSecondaryDescendingImage(IMAGE_BRUSH("Common/SortDownArrows", Icon16x4))
			.SetNormalBrush( FSlateNoResource() )
			.SetHoveredBrush( BOX_BRUSH( "Common/LastColumnHeader_Hovered", 4.f/32.f ) )
			.SetMenuDropdownImage( IMAGE_BRUSH( "Common/ColumnHeader_Arrow", Icon8x8 ) )
			.SetMenuDropdownNormalBorderBrush( BOX_BRUSH( "Common/ColumnHeaderMenuButton_Normal", 4.f/32.f ) )
			.SetMenuDropdownHoveredBorderBrush( BOX_BRUSH( "Common/ColumnHeaderMenuButton_Hovered", 4.f/32.f ) );

		const FSplitterStyle TableHeaderSplitterStyle = FSplitterStyle()
			.SetHandleNormalBrush( FSlateNoResource() )
			.SetHandleHighlightBrush( IMAGE_BRUSH( "Common/HeaderSplitterGrip", Icon8x8 ) );

		Style->Set( "TableView.Header", FHeaderRowStyle()
			.SetColumnStyle( TableColumnHeaderStyle )
			.SetLastColumnStyle( TableLastColumnHeaderStyle )
			.SetColumnSplitterStyle( TableHeaderSplitterStyle )
			.SetBackgroundBrush( BOX_BRUSH( "Common/TableViewHeader", 4.f/32.f ) )
			.SetForegroundColor( DefaultForeground )
			);
	}

	// MultiBox
	{
		Style->Set( "MultiBox.GenericToolBarIcon", new IMAGE_BRUSH( "Icons/icon_generic_toolbar", Icon40x40 ) );
		Style->Set( "MultiBox.GenericToolBarIcon.Small", new IMAGE_BRUSH( "Icons/icon_generic_toolbar", Icon20x20 ) );

		Style->Set( "MultiBox.DeleteButton", FButtonStyle() 
			.SetNormal ( IMAGE_BRUSH( "/Docking/CloseApp_Hovered", Icon16x16 ) )
			.SetPressed( IMAGE_BRUSH( "/Docking/CloseApp_Pressed", Icon16x16 ) )
			.SetHovered( IMAGE_BRUSH( "/Docking/CloseApp_Hovered", Icon16x16 ) )
			);

		Style->Set( "MultiboxHookColor", FLinearColor(0.f, 1.f, 0.f, 1.f) );
	}

	// ToolBar
	{
		Style->Set( "ToolBar.Background", new BOX_BRUSH( "Common/GroupBorder", FMargin(4.0f/16.0f) ) );
		Style->Set( "ToolBar.Icon", new IMAGE_BRUSH( "Icons/icon_tab_toolbar_16px", Icon16x16 ) );
		Style->Set( "ToolBar.Expand", new IMAGE_BRUSH( "Icons/toolbar_expand_16x", Icon16x16) );
		Style->Set( "ToolBar.SubMenuIndicator", new IMAGE_BRUSH( "Common/SubmenuArrow", Icon8x8 ) );
		Style->Set( "ToolBar.SToolBarComboButtonBlock.Padding", FMargin(4.0f,0.0f));
		Style->Set( "ToolBar.SToolBarButtonBlock.Padding", FMargin(4.0f,0.0f));
		Style->Set( "ToolBar.SToolBarCheckComboButtonBlock.Padding", FMargin(4.0f,0.0f));
		Style->Set( "ToolBar.SToolBarButtonBlock.CheckBox.Padding", FMargin(4.0f,0.0f) );
		Style->Set( "ToolBar.SToolBarComboButtonBlock.ComboButton.Color", DefaultForeground );

		Style->Set( "ToolBar.Block.IndentedPadding", FMargin( 18.0f, 2.0f, 4.0f, 4.0f ) );
		Style->Set( "ToolBar.Block.Padding", FMargin( 2.0f, 2.0f, 4.0f, 4.0f ) );

		Style->Set( "ToolBar.Separator", new FSlateColorBrush( FLinearColor(FColor(48, 48, 48)) ) );
		Style->Set( "ToolBar.Separator.Padding", FMargin( 8.f, 0.f, 8.f, 0.f) );

		Style->Set( "ToolBar.Label", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 9 ) ) );
		Style->Set( "ToolBar.EditableText", FEditableTextBoxStyle(NormalEditableTextBoxStyle) .SetFont( DEFAULT_FONT( "Regular", 9 ) ) );
		Style->Set( "ToolBar.Keybinding", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 8 ) ) );

		Style->Set( "ToolBar.Heading", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 8 ) )
			.SetColorAndOpacity( FLinearColor( 0.4f, 0.4, 0.4f, 1.0f ) ) );

		/* Create style for "ToolBar.CheckBox" widget ... */
		const FCheckBoxStyle ToolBarCheckBoxStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Common/SmallCheckBox", Icon14x14 ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Checked",  Icon14x14 ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/SmallCheckBox_Hovered", Icon14x14 ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Checked_Hovered", Icon14x14 ) )
			.SetUncheckedPressedImage(IMAGE_BRUSH( "Common/SmallCheckBox_Hovered", Icon14x14, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Checked_Hovered", Icon14x14, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined", Icon14x14 ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon14x14 ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon14x14, FLinearColor( 0.5f, 0.5f, 0.5f ) ) );
		/* ... and add new style */
		Style->Set( "ToolBar.CheckBox", ToolBarCheckBoxStyle );

		/* Read-only checkbox that appears next to a menu item */
		/* Set images for various SCheckBox states associated with read-only toolbar check box items... */
		const FCheckBoxStyle BasicToolBarCheckStyle = FCheckBoxStyle()
			.SetUncheckedImage(IMAGE_BRUSH("Icons/Empty_14x", Icon14x14))
			.SetUncheckedHoveredImage(IMAGE_BRUSH("Icons/Empty_14x", Icon14x14))
			.SetUncheckedPressedImage(IMAGE_BRUSH("Common/SmallCheckBox_Hovered", Icon14x14))
			.SetCheckedImage(IMAGE_BRUSH("Common/SmallCheck", Icon14x14))
			.SetCheckedHoveredImage(IMAGE_BRUSH("Common/SmallCheck", Icon14x14))
			.SetCheckedPressedImage(IMAGE_BRUSH("Common/SmallCheck", Icon14x14))
			.SetUndeterminedImage(IMAGE_BRUSH("Icons/Empty_14x", Icon14x14))
			.SetUndeterminedHoveredImage(FSlateNoResource())
			.SetUndeterminedPressedImage(FSlateNoResource());
		Style->Set("ToolBar.Check", BasicToolBarCheckStyle);

		// This radio button is actually just a check box with different images
		/* Create style for "ToolBar.RadioButton" widget ... */
		const FCheckBoxStyle ToolbarRadioButtonCheckBoxStyle = FCheckBoxStyle()
				.SetUncheckedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
				.SetCheckedImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x",  Icon16x16 ) )
				.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor ) )
				.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor ) )
				.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor_Pressed ) )
				.SetCheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16, SelectionColor_Pressed ) );
		/* ... and add new style */
		Style->Set( "ToolBar.RadioButton", ToolbarRadioButtonCheckBoxStyle );

		/* Create style for "ToolBar.ToggleButton" widget ... */
		const FCheckBoxStyle ToolBarToggleButtonCheckBoxStyle = FCheckBoxStyle()
				.SetCheckBoxType( ESlateCheckBoxType::ToggleButton )
				.SetUncheckedImage( FSlateNoResource() )
				.SetUncheckedPressedImage( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed ) )
				.SetUncheckedHoveredImage( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor ) )
				.SetCheckedImage( BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) )
				.SetCheckedHoveredImage( BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) )
				.SetCheckedPressedImage( BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor ) );
		/* ... and add new style */
		Style->Set( "ToolBar.ToggleButton", ToolBarToggleButtonCheckBoxStyle );

		Style->Set( "ToolBar.Button", FButtonStyle(Button)
			.SetNormal ( FSlateNoResource() )
			.SetPressed( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed ) )
			.SetHovered( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor ))
			.SetNormalForeground(FSlateColor::UseForeground())
			.SetPressedForeground(FSlateColor::UseForeground())
			.SetHoveredForeground(FSlateColor::UseForeground())
			.SetDisabledForeground(FSlateColor::UseForeground())
		);

	/*	Style->Set( "ToolBar.Button.Normal", new FSlateNoResource() );
		Style->Set( "ToolBar.Button.Pressed", new BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed ) );
		Style->Set( "ToolBar.Button.Hovered", new BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor ) );
*/

		Style->Set( "ToolBar.Button.Checked", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) );
		Style->Set( "ToolBar.Button.Checked_Hovered", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) );
		Style->Set( "ToolBar.Button.Checked_Pressed", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor ) );

		Style->Set("ToolBar.SimpleComboButton", Style->GetWidgetStyle<FComboButtonStyle>("ComboButton"));
		Style->Set("ToolBar.IconSize", Icon20x20);
	}

	// MenuBar
	{
		Style->Set( "Menu.Background",	new BOX_BRUSH( "Old/Menu_Background", FMargin(8.0f/64.0f) ) );
		Style->Set( "Menu.Icon", new IMAGE_BRUSH( "Icons/icon_tab_toolbar_16px", Icon16x16 ) );
		Style->Set( "Menu.Expand", new IMAGE_BRUSH( "Icons/toolbar_expand_16x", Icon16x16) );
		Style->Set( "Menu.SubMenuIndicator", new IMAGE_BRUSH( "Common/SubmenuArrow", Icon8x8 ) );
		Style->Set( "Menu.SToolBarComboButtonBlock.Padding", FMargin(4.0f));
		Style->Set( "Menu.SToolBarButtonBlock.Padding", FMargin(4.0f));
		Style->Set( "Menu.SToolBarCheckComboButtonBlock.Padding", FMargin(4.0f));
		Style->Set( "Menu.SToolBarButtonBlock.CheckBox.Padding", FMargin(0.0f) );
		Style->Set( "Menu.SToolBarComboButtonBlock.ComboButton.Color", DefaultForeground );


		Style->Set( "Menu.Block.IndentedPadding", FMargin( 18.0f, 2.0f, 4.0f, 4.0f ) );
		Style->Set( "Menu.Block.Padding", FMargin( 2.0f, 2.0f, 4.0f, 4.0f ) );

		Style->Set( "Menu.Separator", new BOX_BRUSH( "Old/Button", 4.0f/32.0f ) );
		Style->Set( "Menu.Separator.Padding", FMargin( 0.5f ) );

		Style->Set( "Menu.Label", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 9 ) ) );
		Style->Set( "Menu.EditableText", FEditableTextBoxStyle(NormalEditableTextBoxStyle) .SetFont( DEFAULT_FONT( "Regular", 9 ) ) );
		Style->Set( "Menu.Keybinding", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 8 ) ) );

		Style->Set( "Menu.Heading", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 8 ) )
			.SetColorAndOpacity( FLinearColor( 0.4f, 0.4, 0.4f, 1.0f ) ) );

		const FMargin MenuBlockPadding(10.0f, 3.0f, 5.0f, 2.0f);
		Style->Set("Menu.Heading.Padding", MenuBlockPadding + FMargin(0, 10, 0, 0));

		/* Set images for various SCheckBox states associated with menu check box items... */
		const FCheckBoxStyle BasicMenuCheckBoxStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Common/SmallCheckBox", Icon14x14 ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/SmallCheckBox_Hovered", Icon14x14 ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Hovered", Icon14x14, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Checked", Icon14x14 ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/SmallCheckBox_Checked_Hovered", Icon14x14 ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Checked_Hovered", Icon14x14, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined", Icon14x14 ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon14x14 ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon14x14, FLinearColor( 0.5f, 0.5f, 0.5f ) ) );
 
		/* ...and add the new style */
		Style->Set( "Menu.CheckBox", BasicMenuCheckBoxStyle );
						
		/* Read-only checkbox that appears next to a menu item */
		/* Set images for various SCheckBox states associated with read-only menu check box items... */
		const FCheckBoxStyle BasicMenuCheckStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Icons/Empty_14x", Icon14x14 ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Icons/Empty_14x", Icon14x14 ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Hovered", Icon14x14 ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/SmallCheck", Icon14x14 ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/SmallCheck", Icon14x14 ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheck", Icon14x14 ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Icons/Empty_14x", Icon14x14 ) )
			.SetUndeterminedHoveredImage( FSlateNoResource() )
			.SetUndeterminedPressedImage( FSlateNoResource() );

		/* ...and add the new style */
		Style->Set( "Menu.Check", BasicMenuCheckStyle );

		/* This radio button is actually just a check box with different images */
		/* Set images for various Menu radio button (SCheckBox) states... */
		const FCheckBoxStyle BasicMenuRadioButtonStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16 ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16, SelectionColor ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor_Pressed ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor_Pressed ) );

		/* ...and set new style */
		Style->Set( "Menu.RadioButton", BasicMenuRadioButtonStyle );

		/* Create style for "Menu.ToggleButton" widget ... */
		const FCheckBoxStyle MenuToggleButtonCheckBoxStyle = FCheckBoxStyle()
			.SetCheckBoxType( ESlateCheckBoxType::ToggleButton )
			.SetUncheckedImage( FSlateNoResource() )
			.SetUncheckedPressedImage( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed ) )
			.SetUncheckedHoveredImage( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor ) )
			.SetCheckedImage( BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) )
			.SetCheckedHoveredImage( BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) )
			.SetCheckedPressedImage( BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor ) );
		/* ... and add new style */
		Style->Set( "Menu.ToggleButton", MenuToggleButtonCheckBoxStyle );


		FButtonStyle MenuButton =
			FButtonStyle(NoBorder)
			.SetNormal(FSlateNoResource())
			.SetPressed(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
			.SetHovered(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor))
			.SetNormalPadding(FMargin(0, 1))
			.SetPressedPadding(FMargin(0, 2, 0, 0));

		Style->Set("Menu.Button", MenuButton);

		Style->Set( "Menu.Button.Checked", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) );
		Style->Set( "Menu.Button.Checked_Hovered", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) );
		Style->Set( "Menu.Button.Checked_Pressed", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor ) );

		/* The style of a menu bar button when it has a sub menu open */
		Style->Set( "Menu.Button.SubMenuOpen", new BORDER_BRUSH( "Common/Selection", FMargin(4.f/16.f), FLinearColor(0.10f, 0.10f, 0.10f) ) );

		Style->Set("WindowMenuBar.Background", new BOX_BRUSH("Old/Menu_Background", FMargin(8.0f / 64.0f)));
		Style->Set("WindowMenuBar.Label", FTextBlockStyle(NormalText).SetFont(DEFAULT_FONT("Regular", 9)));
		Style->Set("WindowMenuBar.Expand", new IMAGE_BRUSH("Icons/toolbar_expand_16x", Icon16x16));
		Style->Set("WindowMenuBar.Button", MenuButton);
		Style->Set("WindowMenuBar.Button.SubMenuOpen", new BORDER_BRUSH("Common/Selection", FMargin(4.f / 16.f), FLinearColor(0.10f, 0.10f, 0.10f)));

		Style->Set("WindowMenuBar.MenuBar.Padding", FMargin(12, 2));
	}

	// SExpandableButton defaults...
	{
		Style->Set( "ExpandableButton.Background", new BOX_BRUSH( "Common/Button", 8.0f/32.0f ) );
		
		// Extra padding on the right and bottom to account for image shadow
		Style->Set( "ExpandableButton.Padding", FMargin(3.f, 3.f, 6.f, 6.f) );

		Style->Set( "ExpandableButton.CloseButton", new IMAGE_BRUSH( "Common/ExpansionButton_CloseOverlay", Icon16x16) );
	}

	// SBreadcrumbTrail defaults...
	{
		Style->Set( "BreadcrumbTrail.Delimiter", new IMAGE_BRUSH( "Common/Delimiter", Icon16x16 ) );

		Style->Set( "BreadcrumbButton", FButtonStyle()
			.SetNormal ( FSlateNoResource() )
			.SetPressed( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed ) )
			.SetHovered( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor ) )
			.SetNormalPadding( FMargin(0,0) )
			.SetPressedPadding( FMargin(0,0) )
			);
	}

	// SWizard defaults
	{
		Style->Set("Wizard.PageTitle", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("BoldCondensed", 28))
			.SetShadowOffset(FVector2D(1, 1))
			.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f))
			);
	}

	// SNotificationList defaults...
	{
		Style->Set( "NotificationList.FontBold", DEFAULT_FONT( "Bold", 16 ) );
		Style->Set( "NotificationList.FontLight", DEFAULT_FONT( "Light", 12 ) );
		Style->Set( "NotificationList.ItemBackground", new BOX_BRUSH( "Old/Menu_Background", FMargin(8.0f/64.0f) ) );
		Style->Set( "NotificationList.ItemBackground_Border", new BOX_BRUSH( "Old/Menu_Background_Inverted_Border_Bold", FMargin(8.0f/64.0f) ) );
		Style->Set( "NotificationList.ItemBackground_Border_Transparent", new BOX_BRUSH("Old/Notification_Border_Flash", FMargin(8.0f/64.0f)));
		Style->Set( "NotificationList.SuccessImage", new IMAGE_BRUSH( "Icons/notificationlist_success", Icon16x16 ) );
		Style->Set( "NotificationList.FailImage", new IMAGE_BRUSH( "Icons/notificationlist_fail", Icon16x16 ) );
		Style->Set( "NotificationList.DefaultMessage", new IMAGE_BRUSH( "Common/EventMessage_Default", Icon40x40 ) );
	}

	// SSeparator defaults...
	{
		Style->Set( "Separator", new BOX_BRUSH( "Common/Separator", 1/4.0f, FLinearColor(1,1,1,0.5f) ) );
	}

	// SHeader defaults...
	{
		Style->Set( "Header.Pre", new BOX_BRUSH( "Common/Separator", FMargin(1/4.0f,0,2/4.0f,0), FLinearColor(1,1,1,0.5f) ) );
		Style->Set( "Header.Post", new BOX_BRUSH( "Common/Separator", FMargin(2/4.0f,0,1/4.0f,0), FLinearColor(1,1,1,0.5f) ) );
	}

	// SDockTab, SDockingTarget, SDockingTabStack defaults...
	{
		Style->Set( "Docking.Background", new BOX_BRUSH( "Old/Menu_Background", FMargin(8.0f/64.0f) ) );
		Style->Set( "Docking.Border", new BOX_BRUSH( "Common/GroupBorder", FMargin(4.0f/16.0f) ) );

		FTextBlockStyle DockingTabFont = FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 9 ) )
			.SetColorAndOpacity( FLinearColor(0.72f, 0.72f, 0.72f, 1.f) )
			.SetShadowOffset( FVector2D( 1,1 ) )
			.SetShadowColorAndOpacity( FLinearColor::Black ) ;

		Style->Set( "Docking.UnhideTabwellButton", FButtonStyle(Button)
			.SetNormal ( IMAGE_BRUSH( "/Docking/ShowTabwellButton_Normal", FVector2D(10,10) ) )
			.SetPressed( IMAGE_BRUSH( "/Docking/ShowTabwellButton_Pressed", FVector2D(10,10) ) )
			.SetHovered( IMAGE_BRUSH( "/Docking/ShowTabwellButton_Hovered", FVector2D(10,10) ) )
			.SetNormalPadding(0)
			.SetPressedPadding(0)
			);

		// Flash using the selection color for consistency with the rest of the UI scheme
		const FSlateColor& TabFlashColor = SelectionColor;

		const FButtonStyle CloseButton = FButtonStyle()
			.SetNormal ( IMAGE_BRUSH( "/Docking/CloseApp_Normal", Icon16x16 ) )
			.SetPressed( IMAGE_BRUSH( "/Docking/CloseApp_Pressed", Icon16x16 ) )
			.SetHovered( IMAGE_BRUSH( "/Docking/CloseApp_Hovered", Icon16x16 ) );


		FLinearColor DockColor_Inactive(FColor(45, 45, 45));
		FLinearColor DockColor_Hovered(FColor(54, 54, 54));
		FLinearColor DockColor_Active(FColor(62, 62, 62));

		// Panel Tab
		Style->Set( "Docking.Tab", FDockTabStyle()
			.SetCloseButtonStyle( CloseButton )
			.SetNormalBrush( BOX_BRUSH( "/Docking/Tab_Shape", 2.f /8.0f, DockColor_Inactive ) )
			.SetHoveredBrush( BOX_BRUSH( "/Docking/Tab_Shape", 2.f /8.0f, DockColor_Hovered ) )
			.SetForegroundBrush( BOX_BRUSH( "/Docking/Tab_Shape", 2.f /8.0f, DockColor_Active ) )
			.SetColorOverlayTabBrush( BOX_BRUSH( "/Docking/Tab_ColorOverlay", 4 / 16.0f))
			.SetColorOverlayIconBrush( BOX_BRUSH( "/Docking/Tab_ColorOverlayIcon", 4/16.0f ) )
			.SetContentAreaBrush( FSlateColorBrush( DockColor_Active ) )
			.SetTabWellBrush( FSlateNoResource() )
			.SetTabPadding( FMargin(5, 2, 5, 2) )
			.SetOverlapWidth( -1.0f )
			.SetFlashColor( TabFlashColor )
			.SetTabTextStyle(DockingTabFont)
			);

		// App Tab
		Style->Set( "Docking.MajorTab", FDockTabStyle()
			.SetCloseButtonStyle( CloseButton )
			.SetNormalBrush( BOX_BRUSH( "/Docking/AppTab_Inactive", FMargin(24.0f/64.0f, 4/32.0f) ) )
			.SetColorOverlayTabBrush(BOX_BRUSH("/Docking/AppTab_ColorOverlay", FMargin(24.0f / 64.0f, 4 / 32.0f)))
			.SetColorOverlayIconBrush( BOX_BRUSH( "/Docking/AppTab_ColorOverlayIcon", FMargin(24.0f/64.0f, 4/32.0f) ) )
			.SetForegroundBrush( BOX_BRUSH( "/Docking/AppTab_Foreground", FMargin(24.0f/64.0f, 4/32.0f) ) )
			.SetHoveredBrush( BOX_BRUSH( "/Docking/AppTab_Hovered", FMargin(24.0f/64.0f, 4/32.0f) ) )
			.SetContentAreaBrush( BOX_BRUSH( "/Docking/AppTabContentArea", FMargin(4/16.0f) ) )
			.SetTabWellBrush( FSlateNoResource() )
			.SetTabPadding( FMargin(17, 4, 15, 4) )
			.SetOverlapWidth( 21.0f )
			.SetFlashColor( TabFlashColor )
			.SetTabTextStyle(DockingTabFont)
			);

		// Dock Cross
		Style->Set( "Docking.Cross.DockLeft", new IMAGE_BRUSH( "/Docking/OuterDockingIndicator", FVector2D(6, 6), FLinearColor(1.0f, 0.35f, 0.0f, 0.25f) ) );
		Style->Set( "Docking.Cross.DockLeft_Hovered", new IMAGE_BRUSH( "/Docking/OuterDockingIndicator", FVector2D(6, 6), FLinearColor(1.0f, 0.35f, 0.0f) ) );
		Style->Set( "Docking.Cross.DockTop", new IMAGE_BRUSH( "/Docking/OuterDockingIndicator", FVector2D(6, 6), FLinearColor(1.0f, 0.35f, 0.0f, 0.25f) ) );
		Style->Set( "Docking.Cross.DockTop_Hovered", new IMAGE_BRUSH( "/Docking/OuterDockingIndicator", FVector2D(6, 6), FLinearColor(1.0f, 0.35f, 0.0f) ) );
		Style->Set( "Docking.Cross.DockRight", new IMAGE_BRUSH( "/Docking/OuterDockingIndicator", FVector2D(6, 6), FLinearColor(1.0f, 0.35f, 0.0f, 0.25f) ) );
		Style->Set( "Docking.Cross.DockRight_Hovered", new IMAGE_BRUSH( "/Docking/OuterDockingIndicator", FVector2D(6, 6), FLinearColor(1.0f, 0.35f, 0.0f) ) );
		Style->Set( "Docking.Cross.DockBottom", new IMAGE_BRUSH( "/Docking/OuterDockingIndicator", FVector2D(6, 6), FLinearColor(1.0f, 0.35f, 0.0f, 0.25f) ) );
		Style->Set( "Docking.Cross.DockBottom_Hovered", new IMAGE_BRUSH( "/Docking/OuterDockingIndicator", FVector2D(6, 6), FLinearColor(1.0f, 0.35f, 0.0f) ) );
		Style->Set( "Docking.Cross.DockCenter", new IMAGE_BRUSH( "/Docking/DockingIndicator_Center", Icon64x64,  FLinearColor(1.0f, 0.35f, 0.0f, 0.25f) ) );
		Style->Set( "Docking.Cross.DockCenter_Hovered", new IMAGE_BRUSH( "/Docking/DockingIndicator_Center", Icon64x64,  FLinearColor(1.0f, 0.35f, 0.0f) ) );
		
		Style->Set( "Docking.Cross.BorderLeft",	new FSlateNoResource() );
		Style->Set( "Docking.Cross.BorderTop", new FSlateNoResource() );
		Style->Set( "Docking.Cross.BorderRight", new FSlateNoResource() );
		Style->Set( "Docking.Cross.BorderBottom", new FSlateNoResource() );
		Style->Set( "Docking.Cross.BorderCenter", new FSlateNoResource() );

		Style->Set( "Docking.Cross.PreviewWindowTint", FLinearColor(1.0f, 0.75f, 0.5f) );
		Style->Set( "Docking.Cross.Tint", FLinearColor::White );
		Style->Set( "Docking.Cross.HoveredTint", FLinearColor::White );
	}

	// SScrollBox defaults...
	{
		Style->Set( "ScrollBox", FScrollBoxStyle()
			.SetTopShadowBrush( BOX_BRUSH( "Common/ScrollBoxShadowTop", FVector2D(16, 8), FMargin(0.5, 1, 0.5, 0) ) )
			.SetBottomShadowBrush(BOX_BRUSH("Common/ScrollBoxShadowBottom", FVector2D(16, 8), FMargin(0.5, 0, 0.5, 1)))
			.SetLeftShadowBrush(BOX_BRUSH("Common/ScrollBoxShadowLeft", FVector2D(8, 16), FMargin(1, 0.5, 0, 0.5)))
			.SetRightShadowBrush(BOX_BRUSH("Common/ScrollBoxShadowRight", FVector2D(8, 16), FMargin(0, 0.5, 1, 0.5)))
			);
	}

	// SScrollBorder defaults...
	{
		Style->Set( "ScrollBorder", FScrollBorderStyle()
			.SetTopShadowBrush(BOX_BRUSH("Common/ScrollBorderShadowTop", FVector2D(16, 8), FMargin(0.5, 1, 0.5, 0)))
			.SetBottomShadowBrush(BOX_BRUSH("Common/ScrollBorderShadowBottom", FVector2D(16, 8), FMargin(0.5, 0, 0.5, 1)))
			);
	}

	// SWindow defaults...
	{
#if !PLATFORM_MAC
		const FButtonStyle MinimizeButtonStyle = FButtonStyle(Button)
			.SetNormal (IMAGE_BRUSH("Common/Window/WindowButton_Minimize_Normal", FVector2D(27.0f, 18.0f)))
			.SetHovered(IMAGE_BRUSH("Common/Window/WindowButton_Minimize_Hovered", FVector2D(27.0f, 18.0f)))
			.SetPressed(IMAGE_BRUSH("Common/Window/WindowButton_Minimize_Pressed", FVector2D(27.0f, 18.0f)));

		const FButtonStyle MaximizeButtonStyle = FButtonStyle(Button)
			.SetNormal (IMAGE_BRUSH("Common/Window/WindowButton_Maximize_Normal", FVector2D(23.0f, 18.0f)))
			.SetHovered(IMAGE_BRUSH("Common/Window/WindowButton_Maximize_Hovered", FVector2D(23.0f, 18.0f)))
			.SetPressed(IMAGE_BRUSH("Common/Window/WindowButton_Maximize_Pressed", FVector2D(23.0f, 18.0f)));

		const FButtonStyle RestoreButtonStyle = FButtonStyle(Button)
			.SetNormal (IMAGE_BRUSH("Common/Window/WindowButton_Restore_Normal", FVector2D(23.0f, 18)))
			.SetHovered(IMAGE_BRUSH("Common/Window/WindowButton_Restore_Hovered", FVector2D(23.0f, 18)))
			.SetPressed(IMAGE_BRUSH("Common/Window/WindowButton_Restore_Pressed", FVector2D(23.0f, 18)));

		const FButtonStyle CloseButtonStyle = FButtonStyle(Button)
			.SetNormal (IMAGE_BRUSH("Common/Window/WindowButton_Close_Normal", FVector2D(44.0f, 18.0f)))
			.SetHovered(IMAGE_BRUSH("Common/Window/WindowButton_Close_Hovered", FVector2D(44.0f, 18.0f)))
			.SetPressed(IMAGE_BRUSH("Common/Window/WindowButton_Close_Pressed", FVector2D(44.0f, 18.0f)));
#endif

		const FTextBlockStyle TitleTextStyle = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 9))
			.SetColorAndOpacity(FLinearColor::White)
			.SetShadowOffset(FVector2D(1.0f, 1.0f))
			.SetShadowColorAndOpacity(FLinearColor::Black);

		Style->Set( "Window", FWindowStyle()
#if !PLATFORM_MAC
			.SetMinimizeButtonStyle(MinimizeButtonStyle)
			.SetMaximizeButtonStyle(MaximizeButtonStyle)
			.SetRestoreButtonStyle(RestoreButtonStyle)
			.SetCloseButtonStyle(CloseButtonStyle)
#endif
			.SetTitleTextStyle(TitleTextStyle)
			.SetActiveTitleBrush(IMAGE_BRUSH("Common/Window/WindowTitle", Icon32x32, FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), ESlateBrushTileType::Horizontal))
			.SetInactiveTitleBrush(IMAGE_BRUSH("Common/Window/WindowTitle_Inactive", Icon32x32, FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), ESlateBrushTileType::Horizontal))
			.SetFlashTitleBrush(IMAGE_BRUSH("Common/Window/WindowTitle_Flashing", Icon24x24, FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), ESlateBrushTileType::Horizontal))
			.SetOutlineBrush(BORDER_BRUSH("Common/Window/WindowOutline", FMargin(3.0f / 32.0f)))
			.SetOutlineColor(FLinearColor(0.1f, 0.1f, 0.1f, 1.0f))
			.SetBorderBrush(BOX_BRUSH("Common/Window/WindowBorder", 0.48f))
			.SetBackgroundBrush(IMAGE_BRUSH( "Common/Window/WindowBackground", FVector2D(74.0f, 74.0f), FLinearColor::White, ESlateBrushTileType::Both))
			.SetChildBackgroundBrush(IMAGE_BRUSH( "Common/NoiseBackground", FVector2D(64.0f, 64.0f), FLinearColor::White, ESlateBrushTileType::Both))
			);
	}

	// STutorialWrapper defaults...
	{
		Style->Set("Tutorials.Border", new BOX_BRUSH("Tutorials/TutorialBorder", FVector2D(64.0f, 64.0f), FMargin(25.0f/ 64.0f)));
		Style->Set("Tutorials.Shadow", new BOX_BRUSH("Tutorials/TutorialShadow", FVector2D(256.0f, 256.0f), FMargin(114.0f / 256.0f)));
	}

	// Standard Dialog Settings
	{
		Style->Set("StandardDialog.ContentPadding",FMargin(16.0f, 3.0f));
		Style->Set("StandardDialog.SlotPadding",FMargin(8.0f, 0.0f, 0.0f, 0.0f));
		Style->Set("StandardDialog.MinDesiredSlotWidth", 80.0f);
		Style->Set("StandardDialog.MinDesiredSlotHeight", 0.0f);
		Style->Set("StandardDialog.LargeFont", DEFAULT_FONT("Regular", 11));
	}

	// Widget Reflector Window
	{
		Style->Set("WidgetReflector.TabIcon", new IMAGE_BRUSH("Icons/icon_tab_WidgetReflector_16x", Icon16x16));
		Style->Set("WidgetReflector.Icon", new IMAGE_BRUSH("Icons/icon_tab_WidgetReflector_40x", Icon40x40));
		Style->Set("WidgetReflector.Icon.Small", new IMAGE_BRUSH("Icons/icon_tab_WidgetReflector_40x", Icon20x20));
		Style->Set("WidgetReflector.FocusableCheck", FCheckBoxStyle()
			.SetUncheckedImage(IMAGE_BRUSH("Icons/Empty_14x", Icon14x14))
			.SetUncheckedHoveredImage(IMAGE_BRUSH("Icons/Empty_14x", Icon14x14))
			.SetUncheckedPressedImage(IMAGE_BRUSH("Common/SmallCheckBox_Hovered", Icon14x14))
			.SetCheckedImage(IMAGE_BRUSH("Common/SmallCheck", Icon14x14))
			.SetCheckedHoveredImage(IMAGE_BRUSH("Common/SmallCheck", Icon14x14))
			.SetCheckedPressedImage(IMAGE_BRUSH("Common/SmallCheck", Icon14x14))
			.SetUndeterminedImage(IMAGE_BRUSH("Icons/Empty_14x", Icon14x14))
			.SetUndeterminedHoveredImage(FSlateNoResource())
			.SetUndeterminedPressedImage(FSlateNoResource())
			);
	}

	// Message Log
	{
		Style->Set("MessageLog", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 8))
			.SetShadowOffset(FVector2D::ZeroVector)
		);
		Style->Set("MessageLog.Error", new IMAGE_BRUSH("MessageLog/Log_Error", Icon16x16));
		Style->Set("MessageLog.Warning", new IMAGE_BRUSH("MessageLog/Log_Warning", Icon16x16));
		Style->Set("MessageLog.Note", new IMAGE_BRUSH("MessageLog/Log_Note", Icon16x16));
	}

	// Wizard icons
	{
		Style->Set("Wizard.BackIcon", new IMAGE_BRUSH("Icons/BackIcon", Icon8x8));
		Style->Set("Wizard.NextIcon", new IMAGE_BRUSH("Icons/NextIcon", Icon8x8));
	}

	// Syntax highlighting
	{
		const FTextBlockStyle SmallMonospacedText = FTextBlockStyle(MonospacedText)
			.SetFont(DEFAULT_FONT("Mono", 9));

		Style->Set("SyntaxHighlight.Normal", SmallMonospacedText);
		Style->Set("SyntaxHighlight.Node", FTextBlockStyle(SmallMonospacedText).SetColorAndOpacity(FLinearColor(FColor(0xff006ab4)))); // blue
		Style->Set("SyntaxHighlight.NodeAttributeKey", FTextBlockStyle(SmallMonospacedText).SetColorAndOpacity(FLinearColor(FColor(0xffb40000)))); // red
		Style->Set("SyntaxHighlight.NodeAttribueAssignment", FTextBlockStyle(SmallMonospacedText).SetColorAndOpacity(FLinearColor(FColor(0xffb2b400)))); // yellow
		Style->Set("SyntaxHighlight.NodeAttributeValue", FTextBlockStyle(SmallMonospacedText).SetColorAndOpacity(FLinearColor(FColor(0xffb46100)))); // orange
	}
#endif 
	return Style;

}

UE_ENABLE_OPTIMIZATION_SHIP

#undef RootToContentDir


const TSharedPtr<FSlateDynamicImageBrush> FCoreStyle::GetDynamicImageBrush( FName BrushTemplate, FName TextureName, const ANSICHAR* Specifier )
{
	return Instance->GetDynamicImageBrush(BrushTemplate, TextureName, Specifier);
}


const TSharedPtr<FSlateDynamicImageBrush> FCoreStyle::GetDynamicImageBrush( FName BrushTemplate, const ANSICHAR* Specifier, class UTexture2D* TextureResource, FName TextureName )
{
	return Instance->GetDynamicImageBrush(BrushTemplate, Specifier, TextureResource, TextureName);
}


const TSharedPtr<FSlateDynamicImageBrush> FCoreStyle::GetDynamicImageBrush( FName BrushTemplate, class UTexture2D* TextureResource, FName TextureName )
{
	return Instance->GetDynamicImageBrush(BrushTemplate, TextureResource, TextureName);
}

/* FSlateThrottleManager implementation
 *****************************************************************************/

void FCoreStyle::SetStyle( const TSharedRef< ISlateStyle >& NewStyle )
{
	if (Instance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*Instance.Get());
	}

	Instance = NewStyle;

	if (Instance.IsValid())
	{
		FSlateStyleRegistry::RegisterSlateStyle(*Instance.Get());
	}
	else
	{
		ResetToDefault();
	}
}
