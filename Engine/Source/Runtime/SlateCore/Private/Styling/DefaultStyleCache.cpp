// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/DefaultStyleCache.h"

#include "Misc/LazySingleton.h"
#include "Styling/SlateTypes.h"
#include "Styling/UMGCoreStyle.h"

#if WITH_EDITOR
#include "Styling/CoreStyle.h"
#endif

namespace UE::Slate::Private
{

FDefaultStyleCache& FDefaultStyleCache::Get()
{
	return TLazySingleton<FDefaultStyleCache>::Get();
}

FDefaultStyleCache::FDefaultStyleCache()
{
	if (!IsRunningDedicatedServer())
	{
		Runtime.ButtonStyle = FUMGCoreStyle::Get().GetWidgetStyle<FButtonStyle>("Button");
		Runtime.ButtonStyle.UnlinkColors();

		Runtime.CheckboxStyle = FUMGCoreStyle::Get().GetWidgetStyle<FCheckBoxStyle>("Checkbox");
		Runtime.CheckboxStyle.UnlinkColors();

		Runtime.CircularThrobberBrushStyle = *FUMGCoreStyle::Get().GetBrush("Throbber.CircleChunk");
		Runtime.CircularThrobberBrushStyle.UnlinkColors();

		Runtime.ComboBoxStyle = FUMGCoreStyle::Get().GetWidgetStyle<FComboBoxStyle>("ComboBox");
		Runtime.ComboBoxStyle.UnlinkColors();

		Runtime.EditableTextStyle = FUMGCoreStyle::Get().GetWidgetStyle<FEditableTextStyle>("NormalEditableText");
		Runtime.EditableTextStyle.UnlinkColors();

		Runtime.EditableTextBoxStyle = FUMGCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
		Runtime.EditableTextBoxStyle.UnlinkColors();

		Runtime.ExpandableAreaStyle = FUMGCoreStyle::Get().GetWidgetStyle<FExpandableAreaStyle>("ExpandableArea");
		Runtime.ExpandableAreaStyle.UnlinkColors();
		Runtime.ExpandableAreaBorderBrush = *FUMGCoreStyle::Get().GetBrush("ExpandableArea.Border");
		Runtime.ExpandableAreaBorderBrush.UnlinkColors();

		Runtime.ListViewStyle = FUMGCoreStyle::Get().GetWidgetStyle<FTableViewStyle>("ListView");
		Runtime.ListViewStyle.UnlinkColors();

		Runtime.ProgressBarStyle = FUMGCoreStyle::Get().GetWidgetStyle<FProgressBarStyle>("ProgressBar");
		Runtime.ProgressBarStyle.UnlinkColors();

		Runtime.ScrollBarStyle = FUMGCoreStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar");
		Runtime.ScrollBarStyle.UnlinkColors();

		Runtime.ScrollBoxStyle = FUMGCoreStyle::Get().GetWidgetStyle<FScrollBoxStyle>("ScrollBox");
		Runtime.ScrollBoxStyle.UnlinkColors();

		Runtime.SliderStyle = FUMGCoreStyle::Get().GetWidgetStyle<FSliderStyle>("Slider");
		Runtime.SliderStyle.UnlinkColors();

		Runtime.SpinBoxStyle = FUMGCoreStyle::Get().GetWidgetStyle<FSpinBoxStyle>("SpinBox");
		Runtime.SpinBoxStyle.UnlinkColors();

		Runtime.TableRowStyle = FUMGCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");
		Runtime.TableRowStyle.UnlinkColors();

		Runtime.ThrobberBrush = *FUMGCoreStyle::Get().GetBrush("Throbber.Chunk");
		Runtime.ThrobberBrush.UnlinkColors();

		Runtime.TextBlockStyle = FUMGCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
		Runtime.TextBlockStyle.UnlinkColors();

		Runtime.TreeViewStyle = FUMGCoreStyle::Get().GetWidgetStyle<FTableViewStyle>("TreeView");
		Runtime.TreeViewStyle.UnlinkColors();

#if WITH_EDITOR
		Editor.ButtonStyle = FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("EditorUtilityButton");
		Editor.ButtonStyle.UnlinkColors();

		Editor.CheckboxStyle = FCoreStyle::Get().GetWidgetStyle<FCheckBoxStyle>("Checkbox");
		Editor.CheckboxStyle.UnlinkColors();

		Editor.CircularThrobberBrushStyle = *FCoreStyle::Get().GetBrush("Throbber.CircleChunk");
		Editor.CircularThrobberBrushStyle.UnlinkColors();

		Editor.ComboBoxStyle = FCoreStyle::Get().GetWidgetStyle<FComboBoxStyle>("EditorUtilityComboBox");
		Editor.ComboBoxStyle.UnlinkColors();

		Editor.EditableTextStyle = FCoreStyle::Get().GetWidgetStyle<FEditableTextStyle>("NormalEditableText");
		Editor.EditableTextStyle.UnlinkColors();

		Editor.EditableTextBoxStyle = FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
		Editor.EditableTextBoxStyle.UnlinkColors();

		Editor.ExpandableAreaStyle = FCoreStyle::Get().GetWidgetStyle<FExpandableAreaStyle>("ExpandableArea");
		Editor.ExpandableAreaStyle.UnlinkColors();
		Editor.ExpandableAreaBorderBrush = *FCoreStyle::Get().GetBrush("ExpandableArea.Border");
		Editor.ExpandableAreaBorderBrush.UnlinkColors();

		Editor.ListViewStyle = FCoreStyle::Get().GetWidgetStyle<FTableViewStyle>("ListView");
		Editor.ListViewStyle.UnlinkColors();

		Editor.ProgressBarStyle = FCoreStyle::Get().GetWidgetStyle<FProgressBarStyle>("ProgressBar");
		Editor.ProgressBarStyle.UnlinkColors();

		Editor.ScrollBarStyle = FCoreStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar");
		Editor.ScrollBarStyle.UnlinkColors();

		Editor.ScrollBoxStyle = FCoreStyle::Get().GetWidgetStyle<FScrollBoxStyle>("ScrollBox");
		Editor.ScrollBoxStyle.UnlinkColors();

		Editor.SliderStyle = FCoreStyle::Get().GetWidgetStyle<FSliderStyle>("Slider");
		Editor.SliderStyle.UnlinkColors();

		Editor.SpinBoxStyle = FCoreStyle::Get().GetWidgetStyle<FSpinBoxStyle>("SpinBox");
		Editor.SpinBoxStyle.UnlinkColors();

		Editor.TableRowStyle = FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");
		Editor.TableRowStyle.UnlinkColors();

		Editor.ThrobberBrush = *FCoreStyle::Get().GetBrush("Throbber.Chunk");
		Editor.ThrobberBrush.UnlinkColors();

		Editor.TextBlockStyle = FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
		Editor.TextBlockStyle.UnlinkColors();

		Editor.TreeViewStyle = FCoreStyle::Get().GetWidgetStyle<FTableViewStyle>("TreeView");
		Editor.TreeViewStyle.UnlinkColors();
#endif
	}
}

} //namespace UE::Slate::Private
