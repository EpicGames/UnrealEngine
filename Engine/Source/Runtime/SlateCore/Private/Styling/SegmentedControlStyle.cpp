// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/SegmentedControlStyle.h"
#include "Brushes/SlateNoResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SegmentedControlStyle)

const FName FSegmentedControlStyle::TypeName( TEXT("FSegmentedControlStyle") );

FSegmentedControlStyle::FSegmentedControlStyle()
	: BackgroundBrush(FSlateNoResource())
	, UniformPadding(0)
{
}

void FSegmentedControlStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add(&BackgroundBrush);
	ControlStyle.GetResources(OutBrushes);
	FirstControlStyle.GetResources(OutBrushes);
	LastControlStyle.GetResources(OutBrushes);
}

const FSegmentedControlStyle& FSegmentedControlStyle::GetDefault()
{
	static FSegmentedControlStyle Default;
	return Default;
}
