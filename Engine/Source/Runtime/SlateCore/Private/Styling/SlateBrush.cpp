// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/SlateBrush.h"
#include "SlateGlobals.h"
#include "Application/SlateApplicationBase.h"
#include "Types/SlateVector2.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateBrush)

FSlateBrush::FSlateBrush()
	: TintColor(FLinearColor::White)
	, DrawAs(ESlateBrushDrawType::Image)
	, Tiling(ESlateBrushTileType::NoTile)
	, Mirroring(ESlateBrushMirrorType::NoMirror)
	, ImageType(ESlateBrushImageType::NoImage)
	, ImageSize(SlateBrushDefs::DefaultImageSize, SlateBrushDefs::DefaultImageSize)
	, Margin(0.0f)
#if WITH_EDITORONLY_DATA
	, Tint_DEPRECATED(FLinearColor::White)
#endif
	, ResourceObject(nullptr)
	, UVRegion(ForceInit)
	, bIsDynamicallyLoaded(false)
	, bHasUObject_DEPRECATED(false)
	, bIsSet(true)
	, ResourceName(NAME_None)
{
}

FSlateBrush::FSlateBrush( ESlateBrushDrawType::Type InDrawType, 
						  const FName InResourceName, 
						  const FMargin& InMargin, 
						  ESlateBrushTileType::Type InTiling, 
						  ESlateBrushImageType::Type InImageType, 
						  const UE::Slate::FDeprecateVector2DParameter& InImageSize, 
						  const FLinearColor& InTint, 
						  UObject* InObjectResource, 
						  bool bInDynamicallyLoaded
						)

	: TintColor( InTint )
	, DrawAs(InDrawType)
	, Tiling(InTiling)
	, Mirroring(ESlateBrushMirrorType::NoMirror)
	, ImageType(InImageType)
	, ImageSize( InImageSize )
	, Margin( InMargin )
#if WITH_EDITORONLY_DATA
	, Tint_DEPRECATED(FLinearColor::White)
#endif
	, ResourceObject( InObjectResource )
	, UVRegion( ForceInit )
	, bIsDynamicallyLoaded(bInDynamicallyLoaded)
	, bIsSet(true)
	, ResourceName( InResourceName )
{
	bHasUObject_DEPRECATED = (InObjectResource != nullptr) || InResourceName.ToString().StartsWith(FSlateBrush::UTextureIdentifier());

	//Useful for debugging style breakages
	//if ( !bHasUObject_DEPRECATED && InResourceName.IsValid() && InResourceName != NAME_None )
	//{
	//	checkf( FPaths::FileExists( InResourceName.ToString() ), TEXT("The resource '%s' doesn't exist"), *FPaths::ConvertRelativePathToFull( InResourceName.ToString() ) );
	//}
}

FSlateBrush::FSlateBrush( ESlateBrushDrawType::Type InDrawType,
 						  const FName InResourceName,
 						  const FMargin& InMargin,
 						  ESlateBrushTileType::Type InTiling,
 						  ESlateBrushImageType::Type InImageType,
 						  const UE::Slate::FDeprecateVector2DParameter& InImageSize,
 						  const TSharedRef< FLinearColor >& InTint,
 						  UObject* InObjectResource, 
 						  bool bInDynamicallyLoaded
 						)

	: TintColor( InTint )
	, DrawAs(InDrawType)
	, Tiling(InTiling)
	, Mirroring(ESlateBrushMirrorType::NoMirror)
	, ImageType(InImageType)
	, ImageSize(InImageSize)
	, Margin( InMargin )
#if WITH_EDITORONLY_DATA
	, Tint_DEPRECATED(FLinearColor::White)
#endif
	, ResourceObject( InObjectResource )
	, UVRegion( ForceInit )
	, bIsDynamicallyLoaded(bInDynamicallyLoaded)
	, bIsSet(true)
	, ResourceName( InResourceName )
{
	bHasUObject_DEPRECATED = (InObjectResource != nullptr) || InResourceName.ToString().StartsWith(FSlateBrush::UTextureIdentifier());

	//Useful for debugging style breakages
	//if ( !bHasUObject_DEPRECATED && InResourceName.IsValid() && InResourceName != NAME_None )
	//{
	//	checkf( FPaths::FileExists( InResourceName.ToString() ), TEXT("The resource '%s' doesn't exist"), *FPaths::ConvertRelativePathToFull( InResourceName.ToString() ) );
	//}
}

FSlateBrush::FSlateBrush( ESlateBrushDrawType::Type InDrawType, 
						  const FName InResourceName, 
						  const FMargin& InMargin,
						  ESlateBrushTileType::Type InTiling, 
						  ESlateBrushImageType::Type InImageType, 
						  const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FSlateColor& InTint, 
						  UObject* InObjectResource, 
						  bool bInDynamicallyLoaded
 						)

	: TintColor(InTint)
	, DrawAs(InDrawType)
	, Tiling(InTiling)
	, Mirroring(ESlateBrushMirrorType::NoMirror)
	, ImageType(InImageType)
	, ImageSize(InImageSize)
	, Margin(InMargin)
#if WITH_EDITORONLY_DATA
	, Tint_DEPRECATED(FLinearColor::White)
#endif
	, ResourceObject(InObjectResource)
	, UVRegion(ForceInit)
	, bIsDynamicallyLoaded(bInDynamicallyLoaded)
	, bIsSet(true)
	, ResourceName(InResourceName)
{
	bHasUObject_DEPRECATED = (InObjectResource != nullptr) || InResourceName.ToString().StartsWith(FSlateBrush::UTextureIdentifier());

	//Useful for debugging style breakages
	//if ( !bHasUObject_DEPRECATED && InResourceName.IsValid() && InResourceName != NAME_None )
	//{
	//	checkf( FPaths::FileExists( InResourceName.ToString() ), TEXT("The resource '%s' doesn't exist"), *FPaths::ConvertRelativePathToFull( InResourceName.ToString() ) );
	//}
}

const FString FSlateBrush::UTextureIdentifier()
{
	return FString("texture:/");
}

const FSlateResourceHandle& FSlateBrush::GetRenderingResource() const
{
	if (ImageType == ESlateBrushImageType::Vector)
	{
		UE_LOGF(LogSlate, Warning, "FSlateBrush::GetRenderingResource should be called with a size and scale for vector brushes");
	}
	
	UpdateRenderingResource(GetImageSize(), 1.0f);

	return ResourceHandle;
}

void FSlateBrush::UpdateRenderingResource(FVector2f LocalSize, float DrawScale) const
{
	if (DrawAs != ESlateBrushDrawType::NoDrawType && (ResourceName != NAME_None || ResourceObject != nullptr))
	{
		// Always re-acquire a handle if the current handle is invalid or if its vector graphics.
		// For vector graphics we will rebuild the handle only if the shape needs to be rasterized again and the new size and scale
		if (!ResourceHandle.IsValid() || ImageType == ESlateBrushImageType::Vector)
		{
			ResourceHandle = FSlateApplicationBase::Get().GetRenderer()->GetResourceHandle(*this, LocalSize, DrawScale);
		}
		else if (ResourceHandle.IsValid())
		{
			// Test the resource itself
			if (FSlateShaderResource* Resource = ResourceHandle.GetResourceProxy()->Resource)
			{
				if (!Resource->IsResourceValid())
				{
					ResourceHandle = FSlateApplicationBase::Get().GetRenderer()->GetResourceHandle(*this, LocalSize, DrawScale);
				}
			}
		}
	}

}

bool FSlateBrush::CanRenderResourceObject(UObject* InResourceObject) const
{
	if (InResourceObject)
	{
		if (FSlateApplicationBase::IsInitialized())
		{
			return FSlateApplicationBase::Get().GetRenderer()->CanRenderResource(*InResourceObject);
		}
	}

	return true;
}

void FSlateBrush::SetResourceObject(class UObject* InResourceObject)
{
#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
	// This check is not safe to run from all threads, and would crash in debug
	if (!ensure(!IsThreadSafeForSlateRendering() || CanRenderResourceObject(InResourceObject)))
	{
		// If we can't render the resource return, don't let people use them as brushes, we'll just crash later.
		return;
	}
#endif

	if (ResourceObject != InResourceObject)
	{
		ResourceObject = InResourceObject;
		// Invalidate resource handle
		ResourceHandle = FSlateResourceHandle();
	}
}
