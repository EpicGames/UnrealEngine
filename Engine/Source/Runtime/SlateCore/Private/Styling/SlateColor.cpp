// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/SlateColor.h"
#include "UObject/PropertyTag.h"
#include "Styling/StyleColors.h"
#include "SlateGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateColor)

bool FSlateColor::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Color))
	{
		FColor OldColor;
		Slot << OldColor;
		*this = FSlateColor(FLinearColor(OldColor));

		return true;
	}
	else if (Tag.GetType().IsStruct(NAME_LinearColor))
	{
		FLinearColor OldColor;
		Slot << OldColor;
		*this = FSlateColor(OldColor);

		return true;
	}

	return false;
}

const FLinearColor& FSlateColor::GetColorFromTable() const
{
	return USlateThemeManager::Get().GetColor(ColorTableId);
}

