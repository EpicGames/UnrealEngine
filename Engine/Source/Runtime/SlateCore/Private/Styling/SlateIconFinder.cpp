// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/SlateIconFinder.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

FSlateIcon FSlateIconFinder::FindIconForClass(const UStruct* InStruct, const FName& InDefaultName)
{
	return FindCustomIconForClass(InStruct, TEXT("ClassIcon"), InDefaultName);
}

FSlateIcon FSlateIconFinder::FindCustomIconForClass(const UStruct* InStruct, const TCHAR* InStyleBasePath, const FName& InDefaultName)
{
	FString IconPath(InStyleBasePath);
	IconPath += TEXT(".");
	const int32 BastPathLength = IconPath.Len();

	if (InStruct)
	{
		// walk up struct hierarchy until we find an icon
		const UStruct* CurrentStruct = InStruct;
		while (CurrentStruct)
		{
			CurrentStruct->AppendName(IconPath);
			FSlateIcon Icon = FindIcon(*IconPath);

			if (Icon.IsSet())
			{
				return Icon;
			}

			CurrentStruct = CurrentStruct->GetSuperStruct();
			IconPath.RemoveAt(BastPathLength, IconPath.Len() - BastPathLength, EAllowShrinking::No);
		}
	}

	// If we didn't supply an override name for the default icon use default class icon.
	if (InDefaultName.IsNone())
	{
		IconPath += TEXT("Default");
		return FindIcon(*IconPath);
	}

	return FindIcon(InDefaultName);
}

FSlateIcon FSlateIconFinder::FindIcon(const FName& InIconName)
{
	FSlateIcon Icon;

	FSlateStyleRegistry::IterateAllStyles(
		[&](const ISlateStyle& Style)
		{
			if (Style.GetOptionalBrush(InIconName, nullptr, nullptr))
			{
				Icon = FSlateIcon(Style.GetStyleSetName(), InIconName);
				// terminate iteration
				return false;
			}
			// continue iteration
			return true;
		}
	);

	return Icon;
}
