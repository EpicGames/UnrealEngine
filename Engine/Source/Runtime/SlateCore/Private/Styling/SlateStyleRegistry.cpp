// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/SlateStyleRegistry.h"
#include "Styling/ISlateStyle.h"
#include "Application/SlateApplicationBase.h"


/* Static initialization
 *****************************************************************************/

TMap<FName, const ISlateStyle*> FSlateStyleRegistry::SlateStyleRepository;


/* Static functions
 *****************************************************************************/

void FSlateStyleRegistry::RegisterSlateStyle(const ISlateStyle& InSlateStyle)
{
	const FName& SlateStyleName = InSlateStyle.GetStyleSetName();
	check(SlateStyleName.IsValid());
	check(!FindSlateStyle(SlateStyleName));

	SlateStyleRepository.Add(SlateStyleName, &InSlateStyle);

	if (FSlateApplicationBase::IsInitialized())
	{
		if (FSlateRenderer* Renderer = FSlateApplicationBase::Get().GetRenderer())
		{
			Renderer->LoadStyleResources(InSlateStyle);
		}
	}
}


void FSlateStyleRegistry::UnRegisterSlateStyle(const ISlateStyle& InSlateStyle)
{
	const FName& SlateStyleName = InSlateStyle.GetStyleSetName();
	check(SlateStyleName.IsValid());

	SlateStyleRepository.Remove(SlateStyleName);
}


void FSlateStyleRegistry::UnRegisterSlateStyle(const FName StyleSetName)
{
	check(StyleSetName.IsValid());
	SlateStyleRepository.Remove(StyleSetName);
}


const ISlateStyle* FSlateStyleRegistry::FindSlateStyle(const FName& InSlateStyleName)
{
	const ISlateStyle** SlateStylePtr = SlateStyleRepository.Find(InSlateStyleName);
	return (SlateStylePtr) ? *SlateStylePtr : nullptr;
}

bool FSlateStyleRegistry::IterateAllStyles(const TFunctionRef<bool(const ISlateStyle&)>& Iter)
{
	for (auto& Pair : SlateStyleRepository)
	{
		if (!Iter(*Pair.Value))
		{
			return false;
		}
	}
	return true;
}

void FSlateStyleRegistry::GetAllResources(TArray< const FSlateBrush* >& OutResources)
{
	/* Iterate the style chunks and collect their resources */
	for (auto It = SlateStyleRepository.CreateConstIterator(); It; ++It)
	{
		It->Value->GetResources(OutResources);
	}
}

TArray<FName> FSlateStyleRegistry::GetSylesUsingBrush(const FName BrushName)
{
	TArray<FName> StylesUsingBrush;

	/* Iterate the style chunks and collect their resources */
	for (auto It = SlateStyleRepository.CreateConstIterator(); It; ++It)
	{
		StylesUsingBrush.Append(It->Value->GetEntriesUsingBrush(BrushName));
	}

	return StylesUsingBrush;
}
