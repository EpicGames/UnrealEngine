// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "Fonts/SlateFontInfo.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/StyleDefaults.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"


FSlateStyleSet::FSlateStyleSet(const FName& InStyleSetName)
: StyleSetName(InStyleSetName)
, DefaultBrush(new FSlateImageBrush(FPaths::EngineContentDir() / TEXT("Slate/Checkerboard.png"), FVector2f(16.f, 16.f), FLinearColor::White, ESlateBrushTileType::Both))
{
	// Add a mapping so that this resource will be discovered by GetStyleResources.
	Set(TEXT("Default"), GetDefaultBrush());
}

FSlateStyleSet::FSlateStyleSet(FSlateStyleSet&&) = default;
FSlateStyleSet::FSlateStyleSet(const FSlateStyleSet&) = default;

FSlateStyleSet::~FSlateStyleSet()
{
	// Delete all allocated brush resources.
	for ( TMap< FName, FSlateBrush* >::TIterator It(BrushResources); It; ++It )
	{
		if (!It.Value()->HasUObject())
		{
			delete It.Value();
		}
	}
}

const FName& FSlateStyleSet::GetStyleSetName() const
{
	return StyleSetName;
}

void FSlateStyleSet::GetResources(TArray< const FSlateBrush* >& OutResources) const
{
	// Collection for this style's brush resources.
	TArray< const FSlateBrush* > SlateBrushResources;
	for ( TMap< FName, FSlateBrush* >::TConstIterator It(BrushResources); It; ++It )
	{
		SlateBrushResources.Add(It.Value());
	}

	//Gather resources from our definitions
	for ( TMap< FName, TSharedRef< struct FSlateWidgetStyle > >::TConstIterator It(WidgetStyleValues); It; ++It )
	{
		It.Value()->GetResources(SlateBrushResources);
	}

	// Append this style's resources to OutResources.
	OutResources.Append(SlateBrushResources);
}

TArray<FName> FSlateStyleSet::GetEntriesUsingBrush(const FName BrushName) const
{
	TArray<FName> FoundStyles;

	// Collection for this style's brush resources.
	for (TMap< FName, FSlateBrush* >::TConstIterator It(BrushResources); It; ++It)
	{
		if (It.Value()->GetResourceName() == BrushName)
		{
			FoundStyles.Add(It.Key());
		}
	}

	//Gather resources from our definitions
	TArray<const FSlateBrush*> BrushesInStyle;
	BrushesInStyle.Reserve(20);

	for (TMap< FName, TSharedRef< struct FSlateWidgetStyle > >::TConstIterator It(WidgetStyleValues); It; ++It)
	{
		It.Value()->GetResources(BrushesInStyle);

		for (const FSlateBrush* Brush : BrushesInStyle)
		{
			if (Brush->GetResourceName() == BrushName)
			{
				FoundStyles.Add(It.Key());
			}
		}

		BrushesInStyle.Reset();
	}

	return FoundStyles;
}

void FSlateStyleSet::SetContentRoot(const FString& InContentRootDir)
{
	ContentRootDir = InContentRootDir;
}

FString FSlateStyleSet::RootToContentDir(const ANSICHAR* RelativePath, const TCHAR* Extension)
{
	return ( ContentRootDir / RelativePath ) + Extension;
}

FString FSlateStyleSet::RootToContentDir(const WIDECHAR* RelativePath, const TCHAR* Extension)
{
	return ( ContentRootDir / RelativePath ) + Extension;
}

FString FSlateStyleSet::RootToContentDir(const FString& RelativePath, const TCHAR* Extension)
{
	return ( ContentRootDir / RelativePath ) + Extension;
}

FString FSlateStyleSet::RootToContentDir(const ANSICHAR* RelativePath)
{
	return ( ContentRootDir / RelativePath );
}

FString FSlateStyleSet::RootToContentDir(const WIDECHAR* RelativePath)
{
	return ( ContentRootDir / RelativePath );
}

FString FSlateStyleSet::RootToContentDir(const FString& RelativePath)
{
	return ( ContentRootDir / RelativePath );
}

void FSlateStyleSet::SetCoreContentRoot(const FString& InCoreContentRootDir)
{
	CoreContentRootDir = InCoreContentRootDir;
}

FString FSlateStyleSet::RootToCoreContentDir(const ANSICHAR* RelativePath, const TCHAR* Extension)
{
	return ( CoreContentRootDir / RelativePath ) + Extension;
}

FString FSlateStyleSet::RootToCoreContentDir(const WIDECHAR* RelativePath, const TCHAR* Extension)
{
	return ( CoreContentRootDir / RelativePath ) + Extension;
}

FString FSlateStyleSet::RootToCoreContentDir(const FString& RelativePath, const TCHAR* Extension)
{
	return ( CoreContentRootDir / RelativePath ) + Extension;
}

FString FSlateStyleSet::RootToCoreContentDir(const ANSICHAR* RelativePath)
{
	return ( CoreContentRootDir / RelativePath );
}

FString FSlateStyleSet::RootToCoreContentDir(const WIDECHAR* RelativePath)
{
	return ( CoreContentRootDir / RelativePath );
}

FString FSlateStyleSet::RootToCoreContentDir(const FString& RelativePath)
{
	return ( CoreContentRootDir / RelativePath );
}

void FSlateStyleSet::SetParentStyleName(const FName& InParentStyleName)
{
	ParentStyleName = InParentStyleName;	
}

const ISlateStyle* FSlateStyleSet::GetParentStyle() const
{
	return ParentStyleName.IsNone() ? nullptr : FSlateStyleRegistry::FindSlateStyle(ParentStyleName);
}

float FSlateStyleSet::GetFloat(const FName PropertyName, const ANSICHAR* Specifier, float DefaultValue, const ISlateStyle* RequestingStyle) const
{
	const float* Result = FloatValues.Find(Join(PropertyName, Specifier));

	if (Result == nullptr) 
	{
		if (const ISlateStyle* ParentStyle = GetParentStyle())
		{
			return ParentStyle->GetFloat(PropertyName, Specifier, DefaultValue, RequestingStyle ? RequestingStyle : this);
		}
	}

	// In case the required property is not available, and no DefaultValue was provided, let's log a Warning
	if (Result == nullptr && DefaultValue == FStyleDefaults::GetFloat())
	{
		const ISlateStyle* ReportingStyle = RequestingStyle != nullptr ? RequestingStyle : this;
		ReportingStyle->LogMissingResource(EStyleMessageSeverity::Warning, FText::Format(NSLOCTEXT("SlateStyleSet", "UnknownSlateFloat", "Unable to find float property '{0}' in style."), FText::FromName(PropertyName)), PropertyName);
	}

	return Result ? *Result : DefaultValue;
}

UE::Slate::FDeprecateVector2DResult FSlateStyleSet::GetVector(const FName PropertyName, const ANSICHAR* Specifier, UE::Slate::FDeprecateVector2DParameter DefaultValue, const ISlateStyle* RequestingStyle) const
{
	const FVector2f* const Result = Vector2DValues.Find(Join(PropertyName, Specifier));

	if (Result == nullptr) 
	{
		if (const ISlateStyle* ParentStyle = GetParentStyle())
		{
			return ParentStyle->GetVector(PropertyName, Specifier, DefaultValue);
		}
	}

	// In case the required property is not available, and no DefaultValue was provided, let's log a Warning
	if (Result == nullptr && DefaultValue == FStyleDefaults::GetVector2D())
	{
		const ISlateStyle* ReportingStyle = RequestingStyle != nullptr ? RequestingStyle : this;
		ReportingStyle->LogMissingResource(EStyleMessageSeverity::Warning, FText::Format(NSLOCTEXT("SlateStyleSet", "UnknownSlateVector", "Unable to find vector property '{0}' in style."), FText::FromName(PropertyName)), PropertyName);
	}

	return Result ? FVector2f(*Result) : DefaultValue;
}

const FLinearColor& FSlateStyleSet::GetColor(const FName PropertyName, const ANSICHAR* Specifier, const FLinearColor& DefaultValue, const ISlateStyle* RequestingStyle) const
{
	const FName LookupName(Join(PropertyName, Specifier));
	const FLinearColor* Result = ColorValues.Find(LookupName);

	if (Result == nullptr) 
	{
		if (const ISlateStyle* ParentStyle = GetParentStyle())
		{
			return ParentStyle->GetColor(PropertyName, Specifier, DefaultValue);
		}
	}

	// In case the required property is not available, and no DefaultValue was provided, let's log a Warning
	if (Result == nullptr && DefaultValue == FStyleDefaults::GetColor())
	{
		const ISlateStyle* ReportingStyle = RequestingStyle != nullptr ? RequestingStyle : this;
		ReportingStyle->LogMissingResource(EStyleMessageSeverity::Warning, FText::Format(NSLOCTEXT("SlateStyleSet", "UnknownColor", "Unable to find Color '{0}'."), FText::FromName(LookupName)), LookupName);
	}

	return Result ? *Result : DefaultValue;
}

const FSlateColor FSlateStyleSet::GetSlateColor(const FName PropertyName, const ANSICHAR* Specifier, const FSlateColor& DefaultValue, const ISlateStyle* RequestingStyle) const
{
	const FName StyleName(Join(PropertyName, Specifier));
	const FSlateColor* Result = SlateColorValues.Find(StyleName);
	const FLinearColor* LinearResult = nullptr;

	if ( Result == nullptr )
	{
		LinearResult = ColorValues.Find(StyleName);
	}

	if ( Result == nullptr && LinearResult == nullptr) 
	{
		if (const ISlateStyle* ParentStyle = GetParentStyle())
		{
			return ParentStyle->GetSlateColor(PropertyName, Specifier, DefaultValue);
		}
	}

	// In case the required property is not available, and no DefaultValue was provided, let's log a Warning
	if (Result == nullptr && LinearResult == nullptr && DefaultValue == FStyleDefaults::GetSlateColor())
	{
		const ISlateStyle* ReportingStyle = RequestingStyle != nullptr ? RequestingStyle : this;
		ReportingStyle->LogMissingResource(EStyleMessageSeverity::Warning, FText::Format(NSLOCTEXT("SlateStyleSet", "UnknownSlateColor", "Unable to find SlateColor '{0}'."), FText::FromName(StyleName)), StyleName);
	}

	return Result ? *Result : LinearResult ? *LinearResult : DefaultValue;
}

const FMargin& FSlateStyleSet::GetMargin(const FName PropertyName, const ANSICHAR* Specifier, const FMargin& DefaultValue, const ISlateStyle* RequestingStyle) const
{
	const FName StyleName(Join(PropertyName, Specifier));
	const FMargin* const Result = MarginValues.Find(StyleName);

	if (Result == nullptr) 
	{
		if (const ISlateStyle* ParentStyle = GetParentStyle())
		{
			return ParentStyle->GetMargin(PropertyName, Specifier, DefaultValue);
		}
	}

	// In case the required property is not available, and no DefaultValue was provided, let's log a Warning
	if (Result == nullptr && DefaultValue == FStyleDefaults::GetMargin())
	{
		const ISlateStyle* ReportingStyle = RequestingStyle != nullptr ? RequestingStyle : this;
		ReportingStyle->LogMissingResource(EStyleMessageSeverity::Warning, FText::Format(NSLOCTEXT("SlateStyleSet", "UnknownMargin", "Unable to find Margin '{0}'."), FText::FromName(StyleName)), StyleName);
	}

	return Result ? *Result : DefaultValue;
}

const FSlateBrush* FSlateStyleSet::GetBrush(const FName PropertyName, const ANSICHAR* Specifier, const ISlateStyle* RequestingStyle) const
{
	const FName StyleName = Join(PropertyName, Specifier);
	const FSlateBrush* Result = BrushResources.FindRef(StyleName);

	if ( Result == nullptr )
	{
		TWeakPtr< FSlateDynamicImageBrush > WeakImageBrush = DynamicBrushes.FindRef(StyleName);
		TSharedPtr< FSlateDynamicImageBrush > ImageBrush = WeakImageBrush.Pin();

		if ( ImageBrush.IsValid() )
		{
			Result = ImageBrush.Get();
		}
	}

	if (Result == nullptr) 
	{
		if (const ISlateStyle* ParentStyle = GetParentStyle())
		{
			return ParentStyle->GetBrush(PropertyName, Specifier);
		}
	}

	if ( Result == nullptr )
	{
		const ISlateStyle* ReportingStyle = RequestingStyle != nullptr ? RequestingStyle : this;
		ReportingStyle->LogMissingResource(EStyleMessageSeverity::Warning, FText::Format(NSLOCTEXT("SlateStyleSet", "UnknownBrush", "Unable to find Brush '{0}'."), FText::FromName(StyleName)), StyleName);
	}

	return Result ? Result : GetDefaultBrush();
}

const FSlateBrush* FSlateStyleSet::GetOptionalBrush(const FName PropertyName, const ANSICHAR* Specifier, const FSlateBrush* const InDefaultBrush) const
{
	const FName StyleName = Join(PropertyName, Specifier);
	const FSlateBrush* Result = BrushResources.FindRef(StyleName);

	if ( Result == nullptr )
	{
		TWeakPtr< FSlateDynamicImageBrush > WeakImageBrush = DynamicBrushes.FindRef(StyleName);
		TSharedPtr< FSlateDynamicImageBrush > ImageBrush = WeakImageBrush.Pin();

		if ( ImageBrush.IsValid() )
		{
			Result = ImageBrush.Get();
		}
	}

	if (Result == nullptr) 
	{
		if (const ISlateStyle* ParentStyle = GetParentStyle())
		{
			return ParentStyle->GetOptionalBrush(PropertyName, Specifier, InDefaultBrush);
		}
	}

	return Result ? Result : InDefaultBrush;
}

const TSharedPtr< FSlateDynamicImageBrush > FSlateStyleSet::GetDynamicImageBrush(const FName BrushTemplate, const FName TextureName, const ANSICHAR* Specifier, const ISlateStyle* RequestingStyle) const
{
	return GetDynamicImageBrush(BrushTemplate, Specifier, nullptr, TextureName);
}

const TSharedPtr< FSlateDynamicImageBrush > FSlateStyleSet::GetDynamicImageBrush(const FName BrushTemplate, const ANSICHAR* Specifier, UTexture2D* TextureResource, const FName TextureName, const ISlateStyle* RequestingStyle) const
{
	return GetDynamicImageBrush(Join(BrushTemplate, Specifier), TextureResource, TextureName);
}

const TSharedPtr< FSlateDynamicImageBrush > FSlateStyleSet::GetDynamicImageBrush(const FName BrushTemplate, UTexture2D* TextureResource, const FName TextureName, const ISlateStyle* RequestingStyle) const
{
	//create a resource name
	FName ResourceName;
	ResourceName = (TextureName == NAME_None) ? BrushTemplate : FName(*( BrushTemplate.ToString() + TextureName.ToString() ));

	//see if we already have that brush
	TWeakPtr< FSlateDynamicImageBrush > WeakImageBrush = DynamicBrushes.FindRef(ResourceName);
	TSharedPtr< FSlateDynamicImageBrush > ReturnBrush = WeakImageBrush.Pin();

	// if no brush was found, check referenced styles
	if (!ReturnBrush.IsValid()) 
	{
		if (const ISlateStyle* ParentStyle = GetParentStyle())
		{
			return ParentStyle->GetDynamicImageBrush(BrushTemplate, TextureResource, TextureName);
		}
	}

	// If no DynamicImageBrush was found and this style has no more referenced styles to check, 
	// then notify the RequestingStyle to make the DynamicImageBrush.
	// This is to avoid all DynamicImageBrushes being made on the RootStyle (usually CoreStyle)
	if ( !ReturnBrush.IsValid() )
	{
		const ISlateStyle* ReportingStyle = RequestingStyle != nullptr ? RequestingStyle : this;
		ReturnBrush = ReportingStyle->MakeDynamicImageBrush(BrushTemplate, TextureResource, TextureName);
	}

	return ReturnBrush;
}

FSlateBrush* FSlateStyleSet::GetDefaultBrush() const
{
	return DefaultBrush;
}

const FSlateSound& FSlateStyleSet::GetSound(const FName PropertyName, const ANSICHAR* Specifier, const ISlateStyle* RequestingStyle) const
{
	const FName StyleName = Join(PropertyName, Specifier);
	const FSlateSound* Result = Sounds.Find(StyleName);

	if (Result == nullptr) 
	{
		if (const ISlateStyle* ParentStyle = GetParentStyle())
		{
			return ParentStyle->GetSound(PropertyName, Specifier);
		}
	}

	if ( Result == nullptr )
	{
		const ISlateStyle* ReportingStyle = RequestingStyle != nullptr ? RequestingStyle : this;
		ReportingStyle->LogMissingResource(EStyleMessageSeverity::Warning, FText::Format(NSLOCTEXT("SlateStyleSet", "UnknownSound", "Unable to find Sound '{0}'."), FText::FromName(StyleName)), StyleName);
	}

	return Result ? *Result : FStyleDefaults::GetSound();
}

FSlateFontInfo FSlateStyleSet::GetFontStyle(const FName PropertyName, const ANSICHAR* Specifier) const
{
	const FSlateFontInfo* Result = FontInfoResources.Find(Join(PropertyName, Specifier));

	if (Result == nullptr) 
	{
		if (const ISlateStyle* ParentStyle = GetParentStyle())
		{
			return ParentStyle->GetFontStyle(PropertyName, Specifier);
		}
	}

	return Result ? *Result : FStyleDefaults::GetFontInfo();
}

const FSlateWidgetStyle* FSlateStyleSet::GetWidgetStyleInternal(const FName DesiredTypeName, const FName StyleName, const FSlateWidgetStyle* DefaultStyle, bool bWarnIfNotFound) const
{
	const TSharedRef< struct FSlateWidgetStyle >* StylePtr = WidgetStyleValues.Find(StyleName);

	if (StylePtr == nullptr) 
	{
		if (const ISlateStyle* ParentStyle = GetParentStyle())
		{
			if (const FSlateWidgetStyle* Result = ParentStyle->GetWidgetStyleInternal(DesiredTypeName, StyleName, DefaultStyle, bWarnIfNotFound))
			{
				return Result;
			}
		}
	}

	if ( StylePtr == nullptr )
	{
		if (bWarnIfNotFound)
		{
			Log(EStyleMessageSeverity::Warning, FText::Format(NSLOCTEXT("SlateStyleSet", "UnknownWidgetStyle", "Unable to find Slate Widget Style '{0}'. Using {1} defaults instead."), FText::FromName(StyleName), FText::FromName(DesiredTypeName)));
		}

		return DefaultStyle;
	}

	const TSharedRef< struct FSlateWidgetStyle > Style = *StylePtr;

	if ( Style->GetTypeName() != DesiredTypeName )
	{
		if (bWarnIfNotFound)
		{
			Log(EStyleMessageSeverity::Error, FText::Format(NSLOCTEXT("SlateStyleSet", "WrongWidgetStyleType", "The Slate Widget Style '{0}' is not of the desired type. Desired: '{1}', Actual: '{2}'"), FText::FromName(StyleName), FText::FromName(DesiredTypeName), FText::FromName(Style->GetTypeName())));
		}

		return nullptr;
	}

	return &Style.Get();
}

TSet<FName> FSlateStyleSet::GetStyleKeys() const
{
	TSet<FName> AllKeys;

	{
		TArray<FName> Keys;
		WidgetStyleValues.GenerateKeyArray(Keys);
		AllKeys.Append(Keys);
	}

	{
		TArray<FName> Keys;
		FloatValues.GenerateKeyArray(Keys);
		AllKeys.Append(Keys);
	}

	{
		TArray<FName> Keys;
		Vector2DValues.GenerateKeyArray(Keys);
		AllKeys.Append(Keys);
	}

	{
		TArray<FName> Keys;
		ColorValues.GenerateKeyArray(Keys);
		AllKeys.Append(Keys);
	}

	{
		TArray<FName> Keys;
		SlateColorValues.GenerateKeyArray(Keys);
		AllKeys.Append(Keys);
	}

	{
		TArray<FName> Keys;
		MarginValues.GenerateKeyArray(Keys);
		AllKeys.Append(Keys);
	}

	{
		TArray<FName> Keys;
		BrushResources.GenerateKeyArray(Keys);
		AllKeys.Append(Keys);
	}

	{
		TArray<FName> Keys;
		Sounds.GenerateKeyArray(Keys);
		AllKeys.Append(Keys);
	}

	{
		TArray<FName> Keys;
		FontInfoResources.GenerateKeyArray(Keys);
		AllKeys.Append(Keys);
	}

	return AllKeys;

}

const TSet<FName>& FSlateStyleSet::GetWidgetStyleNames() const
{
	return KnownWidgetStyles;
}

void FSlateStyleSet::Log(ISlateStyle::EStyleMessageSeverity Severity, const FText& Message) const
{
	if ( Severity == EStyleMessageSeverity::Error )
	{
		UE_LOGF(LogSlateStyle, Error, "%ls", *Message.ToString());
	}
	else if ( Severity == EStyleMessageSeverity::PerformanceWarning )
	{
		UE_LOGF(LogSlateStyle, Warning, "%ls", *Message.ToString());
	}
	else if ( Severity == EStyleMessageSeverity::Warning )
	{
		UE_LOGF(LogSlateStyle, Warning, "%ls", *Message.ToString());
	}
	else if ( Severity == EStyleMessageSeverity::Info )
	{
		UE_LOGF(LogSlateStyle, Log, "%ls", *Message.ToString());
	}
	else
	{
		UE_LOGF(LogSlateStyle, Fatal, "%ls", *Message.ToString());
	}
}

void FSlateStyleSet::LogMissingResource(EStyleMessageSeverity Severity, const FText& Message, const FName& MissingResource) const
{
	if (!MissingResources.Contains(MissingResource))
	{
		FText StyleAndMessage = FText::Format(NSLOCTEXT("SlateStyleSet", "SlateSetResourceMissing", "Missing Resource from '{0}' Style: '{1}'"), FText::FromName(GetStyleSetName()), Message);

		MissingResources.Add(MissingResource);
		Log(Severity, StyleAndMessage);
	}
}

const TSharedPtr< FSlateDynamicImageBrush > FSlateStyleSet::MakeDynamicImageBrush( const FName BrushTemplate, UTexture2D* TextureResource, const FName TextureName ) const
{
	// Double check we don't have this brush already defined
	FName ResourceName = (TextureName == NAME_None) ? BrushTemplate : FName(*( BrushTemplate.ToString() + TextureName.ToString() ));
	TWeakPtr< FSlateDynamicImageBrush > WeakImageBrush = DynamicBrushes.FindRef(ResourceName);
	TSharedPtr< FSlateDynamicImageBrush > ReturnBrush = WeakImageBrush.Pin();
	if (ReturnBrush.IsValid())
	{
		return ReturnBrush;
	}

	const FSlateBrush* TemplateResult = GetOptionalBrush(BrushTemplate, nullptr, GetDefaultBrush());

	// create the new brush
	ReturnBrush = MakeShareable(new FSlateDynamicImageBrush(TextureResource, TemplateResult->ImageSize, ResourceName));

	// add it to the dynamic brush list
	DynamicBrushes.Add(ResourceName, ReturnBrush);

	return ReturnBrush;

}

void FSlateStyleSet::LogUnusedBrushResources()
{
	TArray<FString> Filenames;
	IFileManager::Get().FindFilesRecursive(Filenames, *ContentRootDir, *FString("*.png"), true, false, false);

	for ( FString& FilePath : Filenames )
	{
		bool IsUsed = false;
		for ( auto& Entry : BrushResources )
		{
			if ( IsBrushFromFile(FilePath, Entry.Value) )
			{
				IsUsed = true;
				break;
			}
		}

		if ( !IsUsed )
		{
			for ( auto& Entry : WidgetStyleValues )
			{
				TArray< const FSlateBrush* > WidgetBrushes;
				Entry.Value->GetResources(WidgetBrushes);

				for ( const FSlateBrush* Brush : WidgetBrushes )
				{
					if ( IsBrushFromFile(FilePath, Brush) )
					{
						IsUsed = true;
						break;
					}
				}

				if ( IsUsed )
				{
					break;
				}
			}
		}

		if ( !IsUsed )
		{
			Log(EStyleMessageSeverity::Warning, FText::FromString(FilePath));
		}
	}
}

bool FSlateStyleSet::IsBrushFromFile(const FString& FilePath, const FSlateBrush* Brush)
{
	FString Path = Brush->GetResourceName().ToString();
	FPaths::MakeStandardFilename(Path);

	if ( Path.Compare(FilePath, ESearchCase::IgnoreCase) == 0 )
	{
		return true;
	}
	else
	{
		const FString FullFilePath = FPaths::ConvertRelativePathToFull(FilePath);
		const FString FullPath = FPaths::ConvertRelativePathToFull(Path);
		if (FullPath.Compare(FullFilePath, ESearchCase::IgnoreCase) == 0)
		{
			return true;
		}
	}

	return false;
}
