// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/SlateTypes.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateNoResource.h"
#include "Styling/StyleDefaults.h"
#include "Styling/StyleColors.h"
#include "Widgets/InvalidateWidgetReason.h"
#include "Widgets/SWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateTypes)

FCheckBoxStyle::FCheckBoxStyle()
: CheckBoxType(ESlateCheckBoxType::CheckBox)
, UncheckedImage()
, UncheckedHoveredImage()
, UncheckedPressedImage()
, CheckedImage()
, CheckedHoveredImage()
, CheckedPressedImage()
, UndeterminedImage()
, UndeterminedHoveredImage()
, UndeterminedPressedImage()
, Padding(FMargin(2,0,0,0))
, BackgroundImage(FSlateNoResource())
, BackgroundHoveredImage(FSlateNoResource())
, BackgroundPressedImage(FSlateNoResource())
, ForegroundColor(FSlateColor::UseForeground())
, HoveredForeground(FSlateColor::UseForeground())
, PressedForeground(FSlateColor::UseForeground())
, CheckedForeground(FSlateColor::UseForeground())
, CheckedHoveredForeground(FSlateColor::UseForeground())
, CheckedPressedForeground(FSlateColor::UseForeground())
, UndeterminedForeground(FSlateColor::UseForeground())
, BorderBackgroundColor(FLinearColor::White)
{
}

FCheckBoxStyle::FCheckBoxStyle(const FCheckBoxStyle&) = default;
FCheckBoxStyle::~FCheckBoxStyle() = default;

const FName FCheckBoxStyle::TypeName( TEXT("FCheckBoxStyle") );

const FCheckBoxStyle& FCheckBoxStyle::GetDefault()
{
	static FCheckBoxStyle Default;
	return Default;
}

void FCheckBoxStyle::GetResources( TArray< const FSlateBrush* > & OutBrushes ) const
{
	OutBrushes.Add( &UncheckedImage );
	OutBrushes.Add( &UncheckedHoveredImage );
	OutBrushes.Add( &UncheckedPressedImage );
	OutBrushes.Add( &CheckedImage );
	OutBrushes.Add( &CheckedHoveredImage );
	OutBrushes.Add( &CheckedPressedImage );
	OutBrushes.Add( &UndeterminedImage );
	OutBrushes.Add( &UndeterminedHoveredImage );
	OutBrushes.Add( &UndeterminedPressedImage );
	OutBrushes.Add( &BackgroundImage );
	OutBrushes.Add( &BackgroundHoveredImage );
	OutBrushes.Add( &BackgroundPressedImage );
}

#if WITH_EDITORONLY_DATA
void FCheckBoxStyle::PostSerialize(const FArchive& Ar)
{
	if(Ar.IsLoading() && Ar.UEVer() < VER_UE4_FSLATESOUND_CONVERSION)
	{
		// eg, SoundCue'/Game/QA_Assets/Sound/TEST_Music_Ambient.TEST_Music_Ambient'
		CheckedSlateSound = FSlateSound::FromName_DEPRECATED(CheckedSound_DEPRECATED);
		UncheckedSlateSound = FSlateSound::FromName_DEPRECATED(UncheckedSound_DEPRECATED);
		HoveredSlateSound = FSlateSound::FromName_DEPRECATED(HoveredSound_DEPRECATED);
	}
}
#endif

FTextBlockStyle::FTextBlockStyle()
	: Font()
	, ColorAndOpacity()
	, ShadowOffset(FVector2f::ZeroVector)
	, ShadowColorAndOpacity(FLinearColor::Black)
	, SelectedBackgroundColor(FSlateColor::UseForeground())
	, HighlightColor(FLinearColor::Black)
	, TransformPolicy(ETextTransformPolicy::None)
	, OverflowPolicy(ETextOverflowPolicy::Clip)
{
}

FTextBlockStyle::FTextBlockStyle(const FTextBlockStyle&) = default;
FTextBlockStyle::~FTextBlockStyle() = default; // To get around linker errors on one old platform


const FName FTextBlockStyle::TypeName( TEXT("FTextBlockStyle") );

void FTextBlockStyle::GetResources(TArray< const FSlateBrush* >& OutBrushes) const
{
	OutBrushes.Add(&HighlightShape);
	OutBrushes.Add(&UnderlineBrush);
}

const FTextBlockStyle& FTextBlockStyle::GetDefault()
{
	static TSharedPtr< FTextBlockStyle > Default;
	if ( !Default.IsValid() )
	{
		Default = MakeShareable( new FTextBlockStyle() );
		Default->Font = FStyleDefaults::GetFontInfo();
	}

	return *Default.Get();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FTextBlockStyle& FTextBlockStyle::SetFont(const FName& InFontName, float InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
FTextBlockStyle& FTextBlockStyle::SetFont(const FString& InFontName, float InSize) { Font = FSlateFontInfo(*InFontName, InSize); return *this; }
FTextBlockStyle& FTextBlockStyle::SetFont(const WIDECHAR* InFontName, float InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
FTextBlockStyle& FTextBlockStyle::SetFont(const ANSICHAR* InFontName, float InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
FTextBlockStyle& FTextBlockStyle::SetFontName(const FName& InFontName) { Font = FSlateFontInfo(InFontName, Font.GetClampSize()); return *this; }
FTextBlockStyle& FTextBlockStyle::SetFontName(const FString& InFontName) { Font = FSlateFontInfo(InFontName, Font.GetClampSize()); return *this; }
FTextBlockStyle& FTextBlockStyle::SetFontName(const WIDECHAR* InFontName) { Font = FSlateFontInfo(InFontName, Font.GetClampSize()); return *this; }
FTextBlockStyle& FTextBlockStyle::SetFontName(const ANSICHAR* InFontName) { Font = FSlateFontInfo(InFontName, Font.GetClampSize()); return *this; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FButtonStyle::FButtonStyle()
: Normal()
, Hovered()
, Pressed()
, NormalForeground(FSlateColor::UseForeground()) 
, HoveredForeground(FSlateColor::UseForeground())
, PressedForeground(FSlateColor::UseForeground())
, DisabledForeground(FSlateColor::UseForeground())
, NormalPadding()
, PressedPadding()
{
	Disabled = FSlateNoResource();
}

FButtonStyle::FButtonStyle(const FButtonStyle&) = default;
FButtonStyle::~FButtonStyle() = default;

const FName FButtonStyle::TypeName( TEXT("FButtonStyle") );

void FButtonStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &Normal );
	OutBrushes.Add( &Hovered );
	OutBrushes.Add( &Pressed );
	OutBrushes.Add( &Disabled );
}

const FButtonStyle& FButtonStyle::GetDefault()
{
	static FButtonStyle Default;
	return Default;
}

#if WITH_EDITORONLY_DATA
void FButtonStyle::PostSerialize(const FArchive& Ar)
{
	if(Ar.IsLoading() && Ar.UEVer() < VER_UE4_FSLATESOUND_CONVERSION)
	{
		// eg, SoundCue'/Game/QA_Assets/Sound/TEST_Music_Ambient.TEST_Music_Ambient'
		PressedSlateSound = FSlateSound::FromName_DEPRECATED(PressedSound_DEPRECATED);
		HoveredSlateSound = FSlateSound::FromName_DEPRECATED(HoveredSound_DEPRECATED);
	}
}
#endif

FComboButtonStyle::FComboButtonStyle()
	: ShadowOffset(FVector2f::ZeroVector)
	, ShadowColorAndOpacity(FLinearColor::Black)
	, MenuBorderPadding(0.0f)
	, ContentPadding(5.0f)
	, DownArrowPadding(2.0f)
	, DownArrowAlign(EVerticalAlignment::VAlign_Center)
{
}

FComboButtonStyle::FComboButtonStyle(const FComboButtonStyle&) = default;
FComboButtonStyle::~FComboButtonStyle() = default;

const FName FComboButtonStyle::TypeName( TEXT("FComboButtonStyle") );

void FComboButtonStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add(&MenuBorderBrush);
	OutBrushes.Add(&DownArrowImage);
	ButtonStyle.GetResources(OutBrushes);
}

const FComboButtonStyle& FComboButtonStyle::GetDefault()
{
	static FComboButtonStyle Default;
	return Default;
}

FComboBoxStyle::FComboBoxStyle()
	: ContentPadding(FMargin(4.f, 2.f))
	, MenuRowPadding(FMargin(0.f))
{
	ComboButtonStyle.SetMenuBorderPadding(FMargin(1.0));
}

FComboBoxStyle::FComboBoxStyle(const FComboBoxStyle&) = default;
FComboBoxStyle::~FComboBoxStyle() = default;

const FName FComboBoxStyle::TypeName( TEXT("FComboBoxStyle") );

void FComboBoxStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	ComboButtonStyle.GetResources(OutBrushes);
}

const FComboBoxStyle& FComboBoxStyle::GetDefault()
{
	static FComboBoxStyle Default;
	return Default;
}

#if WITH_EDITORONLY_DATA
void FComboBoxStyle::PostSerialize(const FArchive& Ar)
{
	if(Ar.IsLoading() && Ar.UEVer() < VER_UE4_FSLATESOUND_CONVERSION)
	{
		// eg, SoundCue'/Game/QA_Assets/Sound/TEST_Music_Ambient.TEST_Music_Ambient'
		PressedSlateSound = FSlateSound::FromName_DEPRECATED(PressedSound_DEPRECATED);
		SelectionChangeSlateSound = FSlateSound::FromName_DEPRECATED(SelectionChangeSound_DEPRECATED);
	}
}
#endif

FHyperlinkStyle::FHyperlinkStyle()
: UnderlineStyle()
, TextStyle()
, Padding()
{
}

FHyperlinkStyle::FHyperlinkStyle(const FHyperlinkStyle&) = default;
FHyperlinkStyle::~FHyperlinkStyle() = default;

const FName FHyperlinkStyle::TypeName( TEXT("FHyperlinkStyle") );

void FHyperlinkStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	UnderlineStyle.GetResources(OutBrushes);
	TextStyle.GetResources(OutBrushes);
}

const FHyperlinkStyle& FHyperlinkStyle::GetDefault()
{
	static FHyperlinkStyle Default;
	return Default;
}

FEditableTextStyle::FEditableTextStyle()
	: Font(FStyleDefaults::GetFontInfo(9))
	, ColorAndOpacity(FSlateColor::UseForeground())
	, BackgroundImageSelected()
	, CaretImage()
{
}

FEditableTextStyle::FEditableTextStyle(const FEditableTextStyle&) = default;
FEditableTextStyle::~FEditableTextStyle() = default;

void FEditableTextStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &BackgroundImageSelected );
	OutBrushes.Add( &CaretImage );
}

const FName FEditableTextStyle::TypeName( TEXT("FEditableTextStyle") );

const FEditableTextStyle& FEditableTextStyle::GetDefault()
{
	static FEditableTextStyle Default;
	return Default;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FEditableTextStyle& FEditableTextStyle::SetFont(const FName& InFontName, float InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FEditableTextBoxStyle::FEditableTextBoxStyle()
	: BackgroundImageNormal()
	, BackgroundImageHovered()
	, BackgroundImageFocused()
	, BackgroundImageReadOnly()
	, Padding(FMargin(4.0f, 2.0f))
#if WITH_EDITORONLY_DATA
	, Font_DEPRECATED(FStyleDefaults::GetFontInfo(9))
#endif
	, TextStyle(FTextBlockStyle().SetFont(FStyleDefaults::GetFontInfo(9))
				.SetColorAndOpacity(FSlateColor::UseForeground())
				.SetSelectedBackgroundColor(FStyleColors::Highlight)
				.SetHighlightColor(FStyleColors::Black)
				.SetHighlightShape(FSlateColorBrush(FStyleColors::AccentGreen)))
	, ForegroundColor(FSlateColor::UseForeground())
	, BackgroundColor(FLinearColor::White)
	, ReadOnlyForegroundColor(FSlateColor::UseForeground())
	, FocusedForegroundColor(FSlateColor::UseForeground())
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FEditableTextBoxStyle::FEditableTextBoxStyle(const FEditableTextBoxStyle&) = default;
FEditableTextBoxStyle& FEditableTextBoxStyle::operator=(const FEditableTextBoxStyle&) = default;
FEditableTextBoxStyle::~FEditableTextBoxStyle() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FEditableTextBoxStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &BackgroundImageNormal );
	OutBrushes.Add( &BackgroundImageHovered );
	OutBrushes.Add( &BackgroundImageFocused );
	OutBrushes.Add( &BackgroundImageReadOnly );

	ScrollBarStyle.GetResources(OutBrushes);
}

const FName FEditableTextBoxStyle::TypeName( TEXT("FEditableTextBoxStyle") );

const FEditableTextBoxStyle& FEditableTextBoxStyle::GetDefault()
{
	static FEditableTextBoxStyle Default;
	return Default;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FEditableTextBoxStyle& FEditableTextBoxStyle::SetFont(const FName& InFontName, float InSize) { return SetFont(FSlateFontInfo(InFontName, InSize)); }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FInlineEditableTextBlockStyle::FInlineEditableTextBlockStyle()
{
}

FInlineEditableTextBlockStyle::~FInlineEditableTextBlockStyle() = default;

void FInlineEditableTextBlockStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	EditableTextBoxStyle.GetResources(OutBrushes);
	TextStyle.GetResources(OutBrushes);
}

const FName FInlineEditableTextBlockStyle::TypeName( TEXT("FInlineEditableTextBlockStyle") );

const FInlineEditableTextBlockStyle& FInlineEditableTextBlockStyle::GetDefault()
{
	static FInlineEditableTextBlockStyle Default;
	return Default;
}


FProgressBarStyle::FProgressBarStyle()
	: EnableFillAnimation(false)
{
}

FProgressBarStyle::~FProgressBarStyle() = default;

void FProgressBarStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &BackgroundImage );
	OutBrushes.Add( &FillImage );
	OutBrushes.Add( &MarqueeImage );
}

const FName FProgressBarStyle::TypeName( TEXT("FProgressBarStyle") );

const FProgressBarStyle& FProgressBarStyle::GetDefault()
{
	static FProgressBarStyle Default;
	return Default;
}


FScrollBarStyle::FScrollBarStyle()
	: HorizontalBackgroundImage(FSlateNoResource())
	, VerticalBackgroundImage(FSlateNoResource())
	, VerticalTopSlotImage(FSlateNoResource())
	, HorizontalTopSlotImage(FSlateNoResource())
	, VerticalBottomSlotImage(FSlateNoResource())
	, HorizontalBottomSlotImage(FSlateNoResource())
	, NormalThumbImage(FSlateNoResource())
	, HoveredThumbImage(FSlateNoResource())
	, DraggedThumbImage(FSlateNoResource())
	, Thickness(16.0f)
{
}

FScrollBarStyle::FScrollBarStyle(const FScrollBarStyle&) = default;
FScrollBarStyle::~FScrollBarStyle() = default;

void FScrollBarStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &HorizontalBackgroundImage );
	OutBrushes.Add( &VerticalBackgroundImage );
	OutBrushes.Add( &VerticalTopSlotImage);
	OutBrushes.Add( &HorizontalTopSlotImage);
	OutBrushes.Add( &VerticalBottomSlotImage);
	OutBrushes.Add( &HorizontalBottomSlotImage);
	OutBrushes.Add( &NormalThumbImage );
	OutBrushes.Add( &HoveredThumbImage );
	OutBrushes.Add( &DraggedThumbImage );
}

const FName FScrollBarStyle::TypeName( TEXT("FScrollBarStyle") );

const FScrollBarStyle& FScrollBarStyle::GetDefault()
{
	static FScrollBarStyle Default;
	return Default;
}


FExpandableAreaStyle::FExpandableAreaStyle()
: RolloutAnimationSeconds(0.1f)
{
}

FExpandableAreaStyle::~FExpandableAreaStyle() = default;

void FExpandableAreaStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &CollapsedImage );
	OutBrushes.Add( &ExpandedImage );
}

const FName FExpandableAreaStyle::TypeName( TEXT("FExpandableAreaStyle") );

const FExpandableAreaStyle& FExpandableAreaStyle::GetDefault()
{
	static FExpandableAreaStyle Default;
	return Default;
}


FSearchBoxStyle::FSearchBoxStyle()
	: bLeftAlignButtons_DEPRECATED(false)
	, bLeftAlignSearchResultButtons(false)
	, bLeftAlignGlassImageAndClearButton(false)
{
}

FSearchBoxStyle::~FSearchBoxStyle() = default;

FSearchBoxStyle& FSearchBoxStyle::SetTextBoxStyle( const FEditableTextBoxStyle& InTextBoxStyle )
{ 
	TextBoxStyle = InTextBoxStyle;
	if (!ActiveFontInfo.HasValidFont())
	{
		ActiveFontInfo = TextBoxStyle.TextStyle.Font;
	}
	return *this;
}

void FSearchBoxStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	TextBoxStyle.GetResources(OutBrushes);
	OutBrushes.Add( &UpArrowImage );
	OutBrushes.Add( &DownArrowImage );
	OutBrushes.Add( &GlassImage );
	OutBrushes.Add( &ClearImage );
}

const FName FSearchBoxStyle::TypeName( TEXT("FSearchBoxStyle") );

const FSearchBoxStyle& FSearchBoxStyle::GetDefault()
{
	static FSearchBoxStyle Default;
	return Default;
}

FSearchBoxStyle& FSearchBoxStyle::SetLeftAlignButtons(bool bInLeftAlignButtons)
{
	bLeftAlignButtons_DEPRECATED = bInLeftAlignButtons;
	
	bLeftAlignSearchResultButtons = bInLeftAlignButtons;
	bLeftAlignGlassImageAndClearButton= bInLeftAlignButtons;

	return *this;
}

FSliderStyle::FSliderStyle()
	: BarThickness(2.0f)
{
}

FSliderStyle::FSliderStyle(const FSliderStyle&) = default;
FSliderStyle::~FSliderStyle() = default;

void FSliderStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &NormalBarImage );
	OutBrushes.Add( &HoveredBarImage );
	OutBrushes.Add( &DisabledBarImage );
	OutBrushes.Add( &NormalThumbImage );
	OutBrushes.Add( &HoveredThumbImage );
	OutBrushes.Add( &DisabledThumbImage );
}

const FName FSliderStyle::TypeName( TEXT("FSliderStyle") );

const FSliderStyle& FSliderStyle::GetDefault()
{
	static FSliderStyle Default;
	return Default;
}


FVolumeControlStyle::FVolumeControlStyle()
{
}

FVolumeControlStyle::~FVolumeControlStyle() = default;

void FVolumeControlStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	SliderStyle.GetResources(OutBrushes);
	OutBrushes.Add( &HighVolumeImage );
	OutBrushes.Add( &MidVolumeImage );
	OutBrushes.Add( &LowVolumeImage );
	OutBrushes.Add( &NoVolumeImage );
	OutBrushes.Add( &MutedImage );
}

const FName FVolumeControlStyle::TypeName( TEXT("FVolumeControlStyle") );

const FVolumeControlStyle& FVolumeControlStyle::GetDefault()
{
	static FVolumeControlStyle Default;
	return Default;
}

FInlineTextImageStyle::FInlineTextImageStyle()
	: Image()
	, Baseline( 0 )
{
}

FInlineTextImageStyle::FInlineTextImageStyle(const FInlineTextImageStyle&) = default;
FInlineTextImageStyle::~FInlineTextImageStyle() = default;

void FInlineTextImageStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &Image );
}

const FName FInlineTextImageStyle::TypeName( TEXT("FInlineTextImageStyle") );

const FInlineTextImageStyle& FInlineTextImageStyle::GetDefault()
{
	static FInlineTextImageStyle Default;
	return Default;
}

FSpinBoxStyle::FSpinBoxStyle()
	: ActiveBackgroundBrush(FSlateOptionalBrush())
	, HoveredFillBrush(FSlateOptionalBrush())
	, ForegroundColor(FSlateColor::UseForeground())
	, TextPadding(FMargin(1.0f,2.0f))
	, InsetPadding(FMargin(0))
{
}

FSpinBoxStyle::FSpinBoxStyle(const FSpinBoxStyle&) = default;
FSpinBoxStyle::~FSpinBoxStyle() = default;

void FSpinBoxStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add(&BackgroundBrush);
	OutBrushes.Add(&HoveredBackgroundBrush);
	OutBrushes.Add(&ActiveBackgroundBrush);
	OutBrushes.Add(&ActiveFillBrush);
	OutBrushes.Add(&HoveredFillBrush);
	OutBrushes.Add(&InactiveFillBrush);
	OutBrushes.Add(&ArrowsImage);
}

const FName FSpinBoxStyle::TypeName( TEXT("FSpinBoxStyle") );

const FSpinBoxStyle& FSpinBoxStyle::GetDefault()
{
	static FSpinBoxStyle Default;
	return Default;
}


FSplitterStyle::FSplitterStyle()
{
}

FSplitterStyle::~FSplitterStyle() = default;

void FSplitterStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &HandleNormalBrush );
	OutBrushes.Add( &HandleHighlightBrush );
}

const FName FSplitterStyle::TypeName( TEXT("FSplitterStyle") );

const FSplitterStyle& FSplitterStyle::GetDefault()
{
	static FSplitterStyle Default;
	return Default;
}

FTableViewStyle::FTableViewStyle()
	: BackgroundBrush(FSlateNoResource())
{
}

FTableViewStyle::~FTableViewStyle() = default;

void FTableViewStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &BackgroundBrush );	
}

const FName FTableViewStyle::TypeName( TEXT("FTableViewStyle") );

const FTableViewStyle& FTableViewStyle::GetDefault()
{
	static FTableViewStyle Default;
	return Default;
}

FTableRowStyle::FTableRowStyle()
	: bUseParentRowBrush(false)
	, TextColor(FSlateColor::UseForeground())
	, SelectedTextColor(FLinearColor::White)
{
}

FTableRowStyle::~FTableRowStyle() = default;

void FTableRowStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &SelectorFocusedBrush );
	OutBrushes.Add( &ActiveHoveredBrush );
	OutBrushes.Add( &ActiveBrush );
	OutBrushes.Add( &InactiveHoveredBrush );
	OutBrushes.Add( &InactiveBrush );
	OutBrushes.Add( &ParentRowBackgroundHoveredBrush );
	OutBrushes.Add( &ParentRowBackgroundBrush );
	OutBrushes.Add( &EvenRowBackgroundHoveredBrush );
	OutBrushes.Add( &EvenRowBackgroundBrush );
	OutBrushes.Add( &OddRowBackgroundHoveredBrush );
	OutBrushes.Add( &OddRowBackgroundBrush );
	OutBrushes.Add( &DropIndicator_Above );
	OutBrushes.Add( &DropIndicator_Onto );
	OutBrushes.Add( &DropIndicator_Below );
}

const FName FTableRowStyle::TypeName( TEXT("FTableRowStyle") );

const FTableRowStyle& FTableRowStyle::GetDefault()
{
	static FTableRowStyle Default;
	return Default;
}


FTableColumnHeaderStyle::FTableColumnHeaderStyle()
{
}

FTableColumnHeaderStyle::~FTableColumnHeaderStyle() = default;

void FTableColumnHeaderStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &SortPrimaryAscendingImage );
	OutBrushes.Add( &SortPrimaryDescendingImage );
	OutBrushes.Add( &SortSecondaryAscendingImage );
	OutBrushes.Add( &SortSecondaryDescendingImage );
	OutBrushes.Add( &NormalBrush );
	OutBrushes.Add( &HoveredBrush );
	OutBrushes.Add( &MenuDropdownImage );
	OutBrushes.Add( &MenuDropdownNormalBorderBrush );
	OutBrushes.Add( &MenuDropdownHoveredBorderBrush );
}

const FName FTableColumnHeaderStyle::TypeName( TEXT("FTableColumnHeaderStyle") );

const FTableColumnHeaderStyle& FTableColumnHeaderStyle::GetDefault()
{
	static FTableColumnHeaderStyle Default;
	return Default;
}


FHeaderRowStyle::FHeaderRowStyle()
	: SplitterHandleSize(0.0)
	, HorizontalSeparatorBrush(FSlateNoResource())
	, HorizontalSeparatorThickness(0)
{
}

FHeaderRowStyle::~FHeaderRowStyle() = default;

void FHeaderRowStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	ColumnStyle.GetResources(OutBrushes);
	LastColumnStyle.GetResources(OutBrushes);
	ColumnSplitterStyle.GetResources(OutBrushes);
	OutBrushes.Add(&BackgroundBrush);
	OutBrushes.Add(&HorizontalSeparatorBrush);
}

const FName FHeaderRowStyle::TypeName( TEXT("FHeaderRowStyle") );

const FHeaderRowStyle& FHeaderRowStyle::GetDefault()
{
	static FHeaderRowStyle Default;
	return Default;
}


FDockTabStyle::FDockTabStyle()
	: TabTextStyle( FTextBlockStyle::GetDefault() )
	, IconSize(16,16)
	, OverlapWidth(0.0f)
	, NormalForegroundColor(FSlateColor::UseForeground())
	, HoveredForegroundColor(FSlateColor::UseForeground())
	, ActiveForegroundColor(FSlateColor::UseForeground())
	, ForegroundForegroundColor(FSlateColor::UseForeground())
	, IconBorderPadding(1.f)
{
}

FDockTabStyle::~FDockTabStyle() = default;

void FDockTabStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	CloseButtonStyle.GetResources(OutBrushes);
	OutBrushes.Add( &NormalBrush );
	OutBrushes.Add( &ColorOverlayTabBrush );
	OutBrushes.Add( &ColorOverlayIconBrush );
	OutBrushes.Add( &ForegroundBrush );
	OutBrushes.Add( &HoveredBrush );
	OutBrushes.Add( &ContentAreaBrush );
	OutBrushes.Add( &TabWellBrush );
}

const FName FDockTabStyle::TypeName( TEXT("FDockTabStyle") );

const FDockTabStyle& FDockTabStyle::GetDefault()
{
	static FDockTabStyle Default;
	return Default;
}


FScrollBoxStyle::FScrollBoxStyle()
	: BarThickness(9.0f)
{
}

FScrollBoxStyle::~FScrollBoxStyle() = default;

void FScrollBoxStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &TopShadowBrush );
	OutBrushes.Add( &BottomShadowBrush );
	OutBrushes.Add( &LeftShadowBrush );
	OutBrushes.Add( &RightShadowBrush );
}

const FName FScrollBoxStyle::TypeName( TEXT("FScrollBoxStyle") );

const FScrollBoxStyle& FScrollBoxStyle::GetDefault()
{
	static FScrollBoxStyle Default;
	return Default;
}


FScrollBorderStyle::FScrollBorderStyle()
{
}

FScrollBorderStyle::~FScrollBorderStyle() = default;

void FScrollBorderStyle::GetResources( TArray< const FSlateBrush* >& OutBrushes ) const
{
	OutBrushes.Add( &TopShadowBrush );
	OutBrushes.Add( &BottomShadowBrush );
}

const FName FScrollBorderStyle::TypeName( TEXT( "FScrollBorderStyle" ) );

const FScrollBorderStyle& FScrollBorderStyle::GetDefault()
{
	static FScrollBorderStyle Default;
	return Default;
}


FWindowStyle::FWindowStyle()
	: BackgroundColor(FLinearColor::White)
	, OutlineColor(FLinearColor::White)
	, BorderColor(FLinearColor::White)
	, WindowCornerRadius(0)
	, BorderPadding(FMargin(5, 5, 5, 5))
{
}

FWindowStyle::FWindowStyle(const FWindowStyle&) = default;
FWindowStyle::~FWindowStyle() = default;

void FWindowStyle::GetResources(TArray< const FSlateBrush* >& OutBrushes) const
{
	MinimizeButtonStyle.GetResources(OutBrushes);
	MaximizeButtonStyle.GetResources(OutBrushes);
	RestoreButtonStyle.GetResources(OutBrushes);
	CloseButtonStyle.GetResources(OutBrushes);

	TitleTextStyle.GetResources(OutBrushes);

	OutBrushes.Add( &ActiveTitleBrush );
	OutBrushes.Add( &InactiveTitleBrush );
	OutBrushes.Add( &FlashTitleBrush );
	OutBrushes.Add( &BorderBrush );
	OutBrushes.Add( &OutlineBrush );
	OutBrushes.Add( &BackgroundBrush );
	OutBrushes.Add( &ChildBackgroundBrush );
}

const FName FWindowStyle::TypeName( TEXT("FWindowStyle") );

const FWindowStyle& FWindowStyle::GetDefault()
{
	static FWindowStyle Default;
	return Default;
}

const FWindowStyle& FWindowStyle::GetBorderless()
{
	static FWindowStyle BorderlessStyle = []
	{
		FWindowStyle NewBorderlessStyle;
		NewBorderlessStyle
			.SetActiveTitleBrush(FSlateNoResource())
			.SetInactiveTitleBrush(FSlateNoResource())
			.SetFlashTitleBrush(FSlateNoResource())
			.SetOutlineBrush(FSlateNoResource())
			.SetBorderBrush(FSlateNoResource())
			.SetBackgroundBrush(FSlateNoResource())
			.SetChildBackgroundBrush(FSlateNoResource());
		return NewBorderlessStyle;
	}();
	return BorderlessStyle;
}

void FInvalidatableBrushAttribute::SetImage(SWidget& ThisWidget, const TAttribute< const FSlateBrush* >& InImage)
{
	const bool bImagePointerChanged = SetWidgetAttribute(ThisWidget, Image, InImage, EInvalidateWidgetReason::Layout);

	const FSlateBrush* ImagePtr = Image.Get();
	const FSlateBrush NewImageToCache = ImagePtr ? *ImagePtr : FSlateBrush();

	// If the slate brush pointer didn't change, that may not mean nothing changed.  We
	// sometimes reuse the slate brush memory address and change out the texture.  In those
	// circumstances we need to actually look at the data the brush has compared to what it had
	// previously.
	if (!bImagePointerChanged && ImageCache != NewImageToCache)
	{
		ThisWidget.Invalidate(EInvalidateWidgetReason::Layout);
	}

	// Cache the new image's value in case the memory changes later.
	ImageCache = NewImageToCache;
}
