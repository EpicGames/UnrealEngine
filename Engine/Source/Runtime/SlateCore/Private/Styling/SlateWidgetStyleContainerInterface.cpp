// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/SlateWidgetStyleContainerInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateWidgetStyleContainerInterface)

USlateWidgetStyleContainerInterface::USlateWidgetStyleContainerInterface( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	
}


