// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/StarshipCoreStyle.h"
#include "SlateGlobals.h"
#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Fonts/CompositeFont.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/ToolBarStyle.h"
#include "Styling/SegmentedControlStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/CoreStyle.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but Style->RootToContentDir is how the core style is set up
#define RootToContentDir Style->RootToContentDir

/* Static initialization
 *****************************************************************************/

TSharedPtr< ISlateStyle > FStarshipCoreStyle::Instance = nullptr;

using namespace CoreStyleConstants;

TUniquePtr<FStyleFonts> FStyleFonts::Instance = nullptr;

FStyleFonts::FStyleFonts()
	: Normal(DEFAULT_FONT("Regular", 10))
	, NormalItalic(DEFAULT_FONT("Italic", 10))
	, NormalBold(DEFAULT_FONT("Bold", 10))
	, NormalBoldItalic(DEFAULT_FONT("BoldItalic", 10))
	, Small(DEFAULT_FONT("Regular", 8))
	, SmallBold(DEFAULT_FONT("Bold", 8))
	, Large(DEFAULT_FONT("Regular", 14))
	, LargeBold(DEFAULT_FONT("Bold", 14))
	, HeadingMedium(DEFAULT_FONT("BoldCondensed", 33))
	, HeadingSmall(DEFAULT_FONT("BoldCondensed", 21))
	, HeadingExtraSmall(DEFAULT_FONT("BoldCondensed", 15))
{
};


/* FStarshipCoreStyle helper class
 *****************************************************************************/

class FStarshipCoreStyle::FStyle
	: public FSlateStyleSet
{
public:
	FStyle(const FName& InStyleSetName)
		: FSlateStyleSet(InStyleSetName)

		// These are the colors that are updated by the user style customizations
		, SelectorColor_LinearRef(MakeShared<FLinearColor>(0.701f, 0.225f, 0.003f))
		, SelectionColor_LinearRef(MakeShared<FLinearColor>(COLOR("18A0FBFF")))
		, SelectionColor_Inactive_LinearRef(MakeShared<FLinearColor>(0.25f, 0.25f, 0.25f))
		, SelectionColor_Pressed_LinearRef(MakeShared<FLinearColor>(0.701f, 0.225f, 0.003f))
		, HighlightColor_LinearRef(MakeShared<FLinearColor>(0.068f, 0.068f, 0.068f))
	{
	}

	static void SetColor(const TSharedRef<FLinearColor>& Source, const FLinearColor& Value)
	{
		Source->R = Value.R;
		Source->G = Value.G;
		Source->B = Value.B;
		Source->A = Value.A;
	}

	// These are the colors that are updated by the user style customizations
	const TSharedRef<FLinearColor> SelectorColor_LinearRef;
	const TSharedRef<FLinearColor> SelectionColor_LinearRef;
	const TSharedRef<FLinearColor> SelectionColor_Inactive_LinearRef;
	const TSharedRef<FLinearColor> SelectionColor_Pressed_LinearRef;
	const TSharedRef<FLinearColor> HighlightColor_LinearRef;
};


/* FStarshipCoreStyle static functions
 *****************************************************************************/

TSharedRef<const FCompositeFont> FStarshipCoreStyle::GetDefaultFont()
{
	// FStarshipCoreStyle currently uses same default font as core
	return FCoreStyle::GetDefaultFont();
}


FSlateFontInfo FStarshipCoreStyle::GetDefaultFontStyle(const FName InTypefaceFontName, const float InSize, const FFontOutlineSettings& InOutlineSettings)
{
	return FSlateFontInfo(GetDefaultFont(), InSize, InTypefaceFontName, InOutlineSettings);
}


void FStarshipCoreStyle::ResetToDefault()
{
	SetStyle(FStarshipCoreStyle::Create());
}


void FStarshipCoreStyle::SetSelectorColor(const FLinearColor& NewColor)
{
	TSharedPtr<FStyle> Style = StaticCastSharedPtr<FStyle>(Instance);
	check(Style.IsValid());

	FStyle::SetColor(Style->SelectorColor_LinearRef, NewColor);
}


void FStarshipCoreStyle::SetSelectionColor(const FLinearColor& NewColor)
{
	TSharedPtr<FStyle> Style = StaticCastSharedPtr<FStyle>(Instance);
	check(Style.IsValid());

	FStyle::SetColor(Style->SelectionColor_LinearRef, NewColor);
}


void FStarshipCoreStyle::SetInactiveSelectionColor(const FLinearColor& NewColor)
{
	TSharedPtr<FStyle> Style = StaticCastSharedPtr<FStyle>(Instance);
	check(Style.IsValid());

	FStyle::SetColor(Style->SelectionColor_Inactive_LinearRef, NewColor);
}


void FStarshipCoreStyle::SetPressedSelectionColor(const FLinearColor& NewColor)
{
	TSharedPtr<FStyle> Style = StaticCastSharedPtr<FStyle>(Instance);
	check(Style.IsValid());

	FStyle::SetColor(Style->SelectionColor_Pressed_LinearRef, NewColor);
}

void FStarshipCoreStyle::SetFocusBrush(FSlateBrush* NewBrush)
{
	TSharedRef<FStyle> Style = StaticCastSharedRef<FStyle>(Instance.ToSharedRef());
	FSlateStyleRegistry::UnRegisterSlateStyle(Style.Get());

	Style->Set("FocusRectangle", NewBrush);

	FSlateStyleRegistry::RegisterSlateStyle(Style.Get());
}

PRAGMA_PUSH_ATTRIBUTE_MINSIZE_FUNCTIONS

TSharedRef<ISlateStyle> FStarshipCoreStyle::Create()
{
	TSharedRef<FStyle> Style = MakeShareable(new FStyle("CoreStyle"));
	Style->SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FString CanaryPath = RootToContentDir(TEXT("Checkerboard"), TEXT(".png"));

	if (!FPaths::FileExists(CanaryPath))
	{
		// Checkerboard is the default brush so we check for that. No slate fonts are required as those will fall back properly
		UE_LOGF(LogSlate, Warning, "FStarshipCoreStyle assets not detected, skipping FStarshipCoreStyle initialization");

		return Style;
	}


	// These are the Slate colors which reference the dynamic colors in the style;
	const FSlateColor DefaultForeground(FStyleColors::Foreground);
	const FSlateColor InvertedForeground(FStyleColors::ForegroundInverted);
	const FSlateColor SelectorColor(Style->SelectorColor_LinearRef);
	const FSlateColor SelectionColor(Style->SelectionColor_LinearRef);
	const FSlateColor SelectionColor_Inactive(Style->SelectionColor_Inactive_LinearRef);
	const FSlateColor SelectionColor_Pressed(Style->SelectionColor_Pressed_LinearRef);
	const FSlateColor HighlightColor(FStyleColors::Highlight);

	
	const FStyleFonts& StyleFonts = FStyleFonts::Get();

	Style->Set("InvertedForeground", InvertedForeground);

	SetupColors(Style);

	// SScrollBar defaults...
	const FScrollBarStyle ScrollBar = FScrollBarStyle()
		.SetNormalThumbImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f))
		.SetDraggedThumbImage(FSlateRoundedBoxBrush(FStyleColors::Hover2, 4.0f))
		.SetHoveredThumbImage(FSlateRoundedBoxBrush(FStyleColors::Hover2, 4.0f))
		.SetThickness(8.0f);
	;
	{
		Style->Set("Scrollbar", ScrollBar);
	}


	SetupTextStyles(Style);

	// Get these from the text style we just created in order to share them
	const FEditableTextBoxStyle& NormalEditableTextBoxStyle = Style->GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
	const FTextBlockStyle& NormalText = Style->GetWidgetStyle<FTextBlockStyle>("NormalText");


	// Common brushes
	FSlateBrush* GenericWhiteBox = new IMAGE_BRUSH("Old/White", Icon16x16);
	{
		Style->Set("Checkerboard", new IMAGE_BRUSH("Checkerboard", Icon16x16, FLinearColor::White, ESlateBrushTileType::Both));

		Style->Set("GenericWhiteBox", GenericWhiteBox);

		Style->Set("BlackBrush", new FSlateColorBrush(FLinearColor::Black));
		Style->Set("WhiteBrush", new FSlateColorBrush(FLinearColor::White));

		Style->Set("BoxShadow", new BOX_BRUSH("Common/BoxShadow", FMargin(5.0f / 64.0f)));

		Style->Set("FocusRectangle", new FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::Primary, InputFocusThickness));
	}

	// Important colors
	{
		Style->Set("DefaultForeground", DefaultForeground);
		Style->Set("InvertedForeground", InvertedForeground);

		Style->Set("SelectorColor", SelectorColor);
		Style->Set("SelectionColor", SelectionColor);
		Style->Set("SelectionColor_Inactive", SelectionColor_Inactive);
		Style->Set("SelectionColor_Pressed", SelectionColor_Pressed);
	}

	// Invisible buttons, borders, etc.
	const FButtonStyle NoBorder = FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateNoResource())
		.SetPressed(FSlateNoResource())
		.SetNormalForeground(  FStyleColors::ForegroundHover)
		.SetHoveredForeground( FStyleColors::ForegroundHover)
		.SetPressedForeground( FStyleColors::ForegroundHover)
		.SetNormalPadding(FMargin(0.f))
		.SetPressedPadding(FMargin(0.f));


	// Convenient transparent/invisible elements
	{
		Style->Set("NoBrush", new FSlateNoResource());

		Style->Set("NoBorder", new FSlateNoResource());
		Style->Set("NoBorder.Normal", new FSlateNoResource());
		Style->Set("NoBorder.Hovered", new FSlateNoResource());
		Style->Set("NoBorder.Pressed", new FSlateNoResource());

		Style->Set("NoBorder", NoBorder);
	}

	// Demo Recording
	{
		Style->Set("DemoRecording.CursorPing", new IMAGE_BRUSH("Common/CursorPing", FVector2f(31.f, 31.f)));
	}

	// Error Reporting
	{
		Style->Set("ErrorReporting.Box", new BOX_BRUSH("Common/TextBlockHighlightShape", FMargin(3.f / 8.f)));
		Style->Set("ErrorReporting.EmptyBox", new BOX_BRUSH("Common/TextBlockHighlightShape_Empty", FMargin(3.f / 8.f)));
		Style->Set("ErrorReporting.BackgroundColor", FLinearColor(0.35f, 0.0f, 0.0f));
		Style->Set("ErrorReporting.WarningBackgroundColor", FLinearColor(0.828f, 0.364f, 0.003f));
		Style->Set("ErrorReporting.ForegroundColor", FLinearColor::White);
	}

	// Cursor icons
	{
		Style->Set("SoftwareCursor_Grab", new IMAGE_BRUSH("Icons/cursor_grab", Icon24x24));
		Style->Set("SoftwareCursor_CardinalCross", new IMAGE_BRUSH("Icons/cursor_cardinal_cross", Icon24x24));
	}

	// Common icons
	{

		// chrome icon or not (we are testing this)
#if 0
		Style->Set("AppIcon", new IMAGE_BRUSH_SVG("Starship/Common/unreal", FVector2f(36.f, 36.f), FStyleColors::Foreground));
		Style->Set("AppIconPadding", FMargin(11.f, 11.f, 3.f, 5.f));
#else
		Style->Set("AppIcon", new IMAGE_BRUSH_SVG("Starship/Common/UELogo", FVector2f(45.f, 45.f), FStyleColors::White));
		Style->Set("AppIconPadding", FMargin(5.f, 5.f, 5.f, 5.f));
#endif

		Style->Set("UnrealCircle.Thick", new IMAGE_BRUSH_SVG("Starship/Common/unreal-circle-thick", FVector2f(32.f, 32.f), FStyleColors::Foreground));
		Style->Set("UnrealCircle.Thin", new IMAGE_BRUSH_SVG("Starship/Common/unreal-circle-thin", FVector2f(32.f, 32.f)));
		Style->Set("AppIcon.Small", new IMAGE_BRUSH_SVG("Starship/Common/unreal-small", Icon24x24, FStyleColors::Foreground));
		Style->Set("AppIconPadding.Small", FMargin(4.f, 4.f, 0.f, 0.f));

		Style->Set("Checker", new IMAGE_BRUSH("Starship/Common/Checker", Icon16x16, FLinearColor::White, ESlateBrushTileType::Both));

		Style->Set("Icons.Denied", new IMAGE_BRUSH("Icons/denied_16x", Icon16x16));
	
		Style->Set("Icons.Help", new IMAGE_BRUSH_SVG("Starship/Common/help", Icon16x16));
		Style->Set("Icons.Help.Solid", new IMAGE_BRUSH_SVG("Starship/Common/help-solid", Icon16x16));

		Style->Set("Icons.Download", new IMAGE_BRUSH("Icons/icon_Downloads_16x", Icon16x16));
		Style->Set("Icons.Server", new IMAGE_BRUSH_SVG("Starship/Common/server", Icon16x16));
		Style->Set("Icons.Cloud", new IMAGE_BRUSH("Icons/icon_Downloads_16x", Icon16x16));
		Style->Set("Icons.Local", new IMAGE_BRUSH_SVG("Starship/Common/server", Icon16x16));

		Style->Set("Icons.Alert", new IMAGE_BRUSH_SVG("Starship/Common/alert-triangle", Icon16x16, FStyleColors::Foreground));

		Style->Set("Icons.Error", new IMAGE_BRUSH_SVG("Starship/Common/alert-triangle", Icon16x16));
		Style->Set("Icons.Error.Red.Outlined", new IMAGE_BRUSH_SVG("Starship/Common/alert-triangle-red-outlined", Icon16x16));
		Style->Set("Icons.Error.Large", new IMAGE_BRUSH_SVG("Starship/Common/alert-triangle-large", Icon32x32));
		Style->Set("Icons.ErrorWithColor", new IMAGE_BRUSH_SVG("Starship/Common/alert-triangle", Icon16x16, FStyleColors::Error));
		Style->Set("Icons.ErrorWithColor.Large", new IMAGE_BRUSH_SVG("Starship/Common/alert-triangle-large", Icon32x32, FStyleColors::Error));
		
		Style->Set("Icons.Warning", new IMAGE_BRUSH_SVG("Starship/Common/alert-triangle", Icon16x16));
		Style->Set("Icons.Warning.Large", new IMAGE_BRUSH_SVG("Starship/Common/alert-triangle-large", Icon32x32));
		Style->Set("Icons.WarningWithColor", new IMAGE_BRUSH_SVG("Starship/Common/alert-triangle", Icon16x16, FStyleColors::Warning));
		Style->Set("Icons.WarningWithColor.Large", new IMAGE_BRUSH_SVG("Starship/Common/alert-triangle-large", Icon32x32, FStyleColors::Warning));
		Style->Set("Icons.WarningWithColor.Thumbnail", new IMAGE_BRUSH_SVG("Starship/Common/alert-triangle-64", Icon64x64));
		
		Style->Set("Icons.Info", new IMAGE_BRUSH_SVG("Starship/Common/Info", Icon16x16));
		Style->Set("Icons.Info.Small", new IMAGE_BRUSH_SVG("Starship/Common/Info", Icon14x14));
		Style->Set("Icons.Info.Large", new IMAGE_BRUSH_SVG("Starship/Common/Info", Icon32x32));
		Style->Set("Icons.InfoWithColor", new IMAGE_BRUSH_SVG("Starship/Common/Info", Icon16x16, FStyleColors::Foreground));
		Style->Set("Icons.InfoWithColor.Large", new IMAGE_BRUSH_SVG("Starship/Common/Info", Icon32x32, FStyleColors::Foreground));

		Style->Set("Icons.Success", new IMAGE_BRUSH_SVG("Starship/Common/check-circle", Icon16x16));
		Style->Set("Icons.Success.Large", new IMAGE_BRUSH_SVG("Starship/Common/check-circle-large", Icon32x32));
		Style->Set("Icons.SuccessWithColor", new IMAGE_BRUSH_SVG("Starship/Common/check-circle", Icon16x16, FStyleColors::Success));
		Style->Set("Icons.SuccessWithColor.Large", new IMAGE_BRUSH_SVG("Starship/Common/check-circle-large", Icon32x32, FStyleColors::Success));

		Style->Set("Icons.AlertCircle", new IMAGE_BRUSH_SVG("Starship/Common/alert-circle", Icon16x16));
		Style->Set("Icons.AlertCircleWithColor", new IMAGE_BRUSH_SVG("Starship/Common/alert-circle", Icon16x16, FStyleColors::Warning));

		Style->Set("Icons.box-perspective", new IMAGE_BRUSH_SVG("Starship/Common/box-perspective", Icon16x16));
		Style->Set("Icons.cylinder", new IMAGE_BRUSH_SVG("Starship/Common/cylinder", Icon16x16));
		Style->Set("Icons.pyramid", new IMAGE_BRUSH_SVG("Starship/Common/pyriamid", Icon16x16));
		Style->Set("Icons.sphere", new IMAGE_BRUSH_SVG("Starship/Common/sphere", Icon16x16));

		Style->Set("Icons.Settings", new IMAGE_BRUSH_SVG("Starship/Common/settings", Icon16x16));
		Style->Set("Icons.Blueprints", new IMAGE_BRUSH_SVG("Starship/Common/blueprint", Icon16x16));
		Style->Set("Icons.Cross", new IMAGE_BRUSH_SVG("Starship/Common/plus", Icon16x16));
		Style->Set("Icons.Plus", new IMAGE_BRUSH_SVG("Starship/Common/plus", Icon16x16));
		Style->Set("Icons.Minus", new IMAGE_BRUSH_SVG("Starship/Common/minus", Icon16x16));
		Style->Set("Icons.AddCircle", new IMAGE_BRUSH_SVG("Starship/Common/add-circle", Icon16x16));
		Style->Set("Icons.PlusCircle", new IMAGE_BRUSH_SVG("Starship/Common/plus-circle", Icon16x16));
		Style->Set("Icons.MinusCircle", new IMAGE_BRUSH_SVG("Starship/Common/minus-circle", Icon16x16));
		Style->Set("Icons.X", new IMAGE_BRUSH_SVG("Starship/Common/close", Icon16x16));
		Style->Set("Icons.XCircle", new IMAGE_BRUSH_SVG("Starship/Common/close-circle", Icon16x16));
		Style->Set("Icons.Delete", new IMAGE_BRUSH_SVG("Starship/Common/delete-outline", Icon16x16));
		Style->Set("Icons.Save", new IMAGE_BRUSH_SVG("Starship/Common/save", Icon16x16));
		Style->Set("Icons.SaveModified", new IMAGE_BRUSH_SVG("Starship/Common/save-modified", Icon16x16));
		Style->Set("Icons.Favorites.Small", new IMAGE_BRUSH_SVG("Starship/Common/Favorite", Icon20x20));

		Style->Set("Icons.Export", new IMAGE_BRUSH_SVG("Starship/Common/export", Icon16x16));
		Style->Set("Icons.Import", new IMAGE_BRUSH_SVG("Starship/Common/import", Icon16x16));
		Style->Set("Icons.Reimport", new IMAGE_BRUSH_SVG("Starship/Common/reimport", Icon16x16));
		Style->Set("Icons.Filter", new IMAGE_BRUSH_SVG("Starship/Common/filter", Icon16x16));
		Style->Set("Icons.AutoFilter", new IMAGE_BRUSH_SVG("Starship/Common/FilterAuto", Icon16x16));

		Style->Set("Icons.Tag", new IMAGE_BRUSH_SVG("Starship/Common/TagSimple_16", Icon16x16));
		Style->Set("Icons.TagCollection", new IMAGE_BRUSH_SVG("Starship/Common/TagCollection_16", Icon16x16));
		
		Style->Set("Icons.Lock", new IMAGE_BRUSH_SVG("Starship/Common/lock", Icon16x16));
		Style->Set("Icons.Unlock", new IMAGE_BRUSH_SVG("Starship/Common/lock-unlocked", Icon16x16));

		Style->Set("Icons.Normalize", new IMAGE_BRUSH_SVG("Starship/Common/normalize", Icon16x16));

		Style->Set("Icons.CircleArrowLeft", new IMAGE_BRUSH_SVG("Starship/Common/circle-arrow-left", Icon16x16));
		Style->Set("Icons.CircleArrowRight", new IMAGE_BRUSH_SVG("Starship/Common/circle-arrow-right", Icon16x16));

		Style->Set("Icons.CircleArrowUp", new IMAGE_BRUSH_SVG("Starship/Common/circle-arrow-up", Icon16x16));
		Style->Set("Icons.CircleArrowDown", new IMAGE_BRUSH_SVG("Starship/Common/circle-arrow-down", Icon16x16));

		Style->Set("Icons.ArrowLeft", new IMAGE_BRUSH_SVG("Starship/Common/arrow-left", Icon16x16));
		Style->Set("Icons.ArrowRight", new IMAGE_BRUSH_SVG("Starship/Common/arrow-right", Icon16x16));
		Style->Set("Icons.ArrowUp", new IMAGE_BRUSH_SVG("Starship/Common/arrow-up", Icon16x16));
		Style->Set("Icons.ArrowDown", new IMAGE_BRUSH_SVG("Starship/Common/arrow-down", Icon16x16));

		Style->Set("Icons.Check", new IMAGE_BRUSH_SVG("Starship/Common/check", Icon16x16));

		Style->Set("Icons.FolderOpen", new IMAGE_BRUSH_SVG("Starship/Common/folder-open", Icon16x16));
		Style->Set("Icons.FolderClosed", new IMAGE_BRUSH_SVG("Starship/Common/folder-closed", Icon16x16));

		Style->Set("Icons.ChevronLeft", new IMAGE_BRUSH_SVG("Starship/Common/chevron-left", Icon16x16));
		Style->Set("Icons.ChevronRight", new IMAGE_BRUSH_SVG("Starship/Common/chevron-right", Icon16x16));

		Style->Set("Icons.ChevronUp", new IMAGE_BRUSH_SVG("Starship/Common/chevron-up", Icon16x16));
		Style->Set("Icons.ChevronDown", new IMAGE_BRUSH_SVG("Starship/Common/chevron-down", Icon16x16));

		Style->Set("Icons.Search", new IMAGE_BRUSH_SVG("Starship/Common/search", Icon16x16));

		Style->Set("Icons.FilledCircle", new IMAGE_BRUSH_SVG("Starship/Common/filled-circle", Icon16x16));

		Style->Set("Icons.Duplicate", new IMAGE_BRUSH_SVG("Starship/Common/Duplicate", Icon16x16));
		Style->Set("Icons.Edit", new IMAGE_BRUSH_SVG("Starship/Common/edit", Icon16x16));

		Style->Set("Icons.Visible", new IMAGE_BRUSH_SVG("Starship/Common/visible", Icon16x16));
		Style->Set("Icons.Hidden", new IMAGE_BRUSH_SVG("Starship/Common/hidden", Icon16x16));

		Style->Set("Icons.DragHandle", new IMAGE_BRUSH_SVG("Starship/Common/drag-handle", Icon16x16));

		Style->Set("Icons.Refresh", new IMAGE_BRUSH_SVG("Starship/Common/refresh", Icon16x16));

		Style->Set("Icons.Star", new IMAGE_BRUSH_SVG("Starship/Common/Favorite", Icon16x16));
		Style->Set("Icons.Star.Outline", new IMAGE_BRUSH_SVG("Starship/Common/FavoriteOutline", Icon16x16));
		Style->Set("Icons.Link", new IMAGE_BRUSH_SVG("Starship/Common/Linked", Icon16x16));
		Style->Set("Icons.Unlink", new IMAGE_BRUSH_SVG("Starship/Common/Unlinked", Icon16x16));

		Style->Set("Icons.BulletPoint", new IMAGE_BRUSH_SVG("Starship/Common/bullet-point", Icon8x8));
		Style->Set("Icons.BulletPoint16", new IMAGE_BRUSH_SVG("Starship/Common/bullet-point16", Icon16x16));

		Style->Set("Icons.SortDown", new IMAGE_BRUSH_SVG("Starship/Common/SortDown", Icon16x16));
		Style->Set("Icons.SortUp", new IMAGE_BRUSH_SVG("Starship/Common/SortUp", Icon16x16));

		Style->Set("Icons.EyeDropper", new IMAGE_BRUSH_SVG("Starship/Common/EyeDropper", Icon16x16));
		Style->Set("Icons.C++", new IMAGE_BRUSH_SVG("Starship/Common/CPP", Icon16x16));

		Style->Set("Icons.Advanced", new IMAGE_BRUSH_SVG("Starship/Common/Advanced", Icon16x16));
		Style->Set("Icons.Launch", new IMAGE_BRUSH_SVG("Starship/Common/ProjectLauncher", Icon16x16));
		Style->Set("Icons.Rotate90Clockwise", new IMAGE_BRUSH_SVG("Starship/Common/Rotate90Clockwise", Icon16x16));
		Style->Set("Icons.Rotate90CounterClockwise", new IMAGE_BRUSH_SVG("Starship/Common/Rotate90CounterClockwise", Icon16x16));
		Style->Set("Icons.Rotate180", new IMAGE_BRUSH_SVG("Starship/Common/Rotate180", Icon16x16));
		Style->Set("Icons.FlipHorizontal", new IMAGE_BRUSH_SVG("Starship/Common/FlipHorizontal", Icon16x16));
		Style->Set("Icons.FlipVertical", new IMAGE_BRUSH_SVG("Starship/Common/FlipVertical", Icon16x16));
		Style->Set("Icons.Layout", new IMAGE_BRUSH_SVG("Starship/Common/Layout", Icon16x16));
		Style->Set("Icons.Recent", new IMAGE_BRUSH_SVG("Starship/Common/Recent", Icon16x16));

		Style->Set("Icons.Badge", new IMAGE_BRUSH_SVG("Starship/Common/badge", Icon16x16));
		Style->Set("Icons.BadgeModified", new IMAGE_BRUSH_SVG("Starship/Common/badge-modified", Icon16x16));

		Style->Set("Icons.TokenTextBox", new IMAGE_BRUSH_SVG("Starship/Common/TokenTextBox_16", Icon16x16));

		// Toolbar Size Icons
		Style->Set("Icons.Toolbar.Settings", new IMAGE_BRUSH_SVG("Starship/Common/settings", Icon20x20));

		// Constraint Manager Icons
		Style->Set("Icons.ConstraintManager.LookAt", new IMAGE_BRUSH_SVG("Starship/Common/LookAt", Icon16x16));
		Style->Set("Icons.ConstraintManager.ParentHierarchy", new IMAGE_BRUSH_SVG("Starship/Common/ParentHierarchy", Icon16x16));

		Style->Set("Icons.Role", new IMAGE_BRUSH_SVG("Starship/Common/Role", Icon16x16));
		Style->Set("Icons.Merge", new IMAGE_BRUSH_SVG("Starship/Common/Merge", Icon16x16));

		Style->Set("Icons.Calendar", new IMAGE_BRUSH_SVG("Starship/Common/Calendar", Icon16x16));

		// Material Graph Node Icons
		Style->Set("Icons.Preview", new IMAGE_BRUSH_SVG("Starship/Common/preview-default", Icon10x10));
		Style->Set("Icons.DropDown", new IMAGE_BRUSH_SVG("Starship/Common/caret-down", Icon10x10));

	}

	// Tool panels
	{

		Style->Set("ToolPanel.GroupBorder", new FSlateColorBrush(FStyleColors::Panel) );
		Style->Set("ToolPanel.DarkGroupBorder", new BOX_BRUSH("Common/DarkGroupBorder", FMargin(4.0f / 16.0f)));
		Style->Set("ToolPanel.LightGroupBorder", new BOX_BRUSH("Common/LightGroupBorder", FMargin(4.0f / 16.0f)));

		Style->Set("Debug.Border", new BOX_BRUSH("Common/DebugBorder", 4.0f / 16.0f));
		Style->Set("PlainBorder", new BORDER_BRUSH("Common/PlainBorder", 2.f / 8.f));
	}

	// Popup text
	{
		Style->Set("PopupText.Background", new BOX_BRUSH("Old/Menu_Background", FMargin(8.0f / 64.0f)));
	}

	// Generic command icons
	{
		Style->Set("GenericCommands.Undo", new IMAGE_BRUSH_SVG("Starship/Common/Undo", Icon16x16));
		Style->Set("GenericCommands.Redo", new IMAGE_BRUSH_SVG("Starship/Common/Redo", Icon16x16));

		Style->Set("GenericCommands.Copy", new IMAGE_BRUSH_SVG("Starship/Common/Copy", Icon16x16));
		Style->Set("GenericCommands.Cut", new IMAGE_BRUSH_SVG("Starship/Common/Cut", Icon16x16));
		Style->Set("GenericCommands.Delete", new IMAGE_BRUSH_SVG("Starship/Common/Delete", Icon16x16));
		Style->Set("GenericCommands.Paste", new IMAGE_BRUSH_SVG("Starship/Common/Paste", Icon16x16));
		Style->Set("GenericCommands.Duplicate", new IMAGE_BRUSH_SVG("Starship/Common/Duplicate", Icon16x16));

		Style->Set("GenericCommands.Rename", new IMAGE_BRUSH_SVG("Starship/Common/Rename", Icon16x16));

		Style->Set("GenericCommands.SelectAll", new IMAGE_BRUSH_SVG("Starship/Common/SelectAll_16", Icon16x16));
	}

	// SVerticalBox Drag& Drop icon
	Style->Set("VerticalBoxDragIndicator", new IMAGE_BRUSH("Common/VerticalBoxDragIndicator", FVector2f(6.f, 45.f)));
	Style->Set("VerticalBoxDragIndicatorShort", new IMAGE_BRUSH("Common/VerticalBoxDragIndicatorShort", FVector2f(6.f, 15.f)));

	SetupButtonStyles(Style);
	SetupComboButtonStyles(Style);
	SetupCheckboxStyles(Style);

	// SMessageLogListing
	{
		FComboButtonStyle MessageLogListingComboButton = FComboButtonStyle()
			.SetButtonStyle(NoBorder)
			.SetDownArrowImage(IMAGE_BRUSH("Common/ComboArrow", Icon8x8))
			.SetMenuBorderBrush(FSlateNoResource())
			.SetMenuBorderPadding(FMargin(0.0f));
		Style->Set("MessageLogListingComboButton", MessageLogListingComboButton);
	}

	// SSuggestionTextBox defaults...
	{
		Style->Set("SuggestionTextBox.Background", new BOX_BRUSH("Old/Menu_Background", FMargin(8.0f / 64.0f)));
		Style->Set("SuggestionTextBox.Text", FTextBlockStyle()
			.SetFont(StyleFonts.Normal)
			.SetColorAndOpacity(FLinearColor(FColor(0xffaaaaaa)))
		);
	}

	// SToolTip defaults...
	{
		Style->Set("ToolTip.Font", DEFAULT_FONT("Regular", 9));
		Style->Set("ToolTip.Background", new FSlateRoundedBoxBrush(FStyleColors::Dropdown, 0.0f, FStyleColors::DropdownOutline, 1.0f));

		Style->Set("ToolTip.LargerFont", StyleFonts.Normal);
		Style->Set("ToolTip.BrightBackground", new BOX_BRUSH("Old/ToolTip_BrightBackground", FMargin(8.0f / 64.0f)));
	}

	// SBorder defaults...
	{
		Style->Set("Border", new FSlateColorBrush(FStyleColors::Panel));

		FLinearColor TransBackground = FStyleColors::Panel.GetSpecifiedColor();
		TransBackground.A = 0.5f;

		Style->Set("FloatingBorder", new FSlateRoundedBoxBrush(TransBackground, 8.f));
	}

	// SHyperlink defaults...
	{
		FButtonStyle HyperlinkButton = FButtonStyle()
			.SetNormal(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0.f, 0.f, 0.f, 3.f / 16.f)))
			.SetPressed(FSlateNoResource())
			.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0.f, 0.f, 0.f, 3.f / 16.0f)));

		FHyperlinkStyle Hyperlink = FHyperlinkStyle()
			.SetUnderlineStyle(HyperlinkButton)
			.SetTextStyle(NormalText)
			.SetPadding(FMargin(0.0f));
		Style->Set("Hyperlink", Hyperlink);
	}

	// Common styles for blueprint/code references that also need to be exposed to external tools
	{
		FTextBlockStyle InheritedFromNativeTextStyle = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10));

		Style->Set("Common.InheritedFromNativeTextStyle", InheritedFromNativeTextStyle);

		// Go to native class hyperlink
		FButtonStyle EditNativeHyperlinkButton = FButtonStyle()
			.SetNormal(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f)))
			.SetPressed(FSlateNoResource())
			.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0, 0, 0, 3 / 16.0f)));
		FHyperlinkStyle EditNativeHyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(EditNativeHyperlinkButton)
			.SetTextStyle(InheritedFromNativeTextStyle)
			.SetPadding(FMargin(0.0f));

		Style->Set("Common.GotoNativeCodeHyperlink", EditNativeHyperlinkStyle);
	}

	// SProgressBar defaults...
	{
		Style->Set("ProgressBar", FProgressBarStyle()
			.SetBackgroundImage(FSlateColorBrush(FStyleColors::Foldout))
			.SetFillImage(IMAGE_BRUSH("Starship/CoreWidgets/ProgressBar/ProgressMarquee", FVector2f(20.f, 12.f), FStyleColors::PrimaryHover, ESlateBrushTileType::Horizontal))
			.SetMarqueeImage(IMAGE_BRUSH("Starship/CoreWidgets/ProgressBar/ProgressMarquee", FVector2f(20.f, 12.f), FStyleColors::PrimaryHover, ESlateBrushTileType::Horizontal))
			.SetEnableFillAnimation(true)
		);
	}


	// SThrobber, SCircularThrobber defaults...
	{
		Style->Set("Throbber.Chunk", new IMAGE_BRUSH("Common/Throbber_Piece", Icon16x16));
		Style->Set("Throbber.CircleChunk", new IMAGE_BRUSH("Common/Throbber_Piece", Icon8x8));
		Style->Set("Throbber.CircleChunk.Small", new IMAGE_BRUSH("Common/Throbber_Piece", Icon4x4));
	}

	// SExpandableArea defaults...
	{
		Style->Set("ExpandableArea", FExpandableAreaStyle()
			.SetCollapsedImage(IMAGE_BRUSH_SVG("Starship/Common/caret-right", Icon16x16, DefaultForeground))
			.SetExpandedImage(IMAGE_BRUSH_SVG("Starship/Common/caret-down", Icon16x16, DefaultForeground))
		);
		Style->Set("ExpandableArea.TitleFont", StyleFonts.SmallBold);
		Style->Set("ExpandableArea.Border", new FSlateRoundedBoxBrush(FStyleColors::Panel, 4.f) );
	}

	// SSlider and SVolumeControl defaults...
	{
		FSliderStyle SliderStyle = FSliderStyle()
			.SetNormalBarImage(FSlateRoundedBoxBrush(FStyleColors::Input, 2.0f, FStyleColors::Input, 1.0f))
			.SetHoveredBarImage(FSlateRoundedBoxBrush(FStyleColors::Input, 2.0f, FStyleColors::Input, 1.0f))
			.SetDisabledBarImage(FSlateRoundedBoxBrush(FStyleColors::Input, 2.0f, FStyleColors::Recessed, 1.0f))
			.SetNormalThumbImage(FSlateRoundedBoxBrush(FStyleColors::Foreground, Icon8x8))
			.SetHoveredThumbImage(FSlateRoundedBoxBrush(FStyleColors::ForegroundHover, Icon8x8))
			.SetDisabledThumbImage(FSlateRoundedBoxBrush(FStyleColors::Foreground, Icon8x8))
			.SetBarThickness(4.0f);
		Style->Set("Slider", SliderStyle);

		Style->Set("VolumeControl", FVolumeControlStyle()
			.SetSliderStyle(SliderStyle)
			.SetHighVolumeImage(IMAGE_BRUSH("Common/VolumeControl_High", Icon16x16))
			.SetMidVolumeImage(IMAGE_BRUSH("Common/VolumeControl_Mid", Icon16x16))
			.SetLowVolumeImage(IMAGE_BRUSH("Common/VolumeControl_Low", Icon16x16))
			.SetNoVolumeImage(IMAGE_BRUSH("Common/VolumeControl_Off", Icon16x16))
			.SetMutedImage(IMAGE_BRUSH("Common/VolumeControl_Muted", Icon16x16))
		);
	}

	// SpinBox defaults...
	{
		// SSpinBox
		FSpinBoxStyle DefaultSpinBox =
			FSpinBoxStyle()
			.SetBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::InputOutline, InputFocusThickness))
			.SetHoveredBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::Hover, InputFocusThickness))
			.SetActiveBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::Primary, InputFocusThickness))
			.SetActiveFillBrush(FSlateRoundedBoxBrush(FStyleColors::Primary, InputFocusRadius, FLinearColor::Transparent, InputFocusThickness))
			.SetHoveredFillBrush(FSlateRoundedBoxBrush(FStyleColors::Hover, InputFocusRadius, FLinearColor::Transparent, InputFocusThickness))
			.SetInactiveFillBrush(FSlateRoundedBoxBrush(FStyleColors::Dropdown, InputFocusRadius, FLinearColor::Transparent, InputFocusThickness))
			.SetArrowsImage(FSlateNoResource())
			.SetForegroundColor(FStyleColors::ForegroundHover)
			.SetTextPadding(FMargin(7.f, 2.5f, 7.f, 2.5f))
			.SetInsetPadding(FMargin(1));

		Style->Set("SpinBox", DefaultSpinBox);

		// SNumericEntryBox
		Style->Set("NumericEntrySpinBox", DefaultSpinBox);

		Style->Set("NumericEntrySpinBox_Dark", FSpinBoxStyle()
			.SetBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::InputOutline, InputFocusThickness))
			.SetHoveredBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::Hover, InputFocusThickness))
			.SetActiveFillBrush(FSlateRoundedBoxBrush(FStyleColors::Hover, InputFocusRadius, FLinearColor::Transparent, InputFocusThickness))
			.SetInactiveFillBrush(FSlateRoundedBoxBrush(FStyleColors::Secondary, InputFocusRadius, FLinearColor::Transparent, InputFocusThickness))
			.SetArrowsImage(FSlateNoResource())
			.SetTextPadding(FMargin(0.f))
			.SetForegroundColor(FStyleColors::ForegroundHover)
		);

		Style->Set("NumericEntrySpinBox.Decorator", new BOX_BRUSH("Common/TextBoxLabelBorder", FMargin(5.0f / 16.0f)));

		Style->Set("NumericEntrySpinBox.NarrowDecorator", new IMAGE_BRUSH_SVG("Starship/CoreWidgets/NumericEntryBox/NarrowDecorator", FVector2f(2.0f,16.0f)));
	}

	SetupColorPickerStyles(Style);

	// SSplitter
	{
		Style->Set("Splitter", FSplitterStyle()
			.SetHandleNormalBrush(FSlateColorBrush(FStyleColors::Background))
			.SetHandleHighlightBrush(FSlateColorBrush(FStyleColors::Secondary))
		);
		Style->Set("SplitterPanel", FSplitterStyle()
			.SetHandleNormalBrush(FSlateColorBrush(FStyleColors::Panel))
			.SetHandleHighlightBrush(FSlateColorBrush(FLinearColor(FColor(96, 96, 96))))
		);
	}

	// TableView defaults...
	SetupTableViewStyles(Style);

	SetupMultiboxStyles(Style);

	// SExpandableButton defaults...
	{
		Style->Set("ExpandableButton.Background", new BOX_BRUSH("Common/Button", 8.0f / 32.0f));

		// Extra padding on the right and bottom to account for image shadow
		Style->Set("ExpandableButton.Padding", FMargin(3.f, 3.f, 6.f, 6.f));

		Style->Set("ExpandableButton.CloseButton", new IMAGE_BRUSH("Common/ExpansionButton_CloseOverlay", Icon16x16));
	}

	// SBreadcrumbTrail defaults...
	{
		Style->Set("BreadcrumbTrail.Delimiter", new IMAGE_BRUSH("Common/Delimiter", Icon16x16));

		Style->Set("BreadcrumbButton", FButtonStyle()
			.SetNormal(FSlateNoResource())
			.SetPressed(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
			.SetHovered(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor))
			.SetNormalPadding(FMargin(0.f, 0.f))
			.SetPressedPadding(FMargin(0.f, 0.f))
		);
	}

	// SWizard defaults
	{
		Style->Set("Wizard.PageTitle", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("BoldCondensed", 28))
			.SetShadowOffset(FVector2f(1.f, 1.f))
			.SetShadowColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.9f))
		);
	}

	// SNotificationList defaults...
	{
		Style->Set("NotificationList.FontBold", DEFAULT_FONT("Bold", 12));
		Style->Set("NotificationList.FontLight", DEFAULT_FONT("Light", 12));
		Style->Set("NotificationList.ItemBackground", new FSlateColorBrush(FStyleColors::Notifications));

		Style->Set("NotificationList.ItemBackground_Border", new FSlateRoundedBoxBrush(FStyleColors::Notifications, 5.0f, FStyleColors::Black, 1.0f));
		Style->Set("NotificationList.ItemBackground_Border_Transparent", new BOX_BRUSH("Old/Notification_Border_Flash", FMargin(8.0f / 64.0f)));

		Style->Set("NotificationList.Throbber", new IMAGE_BRUSH("Starship/Notifications/Throbber", FVector2f(40.f, 40.f)));

		Style->Set("NotificationList.Watermark", new IMAGE_BRUSH_SVG("Starship/Common/unreal-circle-thin", FVector2f(128.f, 128.f)));

		// Should be deleted. everywhere using this is looking for a generic "alert" icon
		Style->Set("NotificationList.DefaultMessage", new IMAGE_BRUSH("Common/EventMessage_Default", Icon40x40));

		// Should be deleted. everywhere using this should be using Icons.Success
		Style->Set("NotificationList.SuccessImage", new IMAGE_BRUSH("Icons/notificationlist_success", Icon16x16));
		Style->Set("NotificationList.FailImage", new IMAGE_BRUSH("Icons/notificationlist_fail", Icon16x16));

		Style->Set("NotificationList.WidgetText", FTextBlockStyle(NormalText)
		.SetOverflowPolicy(ETextOverflowPolicy::Ellipsis));

	}

	// SSeparator defaults...
	{
		Style->Set("Separator", new FSlateColorBrush(FStyleColors::Recessed));
	}

	// SHeader defaults...
	{
		Style->Set("Header.Pre", new BOX_BRUSH("Common/Separator", FMargin(1.f / 4.0f, 0.f, 2.f / 4.0f, 0.f), FLinearColor(1.f, 1.f, 1.f, 0.5f)));
		Style->Set("Header.Post", new BOX_BRUSH("Common/Separator", FMargin(2.f / 4.0f, 0.f, 1.f / 4.0f, 0.f), FLinearColor(1.f, 1.f, 1.f, 0.5f)));
	}

	SetupDockingStyles(Style);

	// SScrollBox defaults...
	{
		Style->Set("ScrollBox", FScrollBoxStyle()
			.SetTopShadowBrush(BOX_BRUSH("Common/ScrollBoxShadowTop", FVector2f(16.f, 8.f), FMargin(0.5f, 1.f, 0.5f, 0.f)))
			.SetBottomShadowBrush(BOX_BRUSH("Common/ScrollBoxShadowBottom", FVector2f(16.f, 8.f), FMargin(0.5f, 0.f, 0.5f, 1.f)))
			.SetLeftShadowBrush(BOX_BRUSH("Common/ScrollBoxShadowLeft", FVector2f(8.f, 16.f), FMargin(1.f, 0.5f, 0.f, 0.5f)))
			.SetRightShadowBrush(BOX_BRUSH("Common/ScrollBoxShadowRight", FVector2f(8.f, 16.f), FMargin(0.f, 0.5f, 1.f, 0.5f)))
			.SetBarThickness(8.0)
		);

		Style->Set("ScrollBoxNoShadow", FScrollBoxStyle()
			.SetTopShadowBrush(FSlateNoResource())
			.SetBottomShadowBrush(FSlateNoResource())
			.SetLeftShadowBrush(FSlateNoResource())
			.SetRightShadowBrush(FSlateNoResource())
			.SetBarThickness(8.0)
		);
	}

	// SScrollBorder defaults...
	{
		Style->Set("ScrollBorder", FScrollBorderStyle()
			.SetTopShadowBrush(BOX_BRUSH("Common/ScrollBorderShadowTop", FVector2f(16.f, 8.f), FMargin(0.5f, 1.f, 0.5f, 0.f)))
			.SetBottomShadowBrush(BOX_BRUSH("Common/ScrollBorderShadowBottom", FVector2f(16.f, 8.f), FMargin(0.5f, 0.f, 0.5f, 1.f)))
		);

		Style->Set("ScrollBorderNoShadow", FScrollBorderStyle()
			.SetTopShadowBrush(FSlateNoResource())
			.SetBottomShadowBrush(FSlateNoResource())
		);
	}

	// SWindow defaults...
	{
#if !PLATFORM_MAC
		const FButtonStyle MinimizeButtonStyle = FButtonStyle(NoBorder)
			.SetNormal( IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/minimize", FVector2f(42.0f, 23.0f), FStyleColors::Foreground))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/minimize", FVector2f(42.0f, 23.0f), FStyleColors::ForegroundHover))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/minimize", FVector2f(42.0f, 23.0f), FStyleColors::Foreground));

		const FButtonStyle MaximizeButtonStyle = FButtonStyle(NoBorder)
			.SetNormal( IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/maximize", FVector2f(42.0f, 23.0f), FStyleColors::Foreground))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/maximize", FVector2f(42.0f, 23.0f), FStyleColors::ForegroundHover))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/maximize", FVector2f(42.0f, 23.0f), FStyleColors::Foreground));

		const FButtonStyle RestoreButtonStyle = FButtonStyle(NoBorder)
			.SetNormal( IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/restore", FVector2f(42.0f, 23.0f), FStyleColors::Foreground))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/restore", FVector2f(42.0f, 23.0f), FStyleColors::ForegroundHover))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/restore", FVector2f(42.0f, 23.0f), FStyleColors::Foreground));

		const FButtonStyle EnterFullscreenButtonStyle = FButtonStyle(NoBorder)
			.SetNormal( IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/enter_fullscreen", FVector2f(42.0f, 23.0f), FStyleColors::Foreground))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/enter_fullscreen", FVector2f(42.0f, 23.0f), FStyleColors::ForegroundHover))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/enter_fullscreen", FVector2f(42.0f, 23.0f), FStyleColors::Foreground));

		const FButtonStyle ExitFullscreenButtonStyle = FButtonStyle(NoBorder)
			.SetNormal( IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/exit_fullscreen", FVector2f(42.0f, 23.0f), FStyleColors::Foreground))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/exit_fullscreen", FVector2f(42.0f, 23.0f), FStyleColors::ForegroundHover))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/exit_fullscreen", FVector2f(42.0f, 23.0f), FStyleColors::Foreground));

		const FButtonStyle CloseButtonStyle = FButtonStyle(NoBorder)
			.SetNormal( IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/close", FVector2f(42.0f, 23.0f), FStyleColors::Foreground))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/close", FVector2f(42.0f, 23.0f), FStyleColors::ForegroundHover))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/CoreWidgets/Window/close", FVector2f(42.0f, 23.0f), FStyleColors::Foreground));
#endif

		FWindowStyle Window = FWindowStyle()
#if !PLATFORM_MAC
			.SetMinimizeButtonStyle(MinimizeButtonStyle)
			.SetMaximizeButtonStyle(MaximizeButtonStyle)
			.SetRestoreButtonStyle(RestoreButtonStyle)
			.SetEnterFullscreenButtonStyle(EnterFullscreenButtonStyle)
			.SetExitFullscreenButtonStyle(ExitFullscreenButtonStyle)
			.SetCloseButtonStyle(CloseButtonStyle)
#endif
			.SetTitleTextStyle(NormalText)
			.SetActiveTitleBrush(FSlateNoResource())
			.SetInactiveTitleBrush(FSlateNoResource())
			.SetFlashTitleBrush(IMAGE_BRUSH("Common/Window/WindowTitle_Flashing", Icon24x24, FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), ESlateBrushTileType::Horizontal))
			.SetBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed))
			.SetBorderBrush(FSlateRoundedBoxBrush(FStyleColors::Recessed, 2.0f, FStyleColors::WindowBorder, 2.0f))
			.SetOutlineBrush(FSlateRoundedBoxBrush(FStyleColors::Recessed, 2.0f, FStyleColors::InputOutline, 1.0f))
			.SetChildBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed))
			.SetCornerRadius(2)
			.SetBorderPadding(FMargin(3.0f, 3.0f, 3.0f, 3.0f));

		Style->Set("Window", Window);

		const FButtonStyle MinMaxRestoreButtonBackgroundStyle = FButtonStyle(NoBorder)
			.SetHovered(FSlateColorBrush(FColor(255, 255, 255, 40)));
		const FButtonStyle CloseButtonBackgroundStyle = FButtonStyle(NoBorder)
			.SetHovered(FSlateColorBrush(FColor(255, 20, 20, 150)));

		Style->Set("Window.MinMaxRestoreButtonHover", MinMaxRestoreButtonBackgroundStyle);
		Style->Set("Window.CloseButtonHover", CloseButtonBackgroundStyle);

		Window.SetCornerRadius(8.0f);

		Style->Set("NotificationWindow", Window);

		Style->Set("ChildWindow.Background", new FSlateColorBrush(FStyleColors::Recessed));
	}

	// Standard Dialog Settings
	{
		Style->Set("StandardDialog.ContentPadding", FMargin(16.0f, 3.0f));
		Style->Set("StandardDialog.SlotPadding", FMargin(8.0f, 0.0f, 0.0f, 0.0f));
		Style->Set("StandardDialog.MinDesiredSlotWidth", 80.0f);
		Style->Set("StandardDialog.MinDesiredSlotHeight", 0.0f);
		Style->Set("StandardDialog.SmallFont", StyleFonts.Small);
		Style->Set("StandardDialog.LargeFont", StyleFonts.Normal);
	}

	// Widget Reflector Window
	{
		Style->Set("WidgetReflector.TabIcon", new IMAGE_BRUSH("Icons/icon_tab_WidgetReflector_16x", Icon16x16));
		Style->Set("WidgetReflector.Icon", new IMAGE_BRUSH("Icons/icon_tab_WidgetReflector_40x", Icon40x40));
		Style->Set("WidgetReflector.Icon.Small", new IMAGE_BRUSH("Icons/icon_tab_WidgetReflector_40x", Icon20x20));
		Style->Set("WidgetReflector.FocusableCheck", FCheckBoxStyle()
			.SetUncheckedImage(IMAGE_BRUSH("Icons/Empty_14x", Icon14x14))
			.SetUncheckedHoveredImage(IMAGE_BRUSH("Icons/Empty_14x", Icon14x14))
			.SetUncheckedPressedImage(IMAGE_BRUSH("Common/SmallCheckBox_Hovered", Icon14x14))
			.SetCheckedImage(IMAGE_BRUSH("Common/SmallCheck", Icon14x14))
			.SetCheckedHoveredImage(IMAGE_BRUSH("Common/SmallCheck", Icon14x14))
			.SetCheckedPressedImage(IMAGE_BRUSH("Common/SmallCheck", Icon14x14))
			.SetUndeterminedImage(IMAGE_BRUSH("Icons/Empty_14x", Icon14x14))
			.SetUndeterminedHoveredImage(FSlateNoResource())
			.SetUndeterminedPressedImage(FSlateNoResource())
		);
	}

	// Message Log
	{
		Style->Set("MessageLog", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Mono", 9))
			.SetShadowOffset(FVector2f::ZeroVector)
		);
		Style->Set("MessageLog.Error", new IMAGE_BRUSH("MessageLog/Log_Error", Icon16x16));
		Style->Set("MessageLog.Warning", new IMAGE_BRUSH("MessageLog/Log_Warning", Icon16x16));
		Style->Set("MessageLog.Note", new IMAGE_BRUSH("MessageLog/Log_Note", Icon16x16));
	}

	// Wizard icons
	{
		Style->Set("Wizard.BackIcon", new IMAGE_BRUSH("Icons/BackIcon", Icon8x8));
		Style->Set("Wizard.NextIcon", new IMAGE_BRUSH("Icons/NextIcon", Icon8x8));
	}

	// Syntax highlighting
	{
		const FTextBlockStyle SmallMonospacedText = FTextBlockStyle(Style->GetWidgetStyle<FTextBlockStyle>("MonospacedText"))
			.SetFont(DEFAULT_FONT("Mono", 9));

		Style->Set("SyntaxHighlight.Normal", SmallMonospacedText);
		Style->Set("SyntaxHighlight.Node", FTextBlockStyle(SmallMonospacedText).SetColorAndOpacity(FLinearColor(FColor(0xff006ab4)))); // blue
		Style->Set("SyntaxHighlight.NodeAttributeKey", FTextBlockStyle(SmallMonospacedText).SetColorAndOpacity(FLinearColor(FColor(0xffb40000)))); // red
		Style->Set("SyntaxHighlight.NodeAttribueAssignment", FTextBlockStyle(SmallMonospacedText).SetColorAndOpacity(FLinearColor(FColor(0xffb2b400)))); // yellow
		Style->Set("SyntaxHighlight.NodeAttributeValue", FTextBlockStyle(SmallMonospacedText).SetColorAndOpacity(FLinearColor(FColor(0xffb46100)))); // orange
	}

	// Logging defaults
	{
		Style->Set("Log.TabIcon", new IMAGE_BRUSH_SVG("Starship/Common/OutputLog", Icon16x16));

		const FTextBlockStyle NormalLogText = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Mono", 9))
			.SetColorAndOpacity(FStyleColors::Foreground)
			.SetSelectedBackgroundColor(FStyleColors::Highlight)
			.SetHighlightColor(FStyleColors::Black);

		Style->Set("Log.Normal", NormalLogText);

		Style->Set("Log.Warning", FTextBlockStyle(NormalLogText)
			.SetColorAndOpacity(FStyleColors::Warning)
		);

		Style->Set("Log.Error", FTextBlockStyle(NormalLogText)
			.SetColorAndOpacity(FStyleColors::Error)
		);

		Style->Set("Log.TextBox", FEditableTextBoxStyle(NormalEditableTextBoxStyle)
			.SetTextStyle(NormalLogText)
			.SetBackgroundImageNormal(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
			.SetBackgroundImageHovered(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
			.SetBackgroundImageFocused(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
			.SetBackgroundImageReadOnly(BOX_BRUSH("Common/WhiteGroupBorder", FMargin(4.0f / 16.0f)))
			.SetBackgroundColor(FStyleColors::Recessed)
		);
	}

	// Filter Widgets
	{
		FSearchBoxStyle FilterSearchBoxStyle = Style->GetWidgetStyle<FSearchBoxStyle>("SearchBox");
		FEditableTextBoxStyle FilterTextBoxStyle = FilterSearchBoxStyle.TextBoxStyle;

		// Extra padding on the right to account for the Save Search button
		FilterTextBoxStyle.SetPadding(FMargin(8.0f, 3.0f, 30.0f, 4.0f));
		FilterSearchBoxStyle.SetTextBoxStyle(FilterTextBoxStyle);

		Style->Set("FilterBar.SearchBox", FilterSearchBoxStyle);
		
		const FCheckBoxStyle FilterButtonCheckBoxStyle = FCheckBoxStyle()
			.SetUncheckedImage(FSlateNoResource())
			.SetUncheckedHoveredImage(FSlateNoResource())
			.SetUncheckedPressedImage(FSlateNoResource())
			.SetCheckedImage(FSlateNoResource())
			.SetCheckedHoveredImage(FSlateNoResource())
			.SetCheckedPressedImage(FSlateNoResource())
			.SetForegroundColor(FStyleColors::Foreground)
			.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedForegroundColor(FStyleColors::Foreground)
			.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedPressedForegroundColor(FStyleColors::ForegroundHover)
			.SetPressedForegroundColor(FStyleColors::ForegroundHover);

		Style->Set("FilterBar.FilterImage", new IMAGE_BRUSH_SVG("Starship/CoreWidgets/FilterBar/FilterColorSegment", FVector2f(8.f, 22.f)));
		Style->Set("FilterBar.FilterBackground", new FSlateRoundedBoxBrush(FStyleColors::Secondary, 3.0f));

		/* ... and add the new style */
		Style->Set("FilterBar.FilterButton", FilterButtonCheckBoxStyle );
		
		Style->Set( "FilterBar.BasicFilterButton", FCheckBoxStyle( Style->GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
			.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, 1.0f))
			.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Input, 1.0f))
			.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Input, 1.0f))
			.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f, FStyleColors::Input, 1.0f))
			.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::Input, 1.0f))
			.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::Input, 1.0f))
			.SetPadding(FMargin(16, 4))
		);

		// FilteBar implemented as toolbar
		FToolBarStyle FilterToolBar = FToolBarStyle(Style->GetWidgetStyle<FToolBarStyle>("SlimToolBar"))
			.SetIconSize(Icon16x16)
			.SetBackground(FSlateNoResource())
			.SetLabelPadding(FMargin(0))
			.SetComboButtonPadding(FMargin(0))
			.SetBlockPadding(FMargin(3, 0))
			.SetIndentedBlockPadding(FMargin(0))
			.SetBackgroundPadding(FMargin(0))
			.SetButtonPadding(FMargin(0))
			.SetCheckBoxPadding(FMargin(0))
			.SetSeparatorBrush(FSlateNoResource())
			.SetSeparatorPadding(FMargin(0));

		Style->Set("FilterBar.FilterToolBar", FilterToolBar);
	}
	
	// Project Launcher
	{
		Style->Set("Launcher.TabIcon", new IMAGE_BRUSH_SVG("Starship/Common/ProjectLauncher", Icon16x16));
		Style->Set("Launcher.Tabs.Tools", new IMAGE_BRUSH("/Icons/icon_tab_Tools_16x", Icon16x16));
		

		Style->Set("Launcher.Run", new IMAGE_BRUSH("Launcher/Launcher_Run", Icon40x40));
		Style->Set("Launcher.EditSettings", new IMAGE_BRUSH("Launcher/Launcher_EditSettings", Icon40x40));
		Style->Set("Launcher.Back", new IMAGE_BRUSH("Launcher/Launcher_Back", Icon32x32));
		Style->Set("Launcher.Back.Small", new IMAGE_BRUSH("Launcher/Launcher_Back", Icon32x32));
		Style->Set("Launcher.Delete", new IMAGE_BRUSH("Launcher/Launcher_Delete", Icon32x32));

		Style->Set("Launcher.Instance_Commandlet", new IMAGE_BRUSH("Launcher/Instance_Commandlet", Icon25x25));
		Style->Set("Launcher.Instance_Editor", new IMAGE_BRUSH("Launcher/Instance_Editor", Icon25x25));
		Style->Set("Launcher.Instance_Game", new IMAGE_BRUSH("Launcher/Instance_Game", Icon25x25));
		Style->Set("Launcher.Instance_Other", new IMAGE_BRUSH("Launcher/Instance_Other", Icon25x25));
		Style->Set("Launcher.Instance_Server", new IMAGE_BRUSH("Launcher/Instance_Server", Icon25x25));
		Style->Set("Launcher.Instance_Unknown", new IMAGE_BRUSH("Launcher/Instance_Unknown", Icon25x25));
		Style->Set("LauncherCommand.DeployBuild", new IMAGE_BRUSH("Launcher/Launcher_Deploy", Icon40x40));
		Style->Set("LauncherCommand.QuickLaunch", new IMAGE_BRUSH_SVG("Starship/Launcher/PaperAirplane", Icon20x20));
		Style->Set("LauncherCommand.CreateBuild", new IMAGE_BRUSH("Launcher/Launcher_Build", Icon40x40));
		Style->Set("LauncherCommand.AdvancedBuild", new IMAGE_BRUSH("Launcher/Launcher_Advanced", Icon40x40));
		Style->Set("LauncherCommand.AdvancedBuild.Medium", new IMAGE_BRUSH("Launcher/Launcher_Advanced", Icon25x25));
		Style->Set("LauncherCommand.AdvancedBuild.Small", new IMAGE_BRUSH("Launcher/Launcher_Advanced", Icon20x20));

#if DDPI_HAS_EXTENDED_PLATFORMINFO_DATA && (WITH_EDITOR || WITH_UNREAL_DEVELOPER_TOOLS)

		Style->Set("Launcher.Platform.AllPlatforms", new IMAGE_BRUSH("Launcher/All_Platforms_24x", Icon24x24));
		Style->Set("Launcher.Platform.AllPlatforms.Large", new IMAGE_BRUSH("Launcher/All_Platforms_128x", Icon64x64));
		Style->Set("Launcher.Platform.AllPlatforms.XLarge", new IMAGE_BRUSH("Launcher/All_Platforms_128x", Icon128x128));
		for (auto Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
		{
			const FDataDrivenPlatformInfo& PlatformInfo = Pair.Value;

			// some platforms may specify a "rooted" path in the platform extensions directory, so look for that case here, and use a different path for the brush
			FString NormalIconPath = PlatformInfo.GetIconPath(EPlatformIconSize::Normal);
			if (!NormalIconPath.IsEmpty())
			{
				if (NormalIconPath.StartsWith(TEXT("/Platforms/")))
				{
#define PLATFORM_IMAGE_BRUSH( Platform, PlatformPath, ... ) FSlateImageBrush( PlatformPath.Replace(*(FString("/Platforms/") + Platform.ToString() + TEXT("/")), *FPaths::EnginePlatformExtensionDir(*Platform.ToString())) + TEXT(".png") , __VA_ARGS__ )
					Style->Set(PlatformInfo.GetIconStyleName(EPlatformIconSize::Normal), new PLATFORM_IMAGE_BRUSH(PlatformInfo.IniPlatformName, NormalIconPath, Icon24x24));
					Style->Set(PlatformInfo.GetIconStyleName(EPlatformIconSize::Large), new PLATFORM_IMAGE_BRUSH(PlatformInfo.IniPlatformName, PlatformInfo.GetIconPath(EPlatformIconSize::Large), Icon64x64));
					Style->Set(PlatformInfo.GetIconStyleName(EPlatformIconSize::XLarge), new PLATFORM_IMAGE_BRUSH(PlatformInfo.IniPlatformName, PlatformInfo.GetIconPath(EPlatformIconSize::XLarge), Icon128x128));
				}
				else
				{
					const FString PathPrefix = FPaths::EngineContentDir() / TEXT("Editor/Slate");
#define DEVELOPER_IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush(PathPrefix / RelativePath + TEXT(".png"), __VA_ARGS__ )
					Style->Set(PlatformInfo.GetIconStyleName(EPlatformIconSize::Normal), new DEVELOPER_IMAGE_BRUSH(*NormalIconPath, Icon24x24));
					Style->Set(PlatformInfo.GetIconStyleName(EPlatformIconSize::Large), new DEVELOPER_IMAGE_BRUSH(*PlatformInfo.GetIconPath(EPlatformIconSize::Large), Icon64x64));
					Style->Set(PlatformInfo.GetIconStyleName(EPlatformIconSize::XLarge), new DEVELOPER_IMAGE_BRUSH(*PlatformInfo.GetIconPath(EPlatformIconSize::XLarge), Icon128x128));
#undef DEVELOPER_IMAGE_BRUSH
				}
			}
		}

		for (const FPreviewPlatformMenuItem& Item : FDataDrivenPlatformInfoRegistry::GetAllPreviewPlatformMenuItems())
		{
			if (!Item.ActiveIconPath.IsEmpty())
			{
				Style->Set(Item.ActiveIconName, new PLATFORM_IMAGE_BRUSH(Item.PlatformName, Item.ActiveIconPath, Icon40x40));
			}
			if (!Item.InactiveIconPath.IsEmpty())
			{
				Style->Set(Item.InactiveIconName, new PLATFORM_IMAGE_BRUSH(Item.PlatformName, Item.InactiveIconPath, Icon40x40));
			}
		}
#endif

	}

	// Vertical ToolBar
	{
		FToolBarStyle FVerticalToolBarStyle = Style->GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		FTextBlockStyle FVerticalToolBarLabelStyle = FTextBlockStyle(Style->GetWidgetStyle<FTextBlockStyle>("SmallText"));
		FVerticalToolBarLabelStyle.SetOverflowPolicy(ETextOverflowPolicy::Ellipsis);

		FVerticalToolBarStyle.SetLabelStyle(FVerticalToolBarLabelStyle);
		FVerticalToolBarStyle.SetLabelPadding(FMargin(4.0f, 2.0f, 4.0f, 4.0f));
		FVerticalToolBarStyle.SetButtonContentMaxWidth(64.0f);
		FVerticalToolBarStyle.SetButtonContentFillWidth(1.0f);

		FVerticalToolBarStyle.SetButtonPadding(FMargin(0.0f, 0.0f, 0.0f, 0.0f));
		//not this
		FVerticalToolBarStyle.SetCheckBoxPadding(FMargin(0.0f, 0.0f));
		FVerticalToolBarStyle.SetComboButtonPadding(FMargin(0.0f, 0.0f));
		FVerticalToolBarStyle.SetIndentedBlockPadding(FMargin(0.0f, 0.0f));
		FVerticalToolBarStyle.SetBackgroundPadding(FMargin(6.0f, 4.0f));
		FVerticalToolBarStyle.ButtonStyle.SetNormalPadding(FMargin(12.0f, 6.0f));
		FVerticalToolBarStyle.ButtonStyle.SetPressedPadding(FMargin(12.0f, 6.0f));
		FVerticalToolBarStyle.SetIconPadding(FMargin(8.0f, 8.0f));
		FVerticalToolBarStyle.SetIconPaddingWithVisibleLabel(FMargin(8.0f, 8.0f, 8.0f, 2.0f));

		FVerticalToolBarStyle.WrapButtonStyle.SetExpandBrush(IMAGE_BRUSH_SVG("Starship/Common/ellipsis-horizontal-narrow", Icon16x16));
		FCheckBoxStyle CheckBoxStyle = FCheckBoxStyle(FVerticalToolBarStyle.ToggleButton)
			.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.f, FLinearColor(0.f, 0.f, 0.f, .8f), 0.5f))
			.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.f, FLinearColor(0.f, 0.f, 0.f, .8f), 0.5f))
			.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryPress, 4.f, FLinearColor(0.f, 0.f, 0.f, .8f), 0.5f))
			.SetForegroundColor(FStyleColors::Foreground)
			.SetPressedForegroundColor(FStyleColors::ForegroundHover)
			.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedPressedForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetPadding(0.f);

		Style->Set("FVerticalToolBar.ToggleButton", CheckBoxStyle);

		FVerticalToolBarStyle.SetButtonPadding(FMargin(0.0f, 4.0f));
		FVerticalToolBarStyle.SetToggleButtonStyle(CheckBoxStyle);
		FVerticalToolBarStyle.SetSeparatorPadding(FMargin(-5.0f, 4.0f));
		Style->Set("FVerticalToolBar", FVerticalToolBarStyle);
	}

	return Style;
}

PRAGMA_POP_ATTRIBUTE_MINSIZE_FUNCTIONS

const TSharedPtr<FSlateDynamicImageBrush> FStarshipCoreStyle::GetDynamicImageBrush(FName BrushTemplate, FName TextureName, const ANSICHAR* Specifier)
{
	return Instance->GetDynamicImageBrush(BrushTemplate, TextureName, Specifier);
}


const TSharedPtr<FSlateDynamicImageBrush> FStarshipCoreStyle::GetDynamicImageBrush(FName BrushTemplate, const ANSICHAR* Specifier, class UTexture2D* TextureResource, FName TextureName)
{
	return Instance->GetDynamicImageBrush(BrushTemplate, Specifier, TextureResource, TextureName);
}


const TSharedPtr<FSlateDynamicImageBrush> FStarshipCoreStyle::GetDynamicImageBrush(FName BrushTemplate, class UTexture2D* TextureResource, FName TextureName)
{
	return Instance->GetDynamicImageBrush(BrushTemplate, TextureResource, TextureName);
}


/* FSlateThrottleManager implementation
 *****************************************************************************/

void FStarshipCoreStyle::SetStyle(const TSharedRef< ISlateStyle >& NewStyle)
{
	if (Instance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*Instance.Get());
	}

	Instance = NewStyle;

	if (Instance.IsValid())
	{
		FSlateStyleRegistry::RegisterSlateStyle(*Instance.Get());
	}
	else
	{
		ResetToDefault();
	}
}

void FStarshipCoreStyle::SetupColors(TSharedRef<FStyle>& Style)
{
	Style->Set("Colors.Transparent", FStyleColors::Transparent);
	Style->Set("Colors.Black", FStyleColors::Black);
	Style->Set("Colors.Title", FStyleColors::Title);
	Style->Set("Colors.WindowBorder", FStyleColors::WindowBorder);
	Style->Set("Colors.Foldout", FStyleColors::Foldout);
	Style->Set("Colors.Input", FStyleColors::Input);
	Style->Set("Colors.InputOutline", FStyleColors::InputOutline);
	Style->Set("Colors.Recessed", FStyleColors::Recessed);
	Style->Set("Colors.Background", FStyleColors::Background);
	Style->Set("Colors.Panel", FStyleColors::Panel);
	Style->Set("Colors.Header", FStyleColors::Header);
	Style->Set("Colors.Dropdown", FStyleColors::Dropdown);
	Style->Set("Colors.DropdownOutline", FStyleColors::DropdownOutline);
	Style->Set("Colors.Hover", FStyleColors::Hover);
	Style->Set("Colors.Hover2", FStyleColors::Hover2);
	Style->Set("Colors.White", FStyleColors::White);
	Style->Set("Colors.White25", FStyleColors::White25);
	Style->Set("Colors.Highlight", FStyleColors::Highlight);

	Style->Set("Colors.Foreground", FStyleColors::Foreground);
	Style->Set("Colors.ForegroundHover", FStyleColors::ForegroundHover);
	Style->Set("Colors.ForegroundInverted", FStyleColors::ForegroundInverted);
	Style->Set("Colors.ForegroundHeader", FStyleColors::ForegroundHeader);

	Style->Set("Colors.Select", FStyleColors::Select);
	Style->Set("Colors.SelectInactive", FStyleColors::SelectInactive);
	Style->Set("Colors.SelectParent", FStyleColors::SelectParent);
	Style->Set("Colors.SelectHover", FStyleColors::SelectHover);

	Style->Set("Colors.Primary", FStyleColors::Primary);
	Style->Set("Colors.PrimaryHover", FStyleColors::PrimaryHover);
	Style->Set("Colors.PrimaryPress", FStyleColors::PrimaryPress);
	Style->Set("Colors.Secondary", FStyleColors::Secondary);

	Style->Set("Colors.AccentBlue", FStyleColors::AccentBlue);
	Style->Set("Colors.AccentPurple", FStyleColors::AccentPurple);
	Style->Set("Colors.AccentPink", FStyleColors::AccentPink);
	Style->Set("Colors.AccentRed", FStyleColors::AccentRed);
	Style->Set("Colors.AccentOrange", FStyleColors::AccentOrange);
	Style->Set("Colors.AccentYellow", FStyleColors::AccentYellow);
	Style->Set("Colors.AccentGreen", FStyleColors::AccentGreen);
	Style->Set("Colors.AccentBrown", FStyleColors::AccentBrown);
	Style->Set("Colors.AccentBlack", FStyleColors::AccentBlack);
	Style->Set("Colors.AccentGray", FStyleColors::AccentGray);
	Style->Set("Colors.AccentWhite", FStyleColors::AccentWhite);
	Style->Set("Colors.AccentFolder", FStyleColors::AccentFolder);

	Style->Set("Colors.Warning", FStyleColors::Warning);
	Style->Set("Colors.Error", FStyleColors::Error);

	Style->Set("Brushes.Transparent", new FSlateColorBrush(FStyleColors::Transparent));
	Style->Set("Brushes.Black", new FSlateColorBrush(FStyleColors::Black));
	Style->Set("Brushes.Title", new FSlateColorBrush(FStyleColors::Title));
	Style->Set("Brushes.Foldout", new FSlateColorBrush(FStyleColors::Foldout));
	Style->Set("Brushes.Input", new FSlateColorBrush(FStyleColors::Input));
	Style->Set("Brushes.InputOutline", new FSlateColorBrush(FStyleColors::InputOutline));
	Style->Set("Brushes.Recessed", new FSlateColorBrush(FStyleColors::Recessed));
	Style->Set("Brushes.Background", new FSlateColorBrush(FStyleColors::Background));
	Style->Set("Brushes.Panel", new FSlateColorBrush(FStyleColors::Panel));
	Style->Set("Brushes.Header", new FSlateColorBrush(FStyleColors::Header));
	Style->Set("Brushes.Dropdown", new FSlateColorBrush(FStyleColors::Dropdown));
	Style->Set("Brushes.DropdownOutline", new FSlateColorBrush(FStyleColors::DropdownOutline));
	Style->Set("Brushes.Hover", new FSlateColorBrush(FStyleColors::Hover));
	Style->Set("Brushes.Hover2", new FSlateColorBrush(FStyleColors::Hover2));
	Style->Set("Brushes.White", new FSlateColorBrush(FStyleColors::White));
	Style->Set("Brushes.White25", new FSlateColorBrush(FStyleColors::White25));
	Style->Set("Brushes.Highlight", new FSlateColorBrush(FStyleColors::Highlight));

	Style->Set("Brushes.Foreground", new FSlateColorBrush(FStyleColors::Foreground));
	Style->Set("Brushes.ForegroundHover", new FSlateColorBrush(FStyleColors::ForegroundHover));
	Style->Set("Brushes.ForegroundInverted", new FSlateColorBrush(FStyleColors::ForegroundInverted));
	Style->Set("Brushes.ForegroundHeader", new FSlateColorBrush(FStyleColors::ForegroundHeader));

	Style->Set("Brushes.Select", new FSlateColorBrush(FStyleColors::Select));
	Style->Set("Brushes.SelectInactive", new FSlateColorBrush(FStyleColors::SelectInactive));
	Style->Set("Brushes.SelectParent", new FSlateColorBrush(FStyleColors::SelectParent));
	Style->Set("Brushes.SelectHover", new FSlateColorBrush(FStyleColors::SelectHover));

	Style->Set("Brushes.Primary", new FSlateColorBrush(FStyleColors::Primary));
	Style->Set("Brushes.PrimaryHover", new FSlateColorBrush(FStyleColors::PrimaryHover));
	Style->Set("Brushes.PrimaryPress", new FSlateColorBrush(FStyleColors::PrimaryPress));
	Style->Set("Brushes.Secondary", new FSlateColorBrush(FStyleColors::Secondary));
	Style->Set("Brushes.AccentBlue", new FSlateColorBrush(FStyleColors::AccentBlue));
	Style->Set("Brushes.AccentPurple", new FSlateColorBrush(FStyleColors::AccentPurple));
	Style->Set("Brushes.AccentPink", new FSlateColorBrush(FStyleColors::AccentPink));
	Style->Set("Brushes.AccentRed", new FSlateColorBrush(FStyleColors::AccentRed));
	Style->Set("Brushes.AccentOrange", new FSlateColorBrush(FStyleColors::AccentOrange));
	Style->Set("Brushes.AccentYellow", new FSlateColorBrush(FStyleColors::AccentYellow));
	Style->Set("Brushes.AccentGreen", new FSlateColorBrush(FStyleColors::AccentGreen));
	Style->Set("Brushes.AccentBrown", new FSlateColorBrush(FStyleColors::AccentBrown));
	Style->Set("Brushes.AccentBlack", new FSlateColorBrush(FStyleColors::AccentBlack));
	Style->Set("Brushes.AccentGray", new FSlateColorBrush(FStyleColors::AccentGray));
	Style->Set("Brushes.AccentWhite", new FSlateColorBrush(FStyleColors::AccentWhite));
	Style->Set("Brushes.AccentFolder", new FSlateColorBrush(FStyleColors::AccentFolder));

	Style->Set("Brushes.Warning", new FSlateColorBrush(FStyleColors::Warning));
	Style->Set("Brushes.Error", new FSlateColorBrush(FStyleColors::Error));
}

void FStarshipCoreStyle::SetupTextStyles(TSharedRef<FStyle>& Style)
{
	const FScrollBarStyle& ScrollBar = Style->GetWidgetStyle<FScrollBarStyle>("Scrollbar");

	const FStyleFonts& StyleFonts = FStyleFonts::Get();

	const FTextBlockStyle NormalText = FTextBlockStyle()
		.SetFont(StyleFonts.Normal)
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2f::ZeroVector)
		.SetSelectedBackgroundColor(FStyleColors::Highlight)
		.SetHighlightColor(FStyleColors::Black)
		.SetHighlightShape(FSlateColorBrush(FStyleColors::AccentGreen));

	Style->Set("NormalFont", StyleFonts.Normal);
	Style->Set("SmallFont", StyleFonts.Small);
	Style->Set("Font.Large", StyleFonts.Large);
	Style->Set("NormalFontBold", StyleFonts.NormalBold);
	Style->Set("SmallFontBold", StyleFonts.SmallBold);
	Style->Set("Font.Large.Bold", StyleFonts.LargeBold);
	Style->Set("NormalFontItalic", StyleFonts.NormalItalic);
	Style->Set("NormalFontBoldItalic", StyleFonts.NormalBoldItalic);

	Style->Set("HeadingMedium", StyleFonts.HeadingMedium);
	Style->Set("HeadingSmall", StyleFonts.HeadingSmall);
	Style->Set("HeadingExtraSmall", StyleFonts.HeadingExtraSmall);

	FSlateBrush* DefaultTextUnderlineBrush = new IMAGE_BRUSH("Old/White", Icon8x8, FLinearColor::White, ESlateBrushTileType::Both);

	Style->Set("DefaultTextUnderline", DefaultTextUnderlineBrush);

	const FTextBlockStyle NormalUnderlinedText = FTextBlockStyle(NormalText)
		.SetUnderlineBrush(*DefaultTextUnderlineBrush);

	// Monospaced Text
	const FTextBlockStyle MonospacedText = FTextBlockStyle()
		.SetFont(DEFAULT_FONT("Mono", 10))
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2f::ZeroVector)
		.SetShadowColorAndOpacity(FLinearColor::Black)
		.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
		.SetHighlightShape(BOX_BRUSH("Common/TextBlockHighlightShape", FMargin(3.f / 8.f))
		);

	const FTextBlockStyle MonospacedUnderlinedText = FTextBlockStyle(MonospacedText)
		.SetUnderlineBrush(*DefaultTextUnderlineBrush);

	Style->Set("MonospacedText", MonospacedText);
	Style->Set("MonospacedUnderlinedText", MonospacedUnderlinedText);

	// Small Text
	const FTextBlockStyle SmallText = FTextBlockStyle(NormalText)
		.SetFont(StyleFonts.Small);

	const FTextBlockStyle SmallUnderlinedText = FTextBlockStyle(SmallText)
		.SetUnderlineBrush(*DefaultTextUnderlineBrush);

	const FTextBlockStyle LargeText = FTextBlockStyle(NormalText)
		.SetFont(StyleFonts.Large);

	const FTextBlockStyle LargeUnderlinedText = FTextBlockStyle(LargeText)
		.SetUnderlineBrush(*DefaultTextUnderlineBrush);


	// Embossed Text
	Style->Set("EmbossedText", FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 24))
		.SetColorAndOpacity(FLinearColor::Black)
		.SetShadowOffset(FVector2f(0.0f, 1.0f))
		.SetShadowColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.8f, 0.5f))
	);

	const FEditableTextBoxStyle DarkEditableTextBoxStyle = FEditableTextBoxStyle()
		.SetTextStyle(NormalText)
		.SetBackgroundImageNormal(BOX_BRUSH("Common/TextBox_Dark", FMargin(4.0f / 16.0f)))
		.SetBackgroundImageHovered(BOX_BRUSH("Common/TextBox_Hovered_Dark", FMargin(4.0f / 16.0f)))
		.SetBackgroundImageFocused(BOX_BRUSH("Common/TextBox_Hovered_Dark", FMargin(4.0f / 16.0f)))
		.SetBackgroundImageReadOnly(BOX_BRUSH("Common/TextBox_ReadOnly", FMargin(4.0f / 16.0f)))
		.SetScrollBarStyle(ScrollBar);
	{
		Style->Set("DarkEditableTextBox", DarkEditableTextBoxStyle);
		// "NormalFont".
	}


	// STextBlock defaults...
	{
		Style->Set("NormalText", NormalText);
		Style->Set("NormalUnderlinedText", NormalUnderlinedText);

		Style->Set("SmallText", SmallText);
		Style->Set("SmallUnderlinedText", SmallUnderlinedText);

		Style->Set("Text.Large", LargeText);
		Style->Set("Text.Large.Underlined", LargeUnderlinedText);
	}


	// SEditableText defaults...
	{
		FSlateBrush* SelectionBackground = new FSlateColorBrush(FStyleColors::Highlight);
		FSlateBrush* SelectionTarget = new BOX_BRUSH("Old/DashedBorder", FMargin(6.0f / 32.0f), FLinearColor(0.0f, 0.0f, 0.0f, 0.75f));
		FSlateBrush* CompositionBackground = new BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0.f, 0.f, 0.f, 3.f / 16.0f));


		const FEditableTextStyle NormalEditableTextStyle = FEditableTextStyle()
			.SetBackgroundImageSelected(*SelectionBackground)
			.SetBackgroundImageComposing(*CompositionBackground)
			.SetCaretImage(FSlateColorBrush(FStyleColors::White));

		Style->Set("NormalEditableText", NormalEditableTextStyle);

		Style->Set("EditableText.SelectionBackground", SelectionBackground);
		Style->Set("EditableText.SelectionTarget", SelectionTarget);
		Style->Set("EditableText.CompositionBackground", CompositionBackground);
	}

	// SEditableTextBox defaults...
	const FEditableTextBoxStyle NormalEditableTextBoxStyle = FEditableTextBoxStyle()
		.SetTextStyle(NormalText)
		.SetBackgroundImageNormal(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::InputOutline, InputFocusThickness))
		.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::Hover, InputFocusThickness))
		.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::Primary, InputFocusThickness))
		.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Header, InputFocusRadius, FStyleColors::InputOutline, InputFocusThickness))
		.SetFont(StyleFonts.Normal)
		.SetPadding(FMargin(8.0f, 3.0f, 8.0f, 4.0f)) 
		.SetForegroundColor(FStyleColors::Foreground)
		.SetBackgroundColor(FStyleColors::White)
		.SetReadOnlyForegroundColor(FSlateColor::UseForeground())
		.SetFocusedForegroundColor(FStyleColors::White)
		.SetScrollBarStyle(ScrollBar);
	{
		Style->Set("NormalEditableTextBox", NormalEditableTextBoxStyle);
	}

	// SSearchBox defaults...
	{
		const FEditableTextBoxStyle SearchBoxEditStyle =
			FEditableTextBoxStyle(NormalEditableTextBoxStyle)
			.SetBackgroundImageNormal(FSlateRoundedBoxBrush(FStyleColors::Input, FStyleColors::Secondary, InputFocusThickness))
			.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Input, FStyleColors::Hover, InputFocusThickness))
			.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Input, FStyleColors::Primary, InputFocusThickness))
			.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Header, FStyleColors::InputOutline, InputFocusThickness)
		);

		Style->Set("SearchBox", FSearchBoxStyle()
			.SetTextBoxStyle(SearchBoxEditStyle)
			.SetUpArrowImage(IMAGE_BRUSH_SVG("Starship/Common/chevron-up", Icon16x16, FStyleColors::Foreground))
			.SetDownArrowImage(IMAGE_BRUSH_SVG("Starship/Common/chevron-down", Icon16x16, FStyleColors::Foreground))
			.SetGlassImage(IMAGE_BRUSH_SVG("Starship/Common/search", Icon16x16))
			.SetClearImage(IMAGE_BRUSH_SVG("Starship/Common/close", Icon16x16))
			.SetImagePadding(FMargin(4.f, 0.f, 0.f, 0.f))
			.SetImageSizeOverride(FVector2D(14.0f))
			.SetLeftAlignSearchResultButtons(true)
			.SetLeftAlignGlassImageAndClearButton(true)
		);
	}

	const FTextBlockStyle HintText = FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Italic", 8))
		.SetColorAndOpacity(FSlateColor::UseSubduedForeground());
	Style->Set("HintText", HintText);

	// SInlineEditableTextBlock
	{
		// Normal Editable Text 
		FTextBlockStyle InlineEditableTextBlockReadOnly = FTextBlockStyle(NormalText);

		FInlineEditableTextBlockStyle InlineEditableTextBlockStyle = FInlineEditableTextBlockStyle()
			.SetTextStyle(InlineEditableTextBlockReadOnly)
			.SetEditableTextBoxStyle(NormalEditableTextBoxStyle);

		Style->Set("InlineEditableTextBlockStyle", InlineEditableTextBlockStyle);

		// Small Editable Text 
		FTextBlockStyle InlineEditableTextBlockSmallReadOnly = FTextBlockStyle(InlineEditableTextBlockReadOnly)
			.SetFont(StyleFonts.Small);

		FEditableTextBoxStyle InlineEditableTextBlockSmallEditable = FEditableTextBoxStyle(NormalEditableTextBoxStyle)
			.SetFont(StyleFonts.Small);

		FInlineEditableTextBlockStyle InlineEditableTextBlockSmallStyle = FInlineEditableTextBlockStyle()
			.SetTextStyle(InlineEditableTextBlockSmallReadOnly)
			.SetEditableTextBoxStyle(InlineEditableTextBlockSmallEditable);

		Style->Set("InlineEditableTextBlockSmallStyle", InlineEditableTextBlockSmallStyle);

		// Hint Editable Text 
		FInlineEditableTextBlockStyle InlineEditableHintTextBlockStyle = FInlineEditableTextBlockStyle()
			.SetTextStyle(HintText)
			.SetEditableTextBoxStyle(NormalEditableTextBoxStyle);

		Style->Set("InlineEditableTextBlockHintStyle", InlineEditableHintTextBlockStyle);
	}
}

void FStarshipCoreStyle::SetupButtonStyles(TSharedRef<FStyle>& Style)
{
	// SButton defaults
	const FButtonStyle PrimaryButton = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f, FStyleColors::Input, InputFocusThickness))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::Input, InputFocusThickness))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::PrimaryPress, 4.0f, FStyleColors::Input, InputFocusThickness))
		.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f, FStyleColors::Recessed, InputFocusThickness))
		.SetNormalForeground(FStyleColors::ForegroundHover)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::Foreground)
		.SetNormalPadding(ButtonMargins)
		.SetPressedPadding(PressedButtonMargins);

	const FMargin PrimaryButtonLabelAndIconNormalPadding(4.f, 1.5f, 4.f, 1.5f);
	const FMargin PrimaryButtonLabelAndIconPressedPadding(4.f, 2.5f, 4.f, 0.5f);

	const FButtonStyle Button = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.0f, FStyleColors::Input, InputFocusThickness))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Input, InputFocusThickness))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, InputFocusThickness))
		.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f, FStyleColors::Recessed, InputFocusThickness))
		.SetNormalForeground(FStyleColors::ForegroundHover)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::Foreground)
		.SetNormalPadding(ButtonMargins)
		.SetPressedPadding(PressedButtonMargins);

	const FButtonStyle EditorUtilityButton = FButtonStyle(Button)
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.0f, FStyleColors::Input, InputFocusThickness, Icon32x32))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Input, InputFocusThickness, Icon32x32))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, InputFocusThickness, Icon32x32));

	const FButtonStyle SimpleButton = FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f))
		.SetDisabled(FSlateNoResource())
		.SetNormalForeground(FStyleColors::Foreground)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::Foreground)
		.SetNormalPadding(ButtonMargins)
		.SetPressedPadding(PressedButtonMargins);

	const FButtonStyle SimpleButtonLabelAndIcon = FButtonStyle(SimpleButton)
		.SetNormalPadding(FMargin(4.f, 1.f, 4.f, 1.f))
		.SetPressedPadding(FMargin(4.f, 2.0f, 4.f, 0.0f));

	FButtonStyle SimpleButtonIconOnly = FButtonStyle(SimpleButton)
		.SetNormalPadding(FMargin(2.f, 1.f, 2.f, 1.f))
		.SetPressedPadding(FMargin(2.f, 2.0f, 2.f, 0.0f));
	{

		const FTextBlockStyle& NormalText = Style->GetWidgetStyle<FTextBlockStyle>("NormalText");
		const FTextBlockStyle ButtonText = FTextBlockStyle(NormalText).SetOverflowPolicy(ETextOverflowPolicy::Ellipsis);

		Style->Set("ButtonText", ButtonText);

		Style->Set("PrimaryButton", PrimaryButton);
		Style->Set("PrimaryButtonLabelAndIconNormalPadding", PrimaryButtonLabelAndIconNormalPadding);
		Style->Set("PrimaryButtonLabelAndIconPressedPadding", PrimaryButtonLabelAndIconPressedPadding);
		Style->Set("Button", Button);
		Style->Set("EditorUtilityButton", EditorUtilityButton);
		Style->Set("SimpleButton", SimpleButtonIconOnly);
		Style->Set("SimpleButtonLabelAndIcon", SimpleButtonLabelAndIcon);

		Style->Set("PrimaryButtonText", FTextBlockStyle(ButtonText).SetFont(FStyleFonts::Get().NormalBold));

		FSlateFontInfo AddFont = FStyleFonts::Get().SmallBold;
		AddFont.LetterSpacing = 200;

		Style->Set("SecondaryButton", Button);

		//@todo Recolor - This is redundant with button text now. Temporarily here for backwards compat
		Style->Set("DialogButtonText", ButtonText);


		//@todo Recolor - Rename to simple button text
		Style->Set("SmallButtonText", FTextBlockStyle(ButtonText));

		// A button that only responds to clicks and displays nothing
		const FButtonStyle InvisibleButton = FButtonStyle()
			.SetNormal(FSlateNoResource())
			.SetHovered(FSlateNoResource())
			.SetPressed(FSlateNoResource())
			.SetDisabled(FSlateNoResource())
			.SetNormalForeground(FLinearColor::White)
			.SetHoveredForeground(FLinearColor::White)
			.SetPressedForeground(FLinearColor::White)
			.SetDisabledForeground(FLinearColor::White)
			.SetNormalPadding(FMargin(0.f))
			.SetPressedPadding(FMargin(0.f));

		Style->Set("InvisibleButton", InvisibleButton);

		SimpleButtonIconOnly
			.SetHovered(FSlateNoResource())
			.SetPressed(FSlateNoResource())
			.SetNormalPadding(FMargin(2.f, 0))
			.SetPressedPadding(FMargin(2.0f, 0.f));

		Style->Set("HoverOnlyButton", SimpleButtonIconOnly);

		SimpleButtonIconOnly
			.SetNormalPadding(0.0f)
			.SetPressedPadding(0.0f);

		// A HoverOnlyButton that has no padding at all
		Style->Set("ThinHoverOnlyButton", SimpleButtonIconOnly);
	}
}

void FStarshipCoreStyle::SetupComboButtonStyles(TSharedRef<FStyle>& Style)
{
	// SComboButton and SComboBox defaults...
	const FButtonStyle ComboButtonButton = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::InputOutline, InputFocusThickness))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::Hover, InputFocusThickness))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::Hover, InputFocusThickness))
		.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, InputFocusRadius, FStyleColors::DropdownOutline, InputFocusThickness))
		.SetNormalForeground(FStyleColors::Foreground)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::ForegroundHover)
		.SetNormalPadding(ComboButtonMargin)
		.SetPressedPadding(PressedComboButtonMargin);

	const FButtonStyle EditorUtilityComboButtonButton = FButtonStyle(ComboButtonButton)
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::InputOutline, InputFocusThickness, Icon32x32))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::Hover, InputFocusThickness, Icon32x32))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::Hover, InputFocusThickness, Icon32x32));

	// SComboBox 
	FComboButtonStyle ComboButton = FComboButtonStyle()
		.SetButtonStyle(ComboButtonButton)
		.SetContentPadding(0.f)
		.SetDownArrowImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/ComboBox/wide-chevron-down", FVector2f(16.f, 16.f)))
		.SetMenuBorderBrush(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 0.0f, FStyleColors::DropdownOutline, 1.0f))
		.SetMenuBorderPadding(0.f);
	Style->Set("ComboButton", ComboButton);

	// Note: This style is currently not referenced by name in UMG, just used for EditorUtilityComboBox
	FComboButtonStyle EditorUtilityComboButton = FComboButtonStyle(ComboButton)
		.SetButtonStyle(EditorUtilityComboButtonButton);
	Style->Set("EditorUtilityComboButton", ComboButton); 

	FComboBoxStyle ComboBox = FComboBoxStyle()
		.SetContentPadding(0.f)
		.SetMenuRowPadding(FMargin(8.0f, 3.f))
		.SetComboButtonStyle(ComboButton);
	Style->Set("ComboBox", ComboBox);

	FComboBoxStyle EditorUtilityComboBox = FComboBoxStyle(ComboBox)
		.SetComboButtonStyle(EditorUtilityComboButton);
	Style->Set("EditorUtilityComboBox", ComboBox);

	const FButtonStyle& SimpleButton = Style->GetWidgetStyle<FButtonStyle>("SimpleButtonLabelAndIcon");

	// Simple Combo Box  (borderless)
	FComboButtonStyle SimpleComboButton = FComboButtonStyle()
		.SetButtonStyle(SimpleButton)
		.SetContentPadding(0.f)
		.SetDownArrowImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/ComboBox/wide-chevron-down", FVector2f(16.f, 16.f)))
		.SetShadowOffset(FVector2f(0.0f, 0.0f))
		.SetMenuBorderBrush(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 0.0f, FStyleColors::DropdownOutline, 1.0f))
		.SetMenuBorderPadding(0.0f);
	Style->Set("SimpleComboButton", SimpleComboButton);

	SimpleComboButton.SetButtonStyle(Style->GetWidgetStyle<FButtonStyle>("SimpleButton"));
	Style->Set("SimpleComboButtonWithIcon", SimpleComboButton);

	FComboBoxStyle SimpleComboBox = FComboBoxStyle()
		.SetContentPadding(0.f)
		.SetMenuRowPadding(FMargin(8.0f, 2.f))
		.SetComboButtonStyle(SimpleComboButton);
	Style->Set("SimpleComboBox", SimpleComboBox);

	const FTableRowStyle ComboBoxRow = FTableRowStyle()
		.SetEvenRowBackgroundBrush(FSlateNoResource())
		.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::Primary))

		.SetOddRowBackgroundBrush(FSlateNoResource())
		.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::Primary))

		.SetSelectorFocusedBrush(FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::Select, InputFocusThickness))

		.SetActiveBrush(FSlateNoResource())
		.SetActiveHoveredBrush(FSlateColorBrush(FStyleColors::Primary))
		.SetActiveHighlightedBrush(FSlateColorBrush(FStyleColors::Primary))

		.SetInactiveBrush(FSlateNoResource())
		.SetInactiveHoveredBrush(FSlateColorBrush(FStyleColors::Primary))
		.SetInactiveHighlightedBrush(FSlateColorBrush(FStyleColors::Primary))

		.SetTextColor(FStyleColors::White)
		.SetSelectedTextColor(FStyleColors::White)
		
		.SetDropIndicator_Above(FSlateNoResource())
		.SetDropIndicator_Onto(FSlateNoResource())
		.SetDropIndicator_Below(FSlateNoResource());

	Style->Set("ComboBox.Row", ComboBoxRow);

	// SEditableComboBox defaults...
	{
		Style->Set("EditableComboBox.Add", new IMAGE_BRUSH("Icons/PlusSymbol_12x", Icon12x12));
		Style->Set("EditableComboBox.Delete", new IMAGE_BRUSH("Icons/Cross_12x", Icon12x12));
		Style->Set("EditableComboBox.Rename", new IMAGE_BRUSH("Icons/ellipsis_12x", Icon12x12));
		Style->Set("EditableComboBox.Accept", new IMAGE_BRUSH("Common/Check", Icon16x16));
	}

	// SComboBox for Icons Only
	// FComboButtonStyle IconComboButton = FComboButtonStyle()
	// Style->Set("IconComboButton")
}

void FStarshipCoreStyle::SetupCheckboxStyles(TSharedRef<FStyle>& Style)
{
	// SCheckBox defaults...
	const float CheckboxCornerRadius = 3.f;
	const float CheckboxOutlineThickness = 1.0f;

	const FCheckBoxStyle BasicCheckBoxStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::CheckBox)

		.SetForegroundColor(FStyleColors::ForegroundHover)
		.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetPressedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedPressedForegroundColor(FStyleColors::ForegroundHover)
		.SetUndeterminedForegroundColor(FStyleColors::ForegroundHover)

		.SetUncheckedImage(FSlateNoResource())
		.SetUncheckedHoveredImage(FSlateNoResource())
		.SetUncheckedPressedImage(FSlateNoResource())
		.SetCheckedImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/check", Icon16x16, FStyleColors::Primary))
		.SetCheckedHoveredImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/check", Icon16x16, FStyleColors::PrimaryHover))
		.SetCheckedPressedImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/check", Icon16x16, FStyleColors::PrimaryPress))
		.SetUndeterminedImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/indeterminate", Icon16x16, FStyleColors::Primary))
		.SetUndeterminedHoveredImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/indeterminate", Icon16x16, FStyleColors::PrimaryHover))
		.SetUndeterminedPressedImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/indeterminate", Icon16x16, FStyleColors::PrimaryPress))
		.SetBackgroundImage(FSlateRoundedBoxBrush(FStyleColors::Input, CheckboxCornerRadius, FStyleColors::Secondary, CheckboxOutlineThickness, Icon18x18))
		.SetBackgroundHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Input, CheckboxCornerRadius, FStyleColors::Hover, CheckboxOutlineThickness, Icon18x18))
		.SetBackgroundPressedImage(FSlateRoundedBoxBrush(FStyleColors::Input, CheckboxCornerRadius, FStyleColors::Hover, CheckboxOutlineThickness, Icon18x18));

	Style->Set("Checkbox", BasicCheckBoxStyle);

	/* Set images for various transparent SCheckBox states ... */
	const FCheckBoxStyle BasicTransparentCheckBoxStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage(FSlateNoResource())
		.SetUncheckedHoveredImage(FSlateNoResource())
		.SetUncheckedPressedImage(FSlateNoResource())
		.SetCheckedImage(FSlateNoResource())
		.SetCheckedHoveredImage(FSlateNoResource())
		.SetCheckedPressedImage(FSlateNoResource())
		.SetUndeterminedImage(FSlateNoResource())
		.SetUndeterminedHoveredImage(FSlateNoResource())
		.SetUndeterminedPressedImage(FSlateNoResource())
		.SetForegroundColor(FStyleColors::Foreground)
		.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetPressedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedPressedForegroundColor(FStyleColors::ForegroundHover);

	Style->Set("TransparentCheckBox", BasicTransparentCheckBoxStyle);

	/* Default Style for a toggleable button */
	const FCheckBoxStyle ToggleButtonStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f))
		.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f))
		.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f))
		.SetUncheckedImage(FSlateNoResource())
		.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f))
		.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.0f))
		.SetForegroundColor(FStyleColors::Foreground)
		.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetPressedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedPressedForegroundColor(FStyleColors::ForegroundHover)
		.SetPadding(ToggleButtonMargins);

	Style->Set("ToggleButtonCheckbox", ToggleButtonStyle);

	/* Default Style for a toggleable button */
	const FCheckBoxStyle ToggleButtonAltStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius))
		.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, InputFocusRadius))
		.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius))
		.SetUncheckedImage(FSlateNoResource())
		.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, InputFocusRadius))
		.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius))
		.SetForegroundColor(FStyleColors::Foreground)
		.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetPressedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedForegroundColor(FStyleColors::White)
		.SetCheckedHoveredForegroundColor(FStyleColors::White)
		.SetPadding(ToggleButtonMargins);

	//@todo Recolor - Get rid of this. ToggleButtonCheckbox is what we should use
	Style->Set("ToggleButtonCheckboxAlt", ToggleButtonAltStyle);

	FMargin SegmentedControlMargins(8, 3);
	/* Style for a segmented box */
	FCheckBoxStyle SegmentedBoxLeft = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage(FSlateNoResource())
		.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, InputFocusRadius))
		.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Header, InputFocusRadius))
		.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Dropdown, InputFocusRadius))
		.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, InputFocusRadius))
		.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Header, InputFocusRadius))
		.SetForegroundColor(FStyleColors::Foreground)
		.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetPressedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedPressedForegroundColor(FStyleColors::ForegroundHover)
		.SetPadding(SegmentedControlMargins);

	Style->Set("SegmentedControl", FSegmentedControlStyle()
		.SetControlStyle(SegmentedBoxLeft)
		.SetFirstControlStyle(SegmentedBoxLeft)
		.SetLastControlStyle(SegmentedBoxLeft)
		.SetBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius))
		.SetUniformPadding(FMargin(1))
	);


	SegmentedBoxLeft
		.SetUncheckedImage(BOX_BRUSH("/Starship/CoreWidgets/SegmentedBox/left", FVector2f(16.f, 16.f), FMargin(4.0f / 16.0f), FStyleColors::Dropdown))
		.SetUncheckedHoveredImage(BOX_BRUSH("/Starship/CoreWidgets/SegmentedBox/left", FVector2f(16.f, 16.f), FMargin(4.0f / 16.0f), FStyleColors::Hover))
		.SetUncheckedPressedImage(BOX_BRUSH("/Starship/CoreWidgets/SegmentedBox/left", FVector2f(16.f, 16.f), FMargin(4.0f / 16.0f), FStyleColors::Secondary))
		.SetCheckedImage(BOX_BRUSH("/Starship/CoreWidgets/SegmentedBox/left", FVector2f(16.f, 16.f), FMargin(4.0f / 16.0f), FStyleColors::Primary))
		.SetCheckedHoveredImage(BOX_BRUSH("/Starship/CoreWidgets/SegmentedBox/left", FVector2f(16.f, 16.f), FMargin(4.0f / 16.0f), FStyleColors::PrimaryHover))
		.SetCheckedPressedImage(BOX_BRUSH("/Starship/CoreWidgets/SegmentedBox/left", FVector2f(16.f, 16.f), FMargin(4.0f / 16.0f), FStyleColors::PrimaryPress));

	Style->Set("SegmentedCombo.Left", SegmentedBoxLeft);

	const FButtonStyle SegmentedComboRightButton = FButtonStyle()
	.SetNormal(BOX_BRUSH("/Starship/CoreWidgets/SegmentedBox/right", FVector2f(16.f, 16.f), FMargin(4.0f / 16.0f), FStyleColors::Secondary))
	.SetHovered(BOX_BRUSH("/Starship/CoreWidgets/SegmentedBox/right", FVector2f(16.f, 16.f), FMargin(4.0f / 16.0f), FStyleColors::Hover))
	.SetPressed(BOX_BRUSH("/Starship/CoreWidgets/SegmentedBox/right", FVector2f(16.f, 16.f), FMargin(4.0f / 16.0f), FStyleColors::Secondary))
	.SetNormalForeground(FStyleColors::Foreground)
	.SetHoveredForeground(FStyleColors::ForegroundHover)
	.SetPressedForeground(FStyleColors::Primary)
	.SetDisabledForeground(FStyleColors::Foreground)
	.SetNormalPadding(0)
	.SetPressedPadding(0);

	FComboButtonStyle SegmentedComboRight = FComboButtonStyle(Style->GetWidgetStyle<FComboButtonStyle>("ComboButton"))
		.SetContentPadding(FMargin(6.f, 2.f))
		.SetButtonStyle(SegmentedComboRightButton)
		.SetDownArrowImage(IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(6.f, 24.f)));


	Style->Set("SegmentedCombo.Right", SegmentedComboRight);

	// Button that looks/behaves like a SegmentedCombo for cases where SegmentedCombos and 
	// buttons are placed adjacent
	const FCheckBoxStyle SegmentedBoxButtonOnly = FCheckBoxStyle(ToggleButtonStyle)
		.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.f))
		.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.f))
		.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Header, 4.f));

	Style->Set("SegmentedCombo.ButtonOnly", SegmentedBoxButtonOnly);


	/* A radio button is actually just a SCheckBox box with different images */
	/* Set images for various radio button (SCheckBox) states ... */
	const FCheckBoxStyle BasicRadioButtonStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::CheckBox)

		.SetForegroundColor(FLinearColor::White)
		.SetHoveredForegroundColor(FLinearColor::White)
		.SetPressedForegroundColor(FLinearColor::White)
		.SetCheckedForegroundColor(FLinearColor::White)
		.SetCheckedHoveredForegroundColor(FLinearColor::White)
		.SetCheckedPressedForegroundColor(FLinearColor::White)
		.SetUndeterminedForegroundColor(FLinearColor::White)

		.SetUncheckedImage(IMAGE_BRUSH_SVG("/Starship/CoreWidgets/CheckBox/radio-off", Icon16x16, FStyleColors::Dropdown))
		.SetUncheckedHoveredImage(IMAGE_BRUSH_SVG("/Starship/CoreWidgets/CheckBox/radio-off", Icon16x16, FStyleColors::Hover))
		.SetUncheckedPressedImage(IMAGE_BRUSH_SVG("/Starship/CoreWidgets/CheckBox/radio-off", Icon16x16, FStyleColors::Dropdown))
		.SetCheckedImage(IMAGE_BRUSH_SVG("/Starship/CoreWidgets/CheckBox/radio-on", Icon16x16, FStyleColors::Primary))
		.SetCheckedHoveredImage(IMAGE_BRUSH_SVG("/Starship/CoreWidgets/CheckBox/radio-on", Icon16x16, FStyleColors::PrimaryHover))
		.SetCheckedPressedImage(IMAGE_BRUSH_SVG("/Starship/CoreWidgets/CheckBox/radio-on", Icon16x16, FStyleColors::Primary))
		.SetUndeterminedImage(IMAGE_BRUSH_SVG("/Starship/CoreWidgets/CheckBox/radio-off", Icon16x16, FStyleColors::White25))
		.SetUndeterminedHoveredImage(IMAGE_BRUSH_SVG("/Starship/CoreWidgets/CheckBox/radio-off", Icon16x16, FStyleColors::ForegroundHover))
		.SetUndeterminedPressedImage(IMAGE_BRUSH_SVG("/Starship/CoreWidgets/CheckBox/radio-off", Icon16x16, FStyleColors::ForegroundHover))
		.SetPadding(FMargin(4.0f));
	Style->Set("RadioButton", BasicRadioButtonStyle);
}

void FStarshipCoreStyle::SetupDockingStyles(TSharedRef<FStyle>& Style)
{
	const FButtonStyle& Button = Style->GetWidgetStyle<FButtonStyle>("Button");
	const FButtonStyle& NoBorder = Style->GetWidgetStyle<FButtonStyle>("NoBorder");
	const FTextBlockStyle& NormalText = Style->GetWidgetStyle<FTextBlockStyle>("NormalText");
	const FSlateColor SelectionColor = Style->GetSlateColor("SelectionColor");

	// SDockTab, SDockingTarget, SDockingTabStack defaults...
	Style->Set("Docking.Background", new BOX_BRUSH("Old/Menu_Background", FMargin(8.0f / 64.0f)));
	Style->Set("Docking.Border", new FSlateRoundedBoxBrush(FStyleColors::Panel, 4.f));

	Style->Set("Docking.UnhideTabwellButton", FButtonStyle(Button)
		.SetNormal(IMAGE_BRUSH_SVG("Starship/Docking/show-tab-well", Icon8x8, FStyleColors::Primary))
		.SetPressed(IMAGE_BRUSH_SVG("Starship/Docking/show-tab-well", Icon8x8, FStyleColors::PrimaryPress))
		.SetHovered(IMAGE_BRUSH_SVG("Starship/Docking/show-tab-well", Icon8x8, FStyleColors::PrimaryHover))
		.SetNormalPadding(0)
		.SetPressedPadding(0)
	);

	// Flash using the selection color for consistency with the rest of the UI scheme
	const FSlateColor& TabFlashColor = SelectionColor;

	const FButtonStyle CloseButton = FButtonStyle()
		.SetNormal(IMAGE_BRUSH_SVG("Starship/Common/close-small", Icon16x16, FStyleColors::Foreground))
		.SetPressed(IMAGE_BRUSH_SVG("Starship/Common/close-small", Icon16x16, FStyleColors::Foreground))
		.SetHovered(IMAGE_BRUSH_SVG("Starship/Common/close-small", Icon16x16, FStyleColors::ForegroundHover));


	FLinearColor DockColor_Inactive(FColor(45, 45, 45));
	FLinearColor DockColor_Hovered(FColor(54, 54, 54));
	FLinearColor DockColor_Active(FColor(62, 62, 62));

	FDockTabStyle MinorTabStyle =
		FDockTabStyle()
		.SetCloseButtonStyle(CloseButton)
		.SetNormalBrush(FSlateNoResource())
		.SetHoveredBrush(BOX_BRUSH("/Starship/Docking/DockTab_Foreground", 4.0f / 20.0f, FStyleColors::Panel.GetSpecifiedColor().CopyWithNewOpacity(0.8f))) // Panel + 20% Black
		.SetForegroundBrush(BOX_BRUSH("/Starship/Docking/DockTab_Foreground", 4.0f / 20.0f, FStyleColors::Panel))
		.SetColorOverlayTabBrush(FSlateColorBrush(FStyleColors::Foreground))
		.SetColorOverlayIconBrush(FSlateNoResource())
		.SetContentAreaBrush(FSlateColorBrush(FStyleColors::Panel))
		.SetTabWellBrush(FSlateColorBrush(FStyleColors::Background))
		.SetFlashColor(TabFlashColor)

		.SetTabPadding(FMargin(4.f, 3.f, 10.f, 4.f))
		.SetOverlapWidth(-2.0f) // Negative Overlap Width to add space between tabs

		.SetNormalForegroundColor(FStyleColors::Foreground)
		.SetActiveForegroundColor(FStyleColors::ForegroundHover)
		.SetForegroundForegroundColor(FStyleColors::Foreground)
		.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetTabTextStyle(NormalText);


	// Panel Tab
	Style->Set("Docking.Tab", MinorTabStyle);

	// App Tab
	Style->Set("Docking.MajorTab", FDockTabStyle()
		.SetCloseButtonStyle(CloseButton)
		.SetNormalBrush(FSlateNoResource())
		.SetHoveredBrush(BOX_BRUSH("/Starship/Docking/DockTab_Foreground", 4.0f / 20.0f, FStyleColors::Panel.GetSpecifiedColor().CopyWithNewOpacity(0.8f))) // Panel + 20% Black
		.SetForegroundBrush(BOX_BRUSH("/Starship/Docking/DockTab_Foreground", 4.0f / 20.0f, FStyleColors::Panel))

		.SetColorOverlayTabBrush(FSlateColorBrush(FStyleColors::Foreground))
		.SetColorOverlayIconBrush(FSlateRoundedBoxBrush(FStyleColors::Foreground, 4.f))
		.SetContentAreaBrush(FSlateColorBrush(FStyleColors::Background))
		.SetTabWellBrush(FSlateColorBrush(FStyleColors::Background))

		.SetTabPadding(FMargin(4.f, 4.f, 10.f, 4.f))
		.SetOverlapWidth(-2.0f) // Negative Overlap Width to add space between tabs
		.SetFlashColor(TabFlashColor)

		.SetNormalForegroundColor(FStyleColors::Foreground)
		.SetActiveForegroundColor(FStyleColors::White)
		.SetForegroundForegroundColor(FStyleColors::White)
		.SetHoveredForegroundColor(FStyleColors::White)
		.SetTabTextStyle(NormalText)
		.SetIconBorderPadding(4.f)
	);

	Style->Set("Docking.Tab.ContentAreaBrush", new FSlateNoResource());

	Style->Set("Docking.Tab.InactiveTabSeparator", new FSlateColorBrush(FStyleColors::Hover));

	Style->Set("Docking.Tab.ActiveTabIndicatorColor", FStyleColors::Primary);


	FButtonStyle SidebarTabButtonOpened =
		FButtonStyle(NoBorder)
		.SetNormal(FSlateColorBrush(FStyleColors::Panel))
		.SetHovered(FSlateColorBrush(FStyleColors::Panel))
		.SetNormalForeground(MinorTabStyle.NormalForegroundColor)
		.SetPressedForeground(MinorTabStyle.NormalForegroundColor)
		.SetHoveredForeground(MinorTabStyle.HoveredForegroundColor);


	FButtonStyle SidebarTabButtonClosed =
		FButtonStyle(NoBorder)
		.SetNormal(FSlateNoResource())
		.SetHovered(FSlateColorBrush(FStyleColors::Panel))
		.SetNormalForeground(MinorTabStyle.NormalForegroundColor)
		.SetPressedForeground(MinorTabStyle.NormalForegroundColor)
		.SetHoveredForeground(MinorTabStyle.HoveredForegroundColor);
			
	Style->Set("Docking.SidebarButton.Closed", SidebarTabButtonClosed);

	Style->Set("Docking.SidebarButton.Opened", SidebarTabButtonOpened);

	Style->Set("Docking.Sidebar.DrawerShadow", new BOX_BRUSH("/Starship/Docking/drawer-shadow", FMargin(8.f /64.f), FLinearColor(0.f, 0.f, 0.f, 1.f)));
	Style->Set("Docking.Sidebar.DrawerBackground", new FSlateColorBrush(FStyleColors::Panel));
	Style->Set("Docking.Sidebar.Background", new FSlateColorBrush(FStyleColors::Background));

	const FButtonStyle StatusBarButton = FButtonStyle(NoBorder)
		.SetNormalForeground(FStyleColors::Foreground)
		.SetNormalPadding(FMargin(2, 2, 2, 2))
		.SetPressedPadding(FMargin(2, 3, 2, 1));

	const FComboButtonStyle StatusBarComboButton = FComboButtonStyle(Style->GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"));
	
	const FComboButtonStyle StatusBarEllipsisComboButton = FComboButtonStyle(Style->GetWidgetStyle<FComboButtonStyle>("ComboButton"))
		.SetDownArrowImage(IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(6.f, 24.f)))
		.SetButtonStyle(StatusBarButton)
		.SetDownArrowPadding(FMargin(0.0f))
		.SetDownArrowAlignment(EVerticalAlignment::VAlign_Bottom);

	Style->Set("StatusBar.StatusBarButton", StatusBarButton);
	Style->Set("StatusBar.StatusBarComboButton", StatusBarComboButton);
	Style->Set("StatusBar.StatusBarEllipsisComboButton", StatusBarEllipsisComboButton);
	Style->Set("StatusBar.Height", 32.0f);
	Style->Set("StatusBar.DrawerShadow", new BOX_BRUSH("Starship/StatusBar/drawer-shadow-bottom", FMargin(10 / 64.0f, 20 / 64.f, 10 / 64.f, 0), FLinearColor(0, 0, 0, 1)));
	Style->Set("StatusBar.DrawerBackground", new FSlateRoundedBoxBrush(FStyleColors::Panel, 5.0f));

	Style->Set("StatusBar.Background", new FSlateColorBrush(FStyleColors::Panel));
	Style->Set("StatusBar.ProgressOverlay", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 2.0f, FStyleColors::Panel, 1.0f, FVector2f(20.f, 8.f)));

	Style->Set("StatusBar.HelpIcon", new IMAGE_BRUSH_SVG("Starship/Common/help", Icon16x16, FStyleColors::Foreground));

	FToolBarStyle StatusBarToolBarStyle = Style->GetWidgetStyle<FToolBarStyle>("SlimToolBar");

	StatusBarToolBarStyle.SetBackground(FSlateNoResource());
	StatusBarToolBarStyle.SetLabelPadding(FMargin(5, 5, 0, 5));
	StatusBarToolBarStyle.SetIconSize(Icon16x16);
	StatusBarToolBarStyle.SetBackgroundPadding(FMargin(4.f, 4.f));

	Style->Set("StatusBarToolBar", StatusBarToolBarStyle);

	Style->Set("StatusBar.Message.MessageText",
		FTextBlockStyle(Style->GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.SetColorAndOpacity(FStyleColors::Foreground));

	Style->Set("StatusBar.Message.HintText",
		FTextBlockStyle(Style->GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.SetColorAndOpacity(FStyleColors::Primary));
	// Create the regular border brush with all corners rounded and variants with certain corners squared.
	{
		const float CornerRadius = 5.0f;
		const float BorderWidth = 1.0f;
		Style->Set("Docking.Sidebar.Border", new FSlateRoundedBoxBrush(FStyleColors::Transparent, CornerRadius, FStyleColors::Hover, BorderWidth));
		Style->Set("Docking.Sidebar.Border_SquareLeft", new FSlateRoundedBoxBrush(FStyleColors::Transparent, FVector4(0.0f, CornerRadius, CornerRadius, 0.0f), FStyleColors::Hover, BorderWidth));
		Style->Set("Docking.Sidebar.Border_SquareRight", new FSlateRoundedBoxBrush(FStyleColors::Transparent, FVector4(CornerRadius, 0.0f, 0.0f, CornerRadius), FStyleColors::Hover, BorderWidth));
	}

	Style->Set("Docking.Sidebar.PinTabIcon", new IMAGE_BRUSH_SVG("Starship/Docking/pin", Icon12x12));

	// Dock Cross
	Style->Set("Docking.Cross.DockLeft", new IMAGE_BRUSH("/Docking/OuterDockingIndicator", FVector2f(6.f, 6.f), FLinearColor(1.0f, 0.35f, 0.0f, 0.25f)));
	Style->Set("Docking.Cross.DockLeft_Hovered", new IMAGE_BRUSH("/Docking/OuterDockingIndicator", FVector2f(6.f, 6.f), FLinearColor(1.0f, 0.35f, 0.0f)));
	Style->Set("Docking.Cross.DockTop", new IMAGE_BRUSH("/Docking/OuterDockingIndicator", FVector2f(6.f, 6.f), FLinearColor(1.0f, 0.35f, 0.0f, 0.25f)));
	Style->Set("Docking.Cross.DockTop_Hovered", new IMAGE_BRUSH("/Docking/OuterDockingIndicator", FVector2f(6.f, 6.f), FLinearColor(1.0f, 0.35f, 0.0f)));
	Style->Set("Docking.Cross.DockRight", new IMAGE_BRUSH("/Docking/OuterDockingIndicator", FVector2f(6.f, 6.f), FLinearColor(1.0f, 0.35f, 0.0f, 0.25f)));
	Style->Set("Docking.Cross.DockRight_Hovered", new IMAGE_BRUSH("/Docking/OuterDockingIndicator", FVector2f(6.f, 6.f), FLinearColor(1.0f, 0.35f, 0.0f)));
	Style->Set("Docking.Cross.DockBottom", new IMAGE_BRUSH("/Docking/OuterDockingIndicator", FVector2f(6.f, 6.f), FLinearColor(1.0f, 0.35f, 0.0f, 0.25f)));
	Style->Set("Docking.Cross.DockBottom_Hovered", new IMAGE_BRUSH("/Docking/OuterDockingIndicator", FVector2f(6.f, 6.f), FLinearColor(1.0f, 0.35f, 0.0f)));
	Style->Set("Docking.Cross.DockCenter", new IMAGE_BRUSH("/Docking/DockingIndicator_Center", Icon64x64, FLinearColor(1.0f, 0.35f, 0.0f, 0.25f)));
	Style->Set("Docking.Cross.DockCenter_Hovered", new IMAGE_BRUSH("/Docking/DockingIndicator_Center", Icon64x64, FLinearColor(1.0f, 0.35f, 0.0f)));

	Style->Set("Docking.Cross.BorderLeft", new FSlateNoResource());
	Style->Set("Docking.Cross.BorderTop", new FSlateNoResource());
	Style->Set("Docking.Cross.BorderRight", new FSlateNoResource());
	Style->Set("Docking.Cross.BorderBottom", new FSlateNoResource());
	Style->Set("Docking.Cross.BorderCenter", new FSlateNoResource());

	Style->Set("Docking.Cross.PreviewWindowTint", FLinearColor(1.0f, 0.75f, 0.5f));
	Style->Set("Docking.Cross.Tint", FLinearColor::White);
	Style->Set("Docking.Cross.HoveredTint", FLinearColor::White);


	const FButtonStyle OpenTabInWindowButton = FButtonStyle()
		.SetNormal(IMAGE_BRUSH_SVG("Starship/Docking/OpenTabInWindow", Icon12x12, FStyleColors::Foreground))
		.SetPressed(IMAGE_BRUSH_SVG("Starship/Docking/OpenTabInWindow", Icon12x12, FStyleColors::Foreground))
		.SetHovered(IMAGE_BRUSH_SVG("Starship/Docking/OpenTabInWindow", Icon12x12, FStyleColors::ForegroundHover));

	Style->Set("Docking.OpenTabInWindow.Button", OpenTabInWindowButton);
}

void FStarshipCoreStyle::SetupColorPickerStyles(TSharedRef<FStyle>& Style)
{
	// SColorPicker defaults...
	{
		Style->Set("ColorPicker.RoundedSolidBackground", new FSlateRoundedBoxBrush(FStyleColors::White, InputFocusRadius));
		Style->Set("ColorPicker.RoundedAlphaBackground", new FSlateRoundedBoxBrush(FName(*RootToContentDir("Starship/Common/Checker", TEXT(".png"))), FLinearColor::Transparent, InputFocusRadius, FLinearColor::Transparent, 0.0f, Icon16x16, ESlateBrushTileType::Both));
		Style->Set("ColorPicker.RoundedInputBorder", new FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::InputOutline, InputFocusThickness));
		Style->Set("ColorPicker.RoundedInputBorderActive", new FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::Primary, InputFocusThickness));
		Style->Set("ColorPicker.RoundedInputBorderHovered", new FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::Hover, InputFocusThickness));
		Style->Set("ColorPicker.MultipleValuesBackground", new FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius));
		Style->Set("ColorPicker.RecessedBackground", new FSlateRoundedBoxBrush(FStyleColors::Recessed, InputFocusRadius * 2.0f));

		Style->Set("ColorPicker.Font", FStyleFonts::Get().Normal);
		Style->Set("ColorPicker.SmallFont", FStyleFonts::Get().Small);

		Style->Set("ColorPicker.AlphaBackground", new IMAGE_BRUSH("Starship/Common/Checker", Icon16x16, FLinearColor::White, ESlateBrushTileType::Both));
		Style->Set("ColorPicker.EyeDropper", new IMAGE_BRUSH("Icons/eyedropper_16px", Icon16x16));
		Style->Set("ColorPicker.EyeDropperLarge", new IMAGE_BRUSH_SVG("Starship/Common/EyeDropper_20", Icon20x20));
		Style->Set("ColorPicker.Mode", new IMAGE_BRUSH("Common/ColorPicker_Mode_16x", Icon16x16));
		Style->Set("ColorPicker.ModeWheel", new IMAGE_BRUSH("Common/ColorWheel_20x", Icon20x20));
		Style->Set("ColorPicker.ModeSpectrum", new IMAGE_BRUSH("Common/ColorSpectrum_20x", Icon20x20));
		Style->Set("ColorPicker.Separator", new IMAGE_BRUSH("Common/ColorPicker_Separator", FVector2f(2.0f, 2.0f)));
		Style->Set("ColorPicker.Selector", new IMAGE_BRUSH("Common/Circle", Icon8x8));
		Style->Set("ColorPicker.SpinBoxSelectorVertical", new IMAGE_BRUSH("Starship/Common/color-grading-spinbox-selector", FVector2f(3.0f, 1.0f)));
		Style->Set("ColorPicker.SpinBoxSelectorHorizontal", new IMAGE_BRUSH("Starship/Common/color-grading-spinbox-selector-v", FVector2f(1.0f, 3.0f)));

		Style->Set("ColorPicker.Slider", FSliderStyle()
			.SetDisabledThumbImage(IMAGE_BRUSH("Common/ColorPicker_SliderHandle", FVector2f(8.0f, 32.0f)))
			.SetNormalThumbImage(IMAGE_BRUSH("Common/ColorPicker_SliderHandle", FVector2f(8.0f, 32.0f)))
			.SetHoveredThumbImage(IMAGE_BRUSH("Common/ColorPicker_SliderHandle", FVector2f(8.0f, 32.0f)))
		);

		FButtonStyle ComboModeButtonStyle = FButtonStyle()
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Panel, InputFocusRadius, FStyleColors::Panel, InputFocusThickness))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Panel, InputFocusRadius, FStyleColors::Panel, InputFocusThickness))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Panel, InputFocusRadius, FStyleColors::Panel, InputFocusThickness));

		FComboButtonStyle HexModeComboButton = Style->GetWidgetStyle<FComboButtonStyle>("ComboButton");
		HexModeComboButton.SetContentPadding(FMargin(0.0f));
		HexModeComboButton.SetButtonStyle(ComboModeButtonStyle);

		Style->Set("ColorPicker.HexMode", HexModeComboButton);
	}

	// SColorSlider defaults...
	{
		FSpinBoxStyle ColorSliderSpinBox = Style->GetWidgetStyle<FSpinBoxStyle>("SpinBox");
		ColorSliderSpinBox.SetActiveFillBrush(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FLinearColor::Transparent, InputFocusThickness));
		ColorSliderSpinBox.SetHoveredFillBrush(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FLinearColor::Transparent, InputFocusThickness));
		ColorSliderSpinBox.SetInactiveFillBrush(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FLinearColor::Transparent, InputFocusThickness));

		Style->Set("ColorSlider.SpinBox", ColorSliderSpinBox);
	}

	// SColorSpectrum defaults...
	{
		Style->Set("ColorSpectrum.Spectrum", new IMAGE_BRUSH("Common/ColorSpectrum", FVector2f(256.f, 256.f)));
		Style->Set("ColorSpectrum.Selector", new IMAGE_BRUSH("Common/Circle", FVector2f(8.f, 8.f)));
	}

	// SColorThemes defaults...
	{
		Style->Set("ColorPicker.ColorThemes", new IMAGE_BRUSH_SVG("Starship/Common/ColorThemes_20", Icon20x20));
		Style->Set("ColorPicker.ColorThemesSmall", new IMAGE_BRUSH_SVG("Starship/Common/ColorThemes_16", Icon16x16));
		Style->Set("ColorPicker.ColorThemesOff", new IMAGE_BRUSH_SVG("Starship/Common/ColorThemesOff_20", Icon20x20));

		FWindowStyle CursorDecorator = FWindowStyle();
		CursorDecorator.SetCornerRadius(4);
		Style->Set("ColorPicker.CursorDecorator", CursorDecorator);

		FComboButtonStyle ThemesComboButton = Style->GetWidgetStyle<FComboButtonStyle>("ComboButton");
		ThemesComboButton.SetDownArrowPadding(FMargin(0.0f));
		ThemesComboButton.ButtonStyle
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Dropdown, InputFocusRadius, FStyleColors::Input, InputFocusThickness))
			.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, InputFocusRadius, FStyleColors::Input, InputFocusThickness))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Dropdown, InputFocusRadius, FStyleColors::Input, InputFocusThickness))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Dropdown, InputFocusRadius, FStyleColors::Input, InputFocusThickness));

		Style->Set("ColorPicker.ThemesComboButton", ThemesComboButton);

		FButtonStyle AddButton = Style->GetWidgetStyle<FButtonStyle>("Button");
		AddButton.SetNormalPadding(0.0);
		AddButton.SetPressedPadding(0.0);
		AddButton.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::DropdownOutline, InputFocusThickness));
		AddButton.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::DropdownOutline, InputFocusThickness));
		AddButton.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::Hover2, InputFocusThickness));
		AddButton.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::Hover2, InputFocusThickness));

		Style->Set("ColorPicker.AddButton", AddButton);

		FButtonStyle DeleteButton = AddButton;
		DeleteButton.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::Transparent, InputFocusThickness));
		DeleteButton.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::DropdownOutline, InputFocusThickness));

		Style->Set("ColorPicker.DeleteButton", DeleteButton);
	}

	// SColorWheel defaults...
	{
		Style->Set("ColorWheel.HueValueCircle", new IMAGE_BRUSH("Common/ColorWheel", FVector2f(192.f, 192.f)));
		Style->Set("ColorWheel.Selector", new IMAGE_BRUSH("Common/Circle", FVector2f(8.f, 8.f)));
	}

	// SColorGradingWheel defaults...
	{
		Style->Set("ColorGradingWheel.HueValueCircle", new IMAGE_BRUSH("Common/ColorGradingWheel", FVector2f(192.f, 192.f)));
		Style->Set("ColorGradingWheel.Selector", new IMAGE_BRUSH("Common/Circle", FVector2f(8.f, 8.f)));
	}

	// Color Preview Button defaults...
	{
		FButtonStyle ColorPreviewButton = Style->GetWidgetStyle<FButtonStyle>("Button");
		ColorPreviewButton.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::Transparent, InputFocusThickness));
		ColorPreviewButton.SetHovered(FSlateRoundedBoxBrush(FLinearColor(0.0, 0.0, 0.0, 0.4f), InputFocusRadius, FStyleColors::Transparent, InputFocusThickness));
		ColorPreviewButton.SetPressed(FSlateRoundedBoxBrush(FLinearColor(0.0, 0.0, 0.0, 0.4f), InputFocusRadius, FStyleColors::Transparent, InputFocusThickness));
		ColorPreviewButton.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Transparent, InputFocusRadius, FStyleColors::Transparent, InputFocusThickness));

		Style->Set("ColorPicker.ColorPreviewButton", ColorPreviewButton);
	}
}

void FStarshipCoreStyle::SetupTableViewStyles(TSharedRef<FStyle>& Style)
{
	const FSlateColor SelectionColor = Style->GetSlateColor("SelectionColor");
	const FSlateColor SelectorColor = Style->GetSlateColor("SelectorColor");

	const FTableViewStyle DefaultTreeViewStyle = FTableViewStyle();
	Style->Set("ListView", DefaultTreeViewStyle);

	Style->Set("SimpleListView", FTableViewStyle().SetBackgroundBrush(FSlateNoResource()));

	const FTableViewStyle DefaultTableViewStyle = FTableViewStyle();
	Style->Set("TreeView", DefaultTableViewStyle);

	const FTableRowStyle DefaultTableRowStyle = FTableRowStyle()
		.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed))
		.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::SelectHover))

		.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed))
		.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::SelectHover))

		.SetSelectorFocusedBrush(BORDER_BRUSH("Common/Selector", FMargin(4.f / 16.f), SelectorColor))

		.SetActiveBrush(FSlateColorBrush(FStyleColors::Select))
		.SetActiveHoveredBrush(FSlateColorBrush(FStyleColors::Select))

		.SetInactiveBrush(FSlateColorBrush(FStyleColors::SelectInactive))
		.SetInactiveHoveredBrush(FSlateColorBrush(FStyleColors::SelectInactive))

		.SetActiveHighlightedBrush(FSlateColorBrush(FStyleColors::SelectParent)) // This is the parent hightlight
		.SetInactiveHighlightedBrush(FSlateColorBrush(FStyleColors::SelectParent))// This is the parent highlight

		.SetTextColor(FStyleColors::Foreground)
		.SetSelectedTextColor(FStyleColors::ForegroundHover)

		.SetDropIndicator_Above(BOX_BRUSH("Common/DropZoneIndicator_Above", FMargin(10.0f / 16.0f, 10.0f / 16.0f, 0.f, 0.f), SelectionColor))
		.SetDropIndicator_Onto(BOX_BRUSH("Common/DropZoneIndicator_Onto", FMargin(4.0f / 16.0f), SelectionColor))
		.SetDropIndicator_Below(BOX_BRUSH("Common/DropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0.f, 0.f, 10.0f / 16.0f), SelectionColor));

	Style->Set("TableView.Row", DefaultTableRowStyle);

	Style->Set("ListView.PinnedItemShadow", new IMAGE_BRUSH("Starship/CoreWidgets/TableView/PinnedItemShadow", FVector2f(16.f, 8.f)));


	const FTableRowStyle SimpleTableRowStyle = FTableRowStyle()
		.SetEvenRowBackgroundBrush(FSlateNoResource())
		.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::SelectHover))

		.SetOddRowBackgroundBrush(FSlateNoResource())
		.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::SelectHover))

		.SetSelectorFocusedBrush(BORDER_BRUSH("Common/Selector", FMargin(4.f / 16.f), SelectorColor))

		.SetActiveBrush(FSlateColorBrush(FStyleColors::Select))
		.SetActiveHoveredBrush(FSlateColorBrush(FStyleColors::Select))

		.SetInactiveBrush(FSlateColorBrush(FStyleColors::SelectInactive))
		.SetInactiveHoveredBrush(FSlateColorBrush(FStyleColors::SelectInactive))

		.SetActiveHighlightedBrush(FSlateColorBrush(FStyleColors::SelectParent)) // This is the parent hightlight
		.SetInactiveHighlightedBrush(FSlateColorBrush(FStyleColors::SelectParent))// This is the parent highlight

		.SetTextColor(FStyleColors::Foreground)
		.SetSelectedTextColor(FStyleColors::ForegroundInverted)

		.SetDropIndicator_Above(BOX_BRUSH("Common/DropZoneIndicator_Above", FMargin(10.0f / 16.0f, 10.0f / 16.0f, 0, 0), SelectionColor))
		.SetDropIndicator_Onto(BOX_BRUSH("Common/DropZoneIndicator_Onto", FMargin(4.0f / 16.0f), SelectionColor))
		.SetDropIndicator_Below(BOX_BRUSH("Common/DropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0, 0, 10.0f / 16.0f), SelectionColor));

	Style->Set("SimpleTableView.Row", SimpleTableRowStyle);

	const FTableRowStyle ShowParentsTableRowStyle = FTableRowStyle(DefaultTableRowStyle)
		.SetUseParentRowBrush(true)
		.SetParentRowBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Header, 2.f, FStyleColors::Transparent, 1.f))
		.SetParentRowBackgroundHoveredBrush(FSlateRoundedBoxBrush(FStyleColors::SelectHover, 2.f));

	Style->Set("ShowParentsTableView.Row", ShowParentsTableRowStyle);

	const FTableRowStyle DarkTableRowStyle = FTableRowStyle(DefaultTableRowStyle)
		.SetEvenRowBackgroundBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(0.0f, 0.0f, 0.0f, 0.1f)))
		.SetOddRowBackgroundBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(0.0f, 0.0f, 0.0f, 0.1f)));
	Style->Set("TableView.DarkRow", DarkTableRowStyle);

	const FTableRowStyle AlternatingTableRowStyle = FTableRowStyle(DefaultTableRowStyle)
		.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed))
		.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Background));
	Style->Set("TableView.AlternatingRow", AlternatingTableRowStyle);

	Style->Set("TreeArrow_Collapsed", new IMAGE_BRUSH_SVG("Starship/Common/caret-right", Icon16x16, FStyleColors::Foreground));
	Style->Set("TreeArrow_Collapsed_Hovered", new IMAGE_BRUSH_SVG("Starship/Common/caret-right", Icon16x16, FStyleColors::ForegroundHover));
	Style->Set("TreeArrow_Expanded", new IMAGE_BRUSH_SVG("Starship/Common/caret-down", Icon16x16, FStyleColors::Foreground));
	Style->Set("TreeArrow_Expanded_Hovered", new IMAGE_BRUSH_SVG("Starship/Common/caret-down", Icon16x16, FStyleColors::ForegroundHover));


	const FTableColumnHeaderStyle TableColumnHeaderStyle = FTableColumnHeaderStyle()
		.SetSortPrimaryAscendingImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/TableView/sort-up-arrow", Icon12x12))
		.SetSortPrimaryDescendingImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/TableView/sort-down-arrow", Icon12x12))
		.SetSortSecondaryAscendingImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/TableView/sort-up-arrows", Icon12x12))
		.SetSortSecondaryDescendingImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/TableView/sort-down-arrows", Icon12x12))
		.SetNormalBrush(FSlateColorBrush(FStyleColors::Header))
		.SetHoveredBrush(FSlateColorBrush(FStyleColors::Dropdown))

		.SetMenuDropdownImage(IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(6.f, 24.f)))
		.SetMenuDropdownNormalBorderBrush(FSlateNoResource())
		.SetMenuDropdownHoveredBorderBrush(FSlateNoResource());

	Style->Set("TableView.Header.Column", TableColumnHeaderStyle);

	const FTableColumnHeaderStyle TableLastColumnHeaderStyle = FTableColumnHeaderStyle()
		.SetSortPrimaryAscendingImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/TableView/sort-up-arrow", Icon12x12))
		.SetSortPrimaryDescendingImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/TableView/sort-down-arrow", Icon12x12))
		.SetSortSecondaryAscendingImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/TableView/sort-up-arrows", Icon12x12))
		.SetSortSecondaryDescendingImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/TableView/sort-down-arrows", Icon12x12))
		.SetNormalBrush(FSlateColorBrush(FStyleColors::Header))
		.SetHoveredBrush(FSlateColorBrush(FStyleColors::Dropdown))

		.SetMenuDropdownImage(IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(6.f, 24.f)))
		.SetMenuDropdownNormalBorderBrush(FSlateNoResource())
		.SetMenuDropdownHoveredBorderBrush(FSlateNoResource());

	const FSplitterStyle TableHeaderSplitterStyle = FSplitterStyle()
		.SetHandleNormalBrush(FSlateColorBrush(FStyleColors::Recessed))
		.SetHandleHighlightBrush(FSlateColorBrush(FStyleColors::Recessed));

	Style->Set("TableView.Header", FHeaderRowStyle()
		.SetColumnStyle(TableColumnHeaderStyle)
		.SetLastColumnStyle(TableLastColumnHeaderStyle)
		.SetColumnSplitterStyle(TableHeaderSplitterStyle)
		.SetSplitterHandleSize(1.0f)
		.SetBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed))
		.SetForegroundColor(FStyleColors::Foreground)
		.SetHorizontalSeparatorBrush(FSlateColorBrush(FStyleColors::Recessed))
		.SetHorizontalSeparatorThickness(2.0f)
	);
}

void FStarshipCoreStyle::SetupMultiboxStyles(TSharedRef<FStyle>& Style)
{
	const FEditableTextBoxStyle& NormalEditableTextBoxStyle = Style->GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
	const FTextBlockStyle& NormalText = Style->GetWidgetStyle<FTextBlockStyle>("NormalText");
	const FTextBlockStyle& SmallButtonText = Style->GetWidgetStyle<FTextBlockStyle>("SmallButtonText");

	const FSlateColor SelectionColor = Style->GetSlateColor("SelectionColor");
	const FSlateColor SelectionColor_Pressed = Style->GetSlateColor("SelectionColor_Pressed");
	const FSlateColor DefaultForeground = Style->GetSlateColor("SelectionColor_Pressed");

	// MultiBox
	{
		Style->Set("MultiBox.GenericToolBarIcon", new IMAGE_BRUSH_SVG("Starship/Common/EditToolbar", Icon40x40));
		Style->Set("MultiBox.GenericToolBarIcon.Small", new IMAGE_BRUSH_SVG("Starship/Common/EditToolbar", Icon16x16));

		Style->Set("MultiboxHookColor", FLinearColor(0.f, 1.f, 0.f, 1.f));
	}

	// ToolBar
	{
		FToolBarStyle NormalToolbarStyle =
			FToolBarStyle()
			.SetBackground(FSlateColorBrush(FStyleColors::Panel))
			.SetBackgroundPadding(FMargin(0.f, 4.f))
			.SetComboButtonPadding(FMargin(4.0f, 0.0f))
			.SetButtonPadding(FMargin(2.0f, 0.f))
			.SetCheckBoxPadding(FMargin(4.0f, 0.f))
			.SetSeparatorBrush(FSlateColorBrush(FStyleColors::Background))
			.SetSeparatorPadding(FMargin(1.f, 0.f, 1.f, 0.f))
			.SetLabelStyle(FTextBlockStyle(NormalText).SetFont(FStyleFonts::Get().Normal))
			.SetEditableTextStyle(FEditableTextBoxStyle(NormalEditableTextBoxStyle).SetFont(FStyleFonts::Get().Normal))
			.SetComboButtonStyle(Style->GetWidgetStyle<FComboButtonStyle>("ComboButton"))
			.SetBlockPadding(FMargin(2.0f, 2.0f, 4.0f, 4.0f))
			.SetIndentedBlockPadding(FMargin(18.0f, 2.0f, 4.0f, 4.0f));


		/* Create style for "ToolBar.ToggleButton" widget ... */
		const FCheckBoxStyle ToolBarToggleButtonCheckBoxStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(FSlateNoResource())
			.SetUncheckedPressedImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
			.SetUncheckedHoveredImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor))
			.SetCheckedImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
			.SetCheckedHoveredImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
			.SetCheckedPressedImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor));

		NormalToolbarStyle.SetToggleButtonStyle(ToolBarToggleButtonCheckBoxStyle);

		const FButtonStyle ToolbarButton = FButtonStyle()
			.SetNormal(FSlateNoResource())
			.SetPressed(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
			.SetHovered(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor))
			.SetNormalPadding(FMargin(2.f, 2.f, 2.f, 2.f))
			.SetPressedPadding(FMargin(2.f, 3.f, 2.f, 1.f))
			.SetNormalForeground(FSlateColor::UseForeground())
			.SetPressedForeground(FSlateColor::UseForeground())
			.SetHoveredForeground(FSlateColor::UseForeground())
			.SetDisabledForeground(FSlateColor::UseForeground());

		NormalToolbarStyle.SetButtonStyle(ToolbarButton);
		NormalToolbarStyle.SetIconSize(Icon40x40);
		
		NormalToolbarStyle.WrapButtonStyle
			.SetIncludeSeparator(false)
			.SetExpandBrush(IMAGE_BRUSH_SVG("Common/menu", Icon16x16));

		Style->Set("ToolBar", NormalToolbarStyle);

		// Slim Toolbar
		const FButtonStyle& SimpleButtonStyle = Style->GetWidgetStyle<FButtonStyle>("SimpleButton");

		FButtonStyle SlimToolBarButton = FButtonStyle(SimpleButtonStyle)
			.SetNormalForeground(FStyleColors::Foreground)
			.SetPressedForeground(FStyleColors::ForegroundHover)
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetDisabledForeground(FStyleColors::Foreground)
			.SetNormalPadding(FMargin(8.f, 4.f, 8.f, 4.f))
			.SetPressedPadding(FMargin(8.f, 5.f, 8.f, 3.f));


		FComboButtonStyle SlimToolBarComboButton = FComboButtonStyle(Style->GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.SetDownArrowPadding(FMargin(1.f, 0.f, 2.f, 0.f));

		FSlateBrush* SlimToolbarBackground = new FSlateColorBrush(FStyleColors::Panel);
		
		FToolBarStyle SlimToolbarStyle =
			FToolBarStyle()
			.SetBackground(*SlimToolbarBackground)
			.SetBackgroundPadding(FMargin(4.f, 6.f))
			.SetComboButtonPadding(FMargin(0.0f, 0.0f))
			.SetButtonPadding(FMargin(4.0f, 0.0f))
			.SetCheckBoxPadding(FMargin(8.0f, 0.0f))
			.SetSeparatorBrush(FSlateColorBrush(FStyleColors::Recessed))
			.SetSeparatorPadding(FMargin(4.f, -5.0f))
			.SetLabelStyle(FTextBlockStyle(NormalText))
			.SetComboButtonStyle(SlimToolBarComboButton)
			.SetLabelPadding(FMargin(4.f, 0.f, 0.f, 0.f))
			.SetEditableTextStyle(FEditableTextBoxStyle(NormalEditableTextBoxStyle));
			
		SlimToolbarStyle.WrapButtonStyle
			.SetWrapButtonPadding(FMargin(0.0f, -6.f, -4.f, -6.f))
			.SetSeparatorPadding(0.f)
			.SetComboButtonStyle(FComboButtonStyle(SlimToolbarStyle.ComboButtonStyle)
			.SetContentPadding(FMargin(4.0f, 0.f, 0.f, 0.f))
			.SetDownArrowPadding(FMargin(4.f, 0.f, 0.f, 0.f))
			.SetButtonStyle(FButtonStyle()
				.SetNormal(FSlateColorBrush(FStyleColors::Dropdown))
				.SetHovered(FSlateColorBrush(FStyleColors::Hover))
				.SetPressed(FSlateColorBrush(FStyleColors::Header))
				.SetNormalForeground(FStyleColors::Foreground)
				.SetHoveredForeground(FStyleColors::ForegroundHover)
				.SetPressedForeground(FStyleColors::ForegroundHover)
				.SetDisabledForeground(FStyleColors::Hover2)
				.SetNormalPadding(FMargin(6.0f, 4.0f))
				.SetPressedPadding(FMargin(6.0f, 4.0f))
			))
			.SetExpandBrush(IMAGE_BRUSH_SVG("Common/menu", Icon16x16));
			
		FCheckBoxStyle SlimToolBarToggleButtonCheckBoxStyle = FCheckBoxStyle(Style->GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
			.SetPadding(FMargin(8.f, 4.f, 8.f, 4.f));

		SlimToolbarStyle.SetToggleButtonStyle(SlimToolBarToggleButtonCheckBoxStyle);
		SlimToolbarStyle.SetButtonStyle(SlimToolBarButton);

		FButtonStyle SlimToolBarButtonSettingsRegion = FButtonStyle(SimpleButtonStyle)
			.SetPressed(FSlateNoResource())
			.SetHovered(FSlateNoResource())
			.SetNormal(FSlateNoResource())
			.SetNormalForeground(FStyleColors::Foreground)
			.SetPressedForeground(FStyleColors::ForegroundHover)
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetDisabledForeground(FStyleColors::Foreground)
			.SetNormalPadding(FMargin(8.f, 4.f, 8.f, 4.f))
			.SetPressedPadding(FMargin(8.f, 5.f, 8.f, 3.f));


		FCheckBoxStyle SettingsToggleButtonStyle = FCheckBoxStyle(Style->GetWidgetStyle<FCheckBoxStyle>("TransparentCheckBox"))
			.SetPadding(FMargin(8.f, 4.f, 8.f, 4.f));

		SlimToolbarStyle.SetSettingsButtonStyle(SlimToolBarButtonSettingsRegion);
		SlimToolbarStyle.SetSettingsToggleButtonStyle(SettingsToggleButtonStyle);
		SlimToolbarStyle.SetIconSize(Icon20x20);

		SlimToolBarButtonSettingsRegion
			.SetNormalPadding(FMargin(4.f, 4.f, 4.f, 4.f))
			.SetPressedPadding(FMargin(4.f, 5.f, 4.f, 3.f));

		FComboButtonStyle SlimToolBarSettingsComboButton = FComboButtonStyle(SlimToolBarComboButton)
			.SetContentPadding(0)
			.SetButtonStyle(SlimToolBarButtonSettingsRegion)
			.SetDownArrowImage(IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(6.f, 20.f)))
			.SetDownArrowPadding(FMargin(1.f, 0.f));

		SlimToolbarStyle.SetSettingsComboButtonStyle(SlimToolBarSettingsComboButton);


		Style->Set("ToolbarSettingsRegion.LeftHover", new FSlateRoundedBoxBrush(FStyleColors::Hover, FVector4(4.0f, 0.0f, 0.0f, 4.0f)));
		Style->Set("ToolbarSettingsRegion.RightHover", new FSlateRoundedBoxBrush(FStyleColors::Hover, FVector4(0.0f, 4.0f, 4.0f, 0.0f)));
		Style->Set("ToolbarSettingsRegion.Left", new FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(4.0f, 0.0f, 0.0f, 4.0f), FStyleColors::Header, 1.0f));
		Style->Set("ToolbarSettingsRegion.Right", new FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 4.0f, 4.0f, 0.0f), FStyleColors::Header, 1.0f));
		Style->Set("ToolbarSettingsRegion.LeftToggleHover", new FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, FVector4(4.0f, 0.0f, 0.0f, 4.0f)));
		Style->Set("ToolbarSettingsRegion.LeftToggle", new FSlateRoundedBoxBrush(FStyleColors::Primary, FVector4(4.0f, 0.0f, 0.0f, 4.0f)));


		Style->Set("SlimToolBar", SlimToolbarStyle);


		SlimToolbarStyle.SetShowLabels(false);
		
		Style->Set("AssetEditorToolbar", SlimToolbarStyle);

		// Used by empty toolbars and misc widgets to have the same background color
		Style->Set("AssetEditorToolbar.Background", SlimToolbarBackground);
		
		// Styling for secondary toolbars that appear below the main toolbar, currently the same as the main toolbar
		{
			FToolBarStyle SecondaryToolbarStyle(SlimToolbarStyle);
			Style->Set("SecondaryToolbar", SecondaryToolbarStyle);
		}

		// Callout Toolbar - Used to "call out" the toolbar button with text
		{
			SlimToolbarStyle.SetShowLabels(true);

			Style->Set("CalloutToolbar", SlimToolbarStyle);

			FToolBarStyle SlimToolbarStyleWidth100 = SlimToolbarStyle;
			SlimToolbarStyleWidth100.SetButtonTextMaxWidth(100);
			SlimToolbarStyleWidth100.SetButtonTextMinWidth(100);

			Style->Set("CalloutToolbar-Width100", SlimToolbarStyleWidth100);

		}

	}

	// MenuBar
	{
		Style->Set("Menu.WidgetBorder", new FSlateRoundedBoxBrush(FStyleColors::Input, 5.0f));
		Style->Set("Menu.SpinBox", FSpinBoxStyle()
			.SetBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::Secondary, InputFocusThickness))
			.SetHoveredBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Input, InputFocusRadius, FStyleColors::Hover, InputFocusThickness))

			.SetActiveFillBrush(FSlateRoundedBoxBrush(FStyleColors::Hover, InputFocusRadius, FLinearColor::Transparent, InputFocusThickness))
			.SetInactiveFillBrush(FSlateRoundedBoxBrush(FStyleColors::Secondary, InputFocusRadius, FLinearColor::Transparent, InputFocusThickness))
			.SetArrowsImage(FSlateNoResource())
			.SetForegroundColor(FStyleColors::ForegroundHover)
			.SetTextPadding(FMargin(10.f, 3.5f, 10.f, 4.f))
		);


		Style->Set("Menu.Background", new FSlateColorBrush(FStyleColors::Dropdown));
		Style->Set("Menu.Outline", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 0.0f, FStyleColors::DropdownOutline, 1.f));
		Style->Set("Menu.Icon", new IMAGE_BRUSH("Icons/icon_tab_toolbar_16px", Icon16x16));
		Style->Set("Menu.Expand", new IMAGE_BRUSH("Icons/toolbar_expand_16x", Icon16x16));
		Style->Set("Menu.SubMenuIndicator", new IMAGE_BRUSH_SVG("Starship/Common/chevron-right", Icon16x16, FStyleColors::Foreground));
		Style->Set("Menu.SToolBarComboButtonBlock.Padding", FMargin(2.0f));
		Style->Set("Menu.SToolBarButtonBlock.Padding", FMargin(2.0f));
		Style->Set("Menu.SToolBarCheckComboButtonBlock.Padding", FMargin(2.0f));
		Style->Set("Menu.SToolBarButtonBlock.CheckBox.Padding", FMargin(0.0f));
		Style->Set("Menu.SToolBarComboButtonBlock.ComboButton.Color", DefaultForeground);
		Style->Set("Menu.MenuIconSize", 14.f);

		const FMargin MenuBlockPadding(12.0f, 1.0f, 5.0f, 1.0f);
		Style->Set("Menu.Block.IndentedPadding", MenuBlockPadding + FMargin(18.0f, 0, 0, 0));
		Style->Set("Menu.Block.Padding", MenuBlockPadding);

		Style->Set("Menu.Separator", new FSlateColorBrush(FStyleColors::White25));
		Style->Set("Menu.Separator.Padding", FMargin(12.0f, 6.f, 12.0f, 6.f));

		Style->Set("Menu.Label", NormalText);

		Style->Set("Menu.EditableText", FEditableTextBoxStyle(NormalEditableTextBoxStyle).SetFont(FStyleFonts::Get().Normal));
		Style->Set("Menu.Keybinding", FTextBlockStyle(NormalText).SetFont(FStyleFonts::Get().Small));


		FSlateFontInfo XSFont(DEFAULT_FONT("Bold", 7));
		XSFont.LetterSpacing =  250;

		Style->Set("Menu.Heading",
			FTextBlockStyle(SmallButtonText)
			.SetFont(XSFont)
			.SetColorAndOpacity(FStyleColors::White25));
		Style->Set("Menu.Heading.Padding", FMargin(12.0f, 6.f, 12.f, 6.f));

		/* Set images for various SCheckBox states associated with menu check box items... */
		FLinearColor Transparent20 = FLinearColor(1.0f, 1.0f, 1.0f, 0.2f);
		FLinearColor Transparent01 = FLinearColor(1.0f, 1.0f, 1.0f, 0.01f);



		const FCheckBoxStyle MenuCheckIndicator = FCheckBoxStyle()
			.SetUncheckedImage(           FSlateNoResource())
			.SetUncheckedHoveredImage(    FSlateNoResource())
			.SetUncheckedPressedImage(    FSlateNoResource())

			.SetCheckedImage(             IMAGE_BRUSH_SVG("Starship/Common/check", Icon16x16))
			.SetCheckedHoveredImage(      IMAGE_BRUSH_SVG("Starship/Common/check", Icon16x16))
			.SetCheckedPressedImage(      IMAGE_BRUSH_SVG("Starship/Common/check", Icon16x16))

			.SetUndeterminedImage(        IMAGE_BRUSH_SVG("Starship/Common/check", Icon16x16, Transparent01))
			.SetUndeterminedHoveredImage( IMAGE_BRUSH_SVG("Starship/Common/check", Icon16x16, Transparent20))
			.SetUndeterminedPressedImage( IMAGE_BRUSH_SVG("Starship/Common/check", Icon16x16, Transparent20));

		// SCheckBox defaults...
		const float CheckboxCornerRadius = 3.f;
		const float CheckboxOutlineThickness = 1.0f;

		const FCheckBoxStyle MenuCheckBox = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::CheckBox)

			.SetForegroundColor(FStyleColors::White)
			.SetHoveredForegroundColor(FStyleColors::White)
			.SetPressedForegroundColor(FStyleColors::White)
			.SetCheckedForegroundColor(FStyleColors::White)
			.SetCheckedHoveredForegroundColor(FStyleColors::White)
			.SetCheckedPressedForegroundColor(FStyleColors::White)
			.SetUndeterminedForegroundColor(FStyleColors::White)

			.SetUncheckedImage(FSlateNoResource())
			.SetUncheckedHoveredImage(FSlateNoResource())
			.SetUncheckedPressedImage(FSlateNoResource())
			.SetCheckedImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/check", Icon16x16, FStyleColors::White))
			.SetCheckedHoveredImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/check", Icon16x16, FStyleColors::White))
			.SetCheckedPressedImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/check", Icon16x16, FStyleColors::White))
			.SetUndeterminedImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/indeterminate", Icon16x16, FStyleColors::White))
			.SetUndeterminedHoveredImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/indeterminate", Icon16x16, FStyleColors::White))
			.SetUndeterminedPressedImage(IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/indeterminate", Icon16x16, FStyleColors::White))
			.SetBackgroundImage(FSlateRoundedBoxBrush(FStyleColors::Panel, CheckboxCornerRadius, FStyleColors::Hover, CheckboxOutlineThickness, Icon18x18))
			.SetBackgroundHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Panel, CheckboxCornerRadius, FStyleColors::Hover, CheckboxOutlineThickness, Icon18x18))
			.SetBackgroundPressedImage(FSlateRoundedBoxBrush(FStyleColors::Panel, CheckboxCornerRadius, FStyleColors::Hover, CheckboxOutlineThickness, Icon18x18));

		/* ...and add the new style */
		Style->Set("Menu.CheckBox", MenuCheckBox);
		Style->Set("Menu.Check", MenuCheckIndicator);
		Style->Set("ClippingVerticalBox.Check", MenuCheckIndicator);

		/* This radio button is actually just a check box with different images */
		/* Set images for various Menu radio button (SCheckBox) states... */
		const FCheckBoxStyle BasicMenuRadioButtonStyle = FCheckBoxStyle()
			.SetUncheckedImage(           FSlateRoundedBoxBrush(FStyleColors::Input,  Icon8x8))
			.SetUncheckedHoveredImage(    FSlateRoundedBoxBrush(FStyleColors::Input, Icon8x8))
			.SetUncheckedPressedImage(    FSlateRoundedBoxBrush(FStyleColors::Hover2,   Icon8x8))
			.SetCheckedImage(             FSlateRoundedBoxBrush(FStyleColors::White,   Icon8x8))
			.SetCheckedHoveredImage(      FSlateRoundedBoxBrush(FStyleColors::White,   Icon8x8))
			.SetCheckedPressedImage(      FSlateRoundedBoxBrush(FStyleColors::White,   Icon8x8))
			.SetUndeterminedImage(        FSlateRoundedBoxBrush(FStyleColors::Input,  Icon8x8))
			.SetUndeterminedHoveredImage( FSlateRoundedBoxBrush(FStyleColors::White, Icon8x8))
			.SetUndeterminedPressedImage( FSlateRoundedBoxBrush(FStyleColors::White,   Icon8x8));

		/* ...and set new style */
		Style->Set("Menu.RadioButton", BasicMenuRadioButtonStyle);

		/* Create style for "Menu.ToggleButton" widget ... */
		const FCheckBoxStyle MenuToggleButtonCheckBoxStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(FSlateNoResource())
			.SetUncheckedPressedImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
			.SetUncheckedHoveredImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor))
			.SetCheckedImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
			.SetCheckedHoveredImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
			.SetCheckedPressedImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor));

		/* ... and add new style */
		Style->Set("Menu.ToggleButton", MenuToggleButtonCheckBoxStyle);

		FButtonStyle MenuButton =
			FButtonStyle(Style->GetWidgetStyle<FButtonStyle>("NoBorder"))
			.SetNormal(FSlateNoResource())
			.SetPressed(FSlateColorBrush(FStyleColors::Primary))
			.SetHovered(FSlateColorBrush(FStyleColors::Primary))
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetNormalPadding(FMargin(0, 2))
			.SetPressedPadding(FMargin(0, 3, 0, 1));

		Style->Set("Menu.Button", MenuButton);

		Style->Set("Menu.Button.Checked", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed));

		/* The style of a menu bar button when it has a sub menu open */
		Style->Set("Menu.Button.SubMenuOpen", new BORDER_BRUSH("Common/Selection", FMargin(4.f / 16.f), FLinearColor(0.10f, 0.10f, 0.10f)));


		FButtonStyle MenuBarButton =
			FButtonStyle(MenuButton)
			.SetHovered(FSlateColorBrush(FStyleColors::Hover))
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetPressedForeground(FStyleColors::ForegroundHover)
			.SetNormalForeground(FStyleColors::Foreground);

		// For menu bars we need to ignore the button style

		Style->Set("WindowMenuBar.Background", new FSlateNoResource());
		Style->Set("WindowMenuBar.Label", FTextBlockStyle(NormalText).SetFont(FStyleFonts::Get().Normal));
		Style->Set("WindowMenuBar.Expand", new IMAGE_BRUSH("Icons/toolbar_expand_16x", Icon16x16));
		Style->Set("WindowMenuBar.Button", MenuBarButton);
		Style->Set("WindowMenuBar.Button.SubMenuOpen", new FSlateColorBrush(FStyleColors::Primary));
		Style->Set("WindowMenuBar.MenuBar.Padding", FMargin(12, 4));

	}

}

#undef RootToContentDir
