// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/StyleColors.h"

#include "Application/SlateApplicationBase.h"
#include "Misc/Paths.h"
#include "Serialization/JsonWriter.h"
#include "Misc/FileHelper.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StyleColors)

// Note this value is not mutable by the user
const FSlateColor FStyleColors::Transparent = FSlateColor(FLinearColor::Transparent);

const FSlateColor FStyleColors::Black = EStyleColor::Black;
const FSlateColor FStyleColors::Title = EStyleColor::Title;
const FSlateColor FStyleColors::WindowBorder = EStyleColor::WindowBorder;
const FSlateColor FStyleColors::Foldout = EStyleColor::Foldout;
const FSlateColor FStyleColors::Input = EStyleColor::Input;
const FSlateColor FStyleColors::InputOutline = EStyleColor::InputOutline;
const FSlateColor FStyleColors::Recessed = EStyleColor::Recessed;
const FSlateColor FStyleColors::Background = EStyleColor::Background;
const FSlateColor FStyleColors::Panel = EStyleColor::Panel;
const FSlateColor FStyleColors::Header = EStyleColor::Header;
const FSlateColor FStyleColors::Dropdown = EStyleColor::Dropdown;
const FSlateColor FStyleColors::DropdownOutline = EStyleColor::DropdownOutline;
const FSlateColor FStyleColors::Hover = EStyleColor::Hover;
const FSlateColor FStyleColors::Hover2 = EStyleColor::Hover2;

const FSlateColor FStyleColors::White = EStyleColor::White;
const FSlateColor FStyleColors::White25 = EStyleColor::White25;
const FSlateColor FStyleColors::Highlight = EStyleColor::Highlight;

const FSlateColor FStyleColors::Primary = EStyleColor::Primary;
const FSlateColor FStyleColors::PrimaryHover = EStyleColor::PrimaryHover;
const FSlateColor FStyleColors::PrimaryPress = EStyleColor::PrimaryPress;
const FSlateColor FStyleColors::Secondary = EStyleColor::Secondary;

const FSlateColor FStyleColors::Foreground = EStyleColor::Foreground;
const FSlateColor FStyleColors::ForegroundHover = EStyleColor::ForegroundHover;
const FSlateColor FStyleColors::ForegroundInverted = EStyleColor::ForegroundInverted;
const FSlateColor FStyleColors::ForegroundHeader = EStyleColor::ForegroundHeader;

const FSlateColor FStyleColors::Select = EStyleColor::Select;
const FSlateColor FStyleColors::SelectInactive = EStyleColor::SelectInactive;
const FSlateColor FStyleColors::SelectParent = EStyleColor::SelectParent;
const FSlateColor FStyleColors::SelectHover = EStyleColor::SelectHover;

const FSlateColor FStyleColors::Notifications = EStyleColor::Notifications;
// if select ==  primary shouldnt we have a select pressed which is the same as primary press?

const FSlateColor FStyleColors::AccentBlue = EStyleColor::AccentBlue;
const FSlateColor FStyleColors::AccentPurple = EStyleColor::AccentPurple;
const FSlateColor FStyleColors::AccentPink = EStyleColor::AccentPink;
const FSlateColor FStyleColors::AccentRed = EStyleColor::AccentRed;
const FSlateColor FStyleColors::AccentOrange = EStyleColor::AccentOrange;
const FSlateColor FStyleColors::AccentYellow = EStyleColor::AccentYellow;
const FSlateColor FStyleColors::AccentGreen = EStyleColor::AccentGreen;
const FSlateColor FStyleColors::AccentBrown = EStyleColor::AccentBrown;
const FSlateColor FStyleColors::AccentBlack = EStyleColor::AccentBlack;
const FSlateColor FStyleColors::AccentGray = EStyleColor::AccentGray;
const FSlateColor FStyleColors::AccentWhite = EStyleColor::AccentWhite;
const FSlateColor FStyleColors::AccentFolder = EStyleColor::AccentFolder;

const FSlateColor FStyleColors::Warning= EStyleColor::Warning;
const FSlateColor FStyleColors::Error = EStyleColor::Error;
const FSlateColor FStyleColors::Success = EStyleColor::Success;

USlateThemeManager::USlateThemeManager()
{
	InitializeDefaults();
}

void USlateThemeManager::InitializeDefaults()
{
	// This loads defaults for a dark theme

	SetDefaultColor(EStyleColor::Black, COLOR("#000000FF"));
	SetDefaultColor(EStyleColor::Title, COLOR("#151515FF"));
	SetDefaultColor(EStyleColor::Background, COLOR("#151515FF"));
	SetDefaultColor(EStyleColor::WindowBorder, COLOR("0F0F0FFF"));
	SetDefaultColor(EStyleColor::Foldout, COLOR("0F0F0FFF"));
	SetDefaultColor(EStyleColor::Input, COLOR("0F0F0FFF"));
	SetDefaultColor(EStyleColor::InputOutline, COLOR("383838FF"));
	SetDefaultColor(EStyleColor::Recessed, COLOR("#1A1A1AFF"));
	SetDefaultColor(EStyleColor::Panel, COLOR("#242424FF"));
	SetDefaultColor(EStyleColor::Header, COLOR("#2F2F2FFF"));
	SetDefaultColor(EStyleColor::Dropdown, COLOR("#383838FF"));
	SetDefaultColor(EStyleColor::DropdownOutline, COLOR("#4C4C4CFF"));
	SetDefaultColor(EStyleColor::Hover, COLOR("#575757FF"));
	SetDefaultColor(EStyleColor::Hover2, COLOR("#808080FF"));

	SetDefaultColor(EStyleColor::White, COLOR("#FFFFFFFF"));
	SetDefaultColor(EStyleColor::White25, COLOR("#FFFFFF40"));
	SetDefaultColor(EStyleColor::Highlight, COLOR("#0070E0FF"));

	SetDefaultColor(EStyleColor::Primary, COLOR("#0070E0FF"));
	SetDefaultColor(EStyleColor::PrimaryHover, COLOR("#0E86FFFF"));
	SetDefaultColor(EStyleColor::PrimaryPress, COLOR("#0050A0FF"));
	SetDefaultColor(EStyleColor::Secondary, COLOR("#383838FF"));

	SetDefaultColor(EStyleColor::Foreground, COLOR("#C0C0C0FF"));
	SetDefaultColor(EStyleColor::ForegroundHover, COLOR("#FFFFFFFF"));
	SetDefaultColor(EStyleColor::ForegroundInverted, GetDefaultColor(EStyleColor::Input));
	SetDefaultColor(EStyleColor::ForegroundHeader, COLOR("#C8C8C8FF"));

	SetDefaultColor(EStyleColor::Select, GetDefaultColor(EStyleColor::Primary));
	SetDefaultColor(EStyleColor::SelectInactive, COLOR("#40576F"));
	SetDefaultColor(EStyleColor::SelectParent, COLOR("#2C323AFF"));
	SetDefaultColor(EStyleColor::SelectHover, GetDefaultColor(EStyleColor::Panel));

	SetDefaultColor(EStyleColor::Notifications, COLOR("464B50FF"));

	// if select ==  primary shouldnt we have a select pressed which is the same as primary press?

	SetDefaultColor(EStyleColor::AccentBlue, COLOR("#26BBFFFF"));
	SetDefaultColor(EStyleColor::AccentPurple, COLOR("#A139BFFF"));
	SetDefaultColor(EStyleColor::AccentPink, COLOR("#FF729CFF"));
	SetDefaultColor(EStyleColor::AccentRed, COLOR("#FF4040FF"));
	SetDefaultColor(EStyleColor::AccentOrange, COLOR("#FE9B07FF"));
	SetDefaultColor(EStyleColor::AccentYellow, COLOR("#FFDC1AFF"));
	SetDefaultColor(EStyleColor::AccentGreen, COLOR("#8BC24AFF"));
	SetDefaultColor(EStyleColor::AccentBrown, COLOR("#804D39FF"));
	SetDefaultColor(EStyleColor::AccentBlack, COLOR("#242424FF"));
	SetDefaultColor(EStyleColor::AccentGray, COLOR("#808080FF"));
	SetDefaultColor(EStyleColor::AccentWhite, COLOR("#FFFFFFFF"));
	SetDefaultColor(EStyleColor::AccentFolder, COLOR("#B68F55FF"));

	SetDefaultColor(EStyleColor::Warning, COLOR("#FFB800FF"));
	SetDefaultColor(EStyleColor::Error, COLOR("#EF3535FF"));
	SetDefaultColor(EStyleColor::Success, COLOR("#1FE44BFF"));
}

#if ALLOW_THEMES

static const FString ThemesSubDir = "Slate/Themes";


void USlateThemeManager::LoadThemes()
{
	Themes.Empty();

	// Engine wide themes
	LoadThemesFromDirectory(GetEngineThemeDir());

	// Project themes
	LoadThemesFromDirectory(GetProjectThemeDir());

	// User specific themes
	LoadThemesFromDirectory(GetUserThemeDir());

	EnsureValidCurrentTheme();


	ApplyTheme(CurrentThemeId);
}

void USlateThemeManager::SaveCurrentThemeAs(const FString& Filename)
{
	FStyleTheme& CurrentTheme = GetMutableCurrentTheme();
	CurrentTheme.Filename = Filename;
	FString NewPath = CurrentTheme.Filename;
	{
		FString Output;
		WriteCurrentTheme(Output);

		if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*Filename))
		{
			FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*Filename, false);
			// create a new path if the filename has been changed. 
			NewPath = USlateThemeManager::Get().GetUserThemeDir() / CurrentTheme.DisplayName.ToString() + TEXT(".json");

			if (!NewPath.Equals(CurrentTheme.Filename))
			{
				// rename the current .json file with the new name. 
				IFileManager::Get().Move(*NewPath, *Filename);
			}
		}
		FFileHelper::SaveStringToFile(Output, *NewPath);
	}
	
}

bool USlateThemeManager::ExportCurrentThemeTo(const FString& Filename)
{
	FString Output;
	if (!WriteCurrentTheme(Output))
	{
		return false;
	}

	if (IPlatformFile::GetPlatformPhysical().FileExists(*Filename))
	{
		IPlatformFile::GetPlatformPhysical().DeleteFile(*Filename);
	}

	return FFileHelper::SaveStringToFile(Output, *Filename);
}

void USlateThemeManager::ApplyTheme(FGuid ThemeId)
{
	if (ThemeId.IsValid())
	{
		FStyleTheme* CurrentTheme = nullptr;
		if (CurrentThemeId != ThemeId)
		{
			if (CurrentThemeId.IsValid())
			{
				CurrentTheme = &GetMutableCurrentTheme();
				// Unload existing colors
				CurrentTheme->LoadedDefaultColors.Empty();
			}
			
			FStyleTheme* Theme = Themes.FindByKey(ThemeId);
			if (Theme)
			{
				CurrentThemeId = ThemeId;
				SaveConfig();
			}
		}

		//if (CurrentTheme->LoadedDefaultColors.IsEmpty())
		{
			CurrentTheme = &GetMutableCurrentTheme();
			LoadThemeColors(*CurrentTheme);
		}

		// Apply the new colors. Note that if the incoming theme is the same as the current theme we still apply the colors as they may have been overwritten by defaults
		FMemory::Memcpy(ActiveColors.StyleColors, CurrentTheme->LoadedDefaultColors.GetData(), sizeof(FLinearColor)*CurrentTheme->LoadedDefaultColors.Num());

		if (FSlateApplicationBase::IsInitialized())
		{
			FSlateApplicationBase::Get().InvalidateAllWidgets(false /*bClearResourcesImmediately*/);
		}
	}
	OnThemeChanged().Broadcast(CurrentThemeId); 
}

void USlateThemeManager::ApplyDefaultTheme()
{
	ApplyTheme(DefaultDarkTheme.Id); 
}

bool USlateThemeManager::IsEngineTheme() const
{
	// users cannot edit/delete engine-specific themes: 
	const FString EnginePath = GetEngineThemeDir() / GetCurrentTheme().DisplayName.ToString() + TEXT(".json");

	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
	// todo: move default theme check to a standalone function in 5.2
	if (GetCurrentTheme() == DefaultDarkTheme)
	{
		return true;
	}
	else if (FileManager.FileExists(*EnginePath))
	{
		return true;
	}
	return false;
}

bool USlateThemeManager::IsProjectTheme() const
{
	// users cannot edit/delete project-specific themes: 
	const FString ProjectPath = GetProjectThemeDir() / GetCurrentTheme().DisplayName.ToString() + TEXT(".json");

	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile(); 

	if (FileManager.FileExists(*ProjectPath))
	{
		return true; 
	}
	return false; 
}

void USlateThemeManager::RemoveTheme(FGuid ThemeId)
{
	// Current Theme cannot currently be removed.  Apply a new theme first
	if (CurrentThemeId != ThemeId)
	{
		Themes.RemoveAll([&ThemeId](const FStyleTheme& TestTheme) { return TestTheme.Id == ThemeId; });
	}
}

FGuid USlateThemeManager::DuplicateActiveTheme()
{
	const FStyleTheme& CurrentTheme = GetCurrentTheme();

	FGuid NewThemeGuid = FGuid::NewGuid();
	FStyleTheme NewTheme;
	NewTheme.Id = NewThemeGuid;
	NewTheme.DisplayName = FText::Format(NSLOCTEXT("StyleColors", "ThemeDuplicateCopyText", "{0} - Copy"), CurrentTheme.DisplayName);
	NewTheme.LoadedDefaultColors = MakeArrayView<FLinearColor>(ActiveColors.StyleColors, (int32)EStyleColor::MAX);

	Themes.Add(MoveTemp(NewTheme));

	return NewThemeGuid;
}

void USlateThemeManager::SetCurrentThemeDisplayName(FText NewDisplayName)
{
	GetMutableCurrentTheme().DisplayName = NewDisplayName;
}

void USlateThemeManager::ResetActiveColorToDefault(EStyleColor Color)
{
	ActiveColors.StyleColors[(int32)Color] = GetCurrentTheme().LoadedDefaultColors[(int32)Color];

	if (FSlateApplicationBase::IsInitialized())
	{
		FSlateApplicationBase::Get().InvalidateAllWidgets(false /*bClearResourcesImmediately*/);
	}
}

void USlateThemeManager::ResetColorsToDefault()
{
	for (int32 ColorIndex = 0; ColorIndex < (int32)EStyleColor::MAX; ++ColorIndex)
	{
		ActiveColors.StyleColors[ColorIndex] = GetCurrentTheme().LoadedDefaultColors[ColorIndex];
	}

	if (FSlateApplicationBase::IsInitialized())
	{
		FSlateApplicationBase::Get().InvalidateAllWidgets(false /*bClearResourcesImmediately*/);
	}
}

void USlateThemeManager::ValidateActiveTheme()
{
	// This is necessary because the core style loads the color table before ProcessNewlyLoadedUObjects is called which means none of the config properties are in the class property link at that time.
	ReloadConfig();
	EnsureValidCurrentTheme();
	ApplyTheme(USlateThemeManager::Get().GetCurrentTheme().Id);
}

FString USlateThemeManager::GetEngineThemeDir() const
{
	return FPaths::EngineContentDir() / ThemesSubDir;
}

FString USlateThemeManager::GetProjectThemeDir() const
{
	return FPaths::ProjectContentDir() / ThemesSubDir;
}

FString USlateThemeManager::GetUserThemeDir() const
{
	return FPaths::Combine(FPlatformProcess::UserSettingsDir(), *FApp::GetEpicProductIdentifier()) / ThemesSubDir;
}

void USlateThemeManager::LoadThemesFromDirectory(const FString& Directory)
{
	TArray<FString> ThemeFiles;
	IFileManager::Get().FindFiles(ThemeFiles, *Directory, TEXT(".json"));

	for (const FString& ThemeFile : ThemeFiles)
	{
		bool bValidFile = false;
		FString ThemeData;
		FString ThemeFilename = Directory / ThemeFile;
		if (FFileHelper::LoadFileToString(ThemeData, *ThemeFilename))
		{
			FStyleTheme Theme;
			if (ReadTheme(ThemeData, Theme))
			{
				if (FStyleTheme* ExistingTheme = Themes.FindByKey(Theme.Id))
				{
					// Just update the existing theme.  Themes with the same id can override an existing one.  This behavior mimics config file hierarchies
					ExistingTheme->Filename = MoveTemp(ThemeFilename);
				}
				else
				{
					// Theme not found, add a new one
					Theme.Filename = MoveTemp(ThemeFilename);
					Themes.Add(MoveTemp(Theme));
				}
			}
		}
	}
}


bool USlateThemeManager::ReadTheme(const FString& ThemeData, FStyleTheme& Theme)
{
	TSharedRef<TJsonReader<>> ReaderRef = TJsonReaderFactory<>::Create(ThemeData);
	TJsonReader<>& Reader = ReaderRef.Get();

	TSharedPtr<FJsonObject> ObjectPtr;
	if (FJsonSerializer::Deserialize(Reader, ObjectPtr) && ObjectPtr.IsValid())
	{
		int32 Version = 0;
		if (!ObjectPtr->TryGetNumberField(TEXT("Version"), Version))
		{
			// Invalid file
			return false;
		}

		FString IdString;
		if (!ObjectPtr->TryGetStringField(TEXT("Id"), IdString) || !FGuid::Parse(IdString, Theme.Id))
		{
			// Invalid Id;
			return false;
		}

		FString DisplayStr;
		if (!ObjectPtr->TryGetStringField(TEXT("DisplayName"), DisplayStr))
		{
			// Invalid file
			return false;
		}
		Theme.DisplayName = FText::FromString(MoveTemp(DisplayStr));

		// Just check that the theme has colors. We wont load them unless the theme is used
		if (!ObjectPtr->HasField(TEXT("Colors")))
		{
			// No colors
			return false;
		}
	}
	else
	{
		// Log invalid style file
		return false;
	}

	return true;
}

bool USlateThemeManager::WriteCurrentTheme(FString& OutString)
{
	const FStyleTheme& CurrentTheme = GetCurrentTheme();

	TSharedRef<TJsonWriter<>> WriterRef = TJsonWriterFactory<>::Create(&OutString);
	TJsonWriter<>& Writer = WriterRef.Get();
	Writer.WriteObjectStart();
	Writer.WriteValue(TEXT("Version"), 1);
	Writer.WriteValue(TEXT("Id"), CurrentTheme.Id.ToString());
	Writer.WriteValue(TEXT("DisplayName"), CurrentTheme.DisplayName.ToString());

	{
		Writer.WriteObjectStart(TEXT("Colors"));
		UEnum* Enum = StaticEnum<EStyleColor>();
		check(Enum);
		for (int32 ColorIndex = 0; ColorIndex < (int32)EStyleColor::MAX; ++ColorIndex)
		{
			FName EnumName = Enum->GetNameByIndex(ColorIndex);
			Writer.WriteValue(EnumName.ToString(), *ActiveColors.StyleColors[ColorIndex].ToString());
		}
		Writer.WriteObjectEnd();
	}
	Writer.WriteObjectEnd();
	Writer.Close();

	return true;
}

void USlateThemeManager::EnsureValidCurrentTheme()
{
	DefaultDarkTheme.DisplayName = NSLOCTEXT("StyleColors", "DefaultDarkTheme", "Dark");
	// If you change this you invalidate the default dark theme forcing the default theme to be reset and the existing dark theme to become a user theme
	// In general you should not do this. You should instead update the default "dark.json" file 

	DefaultDarkTheme.Id = FGuid(0x13438026, 0x5FBB4A9C, 0xA00A1DC9, 0x770217B8);
	DefaultDarkTheme.Filename = FPaths::EngineContentDir() / TEXT("Slate/Themes/Dark.json");

	int32 ThemeIndex = Themes.AddUnique(DefaultDarkTheme);

	if (!CurrentThemeId.IsValid() || !Themes.Contains(CurrentThemeId))
	{
		CurrentThemeId = DefaultDarkTheme.Id;
	}
}

void USlateThemeManager::LoadThemeColors(FStyleTheme& Theme)
{
	FString ThemeData;

	// Start with the hard coded default colors. They are a fallback if the theme is incomplete 
	if (Theme.LoadedDefaultColors.IsEmpty()) 
	{ 
		Theme.LoadedDefaultColors = MakeArrayView<FLinearColor>(DefaultColors, (int32)EStyleColor::MAX);
	}

	if (FFileHelper::LoadFileToString(ThemeData, *Theme.Filename))
	{
		//Theme.LoadedDefaultColors.Empty();
		TSharedRef<TJsonReader<>> ReaderRef = TJsonReaderFactory<>::Create(ThemeData);
		TJsonReader<>& Reader = ReaderRef.Get();

		TSharedPtr<FJsonObject> ObjectPtr;
		if (FJsonSerializer::Deserialize(Reader, ObjectPtr) && ObjectPtr.IsValid())
		{		
			// Just check that the theme has colors. We wont load them unless the theme is used
			const TSharedPtr<FJsonObject>* ColorsObject = nullptr;
			if(ObjectPtr->TryGetObjectField(TEXT("Colors"), ColorsObject))
			{
				UEnum* Enum = StaticEnum<EStyleColor>();
				check(Enum);
				for (int32 ColorIndex = 0; ColorIndex < (int32)EStyleColor::MAX; ++ColorIndex)
				{
					FName EnumName = Enum->GetNameByIndex(ColorIndex);
					FString ColorString;
					if ((*ColorsObject)->TryGetStringField(EnumName.ToString(), ColorString))
					{
						Theme.LoadedDefaultColors[ColorIndex].InitFromString(ColorString);
					}
				}
			}
		}
	}
}

bool USlateThemeManager::DoesThemeExist(const FGuid& ThemeID) const
{
	for (const FStyleTheme& Theme : Themes)
	{
		if (Theme.Id == ThemeID)
		{
			return true; 
		}
	}
	return false; 
}

/*
const FStyleTheme* UStyleColorTable::GetTheme(const FGuid ThemeId) const
{
	return Themes.FindByKey(ThemeId);
}*/

#endif // ALLOW_THEMES
