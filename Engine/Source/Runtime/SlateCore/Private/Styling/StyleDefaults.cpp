// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/StyleDefaults.h"
#include "Fonts/LegacySlateFontInfoCache.h"

const FSlateBrush* FStyleDefaults::GetNoBrush()
{
	static const FSlateNoResource NoBrush;
	return &NoBrush;
}

const FSlateFontInfo FStyleDefaults::GetFontInfo(uint16 Size)
{
	return FSlateFontInfo(FLegacySlateFontInfoCache::Get().GetDefaultFont(), Size, TEXT("Regular"));
}

float FStyleDefaults::DefaultFloat = 0.0f;
FVector2f FStyleDefaults::DefaultFVector2D = FVector2f::ZeroVector;
FLinearColor FStyleDefaults::DefaultColor = FLinearColor( 1.0f, 1.0f, 1.0f );
FMargin FStyleDefaults::DefaultMargin = 0.0f;
FSlateSound FStyleDefaults::DefaultSound;
FSlateColor FStyleDefaults::DefaultSlateColor = FSlateColor::UseForeground();