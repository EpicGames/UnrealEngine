// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/ToolBarStyle.h"
#include "Brushes/SlateNoResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolBarStyle)

FWrapButtonStyle::FWrapButtonStyle()
	: Padding(FMargin(0.f))
	, WrapButtonIndex(-1) // Default to the right side of a menu
	, SeparatorThickness(2.0f)
	, SeparatorPadding(0)
{
}

FWrapButtonStyle::FWrapButtonStyle(const FWrapButtonStyle&) = default;

void FWrapButtonStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	OutBrushes.Add(&ExpandBrush);
	
	if (const FComboButtonStyle* ComboButton = ComboButtonStyle.GetPtrOrNull())
	{
		ComboButton->GetResources(OutBrushes);
	}
	
	if (const FSlateBrush* Brush = SeparatorBrush.GetPtrOrNull())
	{
		OutBrushes.Add(Brush);
	}
}


const FName FToolBarStyle::TypeName(TEXT("FToolbarStyle"));

FToolBarStyle::FToolBarStyle()
	: BackgroundBrush(FSlateNoResource())
	, ExpandBrush_DEPRECATED(FSlateNoResource())
	, SeparatorBrush(FSlateNoResource())
	, LabelStyle()
	, EditableTextStyle()
	, ToggleButton()
	, SettingsComboButton()
	, ButtonStyle()
	, LabelPadding(0)
	, UniformBlockWidth(0.f)
	, UniformBlockHeight(0.f)
	, NumColumns(0)
	, IconPadding(0)
	, SeparatorPadding(0)
	, SeparatorThickness(2.0f)
	, ComboButtonPadding(0)
	, ButtonPadding(0)
	, CheckBoxPadding(0)
	, BlockPadding(0)
	, IndentedBlockPadding(0)
	, BlockHovered(FSlateNoResource())
	, BackgroundPadding(0)
	, WrapButtonPadding_DEPRECATED(0)
	, WrapButtonIndex_DEPRECATED(-1)
	, IconSize(16, 16)
	, bShowLabels(true)
	, ComboContentMinWidth(0)
	, ComboContentMaxWidth(0)
	, ComboContentHorizontalAlignment(HAlign_Fill)
{}

FToolBarStyle::FToolBarStyle(const FToolBarStyle&) = default;

FToolBarStyle::~FToolBarStyle() = default;

const FToolBarStyle& FToolBarStyle::GetDefault()
{
	static FToolBarStyle Default;
	return Default;
}

void FToolBarStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	OutBrushes.Add(&BackgroundBrush);
	OutBrushes.Add(&ExpandBrush_DEPRECATED);
	OutBrushes.Add(&SeparatorBrush);
	OutBrushes.Add(&BlockHovered);

	LabelStyle.GetResources(OutBrushes);
	EditableTextStyle.GetResources(OutBrushes);
	ToggleButton.GetResources(OutBrushes);
	SettingsComboButton.GetResources(OutBrushes);
	SettingsToggleButton.GetResources(OutBrushes);
	SettingsButtonStyle.GetResources(OutBrushes);
	ButtonStyle.GetResources(OutBrushes);
	WrapButtonStyle.GetResources(OutBrushes);
}
