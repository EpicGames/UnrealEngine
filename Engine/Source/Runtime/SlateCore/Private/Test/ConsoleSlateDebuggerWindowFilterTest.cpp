// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "SlateCoreModule.h"
#include "Debugging/ConsoleSlateDebuggerPaint.h"
#include "Debugging/ConsoleSlateDebuggerPrepass.h"
#include "Debugging/ConsoleSlateDebuggerUpdate.h"
#include "Misc/AutomationTest.h"
#include "Widgets/SWindow.h"
#include "Widgets/SCompoundWidget.h"
#include "Rendering/DrawElements.h"
#include "Debugging/SlateDebugging.h"
#include "HAL/IConsoleManager.h"
#include "Input/HittestGrid.h"
#include "Types/PaintArgs.h"
#include "Misc/App.h"

#if WITH_AUTOMATION_WORKER && WITH_SLATE_DEBUGGING

namespace
{
	/**
	 * Test widget that exposes SetIsProjectContent for testing content-based filtering.
	 * Simulates the behavior of widgets inside SGameLayerManager (IsProjectContent=true) vs widgets outside it (IsProjectContent=false).
	 */
	class SContentTestWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SContentTestWidget) {}
		SLATE_END_ARGS()
		void Construct(const FArguments&) {}
		using SWidget::SetIsProjectContent;

		/** Set AllottedGeometry on the widget's persistent state for testing.
		 *  Normally set by SWidget::Paint(), but test widgets are never painted by the framework.
		 */
		void SetTestAllottedGeometry(const FGeometry& InGeometry)
		{
			const_cast<FSlateWidgetPersistentState&>(GetPersistentState()).AllottedGeometry = InGeometry;
		}
	};

	int32 CountDrawElements(const FSlateDrawElementMap& ElementMap)
	{
		int32 Total = 0;
		VisitTupleElements([&](const auto& ElementType) { Total += ElementType.Num(); }, ElementMap);
		return Total;
	}

	int32 CountPasses(FConsoleSlateDebuggerPassBase& PassDebugger, FConsoleSlateDebuggerUtility::TSWindowId Window)
	{
		int32 Total = 0;
		for (auto& Widget : PassDebugger.GetWidgets())
		{
			if (Widget.Value.Window == Window)
			{
				++Total;
			}
		}
		return Total;
	}
}

namespace UE::Slate::Test
{
	constexpr EAutomationTestFlags TestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext |
		EAutomationTestFlags::ProgramContext | EAutomationTestFlags::LowPriority | EAutomationTestFlags::EngineFilter;

	/**
	 * Integration test for Console Slate Debugger bDebugProjectContentOnly content filtering.
	 * Verifies that when the filter is enabled, only content widgets (those within SGameLayerManager) are tracked, while non-content widgets are
	 * filtered out.
	 */
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FConsoleSlateDebuggerContentFilterTest, "Slate.Debugger.ContentFilter", TestFlags)

	bool FConsoleSlateDebuggerContentFilterTest::RunTest(const FString& Parameters)
	{
		// Get console variables to control the debuggers
		IConsoleVariable* PaintEnableCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SlateDebugger.Paint.Enable"));
		IConsoleVariable* PaintOnlyContentCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SlateDebugger.Paint.OnlyProjectContent"));
		IConsoleVariable* PrepassEnableCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SlateDebugger.Prepass.Enable"));
		IConsoleVariable* PrepassOnlyContentCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SlateDebugger.Prepass.OnlyProjectContent"));
		IConsoleVariable* UpdateEnableCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SlateDebugger.Update.Enable"));
		IConsoleVariable* UpdateOnlyContentCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SlateDebugger.Update.OnlyProjectContent"));

		if (!PaintEnableCVar || !PaintOnlyContentCVar || !PrepassEnableCVar || !PrepassOnlyContentCVar
			|| !UpdateEnableCVar || !UpdateOnlyContentCVar)
		{
			AddError(TEXT("Could not find SlateDebugger console variables"));
			return false;
		}

		const int32 OriginalUpdateEnabled = UpdateEnableCVar->GetInt();
		const int32 OriginalUpdateOnlyContent = UpdateOnlyContentCVar->GetInt();
		const int32 OriginalPaintEnabled = PaintEnableCVar->GetInt();
		const int32 OriginalPaintOnlyContent = PaintOnlyContentCVar->GetInt();
		const int32 OriginalPrepassEnabled = PrepassEnableCVar->GetInt();
		const int32 OriginalPrepassOnlyContent = PrepassOnlyContentCVar->GetInt();

		// Disable legend so Update debugger draw elements come only from tracked widgets,
		// allowing us to observe the content filter.
		FSlateCoreModule& Module = FSlateCoreModule::Get();
		FConsoleSlateDebuggerUpdate& UpdateDebugger = Module.GetUpdateDebugger();
		const bool bLegendWasEnabled = UpdateDebugger.IsLegendShown();
		if (bLegendWasEnabled)
		{
			UpdateDebugger.ToggleDisplayLegend();
		}

		auto TestContentFilter = [&](bool bWidgetIsProjectContent, bool bOnlyContent, bool bExpectRecorded, const TCHAR* Description)
		{
			TSharedPtr<SContentTestWidget> TestWidget;
			TSharedRef<SWindow> TestWindow = SNew(SWindow)
				.ClientSize(FVector2f(100, 100))
				.CreateTitleBar(false)
				.SizingRule(ESizingRule::FixedSize)
				[
					SAssignNew(TestWidget, SContentTestWidget)
				];

			if (bWidgetIsProjectContent)
			{
				TestWidget->SetIsProjectContent(true);
			}

			// Give the widget non-zero geometry so that FSlateDrawElement::MakeBox does not cull it.
			TestWidget->SetTestAllottedGeometry(FGeometry::MakeRoot(FVector2f(50, 50), FSlateLayoutTransform()));

			FSlateWindowElementList ElementList(TestWindow);
			const FConsoleSlateDebuggerUtility::TSWindowId WindowId = FConsoleSlateDebuggerUtility::GetId(TestWindow.Get());

			// Paint
			PaintOnlyContentCVar->Set(bOnlyContent ? 1 : 0, ECVF_SetByTemp);
			PaintEnableCVar->Set(1);

			FConsoleSlateDebuggerPaint& PaintDebugger = Module.GetPaintDebugger();

			const int32 PaintsBefore = CountPasses(PaintDebugger, WindowId);
			FSlateDebugging::EndWidgetPaint.Broadcast(TestWidget.Get(), ElementList, 0);
			const int32 PaintsAfter = CountPasses(PaintDebugger, WindowId);
			const bool bPaintsAdded = PaintsAfter > PaintsBefore;

			if (bExpectRecorded && !bPaintsAdded)
			{
				AddError(FString::Printf(TEXT("%s: Expected debug paints to be added (widget should pass filter), but none were"), Description));
			}
			else if (!bExpectRecorded && bPaintsAdded)
			{
				AddError(FString::Printf(TEXT("%s: Expected widget to be filtered (no paints), but %d paints were added"),
					Description, PaintsAfter - PaintsBefore));
			}

			PaintEnableCVar->Set(0);
			PaintOnlyContentCVar->Unset(ECVF_SetByTemp);

			// Prepass
			PrepassOnlyContentCVar->Set(bOnlyContent ? 1 : 0, ECVF_SetByTemp);
			PrepassEnableCVar->Set(1);

			FConsoleSlateDebuggerPrepass& PrepassDebugger = Module.GetPrepassDebugger();

			const int32 PassesBefore = CountPasses(PrepassDebugger, WindowId);
			FSlateDebugging::EndWidgetPrepass.Broadcast(TestWidget.Get());
			const int32 PassesAfter = CountPasses(PrepassDebugger, WindowId);
			const bool bPassesAdded = PassesAfter > PassesBefore;

			if (bExpectRecorded && !bPassesAdded)
			{
				AddError(FString::Printf(TEXT("%s: Expected debug passes to be added (widget should pass filter), but none were"), Description));
			}
			else if (!bExpectRecorded && bPassesAdded)
			{
				AddError(FString::Printf(TEXT("%s: Expected widget to be filtered (no passes), but %d passes were added"),
					Description, PassesAfter - PassesBefore));
			}

			PrepassEnableCVar->Set(0);
			PrepassOnlyContentCVar->Unset(ECVF_SetByTemp);

			// Update
			UpdateOnlyContentCVar->Set(bOnlyContent ? 1 : 0, ECVF_SetByTemp);
			UpdateEnableCVar->Set(1);

			const int32 ElementsBefore = CountDrawElements(ElementList.GetUncachedDrawElements());
			FSlateDebugging::BroadcastWidgetUpdated(TestWidget.Get(), EWidgetUpdateFlags::NeedsRepaint);
			FHittestGrid HittestGrid;
			FPaintArgs PaintArgs(&TestWindow.Get(), HittestGrid, FVector2f::ZeroVector, FApp::GetCurrentTime(), FApp::GetDeltaTime());
			FGeometry Geometry = FGeometry::MakeRoot(FVector2f(100, 100), FSlateLayoutTransform());
			int32 LayerId = 0;
			FSlateDebugging::PaintDebugElements.Broadcast(PaintArgs, Geometry, ElementList, LayerId);
			const int32 ElementsAfter = CountDrawElements(ElementList.GetUncachedDrawElements());
			const bool bElementsAdded = ElementsAfter > ElementsBefore;

			if (bExpectRecorded && !bElementsAdded)
			{
				AddError(FString::Printf(TEXT("%s: Expected debug elements to be added (widget should pass filter), but none were"), Description));
			}
			else if (!bExpectRecorded && bElementsAdded)
			{
				AddError(FString::Printf(TEXT("%s: Expected widget to be filtered (no elements), but %d elements were added"),
					Description, ElementsAfter - ElementsBefore));
			}

			// Disable debugger between tests to reset state
			UpdateEnableCVar->Set(0);
			UpdateOnlyContentCVar->Unset(ECVF_SetByTemp);
		};

		// Content widgets (simulating inside SGameLayerManager) should pass through filter
		TestContentFilter(/* bWidgetIsContent */ true, /* bOnlyContent */ true, /* bExpect */ true,
			TEXT("Content widget with filter=true"));
		// Non-content widgets (simulating outside SGameLayerManager) should be filtered out
		TestContentFilter(/* bWidgetIsContent */ false, /* bOnlyContent */ true, /* bExpect */ false,
			TEXT("Non-content widget with filter=true"));

		// With filter disabled, all widgets should pass through regardless of content status
		TestContentFilter(/* bWidgetIsContent */ true, /* bOnlyContent */ false, /* bExpect */ true,
			TEXT("Content widget with filter=false"));
		TestContentFilter(/* bWidgetIsContent */ false, /* bOnlyContent */ false, /* bExpect */ true,
			TEXT("Non-content widget with filter=false"));

		// Restore original settings
		if (bLegendWasEnabled)
		{
			UpdateDebugger.ToggleDisplayLegend();
		}
		UpdateEnableCVar->Set(OriginalUpdateEnabled);
		UpdateOnlyContentCVar->Set(OriginalUpdateOnlyContent);
		PaintEnableCVar->Set(OriginalPaintEnabled);
		PaintOnlyContentCVar->Set(OriginalPaintOnlyContent);
		PrepassEnableCVar->Set(OriginalPrepassEnabled);
		PrepassOnlyContentCVar->Set(OriginalPrepassOnlyContent);

		return true;
	}
}

#endif // WITH_AUTOMATION_WORKER && WITH_SLATE_DEBUGGING
