// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Containers/ObservableArray.h"

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace UE::Test
{
	
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FObservableArrayTest, "Slate.ObservableArray", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

/**
 *
 */
bool FObservableArrayTest::RunTest(const FString& Parameters)
{
	::UE::Slate::Containers::EObservableArrayChangedAction ExpectedAction = ::UE::Slate::Containers::EObservableArrayChangedAction::Add;
	int32 ExpectedCounter = 0;
	int32 Counter = 0;
	TArray<int32> ExpectedArray;
	FObservableArrayTest* Self = this;
	auto TestExpectedAction = [Self, &ExpectedAction, &Counter, &ExpectedArray](::UE::Slate::Containers::TObservableArray<int32>::ObservableArrayChangedArgsType Args)
	{
		Self->AddErrorIfFalse(Args.GetAction() == ExpectedAction, TEXT("The notification occurs with the wrong action"));
		if (Args.GetAction() == Slate::Containers::EObservableArrayChangedAction::Add
			|| Args.GetAction() == Slate::Containers::EObservableArrayChangedAction::Remove
			|| Args.GetAction() == Slate::Containers::EObservableArrayChangedAction::RemoveSwap)
		{
			Self->AddErrorIfFalse(ExpectedArray == Args.GetItems(), TEXT("The notification occurs with the GetItems"));
		}
		++Counter;
	};

	{
		{
			TArray<int32> ArrayValues;
			::UE::Slate::Containers::TObservableArray<int32> ObservableValues;
			ObservableValues.OnArrayChanged().AddLambda(TestExpectedAction);
			{
				ExpectedAction = ::UE::Slate::Containers::EObservableArrayChangedAction::Add;

				{
					ExpectedArray.Reset();
					ExpectedArray.Add(2);

					ArrayValues.Add(2);
					ObservableValues.Add(2);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
				{
					ExpectedArray.Reset();
					ExpectedArray.Add(3);

					ArrayValues.Emplace(3);
					ObservableValues.Emplace(3);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
				{
					ExpectedArray.Reset();
					ExpectedArray.Add(4);

					ArrayValues.EmplaceAt(2, 4);
					ObservableValues.EmplaceAt(2, 4);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
				{
					ExpectedArray.Reset();
					ExpectedArray.Add(1);

					ArrayValues.EmplaceAt(0, 1);
					ObservableValues.EmplaceAt(0, 1);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
				{
					ExpectedArray.Reset();
					ExpectedArray.Add(5);

					ArrayValues.Add(5);
					ObservableValues.Add(5);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
				{
					ExpectedArray.Reset();
					ExpectedArray.Append({ 6, 7, 8, 9, 10, 11, 12 });

					ArrayValues.Append(ExpectedArray);
					ObservableValues.Append(ExpectedArray);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
				{
					ExpectedArray.Reset();
					ExpectedArray.Add(9);

					ArrayValues.Append(ExpectedArray);
					ObservableValues.Append(ExpectedArray);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
			}
			{
				int32 ExpectedFirstIndex = INDEX_NONE;
				int32 ExpectedSecondIndex = INDEX_NONE;
				TArray<int32> PreviousObservableArray;
				auto TestExpectedSwapAction = [Self, &ExpectedFirstIndex, &ExpectedSecondIndex, &ObservableValues, &PreviousObservableArray](::UE::Slate::Containers::TObservableArray<int32>::ObservableArrayChangedArgsType Args)
				{
					if (Args.GetAction() == Slate::Containers::EObservableArrayChangedAction::Swap)
					{
						int32 FirstIndex = Args.GetSwapIndex().FirstIndex;
						int32 SecondIndex = Args.GetSwapIndex().SecondIndex;

						Self->AddErrorIfFalse(FirstIndex != SecondIndex, TEXT("SecondIndex != FirstIndex."));
						Self->AddErrorIfFalse(ObservableValues.IsValidIndex(FirstIndex) && ObservableValues.IsValidIndex(SecondIndex), TEXT("The swap index are invalid."));
						Self->AddErrorIfFalse(ObservableValues[FirstIndex] == PreviousObservableArray[SecondIndex], TEXT("The swap of FirstIndex do not matches"));
						Self->AddErrorIfFalse(ObservableValues[SecondIndex] == PreviousObservableArray[FirstIndex], TEXT("The swap of SecondIndex do not matches"));
					}
				};

				ExpectedAction = ::UE::Slate::Containers::EObservableArrayChangedAction::Swap;
				FDelegateHandle SwapHandle = ObservableValues.OnArrayChanged().AddLambda(TestExpectedSwapAction);
				{
					ExpectedArray.Reset();
					PreviousObservableArray = TArray<int32>(ObservableValues.GetData(), ObservableValues.Num());
					ExpectedFirstIndex = 1;
					ExpectedSecondIndex = 2;

					ArrayValues.Swap(ExpectedFirstIndex, ExpectedSecondIndex);
					ObservableValues.Swap(ExpectedFirstIndex, ExpectedSecondIndex);
					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, "The delegate was not executed");
				}
				{
					ExpectedArray.Reset();
					PreviousObservableArray = TArray<int32>(ObservableValues.GetData(), ObservableValues.Num());
					ExpectedFirstIndex = 3;
					ExpectedSecondIndex = 1;

					ArrayValues.Swap(ExpectedFirstIndex, ExpectedSecondIndex);
					ObservableValues.Swap(ExpectedFirstIndex, ExpectedSecondIndex);
					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, "The delegate was not executed");
				}
				ObservableValues.OnArrayChanged().Remove(SwapHandle);
			}
			{
				ExpectedAction = ::UE::Slate::Containers::EObservableArrayChangedAction::Remove;

				{
					ExpectedArray.Reset();
					ExpectedArray.Add(2);

					ArrayValues.RemoveSingle(2);
					ObservableValues.RemoveSingle(2);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
				{
					bool bContains = ObservableValues.Contains(2);
					if (bContains)
					{
						ExpectedArray.Reset();
						ExpectedArray.Add(2);
					}
					ArrayValues.RemoveSingle(2);
					bool bWasRemoved = ObservableValues.RemoveSingle(2) > 0;
					if (bWasRemoved)
					{
						++ExpectedCounter;
					}
					AddErrorIfFalse(bWasRemoved == bContains, TEXT("ObservableValues.Contains == ObservableValues.RemoveSingle"));
					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
				{
					ExpectedArray.Reset();
					ExpectedArray.Add(ObservableValues[2]);

					ArrayValues.RemoveAt(2);
					ObservableValues.RemoveAt(2);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
				{
					ExpectedArray.Reset();
					int32 ToRemoveIndex = 2;
					int32 NumberToRemove = 3;
					for (int32 Index = ToRemoveIndex; Index < NumberToRemove + ToRemoveIndex; ++Index)
					{
						if (ObservableValues.IsValidIndex(Index))
						{
							ExpectedArray.Add(ObservableValues[Index]);
						}
					}

					ArrayValues.RemoveAt(ToRemoveIndex, NumberToRemove);
					ObservableValues.RemoveAt(ToRemoveIndex, NumberToRemove);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
			}
			{
				ExpectedAction = ::UE::Slate::Containers::EObservableArrayChangedAction::Reset;

				ArrayValues.Reset();
				ObservableValues.Reset();
				AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
				++ExpectedCounter;
				AddErrorIfFalse(Counter == ExpectedCounter, "The delegate was not executed");
			}
			{
				ExpectedAction = ::UE::Slate::Containers::EObservableArrayChangedAction::Add;

				{
					ExpectedArray.Reset();
					ExpectedArray.Append({ 6, 6, 7, 7, 7, 8, 9, 10, 11, 12, 12 });

					ArrayValues.Append(ExpectedArray);
					ObservableValues.Append(ExpectedArray);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
			}
			{
				ExpectedAction = ::UE::Slate::Containers::EObservableArrayChangedAction::RemoveSwap;

				bool bExpectPreviousMovedElemenentIndexToBeValid = true;
				int32 ExpectedRemoveIndex = INDEX_NONE;
				TArray<int32> ExpectedBeforeSwapArray;
				auto TestExpectedSwapAction = [Self, &bExpectPreviousMovedElemenentIndexToBeValid , &ExpectedRemoveIndex, &ExpectedBeforeSwapArray, &ObservableValues](::UE::Slate::Containers::TObservableArray<int32>::ObservableArrayChangedArgsType Args)
				{
					if (Args.GetAction() == Slate::Containers::EObservableArrayChangedAction::RemoveSwap)
					{
						int32 RemoveIndex = Args.GetRemovedSwapIndexes().RemoveIndex;
						int32 PreviousMovedElmenentIndex = Args.GetRemovedSwapIndexes().PreviousMovedElmenentIndex;

						Self->AddErrorIfFalse(ExpectedRemoveIndex == RemoveIndex, TEXT("The RemoveIndex is not the expected value."));
						if (ObservableValues.Num() > 0)
						{
							if (bExpectPreviousMovedElemenentIndexToBeValid)
							{
								Self->AddErrorIfFalse(ExpectedBeforeSwapArray.IsValidIndex(PreviousMovedElmenentIndex), TEXT("PreviousMovedElmenentIndex is invalid."));
							}
							else
							{
								Self->AddErrorIfFalse(!ExpectedBeforeSwapArray.IsValidIndex(PreviousMovedElmenentIndex), TEXT("PreviousMovedElmenentIndex is invalid."));
							}

							if (ObservableValues.IsValidIndex(RemoveIndex)
								&& ExpectedBeforeSwapArray.IsValidIndex(PreviousMovedElmenentIndex))
							{
								Self->AddErrorIfFalse(ObservableValues[RemoveIndex] == ExpectedBeforeSwapArray[PreviousMovedElmenentIndex], TEXT("The swap index doesn't matches"));
							}
						}
					}
				};
				FDelegateHandle RemoveSwapHandle = ObservableValues.OnArrayChanged().AddLambda(TestExpectedSwapAction);

				{
					ExpectedArray.Reset();
					ExpectedArray.Append({7});
					ExpectedBeforeSwapArray = TArray<int32>(ObservableValues.GetData(), ObservableValues.Num());
					ExpectedRemoveIndex = ArrayValues.Find(7);
					bExpectPreviousMovedElemenentIndexToBeValid = true;

					ArrayValues.RemoveSingleSwap(7);
					ObservableValues.RemoveSingleSwap(7);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
				{
					ExpectedArray.Reset();
					ExpectedBeforeSwapArray = TArray<int32>(ObservableValues.GetData(), ObservableValues.Num());
					ExpectedRemoveIndex = 0;
					ExpectedArray.Add(ArrayValues[ExpectedRemoveIndex]);
					bExpectPreviousMovedElemenentIndexToBeValid = true;

					ArrayValues.RemoveAtSwap(ExpectedRemoveIndex);
					ObservableValues.RemoveAtSwap(ExpectedRemoveIndex);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
				{
					ExpectedArray.Reset();
					ExpectedBeforeSwapArray = TArray<int32>(ObservableValues.GetData(), ObservableValues.Num());
					ExpectedRemoveIndex = 1;
					for (int32 Index = 0; Index < 4; ++Index)
					{
						ExpectedArray.Add(ArrayValues[Index+ ExpectedRemoveIndex]);
					}
					bExpectPreviousMovedElemenentIndexToBeValid = true;

					ArrayValues.RemoveAtSwap(ExpectedRemoveIndex, 4);
					ObservableValues.RemoveAtSwap(ExpectedRemoveIndex, 4);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}
				{
					ExpectedArray.Reset();
					ExpectedBeforeSwapArray = TArray<int32>(ObservableValues.GetData(), ObservableValues.Num());
					ExpectedRemoveIndex = ArrayValues.Num() - 2;
					for (int32 Index = 0; Index < 2; ++Index)
					{
						ExpectedArray.Add(ArrayValues[Index+ ExpectedRemoveIndex]);
					}
					bExpectPreviousMovedElemenentIndexToBeValid = false;

					ArrayValues.RemoveAtSwap(ExpectedRemoveIndex, 2);
					ObservableValues.RemoveAtSwap(ExpectedRemoveIndex, 2);

					AddErrorIfFalse(ArrayValues == ObservableValues, TEXT("ObservableValues == ArrayValues"));
					++ExpectedCounter;
					AddErrorIfFalse(Counter == ExpectedCounter, TEXT("The delegate was not executed"));
				}

				ObservableValues.OnArrayChanged().Remove(RemoveSwapHandle);
			}
			{
				TArray<int32> ItteratingArrayValues;
				ItteratingArrayValues.Reserve(ObservableValues.Num());
				for (int32 Value : ObservableValues)
				{
					ItteratingArrayValues.Add(Value);
				}
				AddErrorIfFalse(ItteratingArrayValues == ArrayValues, TEXT("ItteratingArrayValues == ArrayValues after itterator."));
			}
		}
	}

	return true;
}
} //namespace

#endif //WITH_DEV_AUTOMATION_TESTS

