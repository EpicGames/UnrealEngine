// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Widgets/SNullWidget.h"

#include <type_traits>
#include <limits>

#if WITH_AUTOMATION_WORKER

#define LOCTEXT_NAMESPACE "Slate.Attribute"

//IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSlateAttributeTest, "Slate.Attribute", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSlateAttributeTest, "Slate.Attribute", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

namespace UE::Slate::Private
{

struct FConstructionCounter
{
	FConstructionCounter() : Value(0) { ++DefaultConstructionCounter; }
	FConstructionCounter(int32 InValue) : Value(InValue) { ++DefaultConstructionCounter; }
	FConstructionCounter(const FConstructionCounter& Other) : Value(Other.Value) { ++CopyConstructionCounter; }
	FConstructionCounter(FConstructionCounter&& Other) :Value(Other.Value) { ++MoveConstructionCounter; }
	FConstructionCounter& operator=(const FConstructionCounter& Other) { Value = Other.Value; ++CopyOperatorCounter; return *this; }
	FConstructionCounter& operator=(FConstructionCounter&& Other) { Value = Other.Value; ++MoveOperatorCounter;  return *this; }

	int32 Value;
	bool operator== (const FConstructionCounter& Other) const { return Other.Value == Value; }

	static int32 DefaultConstructionCounter;
	static int32 CopyConstructionCounter;
	static int32 MoveConstructionCounter;
	static int32 CopyOperatorCounter;
	static int32 MoveOperatorCounter;
	static void ResetCounter()
	{
		DefaultConstructionCounter = 0;
		CopyConstructionCounter = 0;
		MoveConstructionCounter = 0;
		CopyOperatorCounter = 0;
		MoveOperatorCounter = 0;
	}
};
int32 FConstructionCounter::DefaultConstructionCounter = 0;
int32 FConstructionCounter::CopyConstructionCounter = 0;
int32 FConstructionCounter::MoveConstructionCounter = 0;
int32 FConstructionCounter::CopyOperatorCounter = 0;
int32 FConstructionCounter::MoveOperatorCounter = 0;


int32 CallbackForIntAttribute(int32 Value)
{
	return Value;
}
FVector2D CallbackForFVectorAttribute()
{
	return FVector2D(1,1);
}

/**
 * 
 */
class SAttributeLeftWidget_Parent : public SLeafWidget
{
	SLATE_DECLARE_WIDGET(SAttributeLeftWidget_Parent, SLeafWidget)
public:
	SLATE_BEGIN_ARGS(SAttributeLeftWidget_Parent) {}
	SLATE_END_ARGS()


	SAttributeLeftWidget_Parent()
		: IntAttributeA(*this, 99)
		, IntAttributeB(*this, 99)
		, IntAttributeC(*this, 99)
		, IntAttributeD(*this, 99)
	{
		static_assert(std::is_same<TSlateAttribute<bool>, typename TSlateAttributeRef<bool>::SlateAttributeType>::value, "TSlateAttributeRef doesn't have the same type as TSlateAttribute for bool");
		static_assert(std::is_same<TSlateAttribute<int32>, typename TSlateAttributeRef<int32>::SlateAttributeType>::value, "TSlateAttributeRef doesn't have the same type as TSlateAttribute for int32");
		static_assert(std::is_same<TSlateAttribute<FText>, typename TSlateAttributeRef<FText>::SlateAttributeType>::value, "TSlateAttributeRef doesn't have the same type as TSlateAttribute for FText");
		static_assert(std::is_same<TSlateAttribute<FVector>, typename TSlateAttributeRef<FVector>::SlateAttributeType>::value, "TSlateAttributeRef doesn't have the same type as TSlateAttribute for FVector");
	}

	void Construct(const FArguments& InArgs){}
	virtual FVector2D ComputeDesiredSize(float) const override { return FVector2D{ 100, 100 }; }
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		return LayerId;
	}


	TSlateAttribute<int32> IntAttributeA;
	TSlateAttribute<int32> IntAttributeB;
	TSlateAttribute<int32> IntAttributeC;
	TSlateAttribute<int32> IntAttributeD;
	TArray<TSlateManagedAttribute<int32, EInvalidateWidgetReason::ChildOrder>> IntManagedAttributes;
};

SLATE_IMPLEMENT_WIDGET(SAttributeLeftWidget_Parent)
void SAttributeLeftWidget_Parent::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	// The update order is B, A, D, C
	// C updates when D is invalidated, so D needs to be before C
	// A updates after B, so B needs to be before A
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeD, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeC, EInvalidateWidgetReason::Layout)
		.UpdatePrerequisite(GET_MEMBER_NAME_CHECKED(PrivateThisType, IntAttributeD));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeB, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeA, EInvalidateWidgetReason::Layout)
		.UpdatePrerequisite(GET_MEMBER_NAME_CHECKED(PrivateThisType, IntAttributeB));

	AttributeInitializer.OverrideInvalidationReason(GET_MEMBER_NAME_CHECKED(PrivateThisType, IntAttributeD), FSlateAttributeDescriptor::FInvalidateWidgetReasonAttribute{ EInvalidateWidgetReason::Paint});
}


/**
 * 
 */
class SAttributeLeftWidget_Child : public SAttributeLeftWidget_Parent
{
	SLATE_DECLARE_WIDGET(SAttributeLeftWidget_Child, SAttributeLeftWidget_Parent)
public:
	SLATE_BEGIN_ARGS(SAttributeLeftWidget_Child) {}
	SLATE_END_ARGS()


	SAttributeLeftWidget_Child()
		: IntAttributeH(*this, 99)
		, IntAttributeI(*this, 99)
		, IntAttributeJ(*this, 99)
		, IntAttributeK(*this, 99)
		, IntAttributeL(*this, 99)
		, IntAttributeM(*this, 99)
	{
	}

	void Construct(const FArguments& InArgs){}

	TSlateAttribute<int32, EInvalidateWidgetReason::Layout> IntAttributeH;
	TSlateAttribute<int32> IntAttributeI;
	TSlateAttribute<int32> IntAttributeJ;
	TSlateAttribute<int32> IntAttributeK;
	TSlateAttribute<int32> IntAttributeL;
	TSlateAttribute<int32> IntAttributeM;
};

SLATE_IMPLEMENT_WIDGET(SAttributeLeftWidget_Child)
void SAttributeLeftWidget_Child::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	// The update order is M, B, A, I, J, D, C, L, H, K
	//SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeH, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeJ, EInvalidateWidgetReason::Layout)
		.UpdatePrerequisite("IntAttributeA");
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeK, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeI, EInvalidateWidgetReason::Layout)
		.UpdatePrerequisite("IntAttributeB");
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeL, EInvalidateWidgetReason::Layout)
		.UpdatePrerequisite("IntAttributeC");
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeM, EInvalidateWidgetReason::Visibility)
		.UpdatePrerequisite("Visibility")
		.AffectVisibility();
}

/**
 * 
 */
class SAttributeLeftWidget_OnInvalidationParent : public SLeafWidget
{
	SLATE_DECLARE_WIDGET(SAttributeLeftWidget_OnInvalidationParent, SLeafWidget)
public:
	SLATE_BEGIN_ARGS(SAttributeLeftWidget_OnInvalidationParent) {}
	SLATE_END_ARGS()


	SAttributeLeftWidget_OnInvalidationParent()
		: IntAttributeA(*this, 99)
		, IntAttributeB(*this, 99)
		, IntAttributeC(*this, 99)
	{
	}

	using SWidget::IsConstructed;

	void Construct(const FArguments& InArgs)
	{
		IntAttributeA.Assign(*this, 88);
		IntAttributeB.Assign(*this, 88);
		IntAttributeC.Assign(*this, 88);
	}
	virtual FVector2D ComputeDesiredSize(float) const override { return FVector2D{ 100, 100 }; }
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		return LayerId;
	}

	void SetAttributeA(TAttribute<int32> ValueA) { IntAttributeA.Assign(*this, MoveTemp(ValueA)); }
	void SetAttributeB(TAttribute<int32> ValueB) { IntAttributeB.Assign(*this, MoveTemp(ValueB)); }
	void SetAttributeC(TAttribute<int32> ValueC) { IntAttributeC.Assign(*this, MoveTemp(ValueC)); }
	virtual void SetCallbackValue(int32 Value) { CallbackValueA = Value; CallbackValueB = Value; CallbackValueC = Value; }

	TSlateAttribute<int32> IntAttributeA;
	TSlateAttribute<int32> IntAttributeB;
	TSlateAttribute<int32> IntAttributeC;
	int32 CallbackValueA = 555;
	int32 CallbackValueB = 555;
	int32 CallbackValueC = 555;
	int32 CallbackIsConstructedValueA = 0;
	int32 CallbackIsConstructedValueB = 0;
	int32 CallbackIsConstructedValueC = 0;
};

SLATE_IMPLEMENT_WIDGET(SAttributeLeftWidget_OnInvalidationParent)
void SAttributeLeftWidget_OnInvalidationParent::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeA, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				SAttributeLeftWidget_OnInvalidationParent& Instance = static_cast<SAttributeLeftWidget_OnInvalidationParent&>(Widget);
				Instance.CallbackValueA = Instance.IntAttributeA.Get();
				Instance.CallbackIsConstructedValueA += static_cast<SAttributeLeftWidget_OnInvalidationParent&>(Widget).IsConstructed() ? 1 : 0;
			}));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeB, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				SAttributeLeftWidget_OnInvalidationParent& Instance = static_cast<SAttributeLeftWidget_OnInvalidationParent&>(Widget);
				Instance.CallbackValueB = Instance.IntAttributeB.Get();
				Instance.CallbackIsConstructedValueB += static_cast<SAttributeLeftWidget_OnInvalidationParent&>(Widget).IsConstructed() ? 1 : 0;
			}));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, IntAttributeC, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				SAttributeLeftWidget_OnInvalidationParent& Instance = static_cast<SAttributeLeftWidget_OnInvalidationParent&>(Widget);
				Instance.CallbackValueC = Instance.IntAttributeC.Get();
				Instance.CallbackIsConstructedValueC += static_cast<SAttributeLeftWidget_OnInvalidationParent&>(Widget).IsConstructed() ? 1 : 0;
			}));
}

/*
 * 
 */
class SAttributeLeftWidget_OnInvalidationChild : public SAttributeLeftWidget_OnInvalidationParent
{
	SLATE_DECLARE_WIDGET(SAttributeLeftWidget_OnInvalidationChild, SAttributeLeftWidget_OnInvalidationParent)
public:
	SLATE_BEGIN_ARGS(SAttributeLeftWidget_OnInvalidationChild) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs) {}
	virtual FVector2D ComputeDesiredSize(float) const override { return FVector2D{ 100, 100 }; }
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		return LayerId;
	}

	virtual void SetCallbackValue(int32 Value) override
	{
		Super::SetCallbackValue(Value);
		OverrideCallbackValueA = Value;
		OverrideCallbackValueB = Value;
		OverrideCallbackValueC = Value;
	}

	int32 OverrideCallbackValueA;
	int32 OverrideCallbackValueB;
	int32 OverrideCallbackValueC;
};

SLATE_IMPLEMENT_WIDGET(SAttributeLeftWidget_OnInvalidationChild)
void SAttributeLeftWidget_OnInvalidationChild::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	AttributeInitializer.OverrideOnValueChanged("IntAttributeA"
		, FSlateAttributeDescriptor::ECallbackOverrideType::ReplacePrevious
		, FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				SAttributeLeftWidget_OnInvalidationChild& Instance = static_cast<SAttributeLeftWidget_OnInvalidationChild&>(Widget);
				Instance.OverrideCallbackValueA = Instance.IntAttributeA.Get();
			}));
	AttributeInitializer.OverrideOnValueChanged("IntAttributeB"
		, FSlateAttributeDescriptor::ECallbackOverrideType::ExecuteAfterPrevious
		, FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				SAttributeLeftWidget_OnInvalidationChild& Instance = static_cast<SAttributeLeftWidget_OnInvalidationChild&>(Widget);
				Instance.OverrideCallbackValueB = Instance.CallbackValueB;
			}));
	AttributeInitializer.OverrideOnValueChanged("IntAttributeC"
		, FSlateAttributeDescriptor::ECallbackOverrideType::ExecuteBeforePrevious
		, FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				SAttributeLeftWidget_OnInvalidationChild& Instance = static_cast<SAttributeLeftWidget_OnInvalidationChild&>(Widget);
				Instance.OverrideCallbackValueC = Instance.CallbackValueC;
			}));
}

/*
 *
 */
class SContainerAttribute : public SLeafWidget
{
	SLATE_DECLARE_WIDGET(SContainerAttribute, SLeafWidget)
public:
	class FSlot : public TWidgetSlotWithAttributeSupport<FSlot>, public TPaddingWidgetSlotMixin<FSlot>
	{
	public:
		FSlot()
			: TWidgetSlotWithAttributeSupport<FSlot>()
			, TPaddingWidgetSlotMixin<FSlot>(FMargin(99.f))
			, SlotAttribute1(*this, 99)
			, SlotAttribute2(*this, 99)
			, SlotAttribute3(*this, 99)
			, SlotId(INDEX_NONE)
		{ }

		SLATE_SLOT_BEGIN_ARGS_OneMixin(FSlot, TSlotBase<FSlot>, TPaddingWidgetSlotMixin<FSlot>)
			SLATE_ATTRIBUTE(int32, SlotAttribute1)
			SLATE_ATTRIBUTE(int32, SlotAttribute2)
			SLATE_ATTRIBUTE(int32, SlotAttribute3)
			SLATE_ARGUMENT(int32, SlotId)
		SLATE_SLOT_END_ARGS()

		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			TSlotBase<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
			TPaddingWidgetSlotMixin<FSlot>::ConstructMixin(SlotOwner, MoveTemp(InArgs));
			if (InArgs._SlotAttribute1.IsSet())
			{
				SlotAttribute1.Assign(*this, MoveTemp(InArgs._SlotAttribute1));
			}
			if (InArgs._SlotAttribute2.IsSet())
			{
				SlotAttribute2.Assign(*this, MoveTemp(InArgs._SlotAttribute2));
			}
			if (InArgs._SlotAttribute3.IsSet())
			{
				SlotAttribute3.Assign(*this, MoveTemp(InArgs._SlotAttribute3));
			}
			SlotId = InArgs._SlotId;
		}

		static void RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer)
		{
			TPaddingWidgetSlotMixin<FSlot>::RegisterAttributesMixin(AttributeInitializer);
			SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(FSlot, AttributeInitializer, "Slot.Attribute1", SlotAttribute1, EInvalidateWidgetReason::Paint);
			SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(FSlot, AttributeInitializer, "Slot.Attribute2", SlotAttribute2, EInvalidateWidgetReason::Layout);
			SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(FSlot, AttributeInitializer, "Slot.Attribute3", SlotAttribute3, EInvalidateWidgetReason::Prepass);
		}

		TSlateSlotAttribute<int32> SlotAttribute1;
		TSlateSlotAttribute<int32> SlotAttribute2;
		TSlateSlotAttribute<int32> SlotAttribute3;
		int32 SlotId;
	};

	SLATE_BEGIN_ARGS(SContainerAttribute) {}
	SLATE_END_ARGS()

	SContainerAttribute()
		: Slots(this, GET_MEMBER_NAME_CHECKED(SContainerAttribute, Slots))
		, AttributeA(*this, 99)
		, AttributeB(*this, 99)
		, AttributeC(*this, 99)
	{}


	FSlot::FSlotArguments Slot()
	{
		return FSlot::FSlotArguments(MakeUnique<FSlot>());
	}

	TPanelChildren<FSlot>::FScopedWidgetSlotArguments AddSlot()
	{
		return TPanelChildren<FSlot>::FScopedWidgetSlotArguments{ MakeUnique<FSlot>(), Slots, INDEX_NONE };
	}

	bool RemoveSlot(const TSharedRef<SWidget>& SlotWidget)
	{
		return Slots.Remove(SlotWidget) != INDEX_NONE;
	}

	void ClearChildren()
	{
		Slots.Empty();
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override { return LayerId; }
	virtual FVector2D ComputeDesiredSize(float) const override { return FVector2D(0.f, 0.f); }
	void Construct(const FArguments&) {}

public:
	/** The slots that are placed into various grid locations */
	TPanelChildren<FSlot> Slots;
	TSlateAttribute<int32> AttributeA;
	TSlateAttribute<int32> AttributeB;
	TSlateAttribute<int32> AttributeC;
};

SLATE_IMPLEMENT_WIDGET(SContainerAttribute)
void SContainerAttribute::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer SlotInitializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Slots);
	FSlot::RegisterAttributes(SlotInitializer);

	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, AttributeA, EInvalidateWidgetReason::RenderTransform);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, AttributeB, EInvalidateWidgetReason::Visibility)
		.AffectVisibility();
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, AttributeC, EInvalidateWidgetReason::Volatility);
}


// ------------------------------------------------
// -------------- Multi Inheritance ---------------
// ------------------------------------------------

class IMultiInheritanceWidget_Int_Base
{
public:
	virtual void SomeBaseFunction_Int() {};
};
class IMultiInheritanceWidget_UInt_Base
{
public:
	virtual void SomeBaseFunction_Int() {};
};

class SMultiInheritanceWidget_Base
	: public IMultiInheritanceWidget_Int_Base, public SLeafWidget, public IMultiInheritanceWidget_UInt_Base
{
	SLATE_DECLARE_WIDGET(SMultiInheritanceWidget_Base, SLeafWidget)
public:
	SLATE_BEGIN_ARGS(SMultiInheritanceWidget_Base) {}
	SLATE_END_ARGS()

	SMultiInheritanceWidget_Base()
		: BaseIntAttribute(*this, -9999)
		, BaseUIntAttribute(*this, 9999)
	{
	}

	void Construct(const FArguments& InArgs){}
	virtual FVector2D ComputeDesiredSize(float) const override { return FVector2D{ 100, 100 }; }
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		return LayerId;
	}

	TSlateAttribute<int32> BaseIntAttribute;
	TSlateAttribute<uint32> BaseUIntAttribute;
	int32 CallbackIntValue = 0;
	uint32 CallbackUIntValue = 0;
};

SLATE_IMPLEMENT_WIDGET(SMultiInheritanceWidget_Base)
void SMultiInheritanceWidget_Base::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, BaseIntAttribute, EInvalidateWidgetReason::Paint)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			SMultiInheritanceWidget_Base& Instance = static_cast<SMultiInheritanceWidget_Base&>(Widget);
			Instance.CallbackIntValue += Instance.BaseIntAttribute.Get();
		}));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, BaseUIntAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			SMultiInheritanceWidget_Base& Instance = static_cast<SMultiInheritanceWidget_Base&>(Widget);
			Instance.CallbackUIntValue += Instance.BaseUIntAttribute.Get();
		}));
}


class SMultiInheritanceWidget_Parent : public SMultiInheritanceWidget_Base
{
	SLATE_DECLARE_WIDGET(SMultiInheritanceWidget_Parent, SMultiInheritanceWidget_Base)
public:
	SLATE_BEGIN_ARGS(SMultiInheritanceWidget_Parent) {}
	SLATE_END_ARGS()


	SMultiInheritanceWidget_Parent()
		: ParentIntAttribute(*this, -8888)
		, ParentUIntAttribute(*this, 8888)
	{
	}

	void Construct(const FArguments& InArgs){}
	virtual FVector2D ComputeDesiredSize(float) const override { return FVector2D{ 100, 100 }; }
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		return LayerId;
	}


	TSlateAttribute<int32> ParentIntAttribute;
	TSlateAttribute<uint32> ParentUIntAttribute;
};

SLATE_IMPLEMENT_WIDGET(SMultiInheritanceWidget_Parent)
void SMultiInheritanceWidget_Parent::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ParentIntAttribute, EInvalidateWidgetReason::Paint)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			SMultiInheritanceWidget_Parent& Instance = static_cast<SMultiInheritanceWidget_Parent&>(Widget);
			Instance.CallbackIntValue += Instance.ParentIntAttribute.Get();
		}));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ParentUIntAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			SMultiInheritanceWidget_Parent& Instance = static_cast<SMultiInheritanceWidget_Parent&>(Widget);
			Instance.CallbackUIntValue += Instance.ParentUIntAttribute.Get();
		}));
}


class IMultiInheritanceWidget_Int_Parent
{
public:
	virtual void SomeParentFunction_Int() {};
};
class IMultiInheritanceWidget_UInt_Parent
{
public:
	virtual void SomeParentFunction_UInt() {};
};


// Widget with multiple inheritance declared in SII order (SWidget, Interface, Interface)
class SMultiInheritanceWidget_Child_SII
	: public SMultiInheritanceWidget_Parent, public IMultiInheritanceWidget_Int_Parent, public IMultiInheritanceWidget_UInt_Parent
{
	SLATE_DECLARE_WIDGET(SMultiInheritanceWidget_Child_SII, SMultiInheritanceWidget_Parent)
public:
	SLATE_BEGIN_ARGS(SMultiInheritanceWidget_Child_SII) {}
	SLATE_END_ARGS()

	SMultiInheritanceWidget_Child_SII()
		: ChildIntAttribute(*this, -511)
		, ChildUIntAttribute(*this, 511)
	{
	}

	void Construct(const FArguments& InArgs){}

	TSlateAttribute<int32> ChildIntAttribute;
	TSlateAttribute<uint32> ChildUIntAttribute;
};
SLATE_IMPLEMENT_WIDGET(SMultiInheritanceWidget_Child_SII)
void SMultiInheritanceWidget_Child_SII::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ChildIntAttribute, EInvalidateWidgetReason::Paint)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			SMultiInheritanceWidget_Child_SII& Instance = static_cast<SMultiInheritanceWidget_Child_SII&>(Widget);
			Instance.CallbackIntValue += Instance.ChildIntAttribute.Get();
		}));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ChildUIntAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			SMultiInheritanceWidget_Child_SII& Instance = static_cast<SMultiInheritanceWidget_Child_SII&>(Widget);
			Instance.CallbackUIntValue += Instance.ChildUIntAttribute.Get();
		}));
}

// Widget with multiple inheritance declared in ISI order (Interface, SWidget, Interface)
class SMultiInheritanceWidget_Child_ISI
	: public IMultiInheritanceWidget_Int_Parent, public SMultiInheritanceWidget_Parent, public IMultiInheritanceWidget_UInt_Parent
{
	SLATE_DECLARE_WIDGET(SMultiInheritanceWidget_Child_ISI, SMultiInheritanceWidget_Parent)
public:
	SLATE_BEGIN_ARGS(SMultiInheritanceWidget_Child_ISI) {}
	SLATE_END_ARGS()

	SMultiInheritanceWidget_Child_ISI()
		: ChildIntAttribute(*this, -151)
		, ChildUIntAttribute(*this, 151)
	{
	}

	void Construct(const FArguments& InArgs){}

	TSlateAttribute<int32> ChildIntAttribute;
	TSlateAttribute<uint32> ChildUIntAttribute;
};
SLATE_IMPLEMENT_WIDGET(SMultiInheritanceWidget_Child_ISI)
void SMultiInheritanceWidget_Child_ISI::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ChildIntAttribute, EInvalidateWidgetReason::Paint)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			SMultiInheritanceWidget_Child_ISI& Instance = static_cast<SMultiInheritanceWidget_Child_ISI&>(Widget);
			Instance.CallbackIntValue += Instance.ChildIntAttribute.Get();
		}));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ChildUIntAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			SMultiInheritanceWidget_Child_ISI& Instance = static_cast<SMultiInheritanceWidget_Child_ISI&>(Widget);
			Instance.CallbackUIntValue += Instance.ChildUIntAttribute.Get();
		}));
}

// Widget with multiple inheritance declared in IIS order (Interface, Interface, SWidget)
class SMultiInheritanceWidget_Child_IIS
	: public IMultiInheritanceWidget_Int_Parent, public IMultiInheritanceWidget_UInt_Parent, public SMultiInheritanceWidget_Parent
{
	SLATE_DECLARE_WIDGET(SMultiInheritanceWidget_Child_IIS, SMultiInheritanceWidget_Parent)
public:
	SLATE_BEGIN_ARGS(SMultiInheritanceWidget_Child_IIS) {}
	SLATE_END_ARGS()

	SMultiInheritanceWidget_Child_IIS()
		: ChildIntAttribute(*this, -151)
		, ChildUIntAttribute(*this, 151)
	{
	}

	void Construct(const FArguments& InArgs){}

	TSlateAttribute<int32> ChildIntAttribute;
	TSlateAttribute<uint32> ChildUIntAttribute;
};
SLATE_IMPLEMENT_WIDGET(SMultiInheritanceWidget_Child_IIS)
void SMultiInheritanceWidget_Child_IIS::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ChildIntAttribute, EInvalidateWidgetReason::Paint)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			SMultiInheritanceWidget_Child_IIS& Instance = static_cast<SMultiInheritanceWidget_Child_IIS&>(Widget);
			Instance.CallbackIntValue += Instance.ChildIntAttribute.Get();
		}));
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, ChildUIntAttribute, EInvalidateWidgetReason::Layout)
		.OnValueChanged(FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
			SMultiInheritanceWidget_Child_IIS& Instance = static_cast<SMultiInheritanceWidget_Child_IIS&>(Widget);
			Instance.CallbackUIntValue += Instance.ChildUIntAttribute.Get();
		}));
}


class IMultiInheritanceWidget_Int_GrandChild
{
public:
	virtual void SomeGrandChildFunction_Int() {};
};
class IMultiInheritanceWidget_UInt_GrandChild
{
public:
	virtual void SomeGrandChildFunction_UInt() {};
};


class SMultiInheritanceWidget_GrandChild_SII_SII
	: public SMultiInheritanceWidget_Child_SII, public IMultiInheritanceWidget_Int_GrandChild, public IMultiInheritanceWidget_UInt_GrandChild
{
};

class SMultiInheritanceWidget_GrandChild_SII_ISI
	: public IMultiInheritanceWidget_Int_GrandChild, public SMultiInheritanceWidget_Child_SII, public IMultiInheritanceWidget_UInt_GrandChild
{
};
class SMultiInheritanceWidget_GrandChild_SII_IIS
	: public IMultiInheritanceWidget_Int_GrandChild, public IMultiInheritanceWidget_UInt_GrandChild, public SMultiInheritanceWidget_Child_SII
{
};


class SMultiInheritanceWidget_GrandChild_ISI_SII
	: public SMultiInheritanceWidget_Child_ISI, public IMultiInheritanceWidget_Int_GrandChild, public IMultiInheritanceWidget_UInt_GrandChild
{
};
class SMultiInheritanceWidget_GrandChild_ISI_ISI
	: public IMultiInheritanceWidget_Int_GrandChild, public SMultiInheritanceWidget_Child_ISI, public IMultiInheritanceWidget_UInt_GrandChild
{
};
class SMultiInheritanceWidget_GrandChild_ISI_IIS
	: public IMultiInheritanceWidget_Int_GrandChild, public IMultiInheritanceWidget_UInt_GrandChild, public SMultiInheritanceWidget_Child_ISI
{
};


class SMultiInheritanceWidget_GrandChild_IIS_SII
	: public SMultiInheritanceWidget_Child_IIS, public IMultiInheritanceWidget_Int_GrandChild, public IMultiInheritanceWidget_UInt_GrandChild
{
};
class SMultiInheritanceWidget_GrandChild_IIS_ISI
	: public IMultiInheritanceWidget_Int_GrandChild, public SMultiInheritanceWidget_Child_IIS, public IMultiInheritanceWidget_UInt_GrandChild
{
};
class SMultiInheritanceWidget_GrandChild_IIS_IIS
	: public IMultiInheritanceWidget_Int_GrandChild, public IMultiInheritanceWidget_UInt_GrandChild, public SMultiInheritanceWidget_Child_IIS
{
};


} //namespace

bool FSlateAttributeTest::RunTest(const FString& Parameters)
{
	const int NumberOfAttributeInSWidget = SWidget::StaticWidgetClass().GetAttributeDescriptor().GetAttributeNum();
	int32 OrderCounter = 0;
	auto OrderLambda = [this, &OrderCounter]() -> int32
	{
		++OrderCounter;
		return OrderCounter;
	};

	{
		TSharedRef<UE::Slate::Private::SAttributeLeftWidget_Parent> WidgetParent = SNew(UE::Slate::Private::SAttributeLeftWidget_Parent);

		AddErrorIfFalse(&WidgetParent->GetWidgetClass() == &UE::Slate::Private::SAttributeLeftWidget_Parent::StaticWidgetClass()
			, TEXT("The static data do not matches"));

		FSlateAttributeDescriptor const& AttributeDescriptor = WidgetParent->GetWidgetClass().GetAttributeDescriptor();
		AddErrorIfFalse(AttributeDescriptor.GetAttributeNum() == 4 + NumberOfAttributeInSWidget, TEXT("Invalid number of attributes"));

		const int32 IndexA = AttributeDescriptor.IndexOfAttribute("IntAttributeA");
		const int32 IndexB = AttributeDescriptor.IndexOfAttribute("IntAttributeB");
		const int32 IndexC = AttributeDescriptor.IndexOfAttribute("IntAttributeC");
		const int32 IndexD = AttributeDescriptor.IndexOfAttribute("IntAttributeD");
		const int32 IndexI = AttributeDescriptor.IndexOfAttribute("IntAttributeI");
		const int32 IndexJ = AttributeDescriptor.IndexOfAttribute("IntAttributeJ");
		const int32 IndexK = AttributeDescriptor.IndexOfAttribute("IntAttributeK");

		AddErrorIfFalse(IndexA != INDEX_NONE, TEXT("Could not find the Attribute A"));
		AddErrorIfFalse(IndexB != INDEX_NONE, TEXT("Could not find the Attribute B"));
		AddErrorIfFalse(IndexC != INDEX_NONE, TEXT("Could not find the Attribute C"));
		AddErrorIfFalse(IndexD != INDEX_NONE, TEXT("Could not find the Attribute D"));
		AddErrorIfFalse(IndexI == INDEX_NONE, TEXT("Was not supposed to find the Attribute I"));
		AddErrorIfFalse(IndexJ == INDEX_NONE, TEXT("Was not supposed to find the Attribute J"));
		AddErrorIfFalse(IndexK == INDEX_NONE, TEXT("Was not supposed to find the Attribute K"));

		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexA) == AttributeDescriptor.FindAttribute("IntAttributeA"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexB) == AttributeDescriptor.FindAttribute("IntAttributeB"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexC) == AttributeDescriptor.FindAttribute("IntAttributeC"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexD) == AttributeDescriptor.FindAttribute("IntAttributeD"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(AttributeDescriptor.FindAttribute("IntAttributeI") == nullptr, TEXT("Was not supposed to find the Attribute I"));
		AddErrorIfFalse(AttributeDescriptor.FindAttribute("IntAttributeJ") == nullptr, TEXT("Was not supposed to find the Attribute J"));
		AddErrorIfFalse(AttributeDescriptor.FindAttribute("IntAttributeK") == nullptr, TEXT("Was not supposed to find the Attribute K"));

		//B, A, D, C
		AddErrorIfFalse(AttributeDescriptor.GetAttributeAtIndex(IndexB).GetSortOrder() < AttributeDescriptor.GetAttributeAtIndex(IndexA).GetSortOrder(), TEXT("B should have a lower value than A"));
		AddErrorIfFalse(AttributeDescriptor.GetAttributeAtIndex(IndexD).GetSortOrder() < AttributeDescriptor.GetAttributeAtIndex(IndexC).GetSortOrder(), TEXT("D should have a lower value than C"));
		AddErrorIfFalse(AttributeDescriptor.GetAttributeAtIndex(IndexA).GetSortOrder() < AttributeDescriptor.GetAttributeAtIndex(IndexD).GetSortOrder(), TEXT("A should have a lower value than D"));

		{
			OrderCounter = 0;
			WidgetParent->IntAttributeA.Assign(WidgetParent.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetParent->IntAttributeA.Get() == 99, TEXT("A It is not the expected value."));
			WidgetParent->IntAttributeB.Assign(WidgetParent.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetParent->IntAttributeB.Get() == 99, TEXT("B It is not the expected value."));
			WidgetParent->IntAttributeC.Assign(WidgetParent.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetParent->IntAttributeC.Get() == 99, TEXT("C It is not the expected value."));
			WidgetParent->IntAttributeD.Assign(WidgetParent.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetParent->IntAttributeD.Get() == 99, TEXT("D It is not the expected value."));

			OrderCounter = 0;
			WidgetParent->MarkPrepassAsDirty();
			WidgetParent->SlatePrepass(1.f);
			AddErrorIfFalse(WidgetParent->IntAttributeA.Get() == 2, TEXT("A It is not the expected value."));
			AddErrorIfFalse(WidgetParent->IntAttributeB.Get() == 1, TEXT("B It is not the expected value."));
			AddErrorIfFalse(WidgetParent->IntAttributeC.Get() == 4, TEXT("C It is not the expected value."));
			AddErrorIfFalse(WidgetParent->IntAttributeD.Get() == 3, TEXT("D It is not the expected value."));
		}

		{
			WidgetParent->IntAttributeD.Set(WidgetParent.Get(), 8);
			AddErrorIfFalse(WidgetParent->IntAttributeC.Get() == 4, TEXT("C It is not the expected value."));
			AddErrorIfFalse(WidgetParent->IntAttributeD.Get() == 8, TEXT("D It is not the expected value."));

			OrderCounter = 0;
			WidgetParent->MarkPrepassAsDirty();
			WidgetParent->SlatePrepass(1.f);
			AddErrorIfFalse(WidgetParent->IntAttributeA.Get() == 2, TEXT("A It is not the expected value."));
			AddErrorIfFalse(WidgetParent->IntAttributeB.Get() == 1, TEXT("B It is not the expected value."));
			AddErrorIfFalse(WidgetParent->IntAttributeC.Get() == 3, TEXT("C It is not the expected value."));
			AddErrorIfFalse(WidgetParent->IntAttributeD.Get() == 8, TEXT("D It is not the expected value."));
		}
	}

	{
		TSharedRef<UE::Slate::Private::SAttributeLeftWidget_Child> WidgetChild = SNew(UE::Slate::Private::SAttributeLeftWidget_Child);

		AddErrorIfFalse(&WidgetChild->GetWidgetClass() == &UE::Slate::Private::SAttributeLeftWidget_Child::StaticWidgetClass()
			, TEXT("The static data do not matches"));

		FSlateAttributeDescriptor const& AttributeDescriptor = WidgetChild->GetWidgetClass().GetAttributeDescriptor();
		AddErrorIfFalse(AttributeDescriptor.GetAttributeNum() == 9 + NumberOfAttributeInSWidget, TEXT("Invalid number of attributes")); // H is not counted

		const int32 IndexA = AttributeDescriptor.IndexOfAttribute("IntAttributeA");
		const int32 IndexB = AttributeDescriptor.IndexOfAttribute("IntAttributeB");
		const int32 IndexC = AttributeDescriptor.IndexOfAttribute("IntAttributeC");
		const int32 IndexD = AttributeDescriptor.IndexOfAttribute("IntAttributeD");
		const int32 IndexI = AttributeDescriptor.IndexOfAttribute("IntAttributeI");
		const int32 IndexJ = AttributeDescriptor.IndexOfAttribute("IntAttributeJ");
		const int32 IndexK = AttributeDescriptor.IndexOfAttribute("IntAttributeK");
		const int32 IndexL = AttributeDescriptor.IndexOfAttribute("IntAttributeL");
		const int32 IndexM = AttributeDescriptor.IndexOfAttribute("IntAttributeM");

		AddErrorIfFalse(IndexA != INDEX_NONE, TEXT("Could not find the Attribute A"));
		AddErrorIfFalse(IndexB != INDEX_NONE, TEXT("Could not find the Attribute B"));
		AddErrorIfFalse(IndexC != INDEX_NONE, TEXT("Could not find the Attribute C"));
		AddErrorIfFalse(IndexD != INDEX_NONE, TEXT("Could not find the Attribute D"));
		AddErrorIfFalse(IndexI != INDEX_NONE, TEXT("Could not find the Attribute I"));
		AddErrorIfFalse(IndexJ != INDEX_NONE, TEXT("Could not find the Attribute J"));
		AddErrorIfFalse(IndexK != INDEX_NONE, TEXT("Could not find the Attribute K"));
		AddErrorIfFalse(IndexL != INDEX_NONE, TEXT("Could not find the Attribute L"));
		AddErrorIfFalse(IndexM != INDEX_NONE, TEXT("Could not find the Attribute M"));

		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexA) == AttributeDescriptor.FindAttribute("IntAttributeA"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexB) == AttributeDescriptor.FindAttribute("IntAttributeB"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexC) == AttributeDescriptor.FindAttribute("IntAttributeC"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexD) == AttributeDescriptor.FindAttribute("IntAttributeD"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexI) == AttributeDescriptor.FindAttribute("IntAttributeI"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexJ) == AttributeDescriptor.FindAttribute("IntAttributeJ"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexK) == AttributeDescriptor.FindAttribute("IntAttributeK"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexL) == AttributeDescriptor.FindAttribute("IntAttributeL"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexM) == AttributeDescriptor.FindAttribute("IntAttributeM"), TEXT("Index and Attribute should return the same value."));
		AddErrorIfFalse(AttributeDescriptor.FindAttribute("IntAttributeH") == nullptr, TEXT("H exist but is not defined."));

		AddErrorIfFalse(AttributeDescriptor.GetAttributeAtIndex(IndexM).GetSortOrder() < AttributeDescriptor.GetAttributeAtIndex(IndexB).GetSortOrder(), TEXT("M should have a lower value than B"));
		AddErrorIfFalse(AttributeDescriptor.GetAttributeAtIndex(IndexB).GetSortOrder() < AttributeDescriptor.GetAttributeAtIndex(IndexA).GetSortOrder(), TEXT("B should have a lower value than A"));
		AddErrorIfFalse(AttributeDescriptor.GetAttributeAtIndex(IndexA).GetSortOrder() <= AttributeDescriptor.GetAttributeAtIndex(IndexI).GetSortOrder(), TEXT("A should have a lower value than I"));
		AddErrorIfFalse(AttributeDescriptor.GetAttributeAtIndex(IndexI).GetSortOrder() < AttributeDescriptor.GetAttributeAtIndex(IndexJ).GetSortOrder(), TEXT("I should have a lower value than J"));
		AddErrorIfFalse(AttributeDescriptor.GetAttributeAtIndex(IndexJ).GetSortOrder() < AttributeDescriptor.GetAttributeAtIndex(IndexD).GetSortOrder(), TEXT("J should have a lower value than D"));
		AddErrorIfFalse(AttributeDescriptor.GetAttributeAtIndex(IndexD).GetSortOrder() < AttributeDescriptor.GetAttributeAtIndex(IndexC).GetSortOrder(), TEXT("D should have a lower value than C"));
		AddErrorIfFalse(AttributeDescriptor.GetAttributeAtIndex(IndexC).GetSortOrder() < AttributeDescriptor.GetAttributeAtIndex(IndexL).GetSortOrder(), TEXT("C should have a lower value than L"));
		AddErrorIfFalse(AttributeDescriptor.GetAttributeAtIndex(IndexL).GetSortOrder() < AttributeDescriptor.GetAttributeAtIndex(IndexK).GetSortOrder(), TEXT("L should have a lower value than K"));


		{
			OrderCounter = 49;
			WidgetChild->IntAttributeA.Assign(WidgetChild.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetChild->IntAttributeA.Get() == 99, TEXT("A It is not the expected value."));
			WidgetChild->IntAttributeB.Assign(WidgetChild.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetChild->IntAttributeB.Get() == 99, TEXT("B It is not the expected value."));
			WidgetChild->IntAttributeC.Assign(WidgetChild.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetChild->IntAttributeC.Get() == 99, TEXT("C It is not the expected value."));
			WidgetChild->IntAttributeD.Assign(WidgetChild.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetChild->IntAttributeD.Get() == 99, TEXT("D It is not the expected value."));
			WidgetChild->IntAttributeH.Assign(WidgetChild.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetChild->IntAttributeH.Get() == 99, TEXT("I It is not the expected value."));
			WidgetChild->IntAttributeI.Assign(WidgetChild.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetChild->IntAttributeI.Get() == 99, TEXT("I It is not the expected value."));
			WidgetChild->IntAttributeJ.Assign(WidgetChild.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetChild->IntAttributeJ.Get() == 99, TEXT("J It is not the expected value."));
			WidgetChild->IntAttributeK.Assign(WidgetChild.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetChild->IntAttributeK.Get() == 99, TEXT("K It is not the expected value."));
			WidgetChild->IntAttributeL.Assign(WidgetChild.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetChild->IntAttributeL.Get() == 99, TEXT("L It is not the expected value."));
			WidgetChild->IntAttributeM.Assign(WidgetChild.Get(), MakeAttributeLambda(OrderLambda));
			AddErrorIfFalse(WidgetChild->IntAttributeM.Get() == 99, TEXT("L It is not the expected value."));


			OrderCounter = 0;
			WidgetChild->MarkPrepassAsDirty();
			WidgetChild->SlatePrepass(1.f);
			AddErrorIfFalse(WidgetChild->IntAttributeA.Get() == 3 || WidgetChild->IntAttributeA.Get() == 4, TEXT("A It is not the expected value."));
			AddErrorIfFalse(WidgetChild->IntAttributeB.Get() == 2, TEXT("B It is not the expected value."));
			AddErrorIfFalse(WidgetChild->IntAttributeC.Get() == 7, TEXT("C It is not the expected value."));
			AddErrorIfFalse(WidgetChild->IntAttributeD.Get() == 6, TEXT("D It is not the expected value."));
			AddErrorIfFalse(WidgetChild->IntAttributeH.Get() == 9, TEXT("H It is not the expected value."));
			AddErrorIfFalse(WidgetChild->IntAttributeI.Get() == 3 || WidgetChild->IntAttributeI.Get() == 4, TEXT("I It is not the expected value."));
			AddErrorIfFalse(WidgetChild->IntAttributeJ.Get() == 5, TEXT("J It is not the expected value."));
			AddErrorIfFalse(WidgetChild->IntAttributeK.Get() == 10, TEXT("K It is not the expected value."));
			AddErrorIfFalse(WidgetChild->IntAttributeL.Get() == 8, TEXT("L It is not the expected value."));
			AddErrorIfFalse(WidgetChild->IntAttributeM.Get() == 1, TEXT("L It is not the expected value."));
		}
	}


	// Make sure we call all the functions
	{
		{
			// This should just compile to TSlateAttribute
			struct SSlateAttribute : public SLeafWidget
			{
				SSlateAttribute()
					: Toto(6)
					, AttributeA(*this)
					, AttributeB(*this, 5)
					, AttributeC(*this, MoveTemp(Toto))
				{ }
				SLATE_BEGIN_ARGS(SSlateAttribute) {}  SLATE_END_ARGS()
				virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override { return LayerId; }
				virtual FVector2D ComputeDesiredSize(float) const override { return FVector2D(0.f, 0.f); }
				void Construct(const FArguments&) {	}

				int32 Callback() const { return 0; }

				int32 Toto;
				TSlateAttribute<int32, EInvalidateWidgetReason::Paint> AttributeA;
				TSlateAttribute<int32, EInvalidateWidgetReason::Paint> AttributeB;
				TSlateAttribute<int32, EInvalidateWidgetReason::Paint> AttributeC;
			};
			TSharedPtr<SSlateAttribute> Widget = SNew(SSlateAttribute);

			{
				int32 Hello = 7;
				int32 Return1 = Widget->AttributeA.Get();
				Widget->AttributeA.UpdateNow(*Widget);
				Widget->AttributeA.Set(*Widget, 6);
				Widget->AttributeA.Set(*Widget, MoveTemp(Hello));
			}
			{
				auto Getter1 = TAttribute<int32>::FGetter::CreateStatic(UE::Slate::Private::CallbackForIntAttribute, 1);
				Widget->AttributeA.Bind(*Widget, Getter1);
				Widget->AttributeA.Bind(*Widget, MoveTemp(Getter1));
				Widget->AttributeA.Bind(*Widget, &SSlateAttribute::Callback);
			}
			{
				int32 TmpInt1 = 7;
				int32 TmpInt2 = 7;
				auto Getter1 = TAttribute<int32>::FGetter::CreateStatic(UE::Slate::Private::CallbackForIntAttribute, 1);
				TAttribute<int32> Attribute1 = TAttribute<int32>::Create(Getter1);
				TAttribute<int32> Attribute2 = TAttribute<int32>::Create(Getter1);
				TAttribute<int32> Attribute3 = TAttribute<int32>::Create(Getter1);
				Widget->AttributeA.Assign(*Widget, Attribute1);
				Widget->AttributeA.Assign(*Widget, MoveTemp(Attribute1));
				Widget->AttributeA.Assign(*Widget, Attribute2, 7);
				Widget->AttributeA.Assign(*Widget, MoveTemp(Attribute2), 7);
				Widget->AttributeA.Assign(*Widget, Attribute3, MoveTemp(TmpInt1));
				Widget->AttributeA.Assign(*Widget, MoveTemp(Attribute3), MoveTemp(TmpInt2));
			}
			{
				bool bIsBound1 = Widget->AttributeA.IsBound(*Widget);
				bool bIsIdentical1 = Widget->AttributeA.IdenticalTo(*Widget, Widget->AttributeA);
				auto Getter1 = TAttribute<int32>::FGetter::CreateStatic(UE::Slate::Private::CallbackForIntAttribute, 1);
				TAttribute<int32> Attribute1 = TAttribute<int32>::Create(Getter1);
				bool bIsIdentical2 = Widget->AttributeA.IdenticalTo(*Widget, Attribute1);
			}
		}
		{
			typedef UE::Slate::Private::FConstructionCounter FLocalConstructionCounter;

			// This should just compile to TSlateManagedAttribute
			struct SManagedAttribute : public SLeafWidget
			{
				SLATE_BEGIN_ARGS(SManagedAttribute) {} SLATE_END_ARGS()
				virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override { return LayerId; }
				virtual FVector2D ComputeDesiredSize(float) const override { return FVector2D(0.f, 0.f); }
				void Construct(const FArguments&) {}

				FLocalConstructionCounter ReturnDefaultCounter() const { return FLocalConstructionCounter(0); }

				using ManagedSlateAttributeType = TSlateManagedAttribute<FLocalConstructionCounter, EInvalidateWidgetReason::Layout>;
			};
			TSharedPtr<SManagedAttribute>	Widget = SNew(SManagedAttribute);


			auto AddErrorIfCounterDoNotMatches = [this](int32 Construct, int32 Copy, int32 Move, int32 CopyAssign, int32 MoveAssign, const TCHAR* Message)
			{
				bool bSuccess = FLocalConstructionCounter::DefaultConstructionCounter == Construct
					&& FLocalConstructionCounter::CopyConstructionCounter == Copy
					&& FLocalConstructionCounter::MoveConstructionCounter == Move
					&& FLocalConstructionCounter::CopyOperatorCounter == CopyAssign
					&& FLocalConstructionCounter::MoveOperatorCounter == MoveAssign;
				AddErrorIfFalse(bSuccess, FString(Message) +
					FString::Printf(TEXT(" [Construct: %d [exp: %d], Copy: %d [exp: %d], Move: %d [exp: %d], CopyAssign: %d [exp: %d], MoveAssign: %d [exp: %d]]"),
					FLocalConstructionCounter::DefaultConstructionCounter, Construct,
					FLocalConstructionCounter::CopyConstructionCounter, Copy,
					FLocalConstructionCounter::MoveConstructionCounter, Move,
					FLocalConstructionCounter::CopyOperatorCounter, CopyAssign,
					FLocalConstructionCounter::MoveOperatorCounter, MoveAssign
				));
			};

			{
				FLocalConstructionCounter::ResetCounter();
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef() );
				AddErrorIfCounterDoNotMatches(1, 0, 0, 0, 0, TEXT("Default & Copy constructor was not used."));
			}
			{
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef(), Counter);
				AddErrorIfCounterDoNotMatches(0, 1, 0, 0, 0, TEXT("Default & Copy constructor was not used."));
			}
			{
				FLocalConstructionCounter::ResetCounter();
				FLocalConstructionCounter Counter = 1;
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef(), MoveTemp(Counter));
				AddErrorIfCounterDoNotMatches(1, 0, 1, 0, 0, TEXT("Default & Move constructor was not used."));
			}
			{
				TAttribute<FLocalConstructionCounter>::FGetter Getter1 = TAttribute<FLocalConstructionCounter>::FGetter::CreateLambda([](){return FLocalConstructionCounter(1);});
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType{Widget.ToSharedRef(), Getter1, Counter};
				AddErrorIfCounterDoNotMatches(0, 1, 0, 0, 0, TEXT("Getter & Copy constructor was not used."));
			}
			{
				TAttribute<FLocalConstructionCounter>::FGetter Getter1 = TAttribute<FLocalConstructionCounter>::FGetter::CreateLambda([]() {return FLocalConstructionCounter(1);});
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType{Widget.ToSharedRef(), Getter1, MoveTemp(Counter)};
				AddErrorIfCounterDoNotMatches(0, 0, 1, 0, 0, TEXT("Getter & Move constructor was not used."));
			}
			{
				TAttribute<FLocalConstructionCounter>::FGetter Getter1 = TAttribute<FLocalConstructionCounter>::FGetter::CreateLambda([]() {return FLocalConstructionCounter(1);});
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef(), MoveTemp(Getter1), Counter);
				AddErrorIfCounterDoNotMatches(0, 1, 0, 0, 0, TEXT("Move Getter & Copy constructor was not used."));
			}
			{
				TAttribute<FLocalConstructionCounter>::FGetter Getter1 = TAttribute<FLocalConstructionCounter>::FGetter::CreateLambda([]() {return FLocalConstructionCounter(1);});
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef(), MoveTemp(Getter1), MoveTemp(Counter));
				AddErrorIfCounterDoNotMatches(0, 0, 1, 0, 0, TEXT("Move Getter & Move constructor was not used."));
			}
			{
				TAttribute<FLocalConstructionCounter> Attribute1 = MakeAttributeLambda([]() {return FLocalConstructionCounter(1);});
				FLocalConstructionCounter Counter;
				FLocalConstructionCounter::ResetCounter();
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef(), Attribute1, Counter);
				AddErrorIfCounterDoNotMatches(0, 1, 0, 0, 0, TEXT("Attribute & Copy constructor was not used."));
			}
			{
				TAttribute<FLocalConstructionCounter> Attribute1 = MakeAttributeLambda([]() {return FLocalConstructionCounter(1);});
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef(), MoveTemp(Attribute1), MoveTemp(Counter));
				AddErrorIfCounterDoNotMatches(0, 0, 1, 0, 0, TEXT("Move Attribute & Move constructor was not used."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				FLocalConstructionCounter::ResetCounter();
				FLocalConstructionCounter Result = Attribute.Get();
				Attribute.UpdateNow();
				AddErrorIfCounterDoNotMatches(0, 1, 0, 0, 0, TEXT("Get and UpdateNow failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Set(Counter);
				AddErrorIfCounterDoNotMatches(0, 0, 0, 1, 0, TEXT("Set Copy failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Set(MoveTemp(Counter));
				AddErrorIfCounterDoNotMatches(0, 0, 0, 0, 1, TEXT("Set Move failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				auto Getter1 = TAttribute<FLocalConstructionCounter>::FGetter::CreateLambda([]() {return FLocalConstructionCounter(1);});
				FLocalConstructionCounter::ResetCounter();
				Attribute.Bind(Getter1);
				AddErrorIfCounterDoNotMatches(0, 0, 0, 0, 0, TEXT("Bind Copy failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				auto Getter1 = TAttribute<FLocalConstructionCounter>::FGetter::CreateLambda([]() {return FLocalConstructionCounter(1);});
				FLocalConstructionCounter::ResetCounter();
				Attribute.Bind(MoveTemp(Getter1));
				AddErrorIfCounterDoNotMatches(0, 0, 0, 0, 0, TEXT("Bind Copy failed."));
			}
			// Test Assign
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				TAttribute<FLocalConstructionCounter> Attribute1;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(Attribute1);
				AddErrorIfCounterDoNotMatches(0, 0, 0, 0, 0, TEXT("Assign Copy failed."));
				Attribute1.Set({1});
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(Attribute1);
				AddErrorIfCounterDoNotMatches(0, 0, 0, 1, 0, TEXT("Assign Copy failed."));
				Attribute1.Set({ 1 });
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(Attribute1);
				AddErrorIfCounterDoNotMatches(0, 0, 0, 0, 0, TEXT("Assign Move failed."));
				Attribute1.Set({ 2 });
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(MoveTemp(Attribute1));
				AddErrorIfCounterDoNotMatches(0, 0, 1, 0, 1, TEXT("Assign Move failed."));
			}
			// Test unbinded Attribute
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				TAttribute<FLocalConstructionCounter> Attribute1;
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(Attribute1, Counter);
				AddErrorIfCounterDoNotMatches(0, 0, 0, 1, 0, TEXT("Assign Copy/Copy failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				TAttribute<FLocalConstructionCounter> Attribute1;
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(Attribute1, MoveTemp(Counter));
				AddErrorIfCounterDoNotMatches(0, 0, 0, 0, 1, TEXT("Assign Copy/Move failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				TAttribute<FLocalConstructionCounter> Attribute1;
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(MoveTemp(Attribute1), Counter);
				AddErrorIfCounterDoNotMatches(0, 0, 0, 1, 0, TEXT("Assign Move/CCopy failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				TAttribute<FLocalConstructionCounter> Attribute1;
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(MoveTemp(Attribute1), MoveTemp(Counter));
				AddErrorIfCounterDoNotMatches(0, 0, 0, 0, 1, TEXT("Assign Move/Move failed."));
			}
			// Test binded Attribute
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				TAttribute<FLocalConstructionCounter> Attribute1 = MakeAttributeLambda([]() {return FLocalConstructionCounter(1);});
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(Attribute1, Counter);
				AddErrorIfCounterDoNotMatches(0, 0, 0, 0, 0, TEXT("Bind Copy with binded attribute failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				TAttribute<FLocalConstructionCounter> Attribute1 = MakeAttributeLambda([]() {return FLocalConstructionCounter(1);});
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(Attribute1, MoveTemp(Counter));
				AddErrorIfCounterDoNotMatches(0, 0, 0, 0, 0, TEXT("Bind Move with binded attribute failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				TAttribute<FLocalConstructionCounter> Attribute1 = MakeAttributeLambda([]() {return FLocalConstructionCounter(1);});
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(MoveTemp(Attribute1), Counter);
				AddErrorIfCounterDoNotMatches(0, 0, 0, 0, 0, TEXT("Assign Copy with binded attribute failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				TAttribute<FLocalConstructionCounter> Attribute1 = MakeAttributeLambda([]() {return FLocalConstructionCounter(1);});
				FLocalConstructionCounter Counter = 1;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(MoveTemp(Attribute1), MoveTemp(Counter));
				AddErrorIfCounterDoNotMatches(0, 0, 0, 0, 0, TEXT("Assign Move with binded attribute failed."));
			}
			// Test set Attribute
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				FLocalConstructionCounter Counter = 1;
				TAttribute<FLocalConstructionCounter> Attribute1 = Counter;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(Attribute1, Counter);
				AddErrorIfCounterDoNotMatches(0, 0, 0, 1, 0, TEXT("Assign Set Copy failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				FLocalConstructionCounter Counter = 1;
				TAttribute<FLocalConstructionCounter> Attribute1 = Counter;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(Attribute1, MoveTemp(Counter));
				AddErrorIfCounterDoNotMatches(0, 0, 0, 1, 0, TEXT("Assign Set Copy failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				FLocalConstructionCounter Counter = 1;
				TAttribute<FLocalConstructionCounter> Attribute1 = Counter;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(MoveTemp(Attribute1), Counter);
				AddErrorIfCounterDoNotMatches(0, 0, 1, 0, 1, TEXT("Assign Set Move failed."));
			}
			{
				SManagedAttribute::ManagedSlateAttributeType Attribute = SManagedAttribute::ManagedSlateAttributeType(Widget.ToSharedRef());
				FLocalConstructionCounter Counter = 1;
				TAttribute<FLocalConstructionCounter> Attribute1 = Counter;
				FLocalConstructionCounter::ResetCounter();
				Attribute.Assign(MoveTemp(Attribute1), MoveTemp(Counter));
				AddErrorIfCounterDoNotMatches(0, 0, 1, 0, 1, TEXT("Assign Set Move failed."));
			}
		}

		// Slot Attribute test
		{
			TSharedRef<UE::Slate::Private::SContainerAttribute> Widget = SNew(UE::Slate::Private::SContainerAttribute);

			{
				Widget->AttributeA.Assign(Widget.Get(), MakeAttributeLambda(OrderLambda));
				AddErrorIfFalse(Widget->AttributeA.Get() == 99, TEXT("A It is not the expected value."));
				Widget->AttributeB.Assign(Widget.Get(), MakeAttributeLambda(OrderLambda));
				AddErrorIfFalse(Widget->AttributeB.Get() == 99, TEXT("B It is not the expected value."));
				Widget->AttributeC.Assign(Widget.Get(), MakeAttributeLambda(OrderLambda));
				AddErrorIfFalse(Widget->AttributeC.Get() == 99, TEXT("C It is not the expected value."));

				Widget->AddSlot()
					.SlotAttribute1(MakeAttributeLambda(OrderLambda))
					.SlotAttribute2(MakeAttributeLambda(OrderLambda))
					.SlotAttribute3(MakeAttributeLambda(OrderLambda))
					.SlotId(0);
				Widget->AddSlot()
					.SlotAttribute1(MakeAttributeLambda(OrderLambda))
					.SlotAttribute2(MakeAttributeLambda(OrderLambda))
					.SlotAttribute3(MakeAttributeLambda(OrderLambda))
					.SlotId(1);
				Widget->AddSlot()
					.SlotAttribute1(MakeAttributeLambda(OrderLambda))
					.SlotAttribute2(MakeAttributeLambda(OrderLambda))
					.SlotAttribute3(MakeAttributeLambda(OrderLambda))
					.SlotId(2);
			}
			{
				OrderCounter = 0;
				Widget->MarkPrepassAsDirty();
				Widget->SlatePrepass(1.f);

				AddErrorIfFalse(Widget->Slots[0].SlotAttribute1.Get() == 2, TEXT("Slot[0].1 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[0].SlotAttribute2.Get() == 3, TEXT("Slot[0].2 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[0].SlotAttribute3.Get() == 4, TEXT("Slot[0].3 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[0].SlotId == 0, TEXT("Slot[0].SlotId It is not the expected value."));

				AddErrorIfFalse(Widget->Slots[1].SlotAttribute1.Get() == 5, TEXT("Slot[1].1 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[1].SlotAttribute2.Get() == 6, TEXT("Slot[1].2 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[1].SlotAttribute3.Get() == 7, TEXT("Slot[1].3 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[1].SlotId == 1, TEXT("Slot[1].SlotId It is not the expected value."));

				AddErrorIfFalse(Widget->Slots[2].SlotAttribute1.Get() == 8, TEXT("Slot[2].1 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[2].SlotAttribute2.Get() == 9, TEXT("Slot[2].2 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[2].SlotAttribute3.Get() == 10, TEXT("Slot[2].3 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[2].SlotId == 2, TEXT("Slot[2].SlotId It is not the expected value."));
				
				AddErrorIfFalse(Widget->AttributeA.Get() == 11, TEXT("A It is not the expected value."));
				AddErrorIfFalse(Widget->AttributeB.Get() == 1, TEXT("B It is not the expected value."));
				AddErrorIfFalse(Widget->AttributeC.Get() == 12, TEXT("C It is not the expected value."));
			}
			{
				Widget->Slots.Move(0, 1);

				AddErrorIfFalse(Widget->Slots[1].SlotAttribute1.Get() == 2, TEXT("Slot[0].1 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[1].SlotAttribute2.Get() == 3, TEXT("Slot[0].2 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[1].SlotAttribute3.Get() == 4, TEXT("Slot[0].3 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[1].SlotId == 0, TEXT("Slot[1].SlotId It is not the expected value."));

				AddErrorIfFalse(Widget->Slots[0].SlotAttribute1.Get() == 5, TEXT("Slot[1].1 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[0].SlotAttribute2.Get() == 6, TEXT("Slot[1].2 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[0].SlotAttribute3.Get() == 7, TEXT("Slot[1].3 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[0].SlotId == 1, TEXT("Slot[0].SlotId It is not the expected value."));

				AddErrorIfFalse(Widget->Slots[2].SlotAttribute1.Get() == 8, TEXT("Slot[2].1 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[2].SlotAttribute2.Get() == 9, TEXT("Slot[2].2 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[2].SlotAttribute3.Get() == 10, TEXT("Slot[2].3 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[2].SlotId == 2, TEXT("Slot[2].SlotId It is not the expected value."));

				OrderCounter = 0;
				Widget->MarkPrepassAsDirty();
				Widget->SlatePrepass(1.f);

				AddErrorIfFalse(Widget->Slots[0].SlotAttribute1.Get() == 2, TEXT("Slot[0].1 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[0].SlotAttribute2.Get() == 3, TEXT("Slot[0].2 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[0].SlotAttribute3.Get() == 4, TEXT("Slot[0].3 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[0].SlotId == 1, TEXT("Slot[0].SlotId It is not the expected value."));

				AddErrorIfFalse(Widget->Slots[1].SlotAttribute1.Get() == 5, TEXT("Slot[1].1 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[1].SlotAttribute2.Get() == 6, TEXT("Slot[1].2 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[1].SlotAttribute3.Get() == 7, TEXT("Slot[1].3 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[1].SlotId == 0, TEXT("Slot[1].SlotId It is not the expected value."));

				AddErrorIfFalse(Widget->Slots[2].SlotAttribute1.Get() == 8, TEXT("Slot[2].1 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[2].SlotAttribute2.Get() == 9, TEXT("Slot[2].2 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[2].SlotAttribute3.Get() == 10, TEXT("Slot[2].3 It is not the expected value."));
				AddErrorIfFalse(Widget->Slots[2].SlotId == 2, TEXT("Slot[2].SlotId It is not the expected value."));

				AddErrorIfFalse(Widget->AttributeA.Get() == 11, TEXT("A It is not the expected value."));
				AddErrorIfFalse(Widget->AttributeB.Get() == 1, TEXT("B It is not the expected value."));
				AddErrorIfFalse(Widget->AttributeC.Get() == 12, TEXT("C It is not the expected value."));
			}
		}
	}

	// Test OnInvalidation
	{
		{
			TSharedRef<UE::Slate::Private::SAttributeLeftWidget_OnInvalidationParent> WidgetParent
				= SNew(UE::Slate::Private::SAttributeLeftWidget_OnInvalidationParent);


			AddErrorIfFalse(WidgetParent->CallbackIsConstructedValueA == 0, TEXT("The callback for value A was triggered while the widget was constructed."));
			AddErrorIfFalse(WidgetParent->CallbackIsConstructedValueB == 0, TEXT("The callback for value B was triggered while the widget was constructed."));
			AddErrorIfFalse(WidgetParent->CallbackIsConstructedValueC == 0, TEXT("The callback for value C was triggered while the widget was constructed."));

			WidgetParent->SetCallbackValue(10);
			WidgetParent->MarkPrepassAsDirty();
			WidgetParent->SlatePrepass(1.f);
			AddErrorIfFalse(WidgetParent->CallbackValueA == 10, TEXT("The callback for value A was triggered."));
			AddErrorIfFalse(WidgetParent->CallbackValueB == 10, TEXT("The callback for value B was triggered."));
			AddErrorIfFalse(WidgetParent->CallbackValueC == 10, TEXT("The callback for value C was triggered."));

			AddErrorIfFalse(WidgetParent->CallbackIsConstructedValueA == 0, TEXT("The callback for value A was triggered while the widget was constructed."));
			AddErrorIfFalse(WidgetParent->CallbackIsConstructedValueB == 0, TEXT("The callback for value B was triggered while the widget was constructed."));
			AddErrorIfFalse(WidgetParent->CallbackIsConstructedValueC == 0, TEXT("The callback for value C was triggered while the widget was constructed."));
		}
		{
			TSharedRef<UE::Slate::Private::SAttributeLeftWidget_OnInvalidationChild> WidgetChild = SNew(UE::Slate::Private::SAttributeLeftWidget_OnInvalidationChild);

			AddErrorIfFalse(WidgetChild->CallbackIsConstructedValueA == 0, TEXT("The callback for value A was triggered while the widget was constructed."));
			AddErrorIfFalse(WidgetChild->CallbackIsConstructedValueB == 0, TEXT("The callback for value B was triggered while the widget was constructed."));
			AddErrorIfFalse(WidgetChild->CallbackIsConstructedValueC == 0, TEXT("The callback for value C was triggered while the widget was constructed."));

			WidgetChild->SetCallbackValue(10);
			WidgetChild->MarkPrepassAsDirty();
			WidgetChild->SlatePrepass(1.f);
			AddErrorIfFalse(WidgetChild->CallbackValueA == 10, TEXT("The callback for value A was triggered."));
			AddErrorIfFalse(WidgetChild->CallbackValueB == 10, TEXT("The callback for value B was triggered."));
			AddErrorIfFalse(WidgetChild->CallbackValueC == 10, TEXT("The callback for value C was triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueA == 10, TEXT("The callback for override value A was triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueB == 10, TEXT("The callback for override value B was triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueC == 10, TEXT("The callback for override value C was triggered."));

			AddErrorIfFalse(WidgetChild->CallbackIsConstructedValueA == 0, TEXT("The callback for value A was triggered while the widget was constructed."));
			AddErrorIfFalse(WidgetChild->CallbackIsConstructedValueB == 0, TEXT("The callback for value B was triggered while the widget was constructed."));
			AddErrorIfFalse(WidgetChild->CallbackIsConstructedValueC == 0, TEXT("The callback for value C was triggered while the widget was constructed."));
		}

		{
			TSharedRef<UE::Slate::Private::SAttributeLeftWidget_OnInvalidationParent> WidgetParent = SNew(UE::Slate::Private::SAttributeLeftWidget_OnInvalidationParent);

			AddErrorIfFalse(WidgetParent->CallbackIsConstructedValueA == 0, TEXT("The callback for value A was triggered while the widget was constructed."));
			AddErrorIfFalse(WidgetParent->CallbackIsConstructedValueB == 0, TEXT("The callback for value B was triggered while the widget was constructed."));
			AddErrorIfFalse(WidgetParent->CallbackIsConstructedValueC == 0, TEXT("The callback for value C was triggered while the widget was constructed."));

			WidgetParent->SetCallbackValue(10);
			WidgetParent->SetAttributeA(4);
			AddErrorIfFalse(WidgetParent->CallbackValueA == 4, TEXT("The callback for value A was not triggered."));
			AddErrorIfFalse(WidgetParent->CallbackValueB == 10, TEXT("The callback for value B was triggered."));
			AddErrorIfFalse(WidgetParent->CallbackValueC == 10, TEXT("The callback for value C was triggered."));
			WidgetParent->SetAttributeB(5);
			AddErrorIfFalse(WidgetParent->CallbackValueB == 5, TEXT("The callback for value B was not triggered."));
			AddErrorIfFalse(WidgetParent->CallbackValueC == 10, TEXT("The callback for value C was triggered."));
			WidgetParent->SetAttributeC(6);
			AddErrorIfFalse(WidgetParent->CallbackValueC == 6, TEXT("The callback for value C was not triggered."));
		}

		{
			TSharedRef<UE::Slate::Private::SAttributeLeftWidget_OnInvalidationChild> WidgetChild = SNew(UE::Slate::Private::SAttributeLeftWidget_OnInvalidationChild);
			WidgetChild->SetCallbackValue(10);
			WidgetChild->CallbackValueB = 15;
			WidgetChild->SetAttributeA(4);
			AddErrorIfFalse(WidgetChild->CallbackValueA == 10, TEXT("The callback for value A was triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueA == 4, TEXT("The callback for override value A was not triggered."));
			AddErrorIfFalse(WidgetChild->CallbackValueB == 15, TEXT("The callback for value B was triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueB == 10, TEXT("The callback for override value B was triggered."));
			AddErrorIfFalse(WidgetChild->CallbackValueC == 10, TEXT("The callback for value C was triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueC == 10, TEXT("The callback for override value C was triggered."));
			WidgetChild->SetAttributeB(5);
			AddErrorIfFalse(WidgetChild->CallbackValueB == 5, TEXT("The callback for value B was not triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueB == 15, TEXT("The callback for override value B was not triggered."));
			AddErrorIfFalse(WidgetChild->CallbackValueC == 10, TEXT("The callback for value C was triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueC == 10, TEXT("The callback for override value C was triggered."));
			WidgetChild->SetAttributeC(6);
			AddErrorIfFalse(WidgetChild->CallbackValueC == 6, TEXT("The callback for value C was not triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueC == 6, TEXT("The callback for override value C was not triggered."));
		}
		{
			TSharedRef<UE::Slate::Private::SAttributeLeftWidget_OnInvalidationChild> WidgetChild = SNew(UE::Slate::Private::SAttributeLeftWidget_OnInvalidationChild);
			WidgetChild->SetCallbackValue(10);
			WidgetChild->CallbackValueB = 15;
			WidgetChild->SetAttributeA(MakeAttributeLambda([](){ return 4; }));
			WidgetChild->SetAttributeB(MakeAttributeLambda([](){ return 5; }));
			WidgetChild->SetAttributeC(MakeAttributeLambda([](){ return 6; }));

			AddErrorIfFalse(WidgetChild->CallbackValueA == 10, TEXT("The callback for value A was triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueA == 10, TEXT("The callback for override value A was triggered."));
			AddErrorIfFalse(WidgetChild->CallbackValueB == 15, TEXT("The callback for value B was triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueB == 10, TEXT("The callback for override value B was triggered."));
			AddErrorIfFalse(WidgetChild->CallbackValueC == 10, TEXT("The callback for value C was triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueC == 10, TEXT("The callback for override value C was triggered."));

			WidgetChild->MarkPrepassAsDirty();
			WidgetChild->SlatePrepass(1.f);

			AddErrorIfFalse(WidgetChild->CallbackValueA == 10, TEXT("The callback for value A was triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueA == 4, TEXT("The callback for override value A was not triggered."));
			AddErrorIfFalse(WidgetChild->CallbackValueB == 5, TEXT("The callback for value B was not triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueB == 15, TEXT("The callback for override value B was not triggered."));
			AddErrorIfFalse(WidgetChild->CallbackValueC == 6, TEXT("The callback for value C was not triggered."));
			AddErrorIfFalse(WidgetChild->OverrideCallbackValueC == 6, TEXT("The callback for override value C was not triggered."));
		}
	}

	// Test Multi Inheritance
	// Worth noting that the vtable order is not deterministic by the different compilers (especially on Switch)
	{
		auto MultiInheritanceWidget_Child_Test = [this]<typename WidgetType>(TSharedRef<WidgetType> Widget)
		{
			const FSlateAttributeDescriptor& AttributeDescriptor = Widget->GetWidgetClass().GetAttributeDescriptor();
			const int32 IndexBI  = AttributeDescriptor.IndexOfAttribute("BaseIntAttribute");
			const int32 IndexBUI = AttributeDescriptor.IndexOfAttribute("BaseUIntAttribute");
			const int32 IndexPI  = AttributeDescriptor.IndexOfAttribute("ParentIntAttribute");
			const int32 IndexPUI = AttributeDescriptor.IndexOfAttribute("ParentUIntAttribute");
			const int32 IndexI   = AttributeDescriptor.IndexOfAttribute("ChildIntAttribute");
			const int32 IndexUI  = AttributeDescriptor.IndexOfAttribute("ChildUIntAttribute");

			AddErrorIfFalse(IndexBI  != INDEX_NONE, FString::Printf(TEXT("[%s] Could not find the BaseIntAttribute"), *Widget->GetTypeAsString()));
			AddErrorIfFalse(IndexBUI != INDEX_NONE, FString::Printf(TEXT("[%s] Could not find the BaseUIntAttribute"), *Widget->GetTypeAsString()));
			AddErrorIfFalse(IndexPI  != INDEX_NONE, FString::Printf(TEXT("[%s] Could not find the ParentIntAttribute"), *Widget->GetTypeAsString()));
			AddErrorIfFalse(IndexPUI != INDEX_NONE, FString::Printf(TEXT("[%s] Could not find the ParentUIntAttribute"), *Widget->GetTypeAsString()));
			AddErrorIfFalse(IndexI   != INDEX_NONE, FString::Printf(TEXT("[%s] Could not find the IntAttribute"), *Widget->GetTypeAsString()));
			AddErrorIfFalse(IndexUI  != INDEX_NONE, FString::Printf(TEXT("[%s] Could not find the UIntAttribute"), *Widget->GetTypeAsString()));

			AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexBI)  == AttributeDescriptor.FindAttribute("BaseIntAttribute"), FString::Printf( TEXT("[%s] Index and Attribute should return the same value."), *Widget->GetTypeAsString()));
			AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexBUI) == AttributeDescriptor.FindAttribute("BaseUIntAttribute"), FString::Printf( TEXT("[%s] Index and Attribute should return the same value."), *Widget->GetTypeAsString()));
			AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexPI)  == AttributeDescriptor.FindAttribute("ParentIntAttribute"), FString::Printf( TEXT("[%s] Index and Attribute should return the same value."), *Widget->GetTypeAsString()));
			AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexPUI) == AttributeDescriptor.FindAttribute("ParentUIntAttribute"), FString::Printf( TEXT("[%s] Index and Attribute should return the same value."), *Widget->GetTypeAsString()));
			AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexI)   == AttributeDescriptor.FindAttribute("ChildIntAttribute"), FString::Printf( TEXT("[%s] Index and Attribute should return the same value."), *Widget->GetTypeAsString()));
			AddErrorIfFalse(&AttributeDescriptor.GetAttributeAtIndex(IndexUI)  == AttributeDescriptor.FindAttribute("ChildUIntAttribute"), FString::Printf( TEXT("[%s] Index and Attribute should return the same value."), *Widget->GetTypeAsString()));

			AddErrorIfFalse(Widget->CallbackIntValue == 0, FString::Printf( TEXT("[%s] The Int callback was triggered."), *Widget->GetTypeAsString()));
			AddErrorIfFalse(Widget->CallbackUIntValue == 0, FString::Printf( TEXT("[%s] The UInt callback was triggered."), *Widget->GetTypeAsString()));

			Widget->BaseIntAttribute.Assign(*Widget, -8);
			AddErrorIfFalse(Widget->CallbackIntValue == -8, FString::Printf( TEXT("[%s] CallbackIntValue is not the expected value after setting 'BaseIntAttribute': Expected: %d  Value: %d."), *Widget->GetTypeAsString(), -8, Widget->CallbackIntValue));
			Widget->ParentIntAttribute.Assign(*Widget, -80);
			AddErrorIfFalse(Widget->CallbackIntValue == -88, FString::Printf( TEXT("[%s] CallbackIntValue is not the expected value after setting 'ParentIntAttribute': Expected: %d  Value: %d."), *Widget->GetTypeAsString(), -88, Widget->CallbackIntValue));
			Widget->ChildIntAttribute.Assign(*Widget, -800);
			AddErrorIfFalse(Widget->CallbackIntValue == -888, FString::Printf( TEXT("[%s] CallbackIntValue is not the expected value after setting 'ChildIntAttribute': Expected: %d  Value: %d."), *Widget->GetTypeAsString(), -888, Widget->CallbackIntValue));

			Widget->ParentUIntAttribute.Assign(*Widget, 700);
			AddErrorIfFalse(Widget->CallbackUIntValue == 700, FString::Printf( TEXT("[%s] CallbackUIntValue is not the expected value after setting 'ParentUIntAttribute': Expected: %d  Value: %d."), *Widget->GetTypeAsString(), 700, Widget->CallbackUIntValue));
			Widget->ChildUIntAttribute.Assign(*Widget, 70);
			AddErrorIfFalse(Widget->CallbackUIntValue == 770, FString::Printf( TEXT("[%s] CallbackUIntValue is not the expected value after setting 'ChildUIntAttribute': Expected: %d  Value: %d."), *Widget->GetTypeAsString(), 770, Widget->CallbackUIntValue));
			Widget->BaseUIntAttribute.Assign(*Widget, 7);
			AddErrorIfFalse(Widget->CallbackUIntValue == 777, FString::Printf( TEXT("[%s] CallbackUIntValue is not the expected value after setting 'BaseUIntAttribute': Expected: %d  Value: %d."), *Widget->GetTypeAsString(), 777, Widget->CallbackUIntValue));
		};

		MultiInheritanceWidget_Child_Test(SNew(UE::Slate::Private::SMultiInheritanceWidget_Child_SII));
		MultiInheritanceWidget_Child_Test(SNew(UE::Slate::Private::SMultiInheritanceWidget_Child_ISI));
		MultiInheritanceWidget_Child_Test(SNew(UE::Slate::Private::SMultiInheritanceWidget_Child_IIS));

		MultiInheritanceWidget_Child_Test(SNew(UE::Slate::Private::SMultiInheritanceWidget_GrandChild_SII_SII));
		MultiInheritanceWidget_Child_Test(SNew(UE::Slate::Private::SMultiInheritanceWidget_GrandChild_SII_ISI));
		MultiInheritanceWidget_Child_Test(SNew(UE::Slate::Private::SMultiInheritanceWidget_GrandChild_SII_IIS));
		MultiInheritanceWidget_Child_Test(SNew(UE::Slate::Private::SMultiInheritanceWidget_GrandChild_ISI_SII));
		MultiInheritanceWidget_Child_Test(SNew(UE::Slate::Private::SMultiInheritanceWidget_GrandChild_ISI_ISI));
		MultiInheritanceWidget_Child_Test(SNew(UE::Slate::Private::SMultiInheritanceWidget_GrandChild_ISI_IIS));
		MultiInheritanceWidget_Child_Test(SNew(UE::Slate::Private::SMultiInheritanceWidget_GrandChild_IIS_SII));
		MultiInheritanceWidget_Child_Test(SNew(UE::Slate::Private::SMultiInheritanceWidget_GrandChild_IIS_ISI));
		MultiInheritanceWidget_Child_Test(SNew(UE::Slate::Private::SMultiInheritanceWidget_GrandChild_IIS_IIS));
	}
	return true;
}

#undef LOCTEXT_NAMESPACE 

#endif //WITH_AUTOMATION_WORKER
