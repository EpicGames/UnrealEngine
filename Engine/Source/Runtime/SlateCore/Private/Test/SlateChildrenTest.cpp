// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "AutoRTFM/Constants.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/SLeafWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"


#if WITH_AUTOMATION_WORKER

#define LOCTEXT_NAMESPACE "SlateChildrenTest"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSlateChildrenForEachTest, "Slate.Children.ForEach", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

namespace UE::Slate::Test::Private
{

	/** */
	class SEmptyLeftWidget : public SLeafWidget
	{
	public:
		SLATE_BEGIN_ARGS(SEmptyLeftWidget) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
		}
		virtual FVector2D ComputeDesiredSize(float) const override
		{
			return FVector2D{ 100, 100 };
		}
		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
		{
			return LayerId;
		}
	};

	/** */
	class SWeakChild : public SWidget
	{
	public:
		SLATE_BEGIN_ARGS(SWeakChild) {}
			SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_END_ARGS()

	public:
		SWeakChild()
			: WeakChild(this)
		{
		}

		void Construct(const FArguments& InArgs)
		{
			WeakChild.AttachWidget(InArgs._Content.Widget);
		}

		virtual FChildren* GetChildren() override
		{
			return &WeakChild;
		}
		
		virtual void OnArrangeChildren(const FGeometry&, FArrangedChildren&) const override
		{
		}

		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
		{
			return LayerId;
		}

		virtual FVector2D ComputeDesiredSize(float) const override
		{
			return FVector2D{ 100, 100 };
		}

	private:
		TWeakChild<SWidget> WeakChild;
	};

	/** */
	class SSingleChild : public SWidget
	{
	public:
		SLATE_BEGIN_ARGS(SSingleChild) {}
			SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_END_ARGS()

		SSingleChild()
			: ChildSlot(this)
		{
		}

		void Construct(const FArguments& InArgs)
		{
			ChildSlot[InArgs._Content.Widget];
		}

		virtual FChildren* GetChildren() override
		{
			return &ChildSlot;
		}

		virtual void OnArrangeChildren(const FGeometry&, FArrangedChildren&) const override
		{
		}

		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
		{
			return LayerId;
		}

		virtual FVector2D ComputeDesiredSize(float) const override
		{
			return FVector2D{ 100, 100 };
		}

	private:
		FSingleWidgetChildrenWithSlot ChildSlot;
	};

	TSharedRef<SVerticalBox> AddVerticalBox(TSharedPtr<SVerticalBox> VerticalBox, TCHAR Letter)
	{
		FName NewName = *FString::Printf(TEXT("TagVerticalBox-%c"), Letter);
		TSharedRef<SVerticalBox> Result = SNew(SVerticalBox).Tag(NewName);
		if (VerticalBox)
		{
			VerticalBox->AddSlot()[Result];
		}
		return Result;
	}

	TSharedRef<SWidget> AddEmptyWidget(TSharedPtr<SVerticalBox> VerticalBox, int32 Number)
	{
		FName NewName = *FString::Printf(TEXT("TagLeafWidget-%d"), Number);
		TSharedRef<SWidget> Result = SNew(SEmptyLeftWidget).Tag(NewName);
		VerticalBox->AddSlot()[Result];
		return Result;
	}

	TSharedRef<SVerticalBox> BuildNormalTest()
	{
		// A
		//  B (1, 2, 3)
		//  C (4, 5, 6, 7)
		//  Null
		//  D
		//  E (8, 9, 10)
		//  F
		//   G (11, Null, 12)
		//   H (13)
		//   I
		//  J (14)

		TSharedRef<SVerticalBox> Root = AddVerticalBox(nullptr, TEXT('A'));
		{
			TSharedRef<SVerticalBox> Sub = AddVerticalBox(Root, TEXT('B'));
			AddEmptyWidget(Sub, 1);
			AddEmptyWidget(Sub, 2);
			AddEmptyWidget(Sub, 3);
		}
		{
			TSharedRef<SVerticalBox> Sub = AddVerticalBox(Root, TEXT('C'));
			AddEmptyWidget(Sub, 4);
			AddEmptyWidget(Sub, 5);
			AddEmptyWidget(Sub, 6);
			AddEmptyWidget(Sub, 7);
		}
		Root->AddSlot()[SNullWidget::NullWidget];
		AddVerticalBox(Root, TEXT('D'));
		{
			TSharedRef<SVerticalBox> Sub = AddVerticalBox(Root, TEXT('E'));
			AddEmptyWidget(Sub, 8);
			AddEmptyWidget(Sub, 9);
			AddEmptyWidget(Sub, 10);
		}
		{
			TSharedRef<SVerticalBox> Sub = AddVerticalBox(Root, TEXT('F'));
			{
				TSharedRef<SVerticalBox> SubSub = AddVerticalBox(Sub, TEXT('G'));
				AddEmptyWidget(SubSub, 11);
				SubSub->AddSlot()[SNullWidget::NullWidget];
				AddEmptyWidget(SubSub, 12);
			}
			{
				TSharedRef<SVerticalBox> SubSub = AddVerticalBox(Sub, TEXT('H'));
				AddEmptyWidget(SubSub, 13);
			}
			{
				AddVerticalBox(Sub, TEXT('I'));
			}
		}
		{
			TSharedRef<SVerticalBox> Sub = AddVerticalBox(Root, TEXT('J'));
			AddEmptyWidget(Sub, 14);
		}

		return Root;
	}

	TSharedRef<SVerticalBox> BuildChildTest(TSharedPtr<SWidget>& OutWeakWidget)
	{
		// A
		//  B
		//   C-Weak
		//    D (1, 2)
		//   E-Single
		//    F (3, 4)
		//   G (5, 6)
		//  H (7)

		{
			TSharedRef<SVerticalBox> SubD = AddVerticalBox(nullptr, TEXT('D'));
			AddEmptyWidget(SubD, 1);
			AddEmptyWidget(SubD, 2);
			OutWeakWidget = SubD;
		}

		TSharedRef<SVerticalBox> Root = AddVerticalBox(nullptr, TEXT('A'));
		{
			TSharedRef<SVerticalBox> SubB = AddVerticalBox(Root, TEXT('B'));
			{
				FName NewName = "Weak-C";
				TSharedRef<SWidget> Weak = SNew(SWeakChild).Tag(NewName)[OutWeakWidget.ToSharedRef()];
				SubB->AddSlot()[Weak];
			}
			{
				TSharedRef<SVerticalBox> SubF = AddVerticalBox(nullptr, TEXT('F'));
				AddEmptyWidget(SubF, 3);
				AddEmptyWidget(SubF, 4);

				FName NewName = "Single-E";
				TSharedRef<SWidget> SingleChild = SNew(SSingleChild).Tag(NewName)[SubF];
				SubB->AddSlot()[SingleChild];
			}
			{
				TSharedRef<SVerticalBox> SubG = AddVerticalBox(SubB, TEXT('G'));
				AddEmptyWidget(SubG, 5);
				AddEmptyWidget(SubG, 6);
			}
		}
		{
			TSharedRef<SVerticalBox> SubH = AddVerticalBox(Root, TEXT('H'));
			AddEmptyWidget(SubH, 7);
		}
		return Root;
	}

	template<typename Pred>
	AUTORTFM_DISABLE_IF(AutoRTFM::IsDisabled(AutoRTFM::CallMode<Pred, SWidget&>))
	void Recursive(SWidget & Widget, Pred & InPred)
	{
		Widget.GetAllChildren()->ForEachWidget([&InPred](SWidget& Widget)
			{
				InPred(Widget);
				Recursive(Widget, InPred);
			});
	}

} // namespace

bool FSlateChildrenForEachTest::RunTest(const FString& Parameters)
{
	{
		TSharedRef<SVerticalBox> RootChildOrder = UE::Slate::Test::Private::BuildNormalTest();

		{
			int32 Count = 0;
			RootChildOrder->GetAllChildren()->ForEachWidget([&Count](const SWidget& Widget)
				{
					++Count;
				});

			AddErrorIfFalse(Count == 7, FString::Printf(TEXT("Iterated over %d out of 7 SWidget. (BuildNormalTest)"), Count));
		}
		{
			int32 Count = 0;
			auto Pred = [&Count](const SWidget& Widget)
			{
				++Count;
			};
			UE::Slate::Test::Private::Recursive(RootChildOrder.Get(), Pred);
			AddErrorIfFalse(Count == 25, FString::Printf(TEXT("Iterated recursivly over %d out of 25 SWidget. (BuildNormalTest)"), Count));
		}
	}
	{
		TSharedPtr<SWidget> WeakWidget;
		TSharedRef<SVerticalBox> RootChildOrder = UE::Slate::Test::Private::BuildChildTest(WeakWidget);
		{
			int32 Count = 0;
			RootChildOrder->GetAllChildren()->ForEachWidget([&Count](const SWidget& Widget)
				{
					++Count;
				});

			AddErrorIfFalse(Count == 2, FString::Printf(TEXT("Iterated over %d out of 2 SWidget. (BuildChildTest)"), Count));
		}
		{
			int32 Count = 0;
			auto Pred = [&Count](const SWidget& Widget)
			{
				++Count;
			};
			UE::Slate::Test::Private::Recursive(RootChildOrder.Get(), Pred);
			AddErrorIfFalse(Count == 14, FString::Printf(TEXT("Iterated recursivly over %d out of 14 SWidget (BuildChildTest)."), Count));
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE 

#endif //WITH_AUTOMATION_WORKER
