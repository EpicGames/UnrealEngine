// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#include "FastUpdate/SlateInvalidationWidgetList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SLeafWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"


#if WITH_AUTOMATION_WORKER & WITH_SLATE_DEBUGGING

#define LOCTEXT_NAMESPACE "Slate.FastPath.InvalidationWidgetList"

//IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSlateInvalidationWidgetListTest, "Slate.InvalidationWidgetList.AddBuildRemove", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSlateInvalidationWidgetListTest, "Slate.InvalidationWidgetList.AddBuildRemove", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

namespace UE::Slate::Private
{

static const bool bUpdateOnlyWhatIsNeeded = false;

class SEmptyLeftWidget : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SEmptyLeftWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs){}
	virtual FVector2D ComputeDesiredSize(float) const override { return FVector2D{ 100, 100 }; }
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		return LayerId;
	}
};

TSharedRef<SVerticalBox> AddVerticalBox(TSharedPtr<SVerticalBox> VerticalBox, TCHAR Letter)
{
	FName NewName = *FString::Printf(TEXT("TagVerticalBox-%c"), Letter);
	TSharedRef<SVerticalBox> Result = SNew(SVerticalBox).Tag(NewName);
	if (VerticalBox)
	{
		VerticalBox->AddSlot()[Result];
	}
	return Result;
}

TSharedRef<SWidget> AddEmptyWidget(TSharedPtr<SVerticalBox> VerticalBox, int32 Number)
{
	FName NewName = *FString::Printf(TEXT("TagLeafWidget-%d"), Number);
	TSharedRef<SWidget> Result = SNew(SEmptyLeftWidget).Tag(NewName);
	VerticalBox->AddSlot()[Result];
	return Result;
}

TSharedRef<SVerticalBox> BuildTestUI_ChildOrder(TSharedPtr<SVerticalBox>& WidgetC, TSharedPtr<SVerticalBox>& WidgetE, TSharedPtr<SVerticalBox>& WidgetF)
{
	// A
	//  B (1, 2, 3)
	//  C (4, 5, 6, 7)
	//  Null
	//  D
	//  E (8, 9, 10)
	//  F
	//   G (11, Null, 12)
	//   H (13)
	//   I
	//  J (14)

	TSharedRef<SVerticalBox> Root = AddVerticalBox(nullptr, TEXT('A'));
	{
		TSharedRef<SVerticalBox> Sub = AddVerticalBox(Root, TEXT('B'));
		AddEmptyWidget(Sub, 1);
		AddEmptyWidget(Sub, 2);
		AddEmptyWidget(Sub, 3);
	}
	{
		WidgetC = AddVerticalBox(Root, TEXT('C'));
		AddEmptyWidget(WidgetC, 4);
		AddEmptyWidget(WidgetC, 5);
		AddEmptyWidget(WidgetC, 6);
		AddEmptyWidget(WidgetC, 7);
	}
	Root->AddSlot()[SNullWidget::NullWidget];
	AddVerticalBox(Root, TEXT('D'));
	{
		WidgetE = AddVerticalBox(Root, TEXT('E'));
		AddEmptyWidget(WidgetE, 8);
		AddEmptyWidget(WidgetE, 9);
		AddEmptyWidget(WidgetE, 10);
	}
	{
		WidgetF = AddVerticalBox(Root, TEXT('F'));
		{
			TSharedRef<SVerticalBox> SubSub = AddVerticalBox(WidgetF, TEXT('G'));
			AddEmptyWidget(SubSub, 11);
			SubSub->AddSlot()[SNullWidget::NullWidget];
			AddEmptyWidget(SubSub, 12);
		}
		{
			TSharedRef<SVerticalBox> SubSub = AddVerticalBox(WidgetF, TEXT('H'));
			AddEmptyWidget(SubSub, 13);
		}
		{
			AddVerticalBox(WidgetF, TEXT('I'));
		}
	}
	{
		TSharedRef<SVerticalBox> SubSub = AddVerticalBox(Root, TEXT('J'));
		AddEmptyWidget(SubSub, 14);
	}

	return Root;
}

TSharedRef<SVerticalBox> BuildTestUI_Child(TSharedPtr<SVerticalBox>& WidgetA, TArray<TSharedPtr<SWidget>>& ChildOfWidgetA
	, TSharedPtr<SVerticalBox>& WidgetB, TArray<TSharedPtr<SWidget>>& ChildOfWidgetB
	, TSharedPtr<SVerticalBox>& WidgetC, TArray<TSharedPtr<SWidget>>& ChildOfWidgetC
	, TSharedPtr<SVerticalBox>& WidgetD, TArray<TSharedPtr<SWidget>>& ChildOfWidgetD
	, TSharedPtr<SVerticalBox>& WidgetH, TArray<TSharedPtr<SWidget>>& ChildOfWidgetH)
{
	// A
	//  B (1, 2, 3)
	//  C
	//  D
	//   E (4, 5)
	//   F (6)
	//   G (7)
	//  H (8)
	//  I

	WidgetA = AddVerticalBox(nullptr, TEXT('A'));
	{
		WidgetB = AddVerticalBox(WidgetA, TEXT('B'));
		ChildOfWidgetA.Add(WidgetB);
		ChildOfWidgetB.Add(AddEmptyWidget(WidgetB, 1));
		ChildOfWidgetB.Add(AddEmptyWidget(WidgetB, 2));
		ChildOfWidgetB.Add(AddEmptyWidget(WidgetB, 3));
	}
	{
		WidgetC = AddVerticalBox(WidgetA, TEXT('C'));
		ChildOfWidgetA.Add(WidgetC);
	}
	{
		WidgetD = AddVerticalBox(WidgetA, TEXT('D'));
		ChildOfWidgetA.Add(WidgetD);
		{
			TSharedRef<SVerticalBox> SubSub = AddVerticalBox(WidgetD, TEXT('E'));
			ChildOfWidgetD.Add(SubSub);
			AddEmptyWidget(SubSub, 4);
			AddEmptyWidget(SubSub, 5);
		}
		{
			TSharedRef<SVerticalBox> SubSub = AddVerticalBox(WidgetD, TEXT('F'));
			ChildOfWidgetD.Add(SubSub);
			AddEmptyWidget(SubSub, 6);
		}
		{
			TSharedRef<SVerticalBox> SubSub = AddVerticalBox(WidgetD, TEXT('G'));
			ChildOfWidgetD.Add(SubSub);
			AddEmptyWidget(SubSub, 7);
		}
	}
	{
		WidgetH = AddVerticalBox(WidgetA, TEXT('H'));
		ChildOfWidgetA.Add(WidgetH);
		ChildOfWidgetH.Add(AddEmptyWidget(WidgetH, 8));
	}
	ChildOfWidgetA.Add(AddVerticalBox(WidgetA, TEXT('I')));
	return WidgetA.ToSharedRef();
}

} // namespace


bool FSlateInvalidationWidgetListTest::RunTest(const FString& Parameters)
{
	FSlateInvalidationWidgetList::FArguments ArgsToTest[] = {{4,2}, {4,3}, {5,1}, {6, 1}};
	for (FSlateInvalidationWidgetList::FArguments Args : ArgsToTest)
	{
		{
			TSharedPtr<SVerticalBox> WidgetA, WidgetB, WidgetC, WidgetD, WidgetH;
			TArray<TSharedPtr<SWidget>> ChildOfWidgetA, ChildOfWidgetB, ChildOfWidgetC, ChildOfWidgetD, ChildOfWidgetH;
			TSharedRef<SVerticalBox> RootChild = UE::Slate::Private::BuildTestUI_Child(WidgetA, ChildOfWidgetA, WidgetB, ChildOfWidgetB, WidgetC, ChildOfWidgetC, WidgetD, ChildOfWidgetD, WidgetH, ChildOfWidgetH);

			FSlateInvalidationWidgetList List = { FSlateInvalidationRootHandle(), Args };
			List.BuildWidgetList(RootChild);
			AddErrorIfFalse(List.VerifyWidgetsIndex(), TEXT("The widget list integrity has failed."));

			{
				TArray<TSharedPtr<SWidget>> FoundChildren = List.FindChildren(WidgetA.ToSharedRef().Get());
				if (ChildOfWidgetA != FoundChildren)
				{
					AddError(TEXT("Was not able to find the child of VerticalBox A."));
				}
			}
			{
				TArray<TSharedPtr<SWidget>> FoundChildren = List.FindChildren(WidgetB.ToSharedRef().Get());
				if (ChildOfWidgetB != FoundChildren)
				{
					AddError(TEXT("Was not able to find the child of VerticalBox B."));
				}
			}
			{
				TArray<TSharedPtr<SWidget>> FoundChildren = List.FindChildren(WidgetC.ToSharedRef().Get());
				if (ChildOfWidgetC != FoundChildren)
				{
					AddError(TEXT("Was not able to find the child of VerticalBox C."));
				}
			}
			{
				TArray<TSharedPtr<SWidget>> FoundChildren = List.FindChildren(WidgetD.ToSharedRef().Get());
				if (ChildOfWidgetD != FoundChildren)
				{
					AddError(TEXT("Was not able to find the child of VerticalBox D."));
				}
			}
			{
				TArray<TSharedPtr<SWidget>> FoundChildren = List.FindChildren(WidgetH.ToSharedRef().Get());
				if (ChildOfWidgetH != FoundChildren)
				{
					AddError(TEXT("Was not able to find the child of VerticalBox H."));
				}
			}
		}

		{
			TSharedPtr<SVerticalBox> WidgetC, WidgetE, WidgetF;
			TSharedRef<SVerticalBox> RootChildOrder = UE::Slate::Private::BuildTestUI_ChildOrder(WidgetC, WidgetE, WidgetF);

			FSlateInvalidationWidgetList List = { FSlateInvalidationRootHandle(), Args };
			List.BuildWidgetList(RootChildOrder);
			AddErrorIfFalse(List.VerifyWidgetsIndex(), TEXT("The widget list integrity has failed."));

			// Remove Second child of F
			{
				List.RemoveWidget(WidgetF->GetAllChildren()->GetChildAt(1).Get());
				WidgetF->RemoveSlot(WidgetF->GetAllChildren()->GetChildAt(1));
				{
					FSlateInvalidationWidgetList::FArguments ArgsWithoutAssign = Args;
					ArgsWithoutAssign.bAssignedWidgetIndex = false;
					FSlateInvalidationWidgetList TempList = { FSlateInvalidationRootHandle(), ArgsWithoutAssign };
					TempList.BuildWidgetList(RootChildOrder);
					if (!TempList.DeepCompare(List))
					{
						AddError(TEXT("Was not able to remove a child of F."));
					}
				}
			}

			// Remove C and F
			{
				{
					const FSlateInvalidationWidgetIndex WidgetIndexF = List.FindWidget(WidgetF.ToSharedRef().Get());
					if (!List.IsValidIndex(WidgetIndexF) || List[WidgetIndexF].GetWidget() != WidgetF.Get())
					{
						AddError(TEXT("The index of F is not valid anymore."));
					}
					List.RemoveWidget(WidgetIndexF);
				}
				{
					const FSlateInvalidationWidgetIndex WidgetIndexC = List.FindWidget(WidgetC.ToSharedRef().Get());
					if (!List.IsValidIndex(WidgetIndexC) || List[WidgetIndexC].GetWidget() != WidgetC.Get())
					{
						AddError(TEXT("The index of C is not valid anymore."));
					}
					List.RemoveWidget(WidgetIndexC);
				}
				RootChildOrder->RemoveSlot(WidgetF.ToSharedRef());
				RootChildOrder->RemoveSlot(WidgetC.ToSharedRef());
				{
					FSlateInvalidationWidgetList::FArguments ArgsWithoutAssign = Args;
					ArgsWithoutAssign.bAssignedWidgetIndex = false;
					FSlateInvalidationWidgetList TempList = { FSlateInvalidationRootHandle(), ArgsWithoutAssign };
					TempList.BuildWidgetList(RootChildOrder);
					if (!TempList.DeepCompare(List))
					{
						AddError(TEXT("Was not able to remove F and C."));
					}
				}
			}
			// remove last item
			{
				int32 ToRemoveIndex = RootChildOrder->GetAllChildren()->Num() - 1;
				TSharedRef<SWidget> RemovedWidget = RootChildOrder->GetAllChildren()->GetChildAt(ToRemoveIndex);
				RootChildOrder->RemoveSlot(RemovedWidget);
				List.RemoveWidget(List.FindWidget(RemovedWidget.Get()));

				{
					FSlateInvalidationWidgetList::FArguments ArgsWithoutAssign = Args;
					ArgsWithoutAssign.bAssignedWidgetIndex = false;
					FSlateInvalidationWidgetList TempList = { FSlateInvalidationRootHandle(), ArgsWithoutAssign };
					TempList.BuildWidgetList(RootChildOrder);
					if (!TempList.DeepCompare(List))
					{
						AddError(TEXT("Was not able to remove the last item of A."));
					}
				}
			}
		}
		{
			TSharedPtr<SVerticalBox> WidgetC, WidgetE, WidgetF;
			TSharedRef<SVerticalBox> RootChildOrder = UE::Slate::Private::BuildTestUI_ChildOrder(WidgetC, WidgetE, WidgetF);

			// Child invalidation
			FSlateInvalidationWidgetList List = { FSlateInvalidationRootHandle(), Args };
			List.BuildWidgetList(RootChildOrder);
			AddErrorIfFalse(List.VerifyWidgetsIndex(), TEXT("The widget list integrity has failed."));

			struct FChildOrderInvalidationCallback_Count : FSlateInvalidationWidgetList::IProcessChildOrderInvalidationCallback
			{
				mutable int32 ChildRemoveCount = 0;
				mutable int32 ReIndexedCount = 0;
				mutable int32 ResortCount = 0;
				mutable int32 BuiltCount = 0;
				virtual void PreChildRemove(const FSlateInvalidationWidgetList::FIndexRange& Range) override { ++ChildRemoveCount; }
				using FReIndexOperation = FSlateInvalidationWidgetList::IProcessChildOrderInvalidationCallback::FReIndexOperation;
				virtual void ProxiesReIndexed(const FReIndexOperation& Operation) override { ++ReIndexedCount; }
				using FReSortOperation = FSlateInvalidationWidgetList::IProcessChildOrderInvalidationCallback::FReSortOperation;
				virtual void ProxiesPreResort(const FReSortOperation& Operation) override { ++ResortCount; }
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_CHILDORDERCHECK
				virtual void ProxiesBuilt(const FSlateInvalidationWidgetList::FIndexRange& Range) override { ++BuiltCount; }
#endif
			};

			auto TestRemoveWidget = [&](TSharedPtr<SVerticalBox>& Widget, const TCHAR* Message) -> bool
				{
					FChildOrderInvalidationCallback_Count Callback;
					const bool bIsProxyValid = List.ProcessChildOrderInvalidation(Widget->GetProxyHandle().GetWidgetIndex(), Callback);
					AddErrorIfFalse(bIsProxyValid, FString::Printf(TEXT("The proxy should be valid for '%s'."), Message));
					AddErrorIfFalse(List.VerifyWidgetsIndex(), FString::Printf(TEXT("The widget list integrity has failed for '%s'."), Message));
					AddErrorIfFalse(List.VerifySortOrder(), FString::Printf(TEXT("The widget list sort order has failed for '%s'."), Message));

					{
						FSlateInvalidationWidgetList::FArguments ArgsWithoutAssign = Args;
						ArgsWithoutAssign.bAssignedWidgetIndex = false;
						FSlateInvalidationWidgetList TempList = { FSlateInvalidationRootHandle(), ArgsWithoutAssign };
						TempList.BuildWidgetList(RootChildOrder);
						if (!TempList.DeepCompare(List))
						{
							AddError(FString::Printf(TEXT("Was not able to process invalidation of widget '%s'."), Message));
						}
					}
					return Callback.ChildRemoveCount > 0;
				};

			{
				WidgetF->RemoveSlot(WidgetF->GetAllChildren()->GetChildAt(0));
				bool bChildRemoved = TestRemoveWidget(WidgetF, TEXT("F"));
				AddErrorIfFalse(bChildRemoved, TEXT("F has to removed 1 widget"));
			}

			{
				WidgetE->RemoveSlot(WidgetE->GetAllChildren()->GetChildAt(1));
				bool bChildRemoved = TestRemoveWidget(WidgetE, TEXT("E"));
				AddErrorIfFalse(bChildRemoved, TEXT("E has to removed 1 widget"));
			}

			{
				WidgetC->RemoveSlot(WidgetC->GetAllChildren()->GetChildAt(1));
				WidgetC->RemoveSlot(WidgetC->GetAllChildren()->GetChildAt(1));
				bool bChildRemoved = TestRemoveWidget(WidgetC, TEXT("C"));
				AddErrorIfFalse(bChildRemoved, TEXT("C has to removed 2 widget"));
			}

			{
				bool bChildRemoved = TestRemoveWidget(WidgetC, TEXT("C"));
				bool bUpdateOnlyWhatIsNeeded = UE::Slate::Private::bUpdateOnlyWhatIsNeeded;
				AddErrorIfFalse(!bChildRemoved || !bUpdateOnlyWhatIsNeeded, TEXT("C no widget was removed"));
			}
		}
	}


	{
		// A						//[0]A, B, 1, 2
		//  B (1, 2, 3)				//[1]3, C, 4, 5
		//  C (4, 5, 6, 7)			//[2]6, 7, D, E
		//  Null					//[3]8, 9, 0, F
		//  D						//[4]G, 1, 2, H
		//  E (8, 9, 10)			//[5]3, I, J, 4
		//  F
		//   G (11, Null, 12)
		//   H (13)
		//   I
		//  J (14)

		TSharedPtr<SVerticalBox> WidgetC, WidgetE, WidgetF;
		TSharedRef<SVerticalBox> RootChildOrder = UE::Slate::Private::BuildTestUI_ChildOrder(WidgetC, WidgetE, WidgetF);

		FSlateInvalidationWidgetList::FArguments Arg = { 4,2 };
		FSlateInvalidationWidgetList List = { FSlateInvalidationRootHandle(), Arg };
		List.BuildWidgetList(RootChildOrder);
		AddErrorIfFalse(List.VerifyWidgetsIndex(), TEXT("The widget list integrity has failed."));

		// Test FProcessChildOrderInvalidationResult with G
		{
			struct FChildOrderInvalidationCallback_RemoveG : FSlateInvalidationWidgetList::IProcessChildOrderInvalidationCallback
			{
				virtual ~FChildOrderInvalidationCallback_RemoveG() = default;
				FSlateInvalidationWidgetListTest* Self = nullptr;

				TArray<TTuple<FString, FSlateInvalidationWidgetSortOrder, bool>> PreChildRemoveToCheck;
				virtual void PreChildRemove(const FSlateInvalidationWidgetList::FIndexRange& Range) override
				{
					Self->AddErrorIfFalse(Range.IsValid(), TEXT("The range is not valid."));

					if (PreChildRemoveToCheck.Num() == 0)
					{
						Self->AddError(TEXT("No PreChildRemove was expected"));
					}
					else
					{
						for (const auto& Element : PreChildRemoveToCheck)
						{
							bool bIsIncluded = Range.Include(Element.Get<1>());
							if (bIsIncluded != Element.Get<2>())
							{
								Self->AddError(FString::Printf(TEXT("'%s' should be %s in the PreChildRemove range"), *Element.Get<0>(), (bIsIncluded ? TEXT("included") : TEXT("not be included"))));
							}
						}
					}
				}

				TArray<TTuple<FString, FSlateInvalidationWidgetSortOrder, bool>> ProxiesReIndexedToCheck;
				using FReIndexOperation = FSlateInvalidationWidgetList::IProcessChildOrderInvalidationCallback::FReIndexOperation;
				virtual void ProxiesReIndexed(const FReIndexOperation& Operation) override
				{
					Self->AddErrorIfFalse(Operation.GetRange().IsValid(), TEXT("The range is not valid."));

					if (ProxiesReIndexedToCheck.Num() == 0)
					{
						Self->AddError(TEXT("No ProxiesReIndexed was expected"));
					}
					else
					{
						for (const auto& Element : ProxiesReIndexedToCheck)
						{
							bool bIsIncluded = Operation.GetRange().Include(Element.Get<1>());
							if (bIsIncluded != Element.Get<2>())
							{
								Self->AddError(FString::Printf(TEXT("'%s' should be %s in the ProxiesReIndexed range"), *Element.Get<0>(), (bIsIncluded ? TEXT("included") : TEXT("not be included"))));
							}
						}
					}
				}

				TArray<TTuple<FString, FSlateInvalidationWidgetSortOrder, bool>> ProxiesPreResortToCheck;
				using FReSortOperation = FSlateInvalidationWidgetList::IProcessChildOrderInvalidationCallback::FReSortOperation;
				virtual void ProxiesPreResort(const FReSortOperation& Operation) override
				{
					Self->AddErrorIfFalse(Operation.GetRange().IsValid(), TEXT("The range is not valid."));

					if (ProxiesPreResortToCheck.Num() == 0)
					{
						Self->AddError(TEXT("No ProxiesPreResort was expected"));
					}
					else
					{
						for (const auto& Element : ProxiesPreResortToCheck)
						{
							bool bIsIncluded = Operation.GetRange().Include(Element.Get<1>());
							if (bIsIncluded != Element.Get<2>())
							{
								Self->AddError(FString::Printf(TEXT("'%s' should be %s in the ProxiesPreResort range"), *Element.Get<0>(), (bIsIncluded ? TEXT("included") : TEXT("not be included"))));
							}
						}
					}
				}

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_CHILDORDERCHECK
				TArray<TTuple<FString, FSlateInvalidationWidgetSortOrder, bool>> ProxiesBuiltToCheck;
				virtual void ProxiesBuilt(const FSlateInvalidationWidgetList::FIndexRange& Range) override
				{
					Self->AddErrorIfFalse(Range.IsValid(), TEXT("The range is not valid."));

					if (ProxiesBuiltToCheck.Num() == 0)
					{
						Self->AddError(TEXT("No ProxiesBuilt was expected"));
					}
					else
					{
						for (const auto& Element : ProxiesBuiltToCheck)
						{
							bool bIsIncluded = Range.Include(Element.Get<1>());
							if (bIsIncluded != Element.Get<2>())
							{
								Self->AddError(FString::Printf(TEXT("'%s' should be %s in the ProxiesBuilt range"), *Element.Get<0>(), (!bIsIncluded ? TEXT("included") : TEXT("not be included"))));
							}
						}
					}
				}
#endif
			};

			FChildOrderInvalidationCallback_RemoveG CallbackG;
			CallbackG.Self = this;

			const FSlateInvalidationWidgetIndex WidgetIndexC = WidgetC->GetProxyHandle().GetWidgetIndex();
			const FSlateInvalidationWidgetIndex WidgetIndexF = WidgetF->GetProxyHandle().GetWidgetIndex();
			const FSlateInvalidationWidgetIndex WidgetIndexG = WidgetF->GetAllChildren()->GetChildAt(0)->GetProxyHandle().GetWidgetIndex();
			const FSlateInvalidationWidgetIndex WidgetIndex11 = List.IncrementIndex(WidgetIndexG);
			AddErrorIfFalse(WidgetIndex11 == StaticCastSharedRef<SVerticalBox>(WidgetF->GetAllChildren()->GetChildAt(0))->GetAllChildren()->GetChildAt(0)->GetProxyHandle().GetWidgetIndex(), TEXT("Index of 11 is not identical."));
			const FSlateInvalidationWidgetIndex WidgetIndex12 = List.IncrementIndex(WidgetIndex11);
			AddErrorIfFalse(WidgetIndex12 == StaticCastSharedRef<SVerticalBox>(WidgetF->GetAllChildren()->GetChildAt(0))->GetAllChildren()->GetChildAt(2)->GetProxyHandle().GetWidgetIndex(), TEXT("Index of 12 is not identical."));
			const FSlateInvalidationWidgetIndex WidgetIndexH = WidgetF->GetAllChildren()->GetChildAt(1)->GetProxyHandle().GetWidgetIndex();
			const FSlateInvalidationWidgetIndex WidgetIndex13 = List.IncrementIndex(WidgetIndexH);
			AddErrorIfFalse(WidgetIndex13 == StaticCastSharedRef<SVerticalBox>(WidgetF->GetAllChildren()->GetChildAt(1))->GetAllChildren()->GetChildAt(0)->GetProxyHandle().GetWidgetIndex(), TEXT("Index of 13 is not identical."));
			const FSlateInvalidationWidgetIndex WidgetIndexI = List.IncrementIndex(WidgetIndex13);
			const FSlateInvalidationWidgetIndex WidgetIndexJ = List.IncrementIndex(WidgetIndexI);
			const FSlateInvalidationWidgetIndex WidgetIndex14 = List.IncrementIndex(WidgetIndexJ);

			{
				CallbackG.PreChildRemoveToCheck.Emplace(TEXT("C"), FSlateInvalidationWidgetSortOrder{ List, WidgetIndexC }, false);
				CallbackG.PreChildRemoveToCheck.Emplace(TEXT("F"), FSlateInvalidationWidgetSortOrder{ List, WidgetIndexF }, false);
				CallbackG.PreChildRemoveToCheck.Emplace(TEXT("G"), FSlateInvalidationWidgetSortOrder{ List, WidgetIndexG }, true);
				CallbackG.PreChildRemoveToCheck.Emplace(TEXT("11"), FSlateInvalidationWidgetSortOrder{ List, WidgetIndex11 }, true);
				CallbackG.PreChildRemoveToCheck.Emplace(TEXT("12"), FSlateInvalidationWidgetSortOrder{ List, WidgetIndex12 }, true);
				CallbackG.PreChildRemoveToCheck.Emplace(TEXT("H"), FSlateInvalidationWidgetSortOrder{ List, WidgetIndexH }, true);
				CallbackG.PreChildRemoveToCheck.Emplace(TEXT("13"), FSlateInvalidationWidgetSortOrder{ List, WidgetIndex13 }, true);
				CallbackG.PreChildRemoveToCheck.Emplace(TEXT("I"), FSlateInvalidationWidgetSortOrder{ List, WidgetIndexI }, true);
				CallbackG.PreChildRemoveToCheck.Emplace(TEXT("J"), FSlateInvalidationWidgetSortOrder{ List, WidgetIndexJ }, false);
				CallbackG.PreChildRemoveToCheck.Emplace(TEXT("14"), FSlateInvalidationWidgetSortOrder{ List, WidgetIndex14 }, false);
			}

			{
				//No ProxiesReIndexed expected
			}

			{
				//No ProxiesPreResortToCheck expected
			}

			{
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_CHILDORDERCHECK
				CallbackG.ProxiesBuiltToCheck.Emplace(TEXT("C"), FSlateInvalidationWidgetSortOrder{ List, WidgetIndexC }, false);
				CallbackG.ProxiesBuiltToCheck.Emplace(TEXT("F"), FSlateInvalidationWidgetSortOrder{ List, WidgetIndexF }, false);
				const FSlateInvalidationWidgetIndex NewWidgetIndexH = WidgetF->GetAllChildren()->GetChildAt(0)->GetProxyHandle().GetWidgetIndex();
				CallbackG.ProxiesBuiltToCheck.Emplace(TEXT("H"), FSlateInvalidationWidgetSortOrder{ List, NewWidgetIndexH }, true);
				const FSlateInvalidationWidgetIndex NewWidgetIndex13 = List.IncrementIndex(NewWidgetIndexH);
				CallbackG.ProxiesBuiltToCheck.Emplace(TEXT("13"), FSlateInvalidationWidgetSortOrder{ List, NewWidgetIndex13 }, true);
				const FSlateInvalidationWidgetIndex NewWidgetIndexI = List.IncrementIndex(NewWidgetIndex13);
				CallbackG.ProxiesBuiltToCheck.Emplace(TEXT("I"), FSlateInvalidationWidgetSortOrder{ List, NewWidgetIndexI }, true);
				const FSlateInvalidationWidgetIndex NewWidgetIndexJ = List.IncrementIndex(NewWidgetIndexI);
				CallbackG.ProxiesBuiltToCheck.Emplace(TEXT("J"), FSlateInvalidationWidgetSortOrder{ List, NewWidgetIndexJ }, false);
				const FSlateInvalidationWidgetIndex NewWidgetIndex14 = List.IncrementIndex(NewWidgetIndexJ);
				CallbackG.ProxiesBuiltToCheck.Emplace(TEXT("14"), FSlateInvalidationWidgetSortOrder{ List, NewWidgetIndex14 }, false);
#endif
			}

			// Remove G
			{
				WidgetF->RemoveSlot(WidgetF->GetAllChildren()->GetChildAt(0));
				bool bIsValid = List.ProcessChildOrderInvalidation(WidgetIndexF, CallbackG);
				AddErrorIfFalse(bIsValid, TEXT("The F widget should still be valid"));
			}
		}
	}

	return true;
}

#undef LOCTEXT_NAMESPACE 

#endif //WITH_AUTOMATION_WORKER & WITH_SLATE_DEBUGGING
