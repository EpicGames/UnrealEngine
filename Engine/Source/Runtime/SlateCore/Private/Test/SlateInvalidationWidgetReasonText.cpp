// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"

#include "Widgets/InvalidateWidgetReason.h"

#if WITH_AUTOMATION_WORKER

#define LOCTEXT_NAMESPACE "Slate.InvalidationWidgetReason.Lex"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FSlateInvalidationWidgetReasonLexTest, "Slate.ParsingInvalidationWidgetReason", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FSlateInvalidationWidgetReasonLexTest::RunTest(const FString& Parameters)
{
	EInvalidateWidgetReason Value = (EInvalidateWidgetReason)0xFF;
	AddErrorIfFalse(LexToString(Value) == TEXT("All"), TEXT("'All' was not converted properly."));
	Value = EInvalidateWidgetReason::None;
	AddErrorIfFalse(LexToString(Value) == TEXT("None"), TEXT("'None' was not converted properly."));

	auto TestLextToStringTRyParseString = [this](EInvalidateWidgetReason Reason, const TCHAR* ReasonStr)
		{
			EInvalidateWidgetReason ParsedValue = EInvalidateWidgetReason::None;
			FString StringValue = LexToString(Reason);
			AddErrorIfFalse(LexTryParseString(ParsedValue, *StringValue), FString::Printf(TEXT("Was not able to parse '%s'."), ReasonStr));
			AddErrorIfFalse(Reason == ParsedValue, FString::Printf(TEXT("The parsed value and the converted value for '%s' do not match."), ReasonStr));
		};

	TestLextToStringTRyParseString(EInvalidateWidgetReason::None, TEXT("None"));
	TestLextToStringTRyParseString(EInvalidateWidgetReason::Layout, TEXT("Layout"));
	TestLextToStringTRyParseString(EInvalidateWidgetReason::Paint, TEXT("Paint"));
	TestLextToStringTRyParseString(EInvalidateWidgetReason::Volatility, TEXT("Volatility"));
	TestLextToStringTRyParseString(EInvalidateWidgetReason::ChildOrder, TEXT("ChildOrder"));
	TestLextToStringTRyParseString(EInvalidateWidgetReason::RenderTransform, TEXT("RenderTransform"));
	TestLextToStringTRyParseString(EInvalidateWidgetReason::Visibility, TEXT("Visibility"));
	TestLextToStringTRyParseString(EInvalidateWidgetReason::AttributeRegistration, TEXT("AttributeRegistration"));
	TestLextToStringTRyParseString(EInvalidateWidgetReason::Prepass, TEXT("Prepass"));

	TestLextToStringTRyParseString(EInvalidateWidgetReason::LayoutAndVolatility, TEXT("LayoutAndVolatility"));
	TestLextToStringTRyParseString(EInvalidateWidgetReason::PaintAndVolatility, TEXT("PaintAndVolatility"));
	TestLextToStringTRyParseString(EInvalidateWidgetReason::PaintAndVolatility|EInvalidateWidgetReason::Visibility| EInvalidateWidgetReason::None, TEXT("PaintAndVolatility With None"));
	TestLextToStringTRyParseString(EInvalidateWidgetReason::PaintAndVolatility|EInvalidateWidgetReason::Visibility| (EInvalidateWidgetReason)0xFF, TEXT("PaintAndVolatility With All"));

	AddErrorIfFalse(LexTryParseString(Value, TEXT(" Layout | Volatility ")), TEXT("'Layout | Volatility' With spaces"));
	AddErrorIfFalse(Value == EInvalidateWidgetReason::LayoutAndVolatility, TEXT("'Layout | Volatility' With spaces"));


	return true;
}

#undef LOCTEXT_NAMESPACE

#endif //WITH_AUTOMATION_WORKER
