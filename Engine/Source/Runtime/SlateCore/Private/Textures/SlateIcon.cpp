// Copyright Epic Games, Inc. All Rights Reserved.

#include "Textures/SlateIcon.h"
#include "Styling/StyleDefaults.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

/* FSlateIcon structors
 *****************************************************************************/

FSlateIcon::FSlateIcon( )
	: StyleSetName(NAME_None)
	, StyleName(NAME_None)
	, SmallStyleName(NAME_None)
	, StatusOverlayStyleName(NAME_None)
{ }



FSlateIcon::FSlateIcon(const FName InStyleSetName, const FName InStyleName, const FName InSmallStyleName, const FName InStatusOverlayStyleName)
	: StyleSetName(InStyleSetName)
	, StyleName(InStyleName)
	, SmallStyleName((InSmallStyleName == NAME_None) ? ISlateStyle::Join(InStyleName, ".Small") :  InSmallStyleName)
	, StatusOverlayStyleName(InStatusOverlayStyleName)
{ }


/* FSlateIcon interface
 *****************************************************************************/

const FSlateBrush* FSlateIcon::GetSmallIcon( ) const
{
	const ISlateStyle* StyleSet = GetStyleSet();
	
	if (StyleSet)
	{
		const FSlateBrush* FoundBrush = StyleSet->GetOptionalBrush(SmallStyleName);
		if (FoundBrush != FStyleDefaults::GetNoBrush())
		{
			return FoundBrush;
		}
	}

	return FSlateIcon::GetIcon();
}


const ISlateStyle* FSlateIcon::GetStyleSet( ) const
{
	return StyleSetName.IsNone() ? nullptr : FSlateStyleRegistry::FindSlateStyle(StyleSetName);
}


const FSlateBrush* FSlateIcon::GetIcon( ) const
{
	const ISlateStyle* StyleSet = GetStyleSet();
	
	if(StyleSet)
	{
		return StyleSet->GetOptionalBrush(StyleName);
	}

	return FStyleDefaults::GetNoBrush();
}


const FSlateBrush* FSlateIcon::GetOptionalIcon( ) const
{
	const FSlateBrush* Icon = GetIcon();
	return Icon == FStyleDefaults::GetNoBrush() ? nullptr : Icon;
}

const FSlateBrush* FSlateIcon::GetOptionalSmallIcon( ) const
{
	const FSlateBrush* Icon = GetSmallIcon();
	return Icon == FStyleDefaults::GetNoBrush() ? nullptr : Icon;
}

const FSlateBrush* FSlateIcon::GetOverlayIcon() const
{
	const ISlateStyle* StyleSet = GetStyleSet();

	if (StyleSet)
	{
		return StyleSet->GetOptionalBrush(StatusOverlayStyleName);
	}

	return nullptr;
}
