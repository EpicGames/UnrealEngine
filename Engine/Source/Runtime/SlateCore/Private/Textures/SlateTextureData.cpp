// Copyright Epic Games, Inc. All Rights Reserved.

#include "Textures/SlateTextureData.h"
#include "ImageCore.h"

void FSlateTextureData::SetImage( const FImageView & Image )
{
	// if Image is already BGRA8-SRGB then this is just a memcpy
	//	which is what we need anyway to copy the bytes into a new array
	//	so there is no inefficiency in just always using the FImage copy here
	FImage Converted;
	Image.CopyTo(Converted,ERawImageFormat::BGRA8,EGammaSpace::sRGB);
	SetImage( MoveTemp(Converted) );
}
		
void FSlateTextureData::SetImage( FImage && MoveFrom )
{
	// change format if needed, nop if not
	// MoveFrom is discardable so it's okay if we just change it in place
	MoveFrom.ChangeFormat(ERawImageFormat::BGRA8,EGammaSpace::sRGB);

	DEC_MEMORY_STAT_BY( STAT_SlateTextureDataMemory, Bytes.GetAllocatedSize() );

	Width = MoveFrom.SizeX;
	Height = MoveFrom.SizeY;
	BytesPerPixel = 4;		
	Bytes = MoveTemp(MoveFrom.RawData);

	INC_MEMORY_STAT_BY( STAT_SlateTextureDataMemory, Bytes.GetAllocatedSize() );

	MoveFrom.Reset();
}
