// Copyright Epic Games, Inc. All Rights Reserved.

#include "Textures/TextureAtlas.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "Stats/Stats.h"
#include "Textures/SlateShaderResource.h"
#include "Textures/SlateTextureData.h"
#include "Trace/SlateMemoryTags.h"
#include "Misc/MemStack.h"

#include <limits>

DEFINE_STAT(STAT_SlateTextureGPUMemory);
DEFINE_STAT(STAT_SlateTextureDataMemory);
DECLARE_MEMORY_STAT(TEXT("Texture Atlas Memory (CPU)"), STAT_SlateTextureAtlasMemory, STATGROUP_SlateMemory);


const TCHAR* LexToString(ESlateTextureAtlasThreadId ThreadId)
{
	switch (ThreadId)
	{
		case ESlateTextureAtlasThreadId::Unknown: 
			return TEXT("Unknown");
		case ESlateTextureAtlasThreadId::Game: 
			return TEXT("Game");
		case ESlateTextureAtlasThreadId::Render: 
			return TEXT("Render");
		default:
			checkNoEntry();
			break;
	}
	return TEXT("Unsupported thread ID");
}

ESlateTextureAtlasThreadId GetCurrentSlateTextureAtlasThreadId()
{
	// Note: For Game thread ownership, there is a point at which multiple worker threads operate on text simultaneously while the game thread is blocked
	// Access to the font cache is controlled through mutexes so we simply need to check that we are not accessing it from the render thread
	// Game thread access is also allowed when the game thread and render thread are the same
	if (!IsInActualRenderingThread())
	{
		return ESlateTextureAtlasThreadId::Game;
	}
	else if (IsInRenderingThread())
	{
		return ESlateTextureAtlasThreadId::Render;
	}

	return ESlateTextureAtlasThreadId::Unknown;
}

const TCHAR* GetCurrentSlateTextureAtlasThreadIdAsString()
{
	return LexToString(GetCurrentSlateTextureAtlasThreadId());
}

uint32 GetSlateFontAtlasContentBytesPerPixel(ESlateFontAtlasContentType InContentType)
{
	switch (InContentType)
	{
		case ESlateFontAtlasContentType::Alpha:
			return 1;
		case ESlateFontAtlasContentType::Color:
		case ESlateFontAtlasContentType::Msdf:
			return 4;
		default:
			checkNoEntry();
			return 0;
	}
}

/* FSlateTextureAtlas helper class
 *****************************************************************************/

FSlateTextureAtlas::FSlateTextureAtlas( uint32 InWidth, uint32 InHeight, uint32 InBytesPerPixel, ESlateTextureAtlasPaddingStyle InPaddingStyle, bool bInUpdatesAfterInitialization )
	: AtlasData()
	, AtlasUsedSlots(NULL)
	, AtlasEmptySlotsMap()
	, AtlasWidth( InWidth )
	, AtlasHeight( InHeight )
	, BytesPerPixel( InBytesPerPixel )
	, PaddingStyle( InPaddingStyle )
	, bNeedsUpdate( false )
	, bUpdatesAfterInitialization(bInUpdatesAfterInitialization)
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	, AtlasOwnerThread( ESlateTextureAtlasThreadId::Unknown )
#endif
{
	InitAtlasData();
}

FSlateTextureAtlas::~FSlateTextureAtlas()
{
	EmptyAtlasData();
}


/* FSlateTextureAtlas interface
 *****************************************************************************/

void FSlateTextureAtlas::EmptyAtlasData()
{
	// Remove all nodes
	for (FAtlasedTextureSlot* AtlasEmptySlots : AtlasEmptySlotsMap)
	{
		for (FAtlasedTextureSlot::TIterator SlotIt(AtlasEmptySlots); SlotIt;)
		{
			FAtlasedTextureSlot& CurSlot = *SlotIt;
			SlotIt.Next();
			delete &CurSlot;
		}
	}
	AtlasEmptySlotsMap.Reset();

	for (FAtlasedTextureSlot::TIterator SlotIt(AtlasUsedSlots); SlotIt;)
	{
		FAtlasedTextureSlot& CurSlot = *SlotIt;
		SlotIt.Next();
		delete &CurSlot;
	}
	AtlasUsedSlots = nullptr;

	STAT(SIZE_T MemoryBefore = AtlasData.GetAllocatedSize());

	// Clear all raw data
	AtlasData.Empty();

	STAT(SIZE_T MemoryAfter = AtlasData.GetAllocatedSize());
	DEC_MEMORY_STAT_BY(STAT_SlateTextureAtlasMemory, MemoryBefore-MemoryAfter);
}


const FAtlasedTextureSlot* FSlateTextureAtlas::AddTexture( uint32 TextureWidth, uint32 TextureHeight, const TArray<uint8>& Data )
{
	// Find a spot for the character in the texture
	const FAtlasedTextureSlot* NewSlot = FindSlotForTexture(TextureWidth, TextureHeight);

	// handle cases like space, where the size of the glyph is zero. The copy data code doesn't handle zero sized source data with a padding
	// so make sure to skip this call.
	if (NewSlot && TextureWidth > 0 && TextureHeight > 0)
	{
		CopyDataIntoSlot(NewSlot, Data);
		MarkTextureDirty();
	}

	return NewSlot;
}

void FSlateTextureAtlas::MarkTextureDirty()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	{
		const ESlateTextureAtlasThreadId AtlasThreadId = GetCurrentSlateTextureAtlasThreadId();
		check(AtlasThreadId != ESlateTextureAtlasThreadId::Unknown);

		check((GSlateLoadingThreadId != 0) || (AtlasOwnerThread == AtlasThreadId));
	}
#endif

	bNeedsUpdate = true;
}

void FSlateTextureAtlas::InitAtlasData()
{
	LLM_SCOPE_BYTAG(UI_Texture);

	check(AtlasEmptySlotsMap.IsEmpty() && AtlasData.Num() == 0);

	const int32 MapSlotCount = GetFreeSlotSearchIndex(AtlasWidth, AtlasHeight) + 1;
	AtlasEmptySlotsMap.SetNumZeroed(MapSlotCount);
	AddFreeSlot(0, 0, AtlasWidth, AtlasHeight);

	AtlasData.Reserve(AtlasWidth * AtlasHeight * BytesPerPixel);
	AtlasData.AddZeroed(AtlasWidth * AtlasHeight * BytesPerPixel);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	AtlasOwnerThread = GetCurrentSlateTextureAtlasThreadId();
	check(AtlasOwnerThread != ESlateTextureAtlasThreadId::Unknown);
#endif

	INC_MEMORY_STAT_BY(STAT_SlateTextureAtlasMemory, AtlasData.GetAllocatedSize());
}


void FSlateTextureAtlas::CopyRow( const FCopyRowData& CopyRowData )
{
	const uint8* Data = CopyRowData.SrcData;
	uint8* Start = CopyRowData.DestData;
	const uint32 SourceWidth = CopyRowData.SrcTextureWidth;
	const uint32 DestWidth = CopyRowData.DestTextureWidth;
	const uint32 SrcRow = CopyRowData.SrcRow;
	const uint32 DestRow = CopyRowData.DestRow;
	// this can only be one or zero
	const uint32 Padding = GetPaddingAmount();

	const uint8* SourceDataAddr = &Data[(SrcRow * SourceWidth) * BytesPerPixel];
	uint8* DestDataAddr = &Start[(DestRow * DestWidth + Padding) * BytesPerPixel];
	FMemory::Memcpy(DestDataAddr, SourceDataAddr, SourceWidth * BytesPerPixel);

	if (Padding > 0)
	{ 
		uint8* DestPaddingPixelLeft = &Start[(DestRow * DestWidth) * BytesPerPixel];
		uint8* DestPaddingPixelRight = DestPaddingPixelLeft + ((CopyRowData.RowWidth - 1) * BytesPerPixel);
		if (PaddingStyle == ESlateTextureAtlasPaddingStyle::DilateBorder)
		{
			const uint8* FirstPixel = SourceDataAddr; 
			const uint8* LastPixel = SourceDataAddr + ((SourceWidth - 1) * BytesPerPixel);
			FMemory::Memcpy(DestPaddingPixelLeft, FirstPixel, BytesPerPixel);
			FMemory::Memcpy(DestPaddingPixelRight, LastPixel, BytesPerPixel);
		}
		else
		{
			FMemory::Memzero(DestPaddingPixelLeft, BytesPerPixel);
			FMemory::Memzero(DestPaddingPixelRight, BytesPerPixel);
		}

	} 
}

void FSlateTextureAtlas::ZeroRow( const FCopyRowData& CopyRowData )
{
	const uint32 SourceWidth = CopyRowData.SrcTextureWidth;
	const uint32 DestWidth = CopyRowData.DestTextureWidth;
	const uint32 DestRow = CopyRowData.DestRow;

	uint8* DestDataAddr = &CopyRowData.DestData[DestRow * DestWidth * BytesPerPixel];
	FMemory::Memzero(DestDataAddr, CopyRowData.RowWidth * BytesPerPixel);
}


void FSlateTextureAtlas::CopyDataIntoSlot( const FAtlasedTextureSlot* SlotToCopyTo, const TArray<uint8>& Data )
{
	// Copy pixel data to the texture
	uint8* Start = &AtlasData[SlotToCopyTo->Y*AtlasWidth*BytesPerPixel + SlotToCopyTo->X*BytesPerPixel];
	
	// Account for same padding on each sides
	const uint32 Padding = GetPaddingAmount();
	const uint32 AllPadding = Padding*2;

	// Make sure the actual slot is not zero-area (otherwise padding could corrupt other images in the atlas)
	check(SlotToCopyTo->Width > AllPadding);
	check(SlotToCopyTo->Height > AllPadding);

	// The width of the source texture without padding (actual width)
	const uint32 SourceWidth = SlotToCopyTo->Width - AllPadding; 
	const uint32 SourceHeight = SlotToCopyTo->Height - AllPadding;

	FCopyRowData CopyRowData;
	CopyRowData.DestData = Start;
	CopyRowData.SrcData = Data.GetData();
	CopyRowData.DestTextureWidth = AtlasWidth;
	CopyRowData.SrcTextureWidth = SourceWidth;
	CopyRowData.RowWidth = SlotToCopyTo->Width;

	// Apply the padding for bilinear filtering. 
	// Not used if no padding (assumes sampling outside boundaries of the sub texture is not possible)
	if (Padding > 0)
	{
		// Copy first color row into padding.  
		CopyRowData.SrcRow = 0;
		CopyRowData.DestRow = 0;

		if (PaddingStyle == ESlateTextureAtlasPaddingStyle::DilateBorder)
		{
			CopyRow(CopyRowData);
		}
		else
		{
			ZeroRow(CopyRowData);
		}
	}

	// Copy each row of the texture
	for (uint32 Row = Padding; Row < SlotToCopyTo->Height-Padding; ++Row)
	{
		CopyRowData.SrcRow = Row-Padding;
		CopyRowData.DestRow = Row;

		CopyRow(CopyRowData);
	}

	if (Padding > 0)
	{
		// Copy last color row into padding row for bilinear filtering
		CopyRowData.SrcRow = SourceHeight - 1;
		CopyRowData.DestRow = SlotToCopyTo->Height - Padding;

		if (PaddingStyle == ESlateTextureAtlasPaddingStyle::DilateBorder)
		{
			CopyRow(CopyRowData);
		}
		else
		{
			ZeroRow(CopyRowData);
		}
	}
}

#if WITH_ATLAS_DEBUGGING
const FAtlasedTextureSlot* FSlateTextureAtlas::GetSlotAtPosition(FIntPoint InPosition) const
{
	for (FAtlasedTextureSlot::TIterator SlotIt(AtlasUsedSlots); SlotIt; SlotIt++)
	{
		FAtlasedTextureSlot& CurSlot = *SlotIt;

		checkSlow(CurSlot.X <= (uint32)std::numeric_limits<int32>::max());
		checkSlow(CurSlot.Y <= (uint32)std::numeric_limits<int32>::max());
		checkSlow(CurSlot.X + CurSlot.Width <= (uint32)std::numeric_limits<int32>::max());
		checkSlow(CurSlot.Y + CurSlot.Height <= (uint32)std::numeric_limits<int32>::max());
		FIntRect CurSlotRect(FIntPoint((int32)CurSlot.X, (int32)CurSlot.Y), FIntPoint((int32)(CurSlot.X + CurSlot.Width), (int32)(CurSlot.Y + CurSlot.Height)));
		if (CurSlotRect.Contains(InPosition))
		{
			return &CurSlot;
		}
	}

	return nullptr;
}
#endif

const FAtlasedTextureSlot* FSlateTextureAtlas::FindSlotForTexture(uint32 InWidth, uint32 InHeight)
{
	// Account for padding on both sides
	const uint8 Padding = GetPaddingAmount();
	const uint32 TotalPadding = Padding * 2;
	const uint32 PaddedWidth = InWidth + TotalPadding;
	const uint32 PaddedHeight = InHeight + TotalPadding;
	const int32 StartSearchIndex = GetFreeSlotSearchIndex(PaddedWidth, PaddedHeight);
	for (int32 SlotsIndex = StartSearchIndex; SlotsIndex < AtlasEmptySlotsMap.Num(); ++SlotsIndex)
	{
		FAtlasedTextureSlot* const AtlasEmptySlots = AtlasEmptySlotsMap[SlotsIndex];
		for (FAtlasedTextureSlot::TIterator SlotIt(AtlasEmptySlots); SlotIt; SlotIt++)
		{
			FAtlasedTextureSlot& CurSlot = *SlotIt;
			if (PaddedWidth <= CurSlot.Width && PaddedHeight <= CurSlot.Height)
			{
				// The width and height of the new child node(s)
				const uint32 RemainingWidth = CurSlot.Width - PaddedWidth;
				const uint32 RemainingHeight = CurSlot.Height - PaddedHeight;

				// New slots must have a minimum width/height, to avoid excessive slots i.e. excessive memory usage and iteration.
				// No glyphs seem to use slots this small, and cutting these slots out improves performance/memory-usage a fair bit
				const uint32 MinSlotDim = 2;

				// Split the remaining area around this slot into two children.
				if (RemainingHeight >= MinSlotDim || RemainingWidth >= MinSlotDim)
				{
					if (RemainingHeight <= RemainingWidth)
					{
						// Split vertically
						// - - - - - - - - -
						// |       |       |
						// |  Slot |       |
						// |       |       |
						// | - - - | Right |
						// |       |       |
						// |  Left |       |
						// |       |       |
						// - - - - - - - - -
						AddFreeSlot(CurSlot.X + PaddedWidth, CurSlot.Y, RemainingWidth, CurSlot.Height);
						if (RemainingHeight >= MinSlotDim)
						{
							AddFreeSlot(CurSlot.X, CurSlot.Y + PaddedHeight, PaddedWidth, RemainingHeight);
						}
					}
					else
					{
						// Split horizontally
						// - - - - - - - - -
						// |       |       |
						// |  Slot | Left  |
						// |       |       |
						// | - - - - - - - |
						// |               |
						// |     Right     |
						// |               |
						// - - - - - - - - -
						AddFreeSlot(CurSlot.X, CurSlot.Y + PaddedHeight, CurSlot.Width, RemainingHeight);
						if (RemainingWidth >= MinSlotDim)
						{
							AddFreeSlot(CurSlot.X + PaddedWidth, CurSlot.Y, RemainingWidth, PaddedHeight);
						}
					}
				}

				// Shrink and moved to used slot list
				CurSlot.Width = PaddedWidth;
				CurSlot.Height = PaddedHeight;
				CurSlot.Unlink();
				CurSlot.LinkHead(AtlasUsedSlots);
				return &CurSlot;
			}
		}
	}

	return nullptr;
}

int32 FSlateTextureAtlas::GetFreeSlotSearchIndex(uint32 InWidth, uint32 InHeight)
{
	// Currently only bucketing by width, but leaving this open to be extended
	return FMath::CeilLogTwo(InWidth);
}

void FSlateTextureAtlas::AddFreeSlot(uint32 InX, uint32 InY, uint32 InWidth, uint32 InHeight)
{
	FAtlasedTextureSlot* NewSlot = new FAtlasedTextureSlot(InX, InY, InWidth, InHeight, GetPaddingAmount());
	const uint32 SlotIndex = GetFreeSlotSearchIndex(InWidth, InHeight);
	NewSlot->LinkHead(AtlasEmptySlotsMap[SlotIndex]);
} //-V773

FSlateFlushableAtlasCache::FSlateFlushableAtlasCache(const FAtlasFlushParams* InFlushParams)
	: FlushParams(InFlushParams)
{
	check(FlushParams);

	CurrentMaxGrayscaleAtlasPagesBeforeFlushRequest = FlushParams->InitialMaxAtlasPagesBeforeFlushRequest;
	CurrentMaxColorAtlasPagesBeforeFlushRequest = FlushParams->InitialMaxAtlasPagesBeforeFlushRequest;
	CurrentMaxMsdfAtlasPagesBeforeFlushRequest = FlushParams->InitialMaxAtlasPagesBeforeFlushRequest;
	CurrentMaxNonAtlasedTexturesBeforeFlushRequest = FlushParams->InitialMaxNonAtlasPagesBeforeFlushRequest;
}

void FSlateFlushableAtlasCache::ResetFlushCounters()
{
	CurrentMaxGrayscaleAtlasPagesBeforeFlushRequest = FlushParams->InitialMaxAtlasPagesBeforeFlushRequest;
	CurrentMaxColorAtlasPagesBeforeFlushRequest = FlushParams->InitialMaxAtlasPagesBeforeFlushRequest;
	CurrentMaxMsdfAtlasPagesBeforeFlushRequest = FlushParams->InitialMaxAtlasPagesBeforeFlushRequest;
	CurrentMaxNonAtlasedTexturesBeforeFlushRequest = FlushParams->InitialMaxNonAtlasPagesBeforeFlushRequest;
	FrameCounterLastFlushRequest = GFrameCounter;
}

void FSlateFlushableAtlasCache::UpdateFlushCounters(int32 NumGrayscale, int32 NumColor, int32 NumMsdf, int32 NumNonAtlased)
{
	bool bFlushRequested = false;
	bFlushRequested |= UpdateInternal(NumGrayscale, CurrentMaxGrayscaleAtlasPagesBeforeFlushRequest, FlushParams->InitialMaxAtlasPagesBeforeFlushRequest, FlushParams->GrowAtlasFrameWindow);
	bFlushRequested |= UpdateInternal(NumColor, CurrentMaxColorAtlasPagesBeforeFlushRequest, FlushParams->InitialMaxAtlasPagesBeforeFlushRequest, FlushParams->GrowAtlasFrameWindow);
	bFlushRequested |= UpdateInternal(NumMsdf, CurrentMaxMsdfAtlasPagesBeforeFlushRequest, FlushParams->InitialMaxAtlasPagesBeforeFlushRequest, FlushParams->GrowAtlasFrameWindow);
	bFlushRequested |= UpdateInternal(NumNonAtlased, CurrentMaxNonAtlasedTexturesBeforeFlushRequest, FlushParams->InitialMaxNonAtlasPagesBeforeFlushRequest, FlushParams->GrowNonAtlasFrameWindow);

	if (bFlushRequested)
	{
		ResetFlushCounters();
	}
}

bool FSlateFlushableAtlasCache::UpdateInternal(int32 CurrentNum, int32& MaxNum, int32 InitialMax, int32 FrameWindowNum)
{
	const uint64 LastFlushRequestFrameDelta = GFrameCounter - FrameCounterLastFlushRequest;

	if (CurrentNum > MaxNum)
	{
		// The InitialMaxNonAtlasedTexturesBeforeFlushRequest may have changed since last flush,
		// so double check that we're below the current max or initial, if we're under it,
		// update the current to the initial.
		if (CurrentNum <= InitialMax)
		{
			MaxNum = InitialMax;
		}
		// If we grew back up to this number of non-atlased textures within the same or next frame of the previous flush request, then we likely legitimately have 
		// a lot of data cached. We should update CurrentMaxNonAtlasedTexturesBeforeFlushRequest to give us a bit more flexibility before the next flush request
		else if (LastFlushRequestFrameDelta <= FrameWindowNum)
		{
			MaxNum = CurrentNum;
			UE_LOGF(LogSlate, Warning, "Setting the threshold to trigger a flush to %d non-atlased textures as there is a lot of font data being cached.", CurrentNum);
		}
		else
		{
			// We've grown beyond our current stable limit - try and request a flush
			RequestFlushCache(FString::Printf(TEXT("Large atlases out of space; %d/%d Textures; frames since last flush: %llu"), CurrentNum, MaxNum, LastFlushRequestFrameDelta));
			
			return true;
		}
	}

	return false;
}
