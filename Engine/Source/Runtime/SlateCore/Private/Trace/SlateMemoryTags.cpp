// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/SlateMemoryTags.h"

#if ENABLE_LOW_LEVEL_MEM_TRACKER

LLM_DEFINE_TAG(UI_Slate, "UI Slate", TEXT("UI"), GET_STATFNAME(STAT_UI_SlateLLM), GET_STATFNAME(STAT_UISummaryLLM));
LLM_DEFINE_TAG(UI_UMG, "UI UMG", TEXT("UI"), GET_STATFNAME(STAT_UI_UMGLLM), GET_STATFNAME(STAT_UISummaryLLM));
LLM_DEFINE_TAG(UI_Text, "UI Text", TEXT("UI"), GET_STATFNAME(STAT_UI_TextLLM), GET_STATFNAME(STAT_UISummaryLLM));
LLM_DEFINE_TAG(UI_Texture, "UI Texture", TEXT("UI"), GET_STATFNAME(STAT_UI_TextureLLM), GET_STATFNAME(STAT_UISummaryLLM));
LLM_DEFINE_TAG(UI_Style, "UI Style", TEXT("UI"), GET_STATFNAME(STAT_UI_StyleLLM), GET_STATFNAME(STAT_UISummaryLLM));

DEFINE_STAT(STAT_UI_SlateLLM);
DEFINE_STAT(STAT_UI_UMGLLM);
DEFINE_STAT(STAT_UI_TextLLM);
DEFINE_STAT(STAT_UI_TextureLLM);
DEFINE_STAT(STAT_UI_StyleLLM);

#endif // ENABLE_LOW_LEVEL_MEM_TRACKER