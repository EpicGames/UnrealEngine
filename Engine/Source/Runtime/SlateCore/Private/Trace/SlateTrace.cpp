// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/SlateTrace.h"
#include "Async/Async.h"

#if UE_SLATE_TRACE_ENABLED

#include "Application/SlateApplicationBase.h"
#include "HAL/PlatformTime.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/PlatformProcess.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Trace/Trace.inl"
#include "Types/ReflectionMetadata.h"
#include "Widgets/SWidget.h"
#include "FastUpdate/WidgetProxy.h"
#include "Tasks/Task.h"

#if !(UE_BUILD_SHIPPING)

static int32 bCaptureRootInvalidationCallstacks = 0;
static FAutoConsoleVariableRef CVarCaptureRootInvalidationCallstacks(
	TEXT("SlateDebugger.bCaptureRootInvalidationCallstacks"),
	bCaptureRootInvalidationCallstacks,
	TEXT("Whenever a widget is the root cause of an invalidation, capture the callstack for slate insights."));

namespace UE::Slate::Private
{
	static FCriticalSection ModuleLoadLock;
}

#endif // !(UE_BUILD_SHIPPING)

//-----------------------------------------------------------------------------------//

UE_TRACE_CHANNEL_DEFINE(SlateChannel,
	"Slate widget lifecycle and per-frame counts (add/remove, invalidation reasons, tick/timer/repaint/volatile counts); "
	"visualized by the Slate Insights plugin.")

UE_TRACE_EVENT_BEGIN(SlateTrace, ApplicationTickAndDrawWidgets)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(double, DeltaTime)
	UE_TRACE_EVENT_FIELD(uint32, WidgetCount)			// Total amount of widget.
	UE_TRACE_EVENT_FIELD(uint32, TickCount)				// Amount of widget that needed to be tick.
	UE_TRACE_EVENT_FIELD(uint32, TimerCount)			// Amount of widget that needed a timer update.
	UE_TRACE_EVENT_FIELD(uint32, RepaintCount)			// Amount of widget that requested a paint.
	UE_TRACE_EVENT_FIELD(uint32, PaintCount)			// Total amount of widget that got painted
														//This can be higher than RepaintCount+VolatilePaintCount because some widget can get be painted as a side effect of another widget being painted.
	UE_TRACE_EVENT_FIELD(uint32, InvalidateCount)		// Amount of widget that got invalidated.
	UE_TRACE_EVENT_FIELD(uint32, RootInvalidatedCount)	// Amount of InvalidationRoot that got invalidated.
	UE_TRACE_EVENT_FIELD(uint8, SlateFlags)				// Various flags that was enabled for that frame.
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SlateTrace, AddWidget)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, WidgetId)				// Added widget unique ID.
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SlateTrace, WidgetInfo)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, WidgetId)				// Created/Updated widget unique ID.
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Path)	// FReflectionMetaData::GetWidgetPath
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, DebugInfo)// FReflectionMetaData::GetWidgetDebugInfo
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SlateTrace, RemoveWidget)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, WidgetId)				// Removed widget unique ID.
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SlateTrace, WidgetUpdated)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, CycleEnd)
	UE_TRACE_EVENT_FIELD(uint64, WidgetId)				// Updated widget unique ID.
	UE_TRACE_EVENT_FIELD(uint32, AffectedCount)			// The number of widget that got affected by the object
	UE_TRACE_EVENT_FIELD(uint8, UpdateFlags)			// The reason of the update. (EWidgetUpdateFlags)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SlateTrace, WidgetInvalidated)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, WidgetId)					// Invalidated widget unique ID.
	UE_TRACE_EVENT_FIELD(uint64, InvestigatorId)			// Widget unique ID that investigated the invalidation.
	UE_TRACE_EVENT_FIELD(uint8, InvalidateWidgetReason)		// The reason of the invalidation. (EInvalidateWidgetReason)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, ScriptTrace)// Optional script trace for root widget invalidations
	UE_TRACE_EVENT_FIELD(uint64[], Callstack)				// Optional callstack for root widget invalidations
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SlateTrace, RootInvalidated)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, WidgetId)				// Invalidated InvalidationRoot widget unique ID.
	UE_TRACE_EVENT_FIELD(uint64, InvestigatorId)		// Widget unique ID that investigated the invalidation.
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SlateTrace, RootChildOrderInvalidated)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, WidgetId)				// Invalidated InvalidationRoot widget unique ID.
	UE_TRACE_EVENT_FIELD(uint64, InvestigatorId)		// Widget unique ID that investigated the invalidation.
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SlateTrace, InvalidationCallstack)
	UE_TRACE_EVENT_FIELD(uint64, SourceCycle)              // Cycle during which a callstack was captured
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, CallstackText) // Text of the captured callstack
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(SlateTrace, WidgetUpdateSteps)
	UE_TRACE_EVENT_FIELD(uint8[], Buffer)		// Buffer of the pass and the widget id
UE_TRACE_EVENT_END()

//-----------------------------------------------------------------------------------//

namespace SlateTraceDetail
{
	uint32 GWidgetCount = 0;
	uint32 GScopedPaintCount = 0;
	uint32 GScopedUpdateCount = 0;

	uint32 GFramePaintCount = 0;
	uint32 GFrameTickCount = 0;
	uint32 GFrameTimerCount = 0;
	uint32 GFrameRepaintCount = 0;
	//uint32 GFrameVolatileCount = 0;
	uint32 GFrameInvalidateCount = 0;
	uint32 GFrameRootInvalidateCount = 0;

	const int32 SizeOfWidgetUpdateStepsBuffer = 2048*sizeof(uint64);
	uint8 WidgetUpdateStepsBuffer[SizeOfWidgetUpdateStepsBuffer];
	uint32 WidgetUpdateStepsIndex = 0;
	uint32 WidgetUpdateStepsBufferNumber = 0;

	TArray<TWeakPtr<const SWidget>> UpdateWidgetInfos;

	uint8 kWidgetUpdateStepsCommand_NewRootPaint = 0xE0;
	uint8 kWidgetUpdateStepsCommand_StartPaint = 0xE1;	//[cycle][widgetid]
	uint8 kWidgetUpdateStepsCommand_EndPaint = 0xE2;	//[cycle]
	uint8 kWidgetUpdateStepsCommand_NewBuffer = 0xE7;	//[buffer index]

	uint64 GetWidgetId(const SWidget* InWidget)
	{
#if UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
		check(InWidget);
		return InWidget->GetId();
#else
		return 0;
#endif
	}

	uint64 GetWidgetIdIfValid(const SWidget* InWidget)
	{
#if UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
		return InWidget ? InWidget->GetId() : 0;
#else
		return 0;
#endif
	}

	void SerializeToWidgetUpdateSteps(uint8 InNumber)
	{
		WidgetUpdateStepsBuffer[WidgetUpdateStepsIndex] = InNumber;
		++WidgetUpdateStepsIndex;
	}

	void SerializeToWidgetUpdateSteps(uint32 InNumber)
	{
		*reinterpret_cast<uint32*>(WidgetUpdateStepsBuffer + WidgetUpdateStepsIndex) = InNumber;
#if !PLATFORM_LITTLE_ENDIAN
		Algo::Reverse(WidgetUpdateStepsBuffer + WidgetUpdateStepsIndex, sizeof(uint32));
#endif
		WidgetUpdateStepsIndex  += sizeof(uint32);
	}
	
	void SerializeToWidgetUpdateSteps(uint64 InNumber)
	{
		*reinterpret_cast<uint64*>(WidgetUpdateStepsBuffer + WidgetUpdateStepsIndex) = InNumber;
#if !PLATFORM_LITTLE_ENDIAN
		Algo::Reverse(WidgetUpdateStepsBuffer + WidgetUpdateStepsIndex, sizeof(uint64));
#endif
		WidgetUpdateStepsIndex += sizeof(uint64);
	}

	void FlushWidgetUpdateStepsBuffer()
	{
		if (WidgetUpdateStepsIndex > 0)
		{
			UE_TRACE_LOG(SlateTrace, WidgetUpdateSteps, SlateChannel)
				<< WidgetUpdateSteps.Buffer(WidgetUpdateStepsBuffer, WidgetUpdateStepsIndex);

			WidgetUpdateStepsIndex = 0;
			SerializeToWidgetUpdateSteps(kWidgetUpdateStepsCommand_NewBuffer);
			++WidgetUpdateStepsBufferNumber;
			SerializeToWidgetUpdateSteps(WidgetUpdateStepsBufferNumber);
		}
	}

	void AddStartWidgetPaintSteps(const SWidget* InWidget, uint64 InCycle)
	{
		if (WidgetUpdateStepsIndex + sizeof(uint64) + sizeof(uint64) + sizeof(uint8) + sizeof(uint8) >= SizeOfWidgetUpdateStepsBuffer)
		{
			FlushWidgetUpdateStepsBuffer();
		}

		if (SlateTraceDetail::GScopedPaintCount == 0)
		{
			SerializeToWidgetUpdateSteps(kWidgetUpdateStepsCommand_NewRootPaint);
		}

		SerializeToWidgetUpdateSteps(kWidgetUpdateStepsCommand_StartPaint);
		SerializeToWidgetUpdateSteps(InCycle);
		SerializeToWidgetUpdateSteps(GetWidgetId(InWidget));
	}

	void AddEndWidgetPaintPass(const SWidget* InWidget, uint64 InCycle)
	{
		if (WidgetUpdateStepsIndex + sizeof(uint64) + sizeof(uint8) >= SizeOfWidgetUpdateStepsBuffer)
		{
			FlushWidgetUpdateStepsBuffer();
		}
		SerializeToWidgetUpdateSteps(kWidgetUpdateStepsCommand_EndPaint);
		SerializeToWidgetUpdateSteps(InCycle);
	}
}

 //-----------------------------------------------------------------------------------//

FSlateTrace::FScopedWidgetPaintTrace::FScopedWidgetPaintTrace(const SWidget* InWidget)
	: Widget(InWidget)
	, StartPaintCount(SlateTraceDetail::GFramePaintCount)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(SlateChannel))
	{
		StartCycle = FPlatformTime::Cycles64();
		SlateTraceDetail::AddStartWidgetPaintSteps(Widget, StartCycle);
		FSlateTrace::ConditionallyUpdateWidgetInfo(Widget);
	}

	++SlateTraceDetail::GScopedPaintCount;
	++SlateTraceDetail::GFramePaintCount;
}

FSlateTrace::FScopedWidgetPaintTrace::~FScopedWidgetPaintTrace()
{
	--SlateTraceDetail::GScopedPaintCount;
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(SlateChannel))
	{
		const uint64 EndCycle = FPlatformTime::Cycles64();
		const uint32 AffectedCount = SlateTraceDetail::GFramePaintCount - StartPaintCount;

		SlateTraceDetail::AddEndWidgetPaintPass(Widget, EndCycle);

		if (SlateTraceDetail::GScopedPaintCount == 0)
		{
			const EWidgetUpdateFlags UpdateFlags = Widget->IsVolatile() ? EWidgetUpdateFlags::NeedsVolatilePaint : EWidgetUpdateFlags::NeedsRepaint;
			FSlateTrace::OutputWidgetUpdate(Widget, StartCycle, EndCycle, UpdateFlags, AffectedCount);
		}
	}
}

//-----------------------------------------------------------------------------------//

FSlateTrace::FScopedWidgetUpdateTrace::FScopedWidgetUpdateTrace(const SWidget* InWidget)
	: Widget(InWidget)
	, UpdateFlags(EWidgetUpdateFlags::None)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(SlateChannel))
	{
		if (EnumHasAnyFlags(InWidget->UpdateFlags, EWidgetUpdateFlags::NeedsActiveTimerUpdate | EWidgetUpdateFlags::NeedsTick))
		{
			StartCycle = FPlatformTime::Cycles64();
			UpdateFlags = InWidget->UpdateFlags & (EWidgetUpdateFlags::NeedsTick | EWidgetUpdateFlags::NeedsActiveTimerUpdate);
			FSlateTrace::ConditionallyUpdateWidgetInfo(Widget);
		}
	}
	++SlateTraceDetail::GScopedUpdateCount;
}

FSlateTrace::FScopedWidgetUpdateTrace::~FScopedWidgetUpdateTrace()
{
	--SlateTraceDetail::GScopedUpdateCount;
	if (UpdateFlags != EWidgetUpdateFlags::None)
	{
		FSlateTrace::OutputWidgetUpdate(Widget, StartCycle, FPlatformTime::Cycles64(), UpdateFlags, 1);
	}
}

//-----------------------------------------------------------------------------------//

void FSlateTrace::ApplicationTickAndDrawWidgets(float DeltaTime)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(SlateChannel))
	{
		static_assert(sizeof(ESlateTraceApplicationFlags) == sizeof(uint8), "FSlateTrace::ESlateFlags is not a uint8");

		SlateTraceDetail::FlushWidgetUpdateStepsBuffer();

		for (TWeakPtr<const SWidget>& WeakUpdateWidgetInfo : SlateTraceDetail::UpdateWidgetInfos)
		{
			if (TSharedPtr<const SWidget> SharedWidget = WeakUpdateWidgetInfo.Pin())
			{
				const uint64 WidgetId = SlateTraceDetail::GetWidgetId(SharedWidget.Get());
				const FString Path = FReflectionMetaData::GetWidgetPath(SharedWidget.Get());
				const FString DebugInfo = FReflectionMetaData::GetWidgetDebugInfo(SharedWidget.Get());

				if (FTraceAuxiliary::IsConnected())
				{
					SharedWidget->Debug_SetWidgetInfoTraced(FSlateTrace::TraceCounter);
				}

				UE_TRACE_LOG(SlateTrace, WidgetInfo, SlateChannel)
					<< WidgetInfo.WidgetId(WidgetId)
					<< WidgetInfo.Path(*Path)
					<< WidgetInfo.DebugInfo(*DebugInfo);
			}
		}
		SlateTraceDetail::UpdateWidgetInfos.Empty();

		ESlateTraceApplicationFlags LocalFlags = ESlateTraceApplicationFlags::None;
		if (GSlateEnableGlobalInvalidation) { LocalFlags |= ESlateTraceApplicationFlags::GlobalInvalidation; }
		if (GSlateFastWidgetPath) { LocalFlags |= ESlateTraceApplicationFlags::FastWidgetPath; }

		UE_TRACE_LOG(SlateTrace, ApplicationTickAndDrawWidgets, SlateChannel)
			<< ApplicationTickAndDrawWidgets.Cycle(FPlatformTime::Cycles64())
			<< ApplicationTickAndDrawWidgets.DeltaTime(DeltaTime)
			<< ApplicationTickAndDrawWidgets.WidgetCount(SlateTraceDetail::GWidgetCount)
			<< ApplicationTickAndDrawWidgets.TickCount(SlateTraceDetail::GFrameTickCount)
			<< ApplicationTickAndDrawWidgets.TimerCount(SlateTraceDetail::GFrameTimerCount)
			<< ApplicationTickAndDrawWidgets.RepaintCount(SlateTraceDetail::GFrameRepaintCount)
			//<< ApplicationTickAndDrawWidgets.VolatilePaintCount(SlateTraceDetail::GFrameVolatileCount)
			<< ApplicationTickAndDrawWidgets.PaintCount(SlateTraceDetail::GFramePaintCount)
			<< ApplicationTickAndDrawWidgets.InvalidateCount(SlateTraceDetail::GFrameInvalidateCount)
			<< ApplicationTickAndDrawWidgets.RootInvalidatedCount(SlateTraceDetail::GFrameRootInvalidateCount)
			<< ApplicationTickAndDrawWidgets.SlateFlags(static_cast<uint8>(LocalFlags));

		SlateTraceDetail::GFrameTickCount = 0;
		SlateTraceDetail::GFrameTimerCount = 0;
		SlateTraceDetail::GFrameRepaintCount = 0;
		//SlateTraceDetail::GFrameVolatileCount = 0;
		SlateTraceDetail::GFramePaintCount = 0;
		SlateTraceDetail::GFrameInvalidateCount = 0;
		SlateTraceDetail::GFrameRootInvalidateCount = 0;
	}
}

void FSlateTrace::ApplicationRegisterTraceEvents(FSlateApplicationBase& /*SlateApplication*/)
{
	// Registering directly on the slate application would mean slate app headers need to know trace types.
	// Instead register a static here that gets the application on a trace start.
	FTraceAuxiliary::OnTraceStarted.AddStatic(&FSlateTrace::HandleOnTraceStarted);
}

void FSlateTrace::OutputWidgetUpdate(const SWidget* Widget, uint64 StartCycle, uint64 EndCycle, EWidgetUpdateFlags UpdateFlags, uint32 AffectedCount)
{
	if (EnumHasAnyFlags(UpdateFlags, EWidgetUpdateFlags::NeedsActiveTimerUpdate))
	{
		++SlateTraceDetail::GFrameTimerCount;
	}
	else if (EnumHasAnyFlags(UpdateFlags, EWidgetUpdateFlags::NeedsTick))
	{
		++SlateTraceDetail::GFrameTickCount;
	}

	if (EnumHasAnyFlags(UpdateFlags, EWidgetUpdateFlags::NeedsVolatilePaint | EWidgetUpdateFlags::NeedsRepaint))
	{
		++SlateTraceDetail::GFrameRepaintCount;
	}

	static_assert(sizeof(EWidgetUpdateFlags) == sizeof(uint8), "EWidgetUpdateFlags is not a uint8");

	const uint64 WidgetId = SlateTraceDetail::GetWidgetId(Widget);

	UE_TRACE_LOG(SlateTrace, WidgetUpdated, SlateChannel)
		<< WidgetUpdated.Cycle(StartCycle)
		<< WidgetUpdated.CycleEnd(EndCycle)
		<< WidgetUpdated.WidgetId(WidgetId)
		<< WidgetUpdated.AffectedCount(AffectedCount)
		<< WidgetUpdated.UpdateFlags(static_cast<uint8>(UpdateFlags));
}

void FSlateTrace::WidgetInvalidated(const SWidget* Widget, const SWidget* Investigator, EInvalidateWidgetReason Reason)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(SlateChannel) && Reason != EInvalidateWidgetReason::None)
	{
		FSlateTrace::ConditionallyUpdateWidgetInfo(Widget);

		++SlateTraceDetail::GFrameInvalidateCount;

		static_assert(sizeof(EInvalidateWidgetReason) == sizeof(uint8), "EInvalidateWidgetReason is not a uint8");

		TStringBuilder<4096> ScriptTrace;
		constexpr int MAX_DEPTH = 64;
		uint64 StackTrace[MAX_DEPTH] = { 0 };
		uint32 StackTraceDepth = 0;
		uint32 ProcessId = 0;

#if !(UE_BUILD_SHIPPING)
		//@TODO: Could add a CVar to only capture certain callstacks for performance (Widget name, type, etc).
		if (!Investigator && bCaptureRootInvalidationCallstacks)
		{
			FSlowHeartBeatScope SuspendHeartBeat;
			FDisableHitchDetectorScope SuspendGameThreadHitch;

			FFrame::GetScriptCallstack(ScriptTrace, true /* bReturnEmpty */);
			if (ScriptTrace.Len() != 0)
			{
				ScriptTrace.InsertAt(0, TEXT("ScriptTrace: \n"));
			}

			// Walk the stack and dump it to the allocated memory.
			StackTraceDepth = FPlatformStackWalk::CaptureStackBackTrace(StackTrace, MAX_DEPTH);
			ProcessId = FPlatformProcess::GetCurrentProcessId();
		}
#endif // !(UE_BUILD_SHIPPING)

		const uint64 WidgetId = SlateTraceDetail::GetWidgetId(Widget);
		const uint64 InvestigatorId = SlateTraceDetail::GetWidgetIdIfValid(Investigator);
		const uint64 Cycle = FPlatformTime::Cycles64();

		UE_TRACE_LOG(SlateTrace, WidgetInvalidated, SlateChannel)
			<< WidgetInvalidated.Cycle(Cycle)
			<< WidgetInvalidated.WidgetId(WidgetId)
			<< WidgetInvalidated.InvestigatorId(InvestigatorId)
			<< WidgetInvalidated.ScriptTrace(*ScriptTrace)
			<< WidgetInvalidated.Callstack(StackTrace, StackTraceDepth)
			<< WidgetInvalidated.InvalidateWidgetReason(static_cast<uint8>(Reason));

#if !(UE_BUILD_SHIPPING)
		// Don't create an async task that locks when the engine is shutting down
		if (!Investigator && bCaptureRootInvalidationCallstacks && !IsEngineExitRequested())
		{
			LowLevelTasks::ETaskPriority SymbolResolvePriority = LowLevelTasks::ETaskPriority::BackgroundNormal;

			// Note: Done in seperate thread as symbol resolution is very slow, possibly 1~80ms based on widget complexity.
			UE::Tasks::Launch(UE_SOURCE_LOCATION, [Cycle, StackTrace, StackTraceDepth]()
			{
				const SIZE_T CallStackSize = 65535;
				ANSICHAR* CallStack = (ANSICHAR*)FMemory::SystemMalloc(CallStackSize);
				if (CallStack != nullptr)
				{
					CallStack[0] = 0;
				}
				
				// Need to lock on certain platforms when loading symbols, due to single threaded loading
				// See: https://docs.microsoft.com/en-us/windows/win32/api/dbghelp/nf-dbghelp-symloadmoduleexw#remarks
				{
					FScopeLock Lock(&UE::Slate::Private::ModuleLoadLock);

					uint32 StackDepth = 0;
					while (StackDepth < StackTraceDepth)
					{
						// Skip the first two backraces, that's from us.
						if (StackDepth >= 2 && StackTrace[StackDepth])
						{
							FProgramCounterSymbolInfo SymbolInfo;
							FPlatformStackWalk::ProgramCounterToSymbolInfo(StackTrace[StackDepth], SymbolInfo);
							FPlatformStackWalk::SymbolInfoToHumanReadableString(SymbolInfo, CallStack, CallStackSize);
							FCStringAnsi::StrncatTruncateDest(CallStack, CallStackSize, LINE_TERMINATOR_ANSI);
						}
						StackDepth++;
					}
				}

				UE_TRACE_LOG(SlateTrace, InvalidationCallstack, SlateChannel)
					<< InvalidationCallstack.SourceCycle(Cycle)
					<< InvalidationCallstack.CallstackText(CallStack);

				FMemory::SystemFree(CallStack);
			}, SymbolResolvePriority);
		}
#endif // !(UE_BUILD_SHIPPING)
	}
}

void FSlateTrace::RootInvalidated(const SWidget* Widget, const SWidget* Investigator)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(SlateChannel))
	{
		FSlateTrace::ConditionallyUpdateWidgetInfo(Widget);

		++SlateTraceDetail::GFrameInvalidateCount;

		const uint64 WidgetId = SlateTraceDetail::GetWidgetId(Widget);
		const uint64 InvestigatorId = SlateTraceDetail::GetWidgetIdIfValid(Investigator);

		UE_TRACE_LOG(SlateTrace, RootInvalidated, SlateChannel)
			<< RootInvalidated.Cycle(FPlatformTime::Cycles64())
			<< RootInvalidated.WidgetId(WidgetId)
			<< RootInvalidated.InvestigatorId(InvestigatorId);
	}
}

void FSlateTrace::RootChildOrderInvalidated(const SWidget* Widget, const SWidget* Investigator)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(SlateChannel))
	{
		FSlateTrace::ConditionallyUpdateWidgetInfo(Widget);

		++SlateTraceDetail::GFrameInvalidateCount;

		const uint64 WidgetId = SlateTraceDetail::GetWidgetId(Widget);
		const uint64 InvestigatorId = SlateTraceDetail::GetWidgetIdIfValid(Investigator);

		UE_TRACE_LOG(SlateTrace, RootChildOrderInvalidated, SlateChannel)
			<< RootChildOrderInvalidated.Cycle(FPlatformTime::Cycles64())
			<< RootChildOrderInvalidated.WidgetId(WidgetId)
			<< RootChildOrderInvalidated.InvestigatorId(InvestigatorId);
	}
}

void FSlateTrace::AddWidget(const SWidget* Widget)
{
	++SlateTraceDetail::GWidgetCount;

	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(SlateChannel))
	{
		const uint64 WidgetId = SlateTraceDetail::GetWidgetId(Widget);

		UE_TRACE_LOG(SlateTrace, AddWidget, SlateChannel)
			<< AddWidget.Cycle(FPlatformTime::Cycles64())
			<< AddWidget.WidgetId(WidgetId);
	}
}

void FSlateTrace::UpdateWidgetInfo(const SWidget* Widget)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(SlateChannel))
	{
		if (Widget)
		{
			SlateTraceDetail::UpdateWidgetInfos.Add(Widget->AsShared());
		}
	}
}

void FSlateTrace::RemoveWidget(const SWidget* Widget)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(SlateChannel))
	{
		const uint64 WidgetId = SlateTraceDetail::GetWidgetId(Widget);

		UE_TRACE_LOG(SlateTrace, RemoveWidget, SlateChannel)
			<< RemoveWidget.Cycle(FPlatformTime::Cycles64())
			<< RemoveWidget.WidgetId(WidgetId);
		ensure(SlateTraceDetail::GWidgetCount > 0);
	}
	--SlateTraceDetail::GWidgetCount;
}

void FSlateTrace::ConditionallyUpdateWidgetInfo(const SWidget* Widget)
{
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(SlateChannel) && FTraceAuxiliary::IsConnected())
	{
		if (Widget && Widget->Debug_GetWidgetInfoTraced() < FSlateTrace::TraceCounter)
		{
			const uint64 WidgetId = SlateTraceDetail::GetWidgetId(Widget);
			const FString Path = FReflectionMetaData::GetWidgetPath(Widget);
			const FString DebugInfo = FReflectionMetaData::GetWidgetDebugInfo(Widget);

			UE_TRACE_LOG(SlateTrace, WidgetInfo, SlateChannel)
				<< WidgetInfo.WidgetId(WidgetId)
				<< WidgetInfo.Path(*Path)
				<< WidgetInfo.DebugInfo(*DebugInfo);

			Widget->Debug_SetWidgetInfoTraced(FSlateTrace::TraceCounter);
		}
	}
}

void FSlateTrace::HandleOnTraceStarted(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination)
{
	FSlateTrace::TraceCounter++;
}

uint8 FSlateTrace::TraceCounter = 0;

 #endif // UE_SLATE_TRACE_ENABLED
