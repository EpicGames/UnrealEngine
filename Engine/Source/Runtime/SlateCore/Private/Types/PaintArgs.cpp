// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/PaintArgs.h"
#include "Layout/ArrangedWidget.h"
#include "Input/HittestGrid.h"
#include "Widgets/SWindow.h"

FPaintArgs::FPaintArgs(const SWidget* PaintParent, FHittestGrid& InRootHittestGrid, FHittestGrid& InCurrentHitTestGrid, UE::Slate::FDeprecateVector2DParameter InWindowOffset, double InCurrentTime, float InDeltaTime)
	: RootGrid(InRootHittestGrid)
	, CurrentGrid(InCurrentHitTestGrid)
	, WindowOffset(InWindowOffset)
	, PaintParentPtr(PaintParent)
	, CurrentTime(InCurrentTime)
	, DeltaTime(InDeltaTime)
	, bInheritedHittestability(true)
	, bDeferredPainting(false)
{
}

FPaintArgs::FPaintArgs(const SWidget* PaintParent, FHittestGrid& InRootHittestGrid, UE::Slate::FDeprecateVector2DParameter InWindowOffset, double InCurrentTime, float InDeltaTime)
	: FPaintArgs(PaintParent, InRootHittestGrid, InRootHittestGrid, InWindowOffset, InCurrentTime, InDeltaTime)
{

}

FPaintArgs FPaintArgs::InsertCustomHitTestPath(const SWidget* Widget, TSharedRef<ICustomHitTestPath> CustomHitTestPath) const
{
	const_cast<FHittestGrid&>(CurrentGrid).InsertCustomHitTestPath(Widget, CustomHitTestPath);
	return *this;
}
