// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/ReflectionMetadata.h"
#include "Widgets/SWidget.h"

FString FReflectionMetaData::GetWidgetPath(const SWidget* InWidget, bool bShort, bool bNativePathOnly)
{
	if (!InWidget)
	{
		return TEXT("None");
	}

	return GetWidgetPath(*InWidget, bShort, bNativePathOnly);
}

FString FReflectionMetaData::GetWidgetPath(const SWidget& InWidget, bool bShort, bool bNativePathOnly)
{
	FString WidgetPath;

	int32 bWidgetsInPath = 0;

	const SWidget* CurrentWidget = &InWidget;
	while (CurrentWidget)
	{
		TSharedPtr<FReflectionMetaData> MetaData = CurrentWidget->GetMetaData<FReflectionMetaData>();
		if (!bNativePathOnly && MetaData.IsValid())
		{
			WidgetPath.InsertAt(0, MetaData->Name.ToString());
		}
		else
		{
			WidgetPath.InsertAt(0, CurrentWidget->GetReadableLocation());
		}

		CurrentWidget = CurrentWidget->GetParentWidget().Get();

		if (CurrentWidget)
		{
			WidgetPath.InsertAt(0, TEXT("/"));

			bWidgetsInPath++;

			if (bShort && bWidgetsInPath >= 5)
			{
				WidgetPath.InsertAt(0, TEXT("..."));
				break;
			}
		}
	}

	return WidgetPath;
}

FString FReflectionMetaData::GetWidgetDebugInfo(const SWidget* InWidget)
{
	if (!InWidget)
	{
		return TEXT("None");
	}

	return GetWidgetDebugInfo(*InWidget);
}

FString FReflectionMetaData::GetWidgetDebugInfo(const SWidget& InWidget)
{
	// UMG widgets have meta-data to help track them
	TSharedPtr<FReflectionMetaData> MetaData = InWidget.GetMetaData<FReflectionMetaData>();
	if (MetaData.IsValid())
	{
		if (const UObject* AssetPtr = MetaData->Asset.Get())
		{
			const FName AssetName = AssetPtr->GetFName();
			const FName WidgetName = MetaData->Name;

			return FString::Printf(TEXT("%s [%s]"), *AssetName.ToString(), *WidgetName.ToString());
		}
	}

	TSharedPtr<FReflectionMetaData> ParentMetadata = GetWidgetOrParentMetaData(&InWidget);
	if (ParentMetadata.IsValid())
	{
		if (const UObject* AssetPtr = ParentMetadata->Asset.Get())
		{
			const FName AssetName = AssetPtr->GetFName();
			const FName WidgetName = ParentMetadata->Name;

			return FString::Printf(TEXT("%s [%s(%s)]"), *AssetName.ToString(), *WidgetName.ToString(), *InWidget.GetReadableLocation());
		}
	}

	return InWidget.ToString();
}

UObject* FReflectionMetaData::GetWidgetSourceObject(const SWidget* InWidget)
{
	if (!InWidget)
	{
		return nullptr;
	}

	return GetWidgetSourceObject(*InWidget);
}

UObject* FReflectionMetaData::GetWidgetSourceObject(const SWidget& InWidget)
{
	// UMG widgets have meta-data to help track them
	TSharedPtr<FReflectionMetaData> MetaData = InWidget.GetMetaData<FReflectionMetaData>();
	if (MetaData.IsValid())
	{
		return MetaData->SourceObject.Get();
	}

	return nullptr;
}

TSharedPtr<FReflectionMetaData> FReflectionMetaData::GetWidgetOrParentMetaData(const SWidget* InWidget)
{
	const SWidget* CurrentWidget = InWidget;
	while (CurrentWidget != nullptr)
	{
		// UMG widgets have meta-data to help track them
		TSharedPtr<FReflectionMetaData> MetaData = CurrentWidget->GetMetaData<FReflectionMetaData>();
		if (MetaData.IsValid() && MetaData->Asset.Get())
		{
			return MetaData;
		}

		// If the widget we're on doesn't have metadata or asset information, try the parent widgets,
		// sometimes complex widgets create many internal widgets, they should still belong to the
		// corresponding asset/class.
		CurrentWidget = CurrentWidget->GetParentWidget().Get();
	}

	return TSharedPtr<FReflectionMetaData>();
}