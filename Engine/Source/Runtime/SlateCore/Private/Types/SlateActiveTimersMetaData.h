// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Application/ActiveTimerHandle.h"
#include "Types/ISlateMetaData.h"

namespace UE::Slate
{

/** Metadata that holds the active timer content for a widget. */
class FActiveTimersMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FActiveTimersMetaData, ISlateMetaData)

	/** The list of active timer handles for this widget. */
	TArray<TSharedRef<FActiveTimerHandle>> ActiveTimers;
};

}