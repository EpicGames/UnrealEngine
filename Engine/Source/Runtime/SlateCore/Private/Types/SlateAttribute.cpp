// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/SlateAttribute.h"

#include "Types/ReflectionMetadata.h"
#include "Types/SlateAttributeMetaData.h"
#include "Widgets/SWidget.h"
#include "Debugging/WidgetList.h"

namespace SlateAttributePrivate
{
	/*
	 *
	 */
	void TestAttributeAddress(const SWidget& OwningWidget, const FSlateAttributeImpl& Attribute, ESlateAttributeType AttributeType)
	{
#if STATS && DO_CHECK
		//@TODO: DarenC - Using allocsize as proxy for 'IsConstructed' due to encapsulation.
		if (AttributeType == ESlateAttributeType::Member && OwningWidget.GetAllocSize() > 0)
		{
			UPTRINT SlateAttributePtr = (UPTRINT)&Attribute;
			UPTRINT WidgetPtr = (UPTRINT)&OwningWidget;
			checkf(SlateAttributePtr >= WidgetPtr && SlateAttributePtr <= WidgetPtr + OwningWidget.GetAllocSize(),
				TEXT("You can only register Attribute that are defined in a SWidget. ")
				TEXT("Use TAttribute or TSlateExternalAttribute instead. See SWidget: '%s'. See SlateAttribute.h for more info.")
				, *FReflectionMetaData::GetWidgetPath(OwningWidget));
		}
#endif
	}


	/*
	 *
	 */
	void ISlateAttributeContainer::RemoveContainerWidget(SWidget& Widget)
	{
		FSlateAttributeMetaData::RemoveContainerWidget(Widget, *this);
	}


	void ISlateAttributeContainer::UpdateContainerSortOrder(SWidget& Widget)
	{
		FSlateAttributeMetaData::UpdateContainerSortOrder(Widget, *this);
	}


	/*
	 *
	 */
	bool FSlateAttributeImpl::ProtectedIsWidgetInDestructionPath(SWidget* Widget) const
	{
#if UE_WITH_SLATE_DEBUG_WIDGETLIST
		int32 FoundIndex = UE::Slate::FWidgetList::GetAllWidgets().Find(Widget);
		if (FoundIndex == INDEX_NONE)
		{
			// The widget is already destroyed (this is bad) or we are destroying the SWidget base class SlateAttribute (this is ok)
			UPTRINT SlateAttributePtr = (UPTRINT)this;
			UPTRINT WidgetPtr = (UPTRINT)Widget;
			const bool bIsBaseWidgetAttribute = SlateAttributePtr >= WidgetPtr && SlateAttributePtr <= WidgetPtr + sizeof(SWidget);
			return bIsBaseWidgetAttribute;
		}
		// if the shared instance exist, then the widget is not being destroyed
		return !UE::Slate::FWidgetList::GetAllWidgets()[FoundIndex]->DoesSharedInstanceExist();
#endif
		return true;
	}


	bool FSlateAttributeImpl::ProtectedIsImplemented(const SWidget& OwningWidget) const
	{
		const FSlateAttributeDescriptor::OffsetType Offset = (FSlateAttributeDescriptor::OffsetType)((UPTRINT)(this) - (UPTRINT)(&OwningWidget));
		return OwningWidget.GetWidgetClass().GetAttributeDescriptor().FindMemberAttribute(Offset) != nullptr;
	}


	void FSlateAttributeImpl::ProtectedUnregisterAttribute(SWidget& OwningWidget, ESlateAttributeType AttributeType) const
	{
		TestAttributeAddress(OwningWidget, *this, AttributeType);
		
		FSlateAttributeMetaData::UnregisterAttribute(OwningWidget, *this);
	}


	void FSlateAttributeImpl::ProtectedRegisterAttribute(SWidget& OwningWidget, ESlateAttributeType AttributeType, TUniquePtr<ISlateAttributeGetter>&& Wrapper)
	{
		TestAttributeAddress(OwningWidget, *this, AttributeType);

		FSlateAttributeMetaData::RegisterAttribute(OwningWidget, *this, AttributeType, MoveTemp(Wrapper));
	}


	void FSlateAttributeImpl::ProtectedInvalidateWidget(SWidget& OwningWidget, ESlateAttributeType AttributeType, EInvalidateWidgetReason InvalidationReason) const
	{
		TestAttributeAddress(OwningWidget, *this, AttributeType);

		FSlateAttributeMetaData::InvalidateWidget(OwningWidget, *this, AttributeType, InvalidationReason);
	}


	void FSlateAttributeImpl::ProtectedInvalidateWidget(ISlateAttributeContainer& Container, ESlateAttributeType AttributeType, EInvalidateWidgetReason InvalidationReason) const
	{
		FSlateAttributeMetaData::InvalidateWidget(Container, *this, AttributeType, InvalidationReason);
	}


	bool FSlateAttributeImpl::ProtectedIsBound(const SWidget& OwningWidget, ESlateAttributeType AttributeType) const
	{
		TestAttributeAddress(OwningWidget, *this, AttributeType);

		return FSlateAttributeMetaData::IsAttributeBound(OwningWidget, *this);
	}


	bool FSlateAttributeImpl::ProtectedIsBound(const ISlateAttributeContainer& Container, ESlateAttributeType AttributeType) const
	{
		return ProtectedIsBound(Container.GetContainerWidget(), AttributeType);
	}


	ISlateAttributeGetter* FSlateAttributeImpl::ProtectedFindGetter(const SWidget& OwningWidget, ESlateAttributeType AttributeType) const
	{
		TestAttributeAddress(OwningWidget, *this, AttributeType);

		return FSlateAttributeMetaData::GetAttributeGetter(OwningWidget, *this);
	}


	ISlateAttributeGetter* FSlateAttributeImpl::ProtectedFindGetter(const ISlateAttributeContainer& Container, ESlateAttributeType AttributeType) const
	{
		return ProtectedFindGetter(Container.GetContainerWidget(), AttributeType);
	}
	
	
	FDelegateHandle FSlateAttributeImpl::ProtectedFindGetterHandle(const SWidget& OwningWidget, ESlateAttributeType AttributeType) const
	{
		TestAttributeAddress(OwningWidget, *this, AttributeType);

		return FSlateAttributeMetaData::GetAttributeGetterHandle(OwningWidget, *this);
	}


	FDelegateHandle FSlateAttributeImpl::ProtectedFindGetterHandle(const ISlateAttributeContainer& Container, ESlateAttributeType AttributeType) const
	{
		return ProtectedFindGetterHandle(Container.GetContainerWidget(), AttributeType);
	}


	void FSlateAttributeImpl::ProtectedUpdateNow(SWidget& OwningWidget, ESlateAttributeType AttributeType)
	{
		TestAttributeAddress(OwningWidget, *this, AttributeType);

		FSlateAttributeMetaData::UpdateAttribute(OwningWidget, *this);
	}


	void FSlateAttributeImpl::ProtectedUpdateNow(ISlateAttributeContainer& Container, ESlateAttributeType AttributeType)
	{
		ProtectedUpdateNow(Container.GetContainerWidget(), AttributeType);
	}


	void FSlateAttributeImpl::ProtectedMoveAttribute(SWidget& OwningWidget, ESlateAttributeType AttributeType, const FSlateAttributeBase* Other)
	{
		checkf(AttributeType == ESlateAttributeType::Managed, TEXT("Only Managed Attribute cannot be moved."));

		// Other can't be pass as ref because it's already moved.
		//We only need it's pointer to make sure it's not also used in FSlateAttributeMetaData (as a void*).
		check(Other);

		FSlateAttributeMetaData::MoveAttribute(OwningWidget, *this, AttributeType, Other);
	}


	void FSlateAttributeImpl::ProtectedUnregisterAttribute(ISlateAttributeContainer& Container, ESlateAttributeType AttributeType) const
	{
		FSlateAttributeMetaData::UnregisterAttribute(Container, *this);
	}


	void FSlateAttributeImpl::ProtectedRegisterAttribute(ISlateAttributeContainer& Container, ESlateAttributeType AttributeType, TUniquePtr<ISlateAttributeGetter>&& Wrapper)
	{
		FSlateAttributeMetaData::RegisterAttribute(Container, *this, AttributeType, MoveTemp(Wrapper));
	}
}
