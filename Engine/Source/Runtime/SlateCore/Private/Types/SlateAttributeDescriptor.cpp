// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/SlateAttributeDescriptor.h"
#include "Types/SlateAttribute.h"
#include "Containers/ArrayView.h"

#include <limits>


/** */
FSlateAttributeDescriptor::FAttribute::FAttribute(FName InName, OffsetType InOffset, FInvalidateWidgetReasonAttribute InReason)
	: Name(InName)
	, Offset(InOffset)
	, Prerequisite()
	, ContainerName()
	, OnValueChanged()
	, InvalidationReason(MoveTemp(InReason))
	, SortOrder(DefaultSortOrder(InOffset))
	, AttributeType(SlateAttributePrivate::ESlateAttributeType::Member)
	, bAffectVisibility(false)
{
}

FSlateAttributeDescriptor::FAttribute::FAttribute(FName InContainerName, FName InName, OffsetType InOffset, FInvalidateWidgetReasonAttribute InReason)
	: Name(InName)
	, Offset(InOffset)
	, Prerequisite()
	, ContainerName(InContainerName)
	, OnValueChanged()
	, InvalidationReason(MoveTemp(InReason))
	, SortOrder(DefaultSortOrder(InOffset))
	, AttributeType(SlateAttributePrivate::ESlateAttributeType::Contained)
	, bAffectVisibility(false)
{
}


/** */
FSlateAttributeDescriptor::FContainerInitializer::FAttributeEntry::FAttributeEntry(FSlateAttributeDescriptor & InDescriptor, FName InContainerName, int32 InAttributeIndex)
	: Descriptor(InDescriptor)
	, ContainerName(InContainerName)
	, AttributeIndex(InAttributeIndex)
{}


FSlateAttributeDescriptor::FContainerInitializer::FAttributeEntry& FSlateAttributeDescriptor::FContainerInitializer::FAttributeEntry::UpdatePrerequisite(FName Prerequisite)
{
	if (Descriptor.Attributes.IsValidIndex(AttributeIndex))
	{
		Descriptor.SetPrerequisite(ContainerName, Descriptor.Attributes[AttributeIndex], Prerequisite);
	}
	return *this;
}


FSlateAttributeDescriptor::FContainerInitializer::FAttributeEntry& FSlateAttributeDescriptor::FContainerInitializer::FAttributeEntry::OnValueChanged(FAttributeValueChangedDelegate InCallback)
{
	if (Descriptor.Attributes.IsValidIndex(AttributeIndex))
	{
		Descriptor.Attributes[AttributeIndex].OnValueChanged = MoveTemp(InCallback);
	}
	return *this;
}


/** */
FSlateAttributeDescriptor::FInitializer::FAttributeEntry::FAttributeEntry(FSlateAttributeDescriptor& InDescriptor, int32 InAttributeIndex)
	: Descriptor(InDescriptor)
	, AttributeIndex(InAttributeIndex)
{}


FSlateAttributeDescriptor::FInitializer::FAttributeEntry& FSlateAttributeDescriptor::FInitializer::FAttributeEntry::UpdatePrerequisite(FName Prerequisite)
{
	if (Descriptor.Attributes.IsValidIndex(AttributeIndex))
	{
		Descriptor.SetPrerequisite(FName(), Descriptor.Attributes[AttributeIndex], Prerequisite);
	}
	return *this;
}


FSlateAttributeDescriptor::FInitializer::FAttributeEntry& FSlateAttributeDescriptor::FInitializer::FAttributeEntry::AffectVisibility()
{
	if (Descriptor.Attributes.IsValidIndex(AttributeIndex))
	{
		Descriptor.SetAffectVisibility(Descriptor.Attributes[AttributeIndex], true);
	}
	return *this;
}


FSlateAttributeDescriptor::FInitializer::FAttributeEntry& FSlateAttributeDescriptor::FInitializer::FAttributeEntry::OnValueChanged(FAttributeValueChangedDelegate InCallback)
{
	if (Descriptor.Attributes.IsValidIndex(AttributeIndex))
	{
		Descriptor.Attributes[AttributeIndex].OnValueChanged = MoveTemp(InCallback);
	}
	return *this;
}


/** */
FSlateAttributeDescriptor::FContainerInitializer::FContainerInitializer(FSlateAttributeDescriptor& InDescriptor, FName InContainerName)
	: Descriptor(InDescriptor)
	, ContainerName(InContainerName)
{
}

FSlateAttributeDescriptor::FContainerInitializer::FContainerInitializer(FSlateAttributeDescriptor& InDescriptor, const FSlateAttributeDescriptor& ParentDescriptor, FName InContainerName)
	: Descriptor(InDescriptor)
	, ContainerName(InContainerName)
{
	for (const FAttribute& Att : ParentDescriptor.Attributes)
	{
		if (Att.ContainerName == InContainerName)
		{
			bool bNameAlreadyExist = InDescriptor.Attributes.ContainsByPredicate([AttName = Att.GetName()](const FAttribute& Other)
				{
					return Other.GetName() == AttName;
				});
			if (ensureMsgf(!bNameAlreadyExist, TEXT("The attribute '%s' already exists in the FSlateAttributeDescriptor."), *Att.GetName().ToString()))
			{
				InDescriptor.Attributes.Add(Att);
			}
		}
	}
}


FSlateAttributeDescriptor::FContainerInitializer::FAttributeEntry FSlateAttributeDescriptor::FContainerInitializer::AddContainedAttribute(FName AttributeName, OffsetType Offset, const FInvalidateWidgetReasonAttribute& Reason)
{
	return Descriptor.AddContainedAttribute(ContainerName, AttributeName, Offset, Reason);
}

FSlateAttributeDescriptor::FContainerInitializer::FAttributeEntry FSlateAttributeDescriptor::FContainerInitializer::AddContainedAttribute(FName AttributeName, SIZE_T Offset, const FInvalidateWidgetReasonAttribute& Reason)
{
	check(Offset <= std::numeric_limits<OffsetType>::max());
	return Descriptor.AddContainedAttribute(ContainerName, AttributeName, (OffsetType)Offset, Reason);
}

FSlateAttributeDescriptor::FContainerInitializer::FAttributeEntry FSlateAttributeDescriptor::FContainerInitializer::AddContainedAttribute(FName AttributeName, OffsetType Offset, FInvalidateWidgetReasonAttribute&& Reason)
{
	return Descriptor.AddContainedAttribute(ContainerName, AttributeName, Offset, MoveTemp(Reason));
}

FSlateAttributeDescriptor::FContainerInitializer::FAttributeEntry FSlateAttributeDescriptor::FContainerInitializer::AddContainedAttribute(FName AttributeName, SIZE_T Offset, FInvalidateWidgetReasonAttribute&& Reason)
{
	check(Offset <= std::numeric_limits<OffsetType>::max());
	return Descriptor.AddContainedAttribute(ContainerName, AttributeName, (OffsetType)Offset, MoveTemp(Reason));
}


void FSlateAttributeDescriptor::FContainerInitializer::OverrideInvalidationReason(FName AttributeName, const FInvalidateWidgetReasonAttribute& Reason)
{
	Descriptor.OverrideInvalidationReason(ContainerName, AttributeName, Reason);
}


void FSlateAttributeDescriptor::FContainerInitializer::OverrideInvalidationReason(FName AttributeName, FInvalidateWidgetReasonAttribute&& Reason)
{
	Descriptor.OverrideInvalidationReason(ContainerName, AttributeName, MoveTemp(Reason));
}


void FSlateAttributeDescriptor::FContainerInitializer::OverrideOnValueChanged(FName AttributeName, ECallbackOverrideType OverrideType, FAttributeValueChangedDelegate Callback)
{
	Descriptor.OverrideOnValueChanged(ContainerName, AttributeName, OverrideType, MoveTemp(Callback));
}


/** */
FSlateAttributeDescriptor::FInitializer::FInitializer(FSlateAttributeDescriptor& InDescriptor)
	: Descriptor(InDescriptor)
{
}

FSlateAttributeDescriptor::FInitializer::FInitializer(FSlateAttributeDescriptor& InDescriptor, const FSlateAttributeDescriptor& ParentDescriptor)
	: Descriptor(InDescriptor)
{
	InDescriptor.Attributes = ParentDescriptor.Attributes;
	InDescriptor.Containers = ParentDescriptor.Containers;
}


FSlateAttributeDescriptor::FInitializer::~FInitializer()
{
	// Confirm that the Visibility attribute is marked as "bAffectVisibility"
	{
		static const FName NAME_Visibility = "Visibility";
		const FAttribute* FoundVisibilityAttribute = Descriptor.FindAttribute(NAME_Visibility);
		checkf(&Descriptor.GetAttributeAtIndex(0) == FoundVisibilityAttribute, TEXT(""));
		checkf(FoundVisibilityAttribute, TEXT("The visibility attribute doesn't exist."));
		checkf(FoundVisibilityAttribute->bAffectVisibility, TEXT("The Visibility attribute must be marked as 'Affect Visibility'"));
	}


	// Update the sort order for the item that have prerequisite.
	//Because adding the attribute is not required at the moment,
	//try to not change the order in which they were added.
	//Because of that, the algo can creates bad ordering.
	//The AffectVisibility attributes must be in the front of the list.

	struct FPrerequisiteSort
	{
		FPrerequisiteSort() = default;
		FPrerequisiteSort(int32 A, int32 B, int32 InDepth) : AttributeIndex(A), PrerequisitesIndex(B), Depth(InDepth) {}
		int32 AttributeIndex;
		int32 PrerequisitesIndex;
		int32 Depth = -1;

		void CalculateDepth(TArrayView<FPrerequisiteSort> Prerequisites)
		{
			if (Depth < 0)
			{
				check(PrerequisitesIndex != INDEX_NONE);
				if (Prerequisites[PrerequisitesIndex].Depth < 0)
				{
					// calculate the Depth recursively
					Prerequisites[PrerequisitesIndex].CalculateDepth(Prerequisites);
				}
				Depth = Prerequisites[PrerequisitesIndex].Depth + 1;
			}
		}
	};
	struct FPrerequisiteSortPredicate
	{
		const TArray<FAttribute>& Attrbutes;
		bool operator()(const FPrerequisiteSort& A, const FPrerequisiteSort& B) const
		{
			const FAttribute& AttributeA = Attrbutes[A.AttributeIndex];
			const FAttribute& AttributeB = Attrbutes[B.AttributeIndex];

			if (AttributeA.ContainerIndex != AttributeB.ContainerIndex)
			{
				if (AttributeA.ContainerName.IsNone())
				{
					return true;
				}
				else if (AttributeB.ContainerName.IsNone())
				{
					return false;
				}
				return AttributeA.ContainerIndex < AttributeB.ContainerIndex;
			}

			if (AttributeA.bAffectVisibility != AttributeB.bAffectVisibility)
			{
				return AttributeA.bAffectVisibility;
			}

			if (A.Depth != B.Depth)
			{
				return A.Depth < B.Depth;
			}

			if (A.PrerequisitesIndex == B.PrerequisitesIndex)
			{
				return Attrbutes[A.AttributeIndex].SortOrder < Attrbutes[B.AttributeIndex].SortOrder;
			}

			const int32 SortA = A.PrerequisitesIndex != INDEX_NONE ? Attrbutes[A.PrerequisitesIndex].SortOrder : AttributeA.SortOrder;
			const int32 SortB = B.PrerequisitesIndex != INDEX_NONE ? Attrbutes[B.PrerequisitesIndex].SortOrder : AttributeB.SortOrder;
			return SortA < SortB;
		}
	};

	if (Descriptor.Containers.Num() > 0)
	{
		for (FContainer& Container : Descriptor.Containers)
		{
			Container.SortOrder = DefaultSortOrder(Container.Offset);
		}
		Descriptor.Containers.Sort([](const FContainer& A, const FContainer& B) { return A.SortOrder < B.SortOrder; });
	}


	TArray<FPrerequisiteSort, TInlineAllocator<32>> Prerequisites;
	Prerequisites.Reserve(Descriptor.Attributes.Num());

	bool bSortNeeded = false;
	for (int32 Index = 0; Index < Descriptor.Attributes.Num(); ++Index)
	{
		FAttribute& Attribute = Descriptor.Attributes[Index];

		if (!Attribute.ContainerName.IsNone())
		{
			const int32 FoundIndex = Descriptor.Containers.IndexOfByPredicate([ContainerName = Attribute.ContainerName](const FContainer& Other) { return Other.GetName() == ContainerName; });
			check(FoundIndex != INDEX_NONE);
			Attribute.ContainerIndex = (uint8)FoundIndex;
			Attribute.SortOrder = Descriptor.Containers[FoundIndex].SortOrder + (Attribute.Offset * 10);
		}
		else
		{
			Attribute.ContainerIndex = std::numeric_limits<uint8>::max();
			Attribute.SortOrder = DefaultSortOrder(Attribute.Offset);
		}

		if (!Attribute.Prerequisite.IsNone())
		{
			// Find the Prerequisite index
			const FName Prerequisite = Attribute.Prerequisite;
			const int32 PrerequisiteIndex = Descriptor.Attributes.IndexOfByPredicate([Prerequisite](const FAttribute& Other) { return Other.Name == Prerequisite; });
			if (ensureAlwaysMsgf(Descriptor.Attributes.IsValidIndex(PrerequisiteIndex), TEXT("The Prerequisite '%s' doesn't exist"), *Prerequisite.ToString()))
			{
				Prerequisites.Emplace(Index, PrerequisiteIndex, -1);
				bSortNeeded = true;
			}
			else
			{
				Prerequisites.Emplace(Index, INDEX_NONE, 0);
			}
		}
		else
		{
			// index 0 is the visibility attribute
			int32 PrerequisiteIndex = INDEX_NONE;
			if (Index != 0 && Attribute.bAffectVisibility)
			{
				bSortNeeded = true;
				PrerequisiteIndex = 0; // After Visibility
			}
			Prerequisites.Emplace(Index, PrerequisiteIndex, 0);
		}
	}

	if (bSortNeeded)
	{
		// Get the depth order
		for (FPrerequisiteSort& PrerequisiteSort : Prerequisites)
		{
			PrerequisiteSort.CalculateDepth(Prerequisites);
		}

		Prerequisites.Sort(FPrerequisiteSortPredicate{ Descriptor.Attributes });
		int32 PreviousPrerequisiteIndex = INDEX_NONE;
		int32 IncreaseCount = 1;
		for (const FPrerequisiteSort& Element : Prerequisites)
		{
			if (Element.PrerequisitesIndex != INDEX_NONE)
			{
				if (PreviousPrerequisiteIndex == Element.PrerequisitesIndex)
				{
					++IncreaseCount;
				}
				Descriptor.Attributes[Element.AttributeIndex].SortOrder = Descriptor.Attributes[Element.PrerequisitesIndex].SortOrder + IncreaseCount;
			}
			PreviousPrerequisiteIndex = Element.PrerequisitesIndex;
		}

		Descriptor.Attributes.Sort([](const FAttribute& A, const FAttribute& B) { return A.SortOrder < B.SortOrder; });
	}

	// Confirm that the attributes marked as "AffectVisibility" are in front of the list.
#if 0
	{
		bool bLookingForAffectVisibility = true;
		for (const FAttribute& Attribute : Descriptor.Attributes)
		{
			if (!Attribute.bAffectVisibility)
			{
				bLookingForAffectVisibility = false;
			}
			else if (!bLookingForAffectVisibility)
			{
				checkf(false, TEXT("Attribute marked as 'AffectVisibility' should be at the start of the update list or depends on the Visibility attribute."));
			}
		}
	}
#endif
}


FSlateAttributeDescriptor::FInitializer::FAttributeEntry FSlateAttributeDescriptor::FInitializer::AddMemberAttribute(FName AttributeName, OffsetType Offset, const FInvalidateWidgetReasonAttribute& Reason)
{
	return Descriptor.AddMemberAttribute(AttributeName, Offset, Reason);
}


FSlateAttributeDescriptor::FInitializer::FAttributeEntry FSlateAttributeDescriptor::FInitializer::AddMemberAttribute(FName AttributeName, OffsetType Offset, FInvalidateWidgetReasonAttribute&& Reason)
{
	return Descriptor.AddMemberAttribute(AttributeName, Offset, MoveTemp(Reason));
}

FSlateAttributeDescriptor::FContainerInitializer FSlateAttributeDescriptor::FInitializer::AddContainer(FName ContainerName, OffsetType Offset)
{
	return Descriptor.AddContainer(ContainerName, Offset);
}

void FSlateAttributeDescriptor::FInitializer::OverrideInvalidationReason(FName AttributeName, const FInvalidateWidgetReasonAttribute& Reason)
{
	Descriptor.OverrideInvalidationReason(FName(), AttributeName, Reason);
}


void FSlateAttributeDescriptor::FInitializer::OverrideInvalidationReason(FName AttributeName, FInvalidateWidgetReasonAttribute&& Reason)
{
	Descriptor.OverrideInvalidationReason(FName(), AttributeName, MoveTemp(Reason));
}


void FSlateAttributeDescriptor::FInitializer::OverrideOnValueChanged(FName AttributeName, ECallbackOverrideType OverrideType, FAttributeValueChangedDelegate Callback)
{
	Descriptor.OverrideOnValueChanged(FName(), AttributeName, OverrideType, MoveTemp(Callback));
}


void FSlateAttributeDescriptor::FInitializer::SetAffectVisibility(FName AttributeName, bool bAffectVisibility)
{
	FAttribute* Attribute = Descriptor.FindAttribute(AttributeName);
	if (ensureAlwaysMsgf(Attribute, TEXT("The attribute named '%s' doesn't exist"), *AttributeName.ToString()))
	{
		Descriptor.SetAffectVisibility(*Attribute, bAffectVisibility);
	}
}


/** */
FSlateAttributeDescriptor::FAttribute const& FSlateAttributeDescriptor::GetAttributeAtIndex(int32 Index) const
{
	check(Attributes.IsValidIndex(Index));
	FSlateAttributeDescriptor::FAttribute const& Result = Attributes[Index];
	return Result;
}


const FSlateAttributeDescriptor::FContainer* FSlateAttributeDescriptor::FindContainer(FName ContainerName) const
{
	return Containers.FindByPredicate([ContainerName](const FContainer& Other) { return Other.GetName() == ContainerName; });
}


const FSlateAttributeDescriptor::FAttribute* FSlateAttributeDescriptor::FindAttribute(FName AttributeName) const
{
	return Attributes.FindByPredicate([AttributeName](const FAttribute& Other) { return Other.Name == AttributeName; });
}


FSlateAttributeDescriptor::FAttribute* FSlateAttributeDescriptor::FindAttribute(FName AttributeName)
{
	return Attributes.FindByPredicate([AttributeName](const FAttribute& Other) { return Other.Name == AttributeName; });
}


const FSlateAttributeDescriptor::FAttribute* FSlateAttributeDescriptor::FindMemberAttribute(OffsetType AttributeOffset) const
{
	const FSlateAttributeDescriptor::FAttribute* Result = Attributes.FindByPredicate(
		[AttributeOffset](const FAttribute& Other)
		{
			return Other.Offset == AttributeOffset
				&& Other.AttributeType == SlateAttributePrivate::ESlateAttributeType::Member;
		});
	return Result;
}


const FSlateAttributeDescriptor::FAttribute* FSlateAttributeDescriptor::FindContainedAttribute(FName ContainerName, OffsetType AttributeOffset) const
{
	const FSlateAttributeDescriptor::FAttribute* Result = Attributes.FindByPredicate(
		[AttributeOffset, ContainerName](const FAttribute& Other)
		{
			return Other.Offset == AttributeOffset
				&& Other.AttributeType == SlateAttributePrivate::ESlateAttributeType::Contained
				&& Other.ContainerName == ContainerName;
		});
	return Result;
}


int32 FSlateAttributeDescriptor::IndexOfContainer(FName ContainerName) const
{
	return Containers.IndexOfByPredicate([ContainerName](const FContainer& Other) { return Other.GetName() == ContainerName; });
}


int32 FSlateAttributeDescriptor::IndexOfAttribute(FName AttributeName) const
{
	return Attributes.IndexOfByPredicate([AttributeName](const FAttribute& Other) { return Other.Name == AttributeName; });
}


int32 FSlateAttributeDescriptor::IndexOfMemberAttribute(OffsetType AttributeOffset) const
{
	int32 FoundIndex = Attributes.IndexOfByPredicate(
		[AttributeOffset](const FAttribute& Other)
		{
			return Other.Offset == AttributeOffset
				&& Other.AttributeType == SlateAttributePrivate::ESlateAttributeType::Member;
		});
	return FoundIndex;
}


int32 FSlateAttributeDescriptor::IndexOfContainedAttribute(FName ContainerName, OffsetType AttributeOffset) const
{
	int32 FoundIndex = Attributes.IndexOfByPredicate(
		[AttributeOffset, ContainerName](const FAttribute& Other)
		{
			return Other.Offset == AttributeOffset
				&& Other.AttributeType == SlateAttributePrivate::ESlateAttributeType::Contained
				&& Other.ContainerName == ContainerName;
		});
	return FoundIndex;
}


FSlateAttributeDescriptor::FContainerInitializer FSlateAttributeDescriptor::AddContainer(FName ContainerName, OffsetType Offset)
{
	check(!ContainerName.IsNone());

	const FContainer* FoundAttribute = FindContainer(ContainerName);
	if (ensureAlwaysMsgf(FoundAttribute == nullptr, TEXT("The container '%s' already exist. (Do you have the correct parent class in SLATE_DECLARE_WIDGET)"), *ContainerName.ToString()))
	{
		Containers.Emplace(ContainerName, Offset);
	}
	return FContainerInitializer(*this, ContainerName);
}


FSlateAttributeDescriptor::FInitializer::FAttributeEntry FSlateAttributeDescriptor::AddMemberAttribute(FName AttributeName, OffsetType Offset, FInvalidateWidgetReasonAttribute Reason)
{
	check(!AttributeName.IsNone());

	int32 NewIndex = INDEX_NONE;
	FAttribute const* FoundAttribute = FindAttribute(AttributeName);
	if (ensureAlwaysMsgf(FoundAttribute == nullptr, TEXT("The attribute '%s' already exist. (Do you have the correct parent class in SLATE_DECLARE_WIDGET)"), *AttributeName.ToString()))
	{
		NewIndex = Attributes.Emplace(AttributeName, Offset, MoveTemp(Reason));
	}
	return FInitializer::FAttributeEntry(*this, NewIndex);
}


FSlateAttributeDescriptor::FContainerInitializer::FAttributeEntry FSlateAttributeDescriptor::AddContainedAttribute(FName ContainerName, FName AttributeName, OffsetType Offset, FInvalidateWidgetReasonAttribute Reason)
{
	check(!AttributeName.IsNone());
	check(!ContainerName.IsNone());

	int32 NewIndex = INDEX_NONE;
	FAttribute const* FoundAttribute = FindAttribute(AttributeName);
	if (ensureAlwaysMsgf(FoundAttribute == nullptr, TEXT("The attribute '%s' already exist. (Do you have the correct parent class in SLATE_DECLARE_WIDGET)"), *AttributeName.ToString()))
	{
		NewIndex = Attributes.Emplace(ContainerName, AttributeName, Offset, MoveTemp(Reason));
	}
	return FContainerInitializer::FAttributeEntry(*this, ContainerName, NewIndex);
}


void FSlateAttributeDescriptor::OverrideInvalidationReason(FName ContainerName, FName AttributeName, FInvalidateWidgetReasonAttribute Reason)
{
	check(!AttributeName.IsNone());

	FAttribute* FoundAttribute = FindAttribute(AttributeName);
	if (ensureAlwaysMsgf(FoundAttribute != nullptr, TEXT("The attribute '%s' doesn't exist."), *AttributeName.ToString()))
	{
		if (ensureAlwaysMsgf(FoundAttribute->ContainerName == ContainerName, TEXT("You are not allowed to modify an attribute from a different container.")))
		{
			FoundAttribute->InvalidationReason = MoveTemp(Reason);
		}
	}
}


void FSlateAttributeDescriptor::OverrideOnValueChanged(FName ContainerName, FName AttributeName, ECallbackOverrideType OverrideType, FAttributeValueChangedDelegate Callback)
{
	check(!AttributeName.IsNone());

	FAttribute* FoundAttribute = FindAttribute(AttributeName);
	if (ensureAlwaysMsgf(FoundAttribute != nullptr, TEXT("The attribute '%s' doesn't exist."), *AttributeName.ToString()))
	{
		if (ensureAlwaysMsgf(FoundAttribute->ContainerName == ContainerName, TEXT("You are not allowed to modify an attribute from a different container.")))
		{
			switch(OverrideType)
			{
			case ECallbackOverrideType::ReplacePrevious:
				FoundAttribute->OnValueChanged = MoveTemp(Callback);
				break;
			case ECallbackOverrideType::ExecuteAfterPrevious:
			case ECallbackOverrideType::ExecuteBeforePrevious:
				if (FoundAttribute->OnValueChanged.IsBound() && Callback.IsBound())
				{
					FAttributeValueChangedDelegate Previous = FoundAttribute->OnValueChanged;
					FoundAttribute->OnValueChanged = FAttributeValueChangedDelegate::CreateLambda([Previous{MoveTemp(Previous)}, Callback{MoveTemp(Callback)}, OverrideType](SWidget& Widget)
						{
							if (OverrideType == ECallbackOverrideType::ExecuteBeforePrevious)
							{
								Previous.ExecuteIfBound(Widget);
							}
							Callback.ExecuteIfBound(Widget);
							if (OverrideType == ECallbackOverrideType::ExecuteAfterPrevious)
							{
								Previous.ExecuteIfBound(Widget);
							}
						});
				}
				else if (Callback.IsBound())
				{
					FoundAttribute->OnValueChanged = MoveTemp(Callback);
				}
				break;
			default:
				check(false);
			}
		}
	}
}


void FSlateAttributeDescriptor::SetPrerequisite(FName ContainerName, FSlateAttributeDescriptor::FAttribute& Attribute, FName Prerequisite)
{
	if (Prerequisite.IsNone())
	{
		Attribute.Prerequisite = FName();
	}
	else
	{
		const FAttribute* FoundPrerequisite = FindAttribute(Prerequisite);
		if (ensureAlwaysMsgf(FoundPrerequisite, TEXT("The prerequisite '%s' doesn't exist for attribute '%s'"), *Prerequisite.ToString(), *Attribute.Name.ToString()))
		{
			if (ensureAlwaysMsgf(FoundPrerequisite->ContainerName == ContainerName, TEXT("You are not allowed to set a prerequisite from a different container.")))
			{
				Attribute.Prerequisite = Prerequisite;

				// Verify recursion
				{
					TArray<FName, TInlineAllocator<16>> Recursion;
					Recursion.Reserve(Attributes.Num());
					FAttribute const* CurrentAttribute = &Attribute;
					while (!CurrentAttribute->Prerequisite.IsNone())
					{
						if (Recursion.Contains(CurrentAttribute->Name))
						{
							ensureAlwaysMsgf(false, TEXT("The prerequsite '%s' would introduce an infinit loop with attribute '%s'."), *Prerequisite.ToString(), *Attribute.Name.ToString());
							Attribute.Prerequisite = FName();
							break;
						}
						Recursion.Add(CurrentAttribute->Name);
						CurrentAttribute = FindAttribute(CurrentAttribute->Prerequisite);
						check(CurrentAttribute);
					}
				}
			}
		}
		else
		{
			Attribute.Prerequisite = FName();
		}
	}
}


void FSlateAttributeDescriptor::SetAffectVisibility(FAttribute& Attribute, bool bInAffectVisibility)
{
	if (Attribute.Name == "Visibility")
	{
		checkf(bInAffectVisibility, TEXT("The Visibility attribute must be marked at 'Affect Visibility'"));
	}

	if (bInAffectVisibility)
	{
		ensureAlwaysMsgf(EnumHasAnyFlags(Attribute.InvalidationReason.Reason, EInvalidateWidgetReason::Visibility) || Attribute.InvalidationReason.IsBound()
			, TEXT("The attribute '%s' affect the visibility but doesn't have a Visibility as it's InvalidateWidgetReason"), *Attribute.Name.ToString());
	}

	Attribute.bAffectVisibility = bInAffectVisibility;
}