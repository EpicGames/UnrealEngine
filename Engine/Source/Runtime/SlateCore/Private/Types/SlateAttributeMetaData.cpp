// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/SlateAttributeMetaData.h"

#include "Algo/BinarySearch.h"
#include "Layout/Children.h"
#include "Stats/Stats.h"
#include "Types/ReflectionMetadata.h"
#include "Widgets/SWidget.h"

#include <limits>

DECLARE_CYCLE_STAT(TEXT("UpdateVisibilityAttributes"), STAT_Slate_UpdateVisibilityAttributes, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("UpdateExceptVisibilityAttributes"), STAT_Slate_UpdateExceptVisibilityAttributes, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("UpdateChildrenOnlyVisibilityAttributes"), STAT_Slate_UpdateChildrenOnlyVisibilityAttributes, STATGROUP_Slate);
DECLARE_CYCLE_STAT(TEXT("UpdateAllAttributes"), STAT_Slate_UpdateAllAttributes, STATGROUP_Slate);

namespace UE
{
namespace Private
{
	FSlateAttributeDescriptor::OffsetType FindMemberOffset(const SWidget& OwningWidget, const FSlateAttributeBase& Attribute)
	{
		UPTRINT Offset = (UPTRINT)(&Attribute) - (UPTRINT)(&OwningWidget);
		ensure(Offset <= std::numeric_limits<FSlateAttributeDescriptor::OffsetType>::max());
		return (FSlateAttributeDescriptor::OffsetType)(Offset);
	}

	FSlateAttributeDescriptor::OffsetType FindContainOffset(const SlateAttributePrivate::ISlateAttributeContainer& OwningContainer, const FSlateAttributeBase& Attribute)
	{
		UPTRINT Offset = (UPTRINT)(&Attribute) - (UPTRINT)(&OwningContainer);
		ensure(Offset <= std::numeric_limits<FSlateAttributeDescriptor::OffsetType>::max());
		return (FSlateAttributeDescriptor::OffsetType)(Offset);
	}

	EInvalidateWidgetReason InvalidateForMember(SWidget& OwningWidget, const FSlateAttributeBase& Attribute, EInvalidateWidgetReason Reason)
	{
		const FSlateAttributeDescriptor::OffsetType Offset = FindMemberOffset(OwningWidget, Attribute);
		if (const FSlateAttributeDescriptor::FAttribute* FoundAttribute = OwningWidget.GetWidgetClass().GetAttributeDescriptor().FindMemberAttribute(Offset))
		{
			ensureMsgf(Reason == EInvalidateWidgetReason::None, TEXT("SWidget is using the AttributeDescriptor, the invalidation reason should only be defined in one place. See SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION"));
			FoundAttribute->ExecuteOnValueChangedIfBound(OwningWidget);
			return FoundAttribute->GetInvalidationReason(OwningWidget);
		}
		return Reason;
	}

	EInvalidateWidgetReason InvalidateForContained(const SlateAttributePrivate::ISlateAttributeContainer& OwningContainer, SWidget& OwningWidget, const FSlateAttributeBase& Attribute, EInvalidateWidgetReason Reason)
	{
		const FSlateAttributeDescriptor::OffsetType Offset = FindContainOffset(OwningContainer, Attribute);
		if (const FSlateAttributeDescriptor::FAttribute* FoundAttribute = OwningWidget.GetWidgetClass().GetAttributeDescriptor().FindContainedAttribute(OwningContainer.GetContainerName(), Offset))
		{
			ensureMsgf(Reason == EInvalidateWidgetReason::None, TEXT("SWidget is using the AttributeDescriptor, the invalidation reason should only be defined in one place. See SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION"));
			FoundAttribute->ExecuteOnValueChangedIfBound(OwningWidget);
			return FoundAttribute->GetInvalidationReason(OwningWidget);
		}
		return Reason;
	}

	int32 FindAttributeContainerOffset(const SlateAttributePrivate::ISlateAttributeContainer* Container, FSlateAttributeBase* Attribute)
	{
		PTRINT AttributePtr = (PTRINT)Attribute;
		PTRINT ContainerPtr = (PTRINT)Container;
		return AttributePtr - ContainerPtr;
	}
}
}


const SlateAttributePrivate::ISlateAttributeContainer* FSlateAttributeMetaData::FGetterItem::GetAttributeContainer() const
{
	if (AttributeContainerOffset != 0)
	{
		PTRINT AttributePtr = (PTRINT)Attribute;
		return reinterpret_cast<const SlateAttributePrivate::ISlateAttributeContainer*>(AttributePtr - AttributeContainerOffset);
	}
	return nullptr;
}


FSlateAttributeMetaData* FSlateAttributeMetaData::FindMetaData(const SWidget& OwningWidget)
{
	if (OwningWidget.HasRegisteredSlateAttribute())
	{
		checkSlow(OwningWidget.MetaData.Num() > 0);
		const TSharedRef<ISlateMetaData>& SlateMetaData = OwningWidget.MetaData[0];
		checkSlow(SlateMetaData->IsOfType<FSlateAttributeMetaData>());
		return &(static_cast<FSlateAttributeMetaData&>(SlateMetaData.Get()));
	}
#if WITH_SLATE_DEBUGGING
	else if (OwningWidget.MetaData.Num() > 0)
	{
		const TSharedRef<ISlateMetaData>& SlateMetaData = OwningWidget.MetaData[0];
		if (SlateMetaData->IsOfType<FSlateAttributeMetaData>())
		{
			ensureMsgf(false, TEXT("bHasRegisteredSlateAttribute should be set on the SWidget '%s'"), *FReflectionMetaData::GetWidgetDebugInfo(OwningWidget));
			return &(static_cast<FSlateAttributeMetaData&>(SlateMetaData.Get()));
		}
	}
#endif
	return nullptr;
}


void FSlateAttributeMetaData::RegisterAttribute(SWidget& OwningWidget, FSlateAttributeBase& Attribute, ESlateAttributeType AttributeType, TUniquePtr<ISlateAttributeGetter>&& Wrapper)
{
	auto ExecuteRegister = [&](FSlateAttributeMetaData& AttributeMetaData)
	{
		if (AttributeType == ESlateAttributeType::Member)
		{
			AttributeMetaData.RegisterMemberAttributeImpl(OwningWidget, Attribute, MoveTemp(Wrapper));
		}
		else
		{
			check(AttributeType == ESlateAttributeType::Managed);
			AttributeMetaData.RegisterManagedAttributeImpl(OwningWidget, Attribute, MoveTemp(Wrapper));
		}
	};

	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		if (int32 FoundIndex = AttributeMetaData->IndexOfAttribute(Attribute); FoundIndex != INDEX_NONE)
		{
			AttributeMetaData->Attributes[FoundIndex].Getter = MoveTemp(Wrapper);
		}
		else
		{
			ExecuteRegister(*AttributeMetaData);
		}
	}
	else
	{
		TSharedRef<FSlateAttributeMetaData> NewAttributeMetaData = MakeShared<FSlateAttributeMetaData>();
		ExecuteRegister(NewAttributeMetaData.Get());
		if (NewAttributeMetaData->GetRegisteredAttributeCount())
		{
			OwningWidget.bHasRegisteredSlateAttribute = true;
			OwningWidget.MetaData.Insert(NewAttributeMetaData, 0);
			if (OwningWidget.IsConstructed() && OwningWidget.IsAttributesUpdatesEnabled())
			{
				OwningWidget.Invalidate(EInvalidateWidgetReason::AttributeRegistration);
			}
		}
	}
}


void FSlateAttributeMetaData::RegisterAttribute(ISlateAttributeContainer& OwningContainer, FSlateAttributeBase& Attribute, ESlateAttributeType AttributeType, TUniquePtr<ISlateAttributeGetter>&& Wrapper)
{
	SWidget& OwningWidget = OwningContainer.GetContainerWidget();
	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		if (int32 FoundIndex = AttributeMetaData->IndexOfAttribute(Attribute); FoundIndex != INDEX_NONE)
		{
			AttributeMetaData->Attributes[FoundIndex].Getter = MoveTemp(Wrapper);
		}
		else
		{
			AttributeMetaData->RegisterContainAttributeImpl(OwningContainer, OwningWidget, Attribute, MoveTemp(Wrapper));
		}
	}
	else
	{
		TSharedRef<FSlateAttributeMetaData> NewAttributeMetaData = MakeShared<FSlateAttributeMetaData>();
		NewAttributeMetaData->RegisterContainAttributeImpl(OwningContainer, OwningWidget, Attribute, MoveTemp(Wrapper));
		if (NewAttributeMetaData->GetRegisteredAttributeCount())
		{
			OwningWidget.bHasRegisteredSlateAttribute = true;
			OwningWidget.MetaData.Insert(NewAttributeMetaData, 0);
			if (OwningWidget.IsConstructed() && OwningWidget.IsAttributesUpdatesEnabled())
			{
				OwningWidget.Invalidate(EInvalidateWidgetReason::AttributeRegistration);
			}
		}
	}
}


void FSlateAttributeMetaData::RegisterMemberAttributeImpl(SWidget& OwningWidget, FSlateAttributeBase& Attribute, TUniquePtr<ISlateAttributeGetter>&& Getter)
{
	// MemberAttribute are optional for now but will be needed in the future
	const FSlateAttributeDescriptor::OffsetType AttributeOffset = UE::Private::FindMemberOffset(OwningWidget, Attribute);
	const FSlateAttributeDescriptor::FAttribute* FoundMemberAttribute = OwningWidget.GetWidgetClass().GetAttributeDescriptor().FindMemberAttribute(AttributeOffset);
	if (FoundMemberAttribute)
	{
		const int32 InsertLocation = Algo::LowerBoundBy(Attributes, FoundMemberAttribute->GetSortOrder(), [](const FGetterItem& Item) { return Item.SortOrder; }, TLess<>());
		FGetterItem& GetterItem = Attributes.Insert_GetRef({ &Attribute, FoundMemberAttribute->GetSortOrder(), MoveTemp(Getter), FoundMemberAttribute }, InsertLocation);
		if (FoundMemberAttribute->DoesAffectVisibility())
		{
			++AffectVisibilityCounter;
		}
	}
	else
	{
		const uint32 SortOrder = FSlateAttributeDescriptor::DefaultSortOrder(AttributeOffset);
		const  int32 InsertLocation = Algo::LowerBoundBy(Attributes, SortOrder, [](const FGetterItem& Item) { return Item.SortOrder; }, TLess<>());
		Attributes.Insert({ &Attribute, SortOrder, MoveTemp(Getter) }, InsertLocation);
	}
}


void FSlateAttributeMetaData::RegisterManagedAttributeImpl(SWidget& OwningWidget, FSlateAttributeBase& Attribute, TUniquePtr<ISlateAttributeGetter>&& Getter)
{
	const uint32 ManagedSortOrder = std::numeric_limits<uint32>::max();
	Attributes.Emplace(&Attribute, ManagedSortOrder, MoveTemp(Getter));
}


void FSlateAttributeMetaData::RegisterContainAttributeImpl(ISlateAttributeContainer& OwningContainer, SWidget& OwningWidget, FSlateAttributeBase& Attribute, TUniquePtr<ISlateAttributeGetter>&& Getter)
{
	const FSlateAttributeDescriptor& Descriptor = OwningWidget.GetWidgetClass().GetAttributeDescriptor();
	const FName ContainerName = OwningContainer.GetContainerName();
	const FSlateAttributeDescriptor::OffsetType AttributeOffset = UE::Private::FindContainOffset(OwningContainer, Attribute);
	const FSlateAttributeDescriptor::FAttribute* FoundAttribute = Descriptor.FindContainedAttribute(ContainerName, AttributeOffset);
	ensureAlwaysMsgf(FoundAttribute, TEXT("Attribute with offset '%d' is not registered. All Slot Attributes needs to be registered."), AttributeOffset);
	if (FoundAttribute)
	{
		const FSlateAttributeDescriptor::FContainer* FoundCountainer = Descriptor.FindContainer(ContainerName);
		ensureAlwaysMsgf(FoundCountainer, TEXT("The container '%s' doesn't not exist."), *ContainerName.ToString());
		if (FoundCountainer)
		{
			FGetterItem NewItem{ &Attribute, FoundAttribute->GetSortOrder(), MoveTemp(Getter), FoundAttribute };
			NewItem.AttributeContainerOffset = UE::Private::FindAttributeContainerOffset(&OwningContainer, &Attribute);
			const int32 InsertLocation = Algo::LowerBound(Attributes, NewItem, TLess<>());
			Attributes.Insert(MoveTemp(NewItem), InsertLocation);
		}
	}
}


bool FSlateAttributeMetaData::UnregisterAttribute(SWidget& OwningWidget, const FSlateAttributeBase& Attribute)
{
	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		check(AttributeMetaData->Attributes.Num() != 0);
		const bool bResult = AttributeMetaData->UnregisterAttributeImpl(Attribute);
		AttributeMetaData->RemoveMetaDataIfNeeded(OwningWidget, bResult);
		return bResult;
	}
	return false;
}


void FSlateAttributeMetaData::RemoveMetaDataIfNeeded(SWidget& OwningWidget, bool bRemoved) const
{
	if (Attributes.Num() == 0)
	{
		check(bRemoved); // if the num is 0 then we should have remove an item.
		OwningWidget.bHasRegisteredSlateAttribute = false;
		OwningWidget.MetaData.RemoveAtSwap(0);
		if (OwningWidget.IsConstructed() && OwningWidget.IsAttributesUpdatesEnabled())
		{
			OwningWidget.Invalidate(EInvalidateWidgetReason::AttributeRegistration);
		}
	}
}


bool FSlateAttributeMetaData::UnregisterAttribute(ISlateAttributeContainer& OwningContainer, const FSlateAttributeBase& Attribute)
{
	SWidget& OwningWidget = OwningContainer.GetContainerWidget();
	return UnregisterAttribute(OwningWidget, Attribute);
}


bool FSlateAttributeMetaData::UnregisterAttributeImpl(const FSlateAttributeBase& Attribute)
{
	const int32 FoundIndex = IndexOfAttribute(Attribute);
	if (FoundIndex != INDEX_NONE)
	{
		if (Attributes[FoundIndex].CachedAttributeDescriptor && Attributes[FoundIndex].CachedAttributeDescriptor->DoesAffectVisibility())
		{
			check(AffectVisibilityCounter > 0);
			--AffectVisibilityCounter;
		}
		Attributes.RemoveAt(FoundIndex); // keep the order valid
		return true;
	}
	return false;
}


TArray<FName> FSlateAttributeMetaData::GetAttributeNames(const SWidget& OwningWidget)
{
	TArray<FName> Names;
	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		Names.Reserve(AttributeMetaData->Attributes.Num());
		for (const FGetterItem& Getter : AttributeMetaData->Attributes)
		{
			const FName Name = Getter.GetAttributeName(OwningWidget);
			if (Name.IsValid())
			{
				Names.Add(Name);
			}
		}
	}
	return Names;
}


const TCHAR* DebugSlateAttribute(const SWidget* Widget, int32 Index)
{
	if (Widget)
	{
		if (const FSlateAttributeMetaData* MetaData = FSlateAttributeMetaData::FindMetaData(*Widget))
		{
			if (MetaData->Attributes.IsValidIndex(Index))
			{
				FName AttributeName = MetaData->Attributes[Index].GetAttributeName(*Widget);

				// Hardcoded static array. This function is only used inside the debugger so it should be fine to return it.
				static TCHAR TempName[FName::StringBufferSize];
				FCString::Strcpy(TempName, *FName::SafeString(AttributeName.GetDisplayIndex(), AttributeName.GetNumber()));
				return TempName;
			}
		}
	}
	return nullptr;
}


FName FSlateAttributeMetaData::FGetterItem::GetAttributeName(const SWidget& OwningWidget) const
{
	return CachedAttributeDescriptor ? CachedAttributeDescriptor->GetName() : FName();
}


void FSlateAttributeMetaData::InvalidateWidget(SWidget& OwningWidget, const FSlateAttributeBase& Attribute, ESlateAttributeType AttributeType, EInvalidateWidgetReason Reason)
{
	// The widget is in the construction phase or is building in the WidgetList.
	//It's already invalidated... no need to keep invalidating it.
	//N.B. no needs to set the bUpatedManually in this case because
	//	1. they are in construction, so they will all be called anyway
	//	2. they are in WidgetList, so the SlateAttribute.Set will not be called
	if (!OwningWidget.IsConstructed())
	{
		return;
	}

	const FSlateAttributeDescriptor::FAttributeValueChangedDelegate* OnValueChangedCallback = nullptr;

	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		const int32 FoundIndex = AttributeMetaData->IndexOfAttribute(Attribute);
		if (FoundIndex != INDEX_NONE)
		{
			const FGetterItem& GetterItem = AttributeMetaData->Attributes[FoundIndex];
			if (GetterItem.CachedAttributeDescriptor)
			{
				ensureMsgf(Reason == EInvalidateWidgetReason::None, TEXT("SWidget is using the AttributeDescriptor, the invalidation reason should only be defined in one place. See SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION"));
				GetterItem.CachedAttributeDescriptor->ExecuteOnValueChangedIfBound(OwningWidget);
				Reason = GetterItem.CachedAttributeDescriptor->GetInvalidationReason(OwningWidget);
			}
		}
		// Not registered/bound but may be defined in the Descriptor
		else if (AttributeType == ESlateAttributeType::Member)
		{
			Reason = UE::Private::InvalidateForMember(OwningWidget, Attribute, Reason);
		}

		Reason |= AttributeMetaData->CachedInvalidationReason;
		AttributeMetaData->CachedInvalidationReason = EInvalidateWidgetReason::None;
	}
	else if (AttributeType == ESlateAttributeType::Member)
	{
		Reason = UE::Private::InvalidateForMember(OwningWidget, Attribute, Reason);
	}

#if WITH_SLATE_DEBUGGING
	ensureAlwaysMsgf(FSlateAttributeBase::IsInvalidateWidgetReasonSupported(Reason), TEXT("%s is not an EInvalidateWidgetReason supported by SlateAttribute."), *LexToString(Reason));
#endif

	OwningWidget.Invalidate(Reason);
}


void FSlateAttributeMetaData::InvalidateWidget(ISlateAttributeContainer& OwningContainer, const FSlateAttributeBase& Attribute, ESlateAttributeType AttributeType, EInvalidateWidgetReason Reason)
{
	check(AttributeType == SlateAttributePrivate::ESlateAttributeType::Contained);

	SWidget& Widget = OwningContainer.GetContainerWidget();
	if (!Widget.IsConstructed())
	{
		return;
	}

	const FSlateAttributeDescriptor::FAttributeValueChangedDelegate* OnValueChangedCallback = nullptr;

	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(Widget))
	{
		const int32 FoundIndex = AttributeMetaData->IndexOfAttribute(Attribute);
		if (FoundIndex != INDEX_NONE)
		{
			const FGetterItem& GetterItem = AttributeMetaData->Attributes[FoundIndex];
			if (GetterItem.CachedAttributeDescriptor)
			{
				ensureMsgf(Reason == EInvalidateWidgetReason::None, TEXT("SWidget is using the AttributeDescriptor, the invalidation reason should only be defined in one place. See SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION"));
				GetterItem.CachedAttributeDescriptor->ExecuteOnValueChangedIfBound(Widget);
				Reason = GetterItem.CachedAttributeDescriptor->GetInvalidationReason(Widget);
			}
			// else, it's not in the descriptor
		}
		// Not registered/bound but may be defined in the Descriptor
		else
		{
			Reason = UE::Private::InvalidateForContained(OwningContainer, Widget, Attribute, Reason);
		}

		Reason |= AttributeMetaData->CachedInvalidationReason;
		AttributeMetaData->CachedInvalidationReason = EInvalidateWidgetReason::None;
	}
	else
	{
		Reason = UE::Private::InvalidateForContained(OwningContainer, Widget, Attribute, Reason);
	}

#if WITH_SLATE_DEBUGGING
	ensureAlwaysMsgf(FSlateAttributeBase::IsInvalidateWidgetReasonSupported(Reason), TEXT("%s is not an EInvalidateWidgetReason supported by SlateAttribute."), *LexToString(Reason));
#endif

	Widget.Invalidate(Reason);
}


void FSlateAttributeMetaData::UpdateAllAttributes(SWidget& OwningWidget, EInvalidationPermission InvalidationStyle)
{
	SCOPE_CYCLE_COUNTER(STAT_Slate_UpdateAllAttributes);

	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		AttributeMetaData->UpdateAttributesImpl(OwningWidget, InvalidationStyle, 0, AttributeMetaData->Attributes.Num());
	}
}


void FSlateAttributeMetaData::UpdateOnlyVisibilityAttributes(SWidget& OwningWidget, EInvalidationPermission InvalidationStyle)
{
	SCOPE_CYCLE_COUNTER(STAT_Slate_UpdateVisibilityAttributes);

	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		if (AttributeMetaData->AffectVisibilityCounter > 0)
		{
			const int32 StartIndex = 0;
			const int32 EndIndex = AttributeMetaData->AffectVisibilityCounter;
			AttributeMetaData->UpdateAttributesImpl(OwningWidget, InvalidationStyle, StartIndex, EndIndex);
		}
	}
}


void FSlateAttributeMetaData::UpdateExceptVisibilityAttributes(SWidget& OwningWidget, EInvalidationPermission InvalidationStyle)
{
	SCOPE_CYCLE_COUNTER(STAT_Slate_UpdateExceptVisibilityAttributes);

	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		if (AttributeMetaData->AffectVisibilityCounter < AttributeMetaData->Attributes.Num())
		{
			const int32 StartIndex = AttributeMetaData->AffectVisibilityCounter;
			const int32 EndIndex = AttributeMetaData->Attributes.Num();
			AttributeMetaData->UpdateAttributesImpl(OwningWidget, InvalidationStyle, StartIndex, EndIndex);
		}
	}
}


void FSlateAttributeMetaData::UpdateChildrenOnlyVisibilityAttributes(SWidget& OwningWidget, EInvalidationPermission InvalidationStyle, bool bRecursive)
{
	SCOPE_CYCLE_COUNTER(STAT_Slate_UpdateChildrenOnlyVisibilityAttributes);

	OwningWidget.GetChildren()->ForEachWidget([InvalidationStyle, bRecursive](SWidget& Child)
		{
			UpdateOnlyVisibilityAttributes(Child, InvalidationStyle);
			if (bRecursive)
			{
				UpdateChildrenOnlyVisibilityAttributes(Child, InvalidationStyle, bRecursive);
			}
		});
}


void FSlateAttributeMetaData::ApplyDelayedInvalidation(SWidget& OwningWidget)
{
	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		if (AttributeMetaData->CachedInvalidationReason != EInvalidateWidgetReason::None)
		{
			OwningWidget.Invalidate(AttributeMetaData->CachedInvalidationReason);
		}
	}
}


void FSlateAttributeMetaData::UpdateAttributesImpl(SWidget& OwningWidget, EInvalidationPermission InvalidationStyle, int32 StartIndex, int32 IndexNum)
{
	bool bInvalidateIfNeeded = (InvalidationStyle == EInvalidationPermission::AllowInvalidation) || (InvalidationStyle == EInvalidationPermission::AllowInvalidationIfConstructed && OwningWidget.IsConstructed());
	bool bAllowInvalidation = bInvalidateIfNeeded || InvalidationStyle == EInvalidationPermission::DelayInvalidation;
	EInvalidateWidgetReason InvalidationReason = EInvalidateWidgetReason::None;
	for (int32 Index = StartIndex; Index < IndexNum; ++Index)
	{
		FGetterItem& GetterItem = Attributes[Index];

		ISlateAttributeGetter::FUpdateAttributeResult Result = GetterItem.Getter->UpdateAttribute(OwningWidget);
		if (Result.bInvalidationRequested)
		{
			if (GetterItem.CachedAttributeDescriptor)
			{
				GetterItem.CachedAttributeDescriptor->ExecuteOnValueChangedIfBound(OwningWidget);
				if (bAllowInvalidation)
				{
					InvalidationReason |= GetterItem.CachedAttributeDescriptor->GetInvalidationReason(OwningWidget);
				}
			}
			else if (bAllowInvalidation)
			{
				InvalidationReason |= Result.InvalidationReason;
			}
		}
	}

#if WITH_SLATE_DEBUGGING
	ensureAlwaysMsgf(FSlateAttributeBase::IsInvalidateWidgetReasonSupported(InvalidationReason), TEXT("'%s' is not an EInvalidateWidgetReason supported by SlateAttribute."), *LexToString(InvalidationReason));
#endif

	if (bInvalidateIfNeeded)
	{
		OwningWidget.Invalidate(InvalidationReason | CachedInvalidationReason);
		CachedInvalidationReason = EInvalidateWidgetReason::None;
	}
	else if (InvalidationStyle == EInvalidationPermission::DelayInvalidation)
	{
		CachedInvalidationReason |= InvalidationReason;
	}
	else if (InvalidationStyle == EInvalidationPermission::DenyAndClearDelayedInvalidation)
	{
		CachedInvalidationReason = EInvalidateWidgetReason::None;
	}
}


void FSlateAttributeMetaData::UpdateAttribute(SWidget& OwningWidget, FSlateAttributeBase& Attribute)
{
	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		const int32 FoundIndex = AttributeMetaData->IndexOfAttribute(Attribute);
		if (FoundIndex != INDEX_NONE)
		{
			FGetterItem& GetterItem = AttributeMetaData->Attributes[FoundIndex];
			checkSlow(GetterItem.Getter.Get());
			ISlateAttributeGetter::FUpdateAttributeResult Result = GetterItem.Getter->UpdateAttribute(OwningWidget);
			if (Result.bInvalidationRequested && OwningWidget.IsConstructed())
			{
				EInvalidateWidgetReason InvalidationReason = Result.InvalidationReason;
				if (GetterItem.CachedAttributeDescriptor)
				{
					GetterItem.CachedAttributeDescriptor->ExecuteOnValueChangedIfBound(OwningWidget);
					InvalidationReason = GetterItem.CachedAttributeDescriptor->GetInvalidationReason(OwningWidget);
				}

#if WITH_SLATE_DEBUGGING
				ensureAlwaysMsgf(FSlateAttributeBase::IsInvalidateWidgetReasonSupported(InvalidationReason), TEXT("%s is not an EInvalidateWidgetReason supported by SlateAttribute."), *LexToString(InvalidationReason));
#endif
				OwningWidget.Invalidate(InvalidationReason | AttributeMetaData->CachedInvalidationReason);
				AttributeMetaData->CachedInvalidationReason = EInvalidateWidgetReason::None;
			}
		}
	}
}


bool FSlateAttributeMetaData::IsAttributeBound(const SWidget& OwningWidget, const FSlateAttributeBase& Attribute)
{
	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		return AttributeMetaData->IndexOfAttribute(Attribute) != INDEX_NONE;
	}
	return false;
}


SlateAttributePrivate::ISlateAttributeGetter* FSlateAttributeMetaData::GetAttributeGetter(const SWidget& OwningWidget, const FSlateAttributeBase& Attribute)
{
	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		const int32 FoundIndex = AttributeMetaData->IndexOfAttribute(Attribute);
		if (FoundIndex != INDEX_NONE)
		{
			return AttributeMetaData->Attributes[FoundIndex].Getter.Get();
		}
	}
	return nullptr;
}


FDelegateHandle FSlateAttributeMetaData::GetAttributeGetterHandle(const SWidget& OwningWidget, const FSlateAttributeBase& Attribute)
{
	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		const int32 FoundIndex = AttributeMetaData->IndexOfAttribute(Attribute);
		if (FoundIndex != INDEX_NONE)
		{
			return AttributeMetaData->Attributes[FoundIndex].Getter->GetDelegateHandle();
		}
	}
	return FDelegateHandle();
}


void FSlateAttributeMetaData::MoveAttribute(const SWidget& OwningWidget, FSlateAttributeBase& NewAttribute, ESlateAttributeType AttributeType, const FSlateAttributeBase* PreviousAttribute)
{
	checkf(AttributeType == ESlateAttributeType::Managed, TEXT("TSlateAttribute cannot be moved. This should be already prevented in SlateAttribute.h"));
	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		const int32 FoundIndex = AttributeMetaData->Attributes.IndexOfByPredicate([PreviousAttribute](const FGetterItem& Item) { return Item.Attribute == PreviousAttribute; });
		if (FoundIndex != INDEX_NONE)
		{
			AttributeMetaData->Attributes[FoundIndex].Attribute = &NewAttribute;
			AttributeMetaData->Attributes[FoundIndex].Getter->SetAttribute(NewAttribute);
			//Attributes.Sort(); // Managed are always at the end and there order is not realiable.
		}
	}
}


void FSlateAttributeMetaData::RemoveContainerWidget(SWidget& OwningWidget, ISlateAttributeContainer& Container)
{
	// NB. This code can be called when we remove a FSlot or if the SWidget is destroyed.
	//If the SWidget is being destroyed, the FindMetaData return nullptr.
	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		int32 RemoveCount = AttributeMetaData->Attributes.RemoveAll([ContainerPtr = &Container](const FGetterItem& Item)
			{
				return Item.GetAttributeContainer() == ContainerPtr;
			});

		AttributeMetaData->RemoveMetaDataIfNeeded(OwningWidget, RemoveCount>0);
	}
}


void FSlateAttributeMetaData::UpdateContainerSortOrder(SWidget& OwningWidget, ISlateAttributeContainer& Container)
{
	if (FSlateAttributeMetaData* AttributeMetaData = FSlateAttributeMetaData::FindMetaData(OwningWidget))
	{
		AttributeMetaData->Attributes.Sort(TLess<>());
	}
}
