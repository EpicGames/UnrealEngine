// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/ICursor.h"
#include "Misc/Attribute.h"
#include "Types/ISlateMetaData.h"
#include "Widgets/IToolTip.h"


/** Metadata that holds the cursor content for a widget. */
class FSlateCursorMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FSlateCursorMetaData, ISlateMetaData)
	
	/** The cursor to show when the mouse is hovering over this widget. */
	TAttribute<TOptional<EMouseCursor::Type>> Cursor;
};
