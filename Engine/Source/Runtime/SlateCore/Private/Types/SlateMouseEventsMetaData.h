// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/ISlateMetaData.h"
#include "Types/WidgetMouseEventsDelegate.h"

/** Metadata to hold mouse event set on the SWidget */
class FSlateMouseEventsMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FSlateMouseEventsMetaData, ISlateMetaData)
	
	FPointerEventHandler MouseButtonDownHandle;
	FPointerEventHandler MouseButtonUpHandle;
	FPointerEventHandler MouseMoveHandle;
	FPointerEventHandler MouseDoubleClickHandle;
	FNoReplyPointerEventHandler MouseEnterHandler;
	FSimpleNoReplyPointerEventHandler MouseLeaveHandler;
};
