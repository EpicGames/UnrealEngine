// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/SlateRect.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Types/ISlateMetaData.h"

/** 
* Metadata that holds the tool tip force field expansion for a widget 
* Used to extend tooltip repel zone
*/
class FSlateToolTipForceFieldExpansionMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FSlateToolTipForceFieldExpansionMetaData, ISlateMetaData)

	TAttribute<TOptional<FSlateRect>> Attribute;
};
