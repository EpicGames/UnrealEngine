// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/ISlateMetaData.h"
#include "Widgets/IToolTip.h"


/** Metadata that holds the tool tip content for a widget. */
class FSlateToolTipMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FSlateToolTipMetaData, ISlateMetaData)
	
	TAttribute<TSharedPtr<IToolTip>> ToolTip;
};
