// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/SlateVector2.h"
#include "UObject/ObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateVector2)

namespace UE::Slate
{
	
FDeprecateSlateVectorPtrVariant::FDeprecateSlateVectorPtrVariant(FDeprecateSlateVector2D* InInstance)
	: Instance(InInstance)
{
}

FDeprecateSlateVectorPtrVariant::operator const FVector2D*() const &
{
	DoubleVector = FVector2D(Instance->X, Instance->Y);
	return &DoubleVector;
}

FDeprecateSlateVectorPtrVariant::operator const FVector2f*() const
{
	return Instance;
}

FDeprecateSlateVectorPtrVariant::operator FVector2f*()
{
	return Instance;
}

} // namespace UE::Slate

bool FDeprecateSlateVector2D::Serialize(FStructuredArchive::FSlot Slot)
{
	// Always serialize as a FVector2f
	Slot << static_cast<FVector2f&>(*this);
	return true;
}

bool FDeprecateSlateVector2D::SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.GetType().IsStruct(NAME_Vector2d))
	{
		FVector2d OldVector;
		Slot << OldVector;
		*this = UE::Slate::CastToVector2f(OldVector);
		return true;
	}
	else if (Tag.GetType().IsStruct(NAME_Vector2f))
	{
		FVector2f OldVector;
		Slot << OldVector;
		*this = OldVector;
		return true;
	}
	else if (Tag.GetType().IsStruct(NAME_Vector2D))
	{
		if (Slot.GetUnderlyingArchive().UEVer() < EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
		{
			FVector2f OldVector;
			Slot << OldVector;
			*this = OldVector;
			return true;
		}
		else
		{
			FVector2d OldVector;
			Slot << OldVector;
			*this = UE::Slate::CastToVector2f(OldVector);
			return true;
		}
	}

	return false;
}

bool FDeprecateSlateVector2D::ComponentwiseAllLessThan(const UE::Slate::FDeprecateVector2DParameter& Other) const
{
	return FVector2f::ComponentwiseAllLessThan(Other);
}

bool FDeprecateSlateVector2D::ComponentwiseAllGreaterThan(const UE::Slate::FDeprecateVector2DParameter& Other) const
{
	return FVector2f::ComponentwiseAllGreaterThan(Other);
}

bool FDeprecateSlateVector2D::ComponentwiseAllLessOrEqual(const UE::Slate::FDeprecateVector2DParameter& Other) const
{
	return FVector2f::ComponentwiseAllLessOrEqual(Other);
}

bool FDeprecateSlateVector2D::ComponentwiseAllGreaterOrEqual(const UE::Slate::FDeprecateVector2DParameter& Other) const
{
	return FVector2f::ComponentwiseAllGreaterOrEqual(Other);
}
