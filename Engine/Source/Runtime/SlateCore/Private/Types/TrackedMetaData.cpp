// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/TrackedMetaData.h"
#include "Widgets/Accessibility/SlateWidgetTracker.h"

FTrackedMetaData::FTrackedMetaData(const SWidget* InTrackedWidget, TArray<FName>&& InTags)
{
	if (FSlateWidgetTracker::Get().IsEnabled())
	{
		if (ensure(InTrackedWidget != nullptr && InTags.Num() > 0))
		{
			TrackedWidget = InTrackedWidget;
			Tags = MoveTemp(InTags);
			FSlateWidgetTracker::Get().AddTrackedWidget(TrackedWidget, Tags);
		}
	}
}

FTrackedMetaData::FTrackedMetaData(const SWidget* InTrackedWidget, FName InTags)
{
	if (FSlateWidgetTracker::Get().IsEnabled())
	{
		if (ensure(InTrackedWidget != nullptr && !InTags.IsNone()))
		{
			TrackedWidget = InTrackedWidget;
			Tags.Emplace(MoveTemp(InTags));
			FSlateWidgetTracker::Get().AddTrackedWidget(TrackedWidget, Tags);
		}
	}
}

FTrackedMetaData::~FTrackedMetaData()
{
	if (FSlateWidgetTracker::Get().IsEnabled())
	{
		FSlateWidgetTracker::Get().RemoveTrackedWidget(TrackedWidget);
	}
}
