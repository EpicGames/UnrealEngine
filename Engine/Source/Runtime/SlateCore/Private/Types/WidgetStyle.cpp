// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/WidgetStyle.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"


/* Static initialization
 *****************************************************************************/

const float FWidgetStyle::SubdueAmount = 0.6f;


/* FWidgetStyle interface
 *****************************************************************************/

FWidgetStyle& FWidgetStyle::SetForegroundColor( const TAttribute<struct FSlateColor>& InForeground )
{
	ForegroundColor = InForeground.Get().GetColor(*this);

	SubduedForeground = ForegroundColor;
	SubduedForeground.A *= SubdueAmount;

	return *this;
}
