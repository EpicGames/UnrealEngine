// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY

#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#include "Widgets/Accessibility/SlateAccessibleWidgetCache.h"
#include "Application/SlateApplicationBase.h"
#include "Application/SlateWindowHelper.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Input/HittestGrid.h"
#include "HAL/IConsoleManager.h"
#include "GenericPlatform/GenericPlatformProcess.h"


DECLARE_CYCLE_STAT(TEXT("Slate Accessibility: Tick"), STAT_AccessibilitySlateTick, STATGROUP_Accessibility);
DECLARE_CYCLE_STAT(TEXT("Slate Accessibility: Event Raised"), STAT_AccessibilitySlateEventRaised, STATGROUP_Accessibility);
DECLARE_CYCLE_STAT(TEXT("Slate Accessibility: Run In Thread"), STAT_AccessibilitySlateRunInThread, STATGROUP_Accessibility);
DECLARE_CYCLE_STAT(TEXT("Slate Accessibility: Enqueue Accessible Task"), STAT_AccessibilitySlateEnqueueAccessibleTask, STATGROUP_Accessibility);
DECLARE_CYCLE_STAT(TEXT("Slate Accessibility: Process Accessible Tasks"), STAT_AccessibilitySlateProcessAccessibleTasks, STATGROUP_Accessibility);

FSlateAccessibleMessageHandler::FSlateAccessibleMessageHandler()
	: FGenericAccessibleMessageHandler()
#if ACCESSIBILITY_DEBUG_RESPONSIVENESS
	, EnqueueEvent(FGenericPlatformProcess::GetSynchEventFromPool(false))
#endif
	, bDirty(false)
{
	bApplicationIsAccessible = true;
}

void FSlateAccessibleMessageHandler::OnActivate()
{
	bDirty = true;
}

void FSlateAccessibleMessageHandler::OnDeactivate()
{
	FSlateAccessibleWidgetCache::ClearAll();
}

TSharedPtr<IAccessibleWidget> FSlateAccessibleMessageHandler::GetAccessibleWindow(const TSharedRef<FGenericWindow>& InWindow) const
{
	if (IsActive())
	{
		TSharedPtr<SWindow> SlateWindow = FSlateWindowHelper::FindWindowByPlatformWindow(FSlateApplicationBase::Get().GetTopLevelWindows(), InWindow);
		return FSlateAccessibleWidgetCache::GetAccessibleWidgetChecked(SlateWindow);
	}
	return nullptr;
}

AccessibleWidgetId FSlateAccessibleMessageHandler::GetAccessibleWindowId(const TSharedRef<FGenericWindow>& InWindow) const
{
	TSharedPtr<IAccessibleWidget> AccessibleWindow = GetAccessibleWindow(InWindow);
	if (AccessibleWindow.IsValid())
	{
		return AccessibleWindow->GetId();
	}
	return IAccessibleWidget::InvalidAccessibleWidgetId;
}

TSharedPtr<IAccessibleWidget> FSlateAccessibleMessageHandler::GetAccessibleWidgetFromId(AccessibleWidgetId Id) const
{
	return FSlateAccessibleWidgetCache::GetAccessibleWidgetFromId(Id);
}

void FSlateAccessibleMessageHandler::RunInThread(const TFunction<void()>& InFunction, bool bWaitForCompletion, ENamedThreads::Type InThread)
{
	if (IsActive())
	{
		SCOPE_CYCLE_COUNTER(STAT_AccessibilitySlateRunInThread);
		// For Slate Accessible Message Handler, we HAVE to fulfill all these requests on the game thread 
		check(InThread == ENamedThreads::GameThread);
		check(InFunction);
		
		ENamedThreads::Type CurrentThreadID = FTaskGraphInterface::Get().GetCurrentThreadIfKnown();
		// If we are already on the correct thread, just execute the task 
		if (CurrentThreadID == ENamedThreads::GameThread)
		{
			InFunction();
		}
		else
		{
			// otherwise queue the event to be run in the game thread during FSlateApplication::Tick() 
			if (bWaitForCompletion)
			{
				FEvent* CompletionEvent = FGenericPlatformProcess::GetSynchEventFromPool(false);
				//we should always be able to get an event from the platform 
				check(CompletionEvent);
				EnqueueAccessibleTask(FSlateAccessibleTask(InFunction, CompletionEvent));
				CompletionEvent->Wait();
				FGenericPlatformProcess::ReturnSynchEventToPool(CompletionEvent);
			}
			else
			{
				EnqueueAccessibleTask(FSlateAccessibleTask(InFunction));
			}
		}
	}
}

void FSlateAccessibleMessageHandler::EnqueueAccessibleTask(const FSlateAccessibleTask& InAccessibleTask)
{
	SCOPE_CYCLE_COUNTER(STAT_AccessibilitySlateEnqueueAccessibleTask);
	FScopeLock Lock(&QueueCriticalSection);
	AccessibleTaskStorageQueue.Push(InAccessibleTask);
#if ACCESSIBILITY_DEBUG_RESPONSIVENESS
	EnqueueEvent->Trigger();
#endif 
}

void FSlateAccessibleMessageHandler::ProcessAccessibleTasks()
{
	if (!IsActive())
	{
		return;
	}
	SCOPE_CYCLE_COUNTER(STAT_AccessibilitySlateProcessAccessibleTasks);
#if !ACCESSIBILITY_DEBUG_RESPONSIVENESS
	// The processing queue should always be empty at this point as everything was processed last call 
	check(AccessibleTaskProcessingQueue.Num() == 0);
	// We swap the content of the queues to avoid constantly holding the lock while processing all tasks 
	{
		FScopeLock Lock(&QueueCriticalSection);
		Swap(AccessibleTaskStorageQueue, AccessibleTaskProcessingQueue);
	}
	while (AccessibleTaskProcessingQueue.Num() > 0)
	{
		FSlateAccessibleTask TaskToRun = AccessibleTaskProcessingQueue.Pop();
		TaskToRun.DoTask();
	}
#else
	// We run all tasks in a time slice to improve responsiveness 
	check(AccessibleTaskProcessingQueue.Num() == 0);
	{
		FScopeLock Lock(&QueueCriticalSection);
		Swap(AccessibleTaskStorageQueue, AccessibleTaskProcessingQueue);
	}
	if (AccessibleTaskProcessingQueue.Num() == 0)
	{
		return; 
	}
	// Change this value to alter the time slice. The longer the time slice, the better the accessibility responsiveness, but the worse the performance for everything else  
	static const double TimeSliceMilliseconds = 0.5;
	const double TimeSliceSeconds = TimeSliceMilliseconds * 0.001;
	const double StartTime = FPlatformTime::Seconds();
	double RemainingTime = TimeSliceSeconds;
	while (RemainingTime > 0)
	{
		while (AccessibleTaskProcessingQueue.Num() > 0)
		{
			FSlateAccessibleTask TaskToRun = AccessibleTaskProcessingQueue.Pop();
			TaskToRun.DoTask();
		}
		double ElapsedTime = FPlatformTime::Seconds() - StartTime;
		RemainingTime = TimeSliceSeconds - ElapsedTime;
		if (RemainingTime > 0)
		{

#if ACCESSIBILITY_DEBUG_RESPONSIVENESS
			EnqueueEvent->Wait(FTimespan::FromSeconds(RemainingTime));
#endif
			ElapsedTime = FPlatformTime::Seconds() - StartTime;
			RemainingTime = TimeSliceSeconds - ElapsedTime;
		}
	}
#endif 
}

void FSlateAccessibleMessageHandler::OnWidgetRemoved(SWidget* Widget)
{
	if (IsActive())
	{
		TSharedPtr<FSlateAccessibleWidget> RemovedWidget = FSlateAccessibleWidgetCache::RemoveWidget(Widget);
		if (RemovedWidget.IsValid())
		{
			RaiseEvent(FAccessibleEventArgs(RemovedWidget.ToSharedRef(), EAccessibleEvent::WidgetRemoved));
			// If this ensure fails, bDirty = true must be called to ensure the tree is kept up to date.
			ensureMsgf(!Widget->GetParentWidget().IsValid(), TEXT("A widget was unexpectedly deleted before detaching from its parent."));
		}
	}
}

void FSlateAccessibleMessageHandler::HandleAccessibleWidgetFocusChangeEvent(const TSharedRef<IAccessibleWidget>& Widget, bool bIsWidgetGainingFocus, FAccessibleUserIndex UserIndex)
{
	// we should have already checked that the user index is registered before this function is called
	check(GetAccessibleUserRegistry().IsUserRegistered(UserIndex));
	if (bIsWidgetGainingFocus)
	{
		TSharedPtr<FGenericAccessibleUser> AccessibleUser = GetAccessibleUserRegistry().GetUser(UserIndex);
		// For widgets that are non-keyboard focusable but support accessibility,
		// we have to manually raise a focus change event
		// to signal the platform that the currently accessible focused widget is losing focus
		// As regular navigation focus in FSlateApplication doesn't apply to these widgets
		if (!Widget->SupportsFocus() && Widget->SupportsAccessibleFocus())
		{
			TSharedPtr<IAccessibleWidget> CurrentAccessibilityFocusedWidget = AccessibleUser->GetFocusedAccessibleWidget();
			if (CurrentAccessibilityFocusedWidget && (Widget != CurrentAccessibilityFocusedWidget))
			{
				// let the currently focused widget lose focus 
				HandleAccessibleWidgetFocusChangeEvent(CurrentAccessibilityFocusedWidget.ToSharedRef(), false, UserIndex);
			}
		}
		// Update the widget that currently has accessible focus for the user
		AccessibleUser->SetFocusedAccessibleWidget(Widget);
	}
	// if the widget is losing focus and is not focusable by keyboard/gamepad
	else if (!Widget->SupportsFocus())
	{
		// We manually raise a focus change event here   to allow modules listening to FAccessibleEventDelegate
		// to do additional processing to unfocus from the widget
		// E.g Platforms need this information to unfocus in the platform accessibility API
		RaiseEvent(FAccessibleEventArgs(Widget, EAccessibleEvent::FocusChange, true, false, UserIndex));
	}
}

void FSlateAccessibleMessageHandler::OnWidgetEventRaised(const FSlateWidgetAccessibleEventArgs& Args)
{
	// we only process events for accessible users that have been registered
	if (IsActive() && GetAccessibleUserRegistry().IsUserRegistered(Args.SlateUserIndex))
	{
		SCOPE_CYCLE_COUNTER(STAT_AccessibilitySlateEventRaised);
		// todo: not sure what to do for a case like focus changed to not-accessible widget. maybe pass through a nullptr?
		if (Args.Widget->IsAccessible())
		{
			TSharedRef<FSlateAccessibleWidget> AccessibleWidget = FSlateAccessibleWidgetCache::GetAccessibleWidget(Args.Widget);
			// Perform FSlateAccessibleMessageHandler preprocessing here  before platform processing
			// Most of the time, accessible focus is the same as Slate focus. We sync them here
			// as a focus event is raised from SlateApplication for keyboard/gamepad focusable widgets.
			// However, we can also raise a focus change event manually on non-Slate focusable widgets e.g STableRow
			// That's where accessible focus and Slate focus can diverge
			if (Args.Event == EAccessibleEvent::FocusChange)
			{
				// The old focus value is the same as the new focus value.
				// This is most likely a user error when raising the event.
				ensure(Args.OldValue.GetValue<bool>() != Args.NewValue.GetValue<bool>());
				// If the Widget is gaining focus
				if (Args.NewValue.GetValue<bool>())
				{
					HandleAccessibleWidgetFocusChangeEvent(AccessibleWidget, true, Args.SlateUserIndex);
				}
			}
			RaiseEvent(FAccessibleEventArgs(AccessibleWidget, Args.Event, Args.OldValue, Args.NewValue, Args.SlateUserIndex));
		}
	}
}

int32 GAccessibleWidgetsProcessedPerTick = 100;
FAutoConsoleVariableRef AccessibleWidgetsProcessedPerTickRef(
	TEXT("Slate.AccessibleWidgetsProcessedPerTick"),
	GAccessibleWidgetsProcessedPerTick,
	TEXT("To reduce performance spikes, generating the accessible widget tree is limited to this many widgets per tick to update.")
);

void FSlateAccessibleMessageHandler::Tick()
{
	if (IsActive())
	{
		SCOPE_CYCLE_COUNTER(STAT_AccessibilitySlateTick);
		if (bDirty && ToProcess.Num() == 0)
		{
			bDirty = false;
			// Process ALL windows, not just the top level ones. Otherwise we miss things like combo boxes.
			TArray<TSharedRef<SWindow>> SlateWindows = FSlateApplicationBase::Get().GetTopLevelWindows();
			while (SlateWindows.Num() > 0)
			{
				const TSharedRef<SWindow> CurrentWindow = SlateWindows.Pop(EAllowShrinking::No);
				ToProcess.Emplace(CurrentWindow, FSlateAccessibleWidgetCache::GetAccessibleWidget(CurrentWindow));
				SlateWindows.Append(CurrentWindow->GetChildWindows());
			}
		}

		if (ToProcess.Num() > 0)
		{
			for (int32 Counter = 0; ToProcess.Num() > 0 && Counter < GAccessibleWidgetsProcessedPerTick; ++Counter)
			{
				FWidgetAndParent WidgetAndParent = ToProcess.Pop(EAllowShrinking::No);
				if (WidgetAndParent.Widget.IsValid())
				{
					TSharedPtr<SWidget> SharedWidget = WidgetAndParent.Widget.Pin();
					if (SharedWidget->CanChildrenBeAccessible())
					{
						FChildren* SharedChildren = SharedWidget->GetChildren();
						for (int32 i = 0; i < SharedChildren->Num(); ++i)
						{
							TSharedRef<SWidget> Child = SharedChildren->GetChildAt(i);
							if (Child->GetAccessibleBehavior() != EAccessibleBehavior::NotAccessible)
							{
								TSharedRef<FSlateAccessibleWidget> AccessibleChild = FSlateAccessibleWidgetCache::GetAccessibleWidget(Child);
								AccessibleChild->SiblingIndex = WidgetAndParent.Parent->ChildrenBuffer.Num();
								AccessibleChild->UpdateParent(WidgetAndParent.Parent);
								// A separate children buffer is filled instead of the children array itself
								// so that accessibility queries still work (using the old data) while updating
								// accessible widget data.
								WidgetAndParent.Parent->ChildrenBuffer.Add(AccessibleChild);
								ToProcess.Emplace(Child, AccessibleChild);
							}
							else
							{
								// Keep a reference to the last-known accessible parent
								ToProcess.Emplace(Child, WidgetAndParent.Parent);
							}
						}
					}
				}
			}

			// Once processing is finished, update each widget's children array with its children buffer
			if (ToProcess.Num() == 0)
			{
				for (auto WidgetIterator = FSlateAccessibleWidgetCache::GetAllWidgets(); WidgetIterator; ++WidgetIterator)
				{
					TSharedRef<FSlateAccessibleWidget> AccessibleWidget = WidgetIterator.Value();
					AccessibleWidget->Children = MoveTemp(AccessibleWidget->ChildrenBuffer);
				}
			}
		}
	}
}

void FSlateAccessibleMessageHandler::MakeAccessibleAnnouncement(const FString& AnnouncementString)
{
	if(IsActive())
	{
		// Making an announcement doesn't have to be tied to any specific widget.
		// We just pass in the accessible widget for the top levle window for convenience
		TSharedPtr<SWidget> ActiveTopLevelWindow = FSlateApplicationBase::Get().GetActiveTopLevelWindow();
		if(ActiveTopLevelWindow.IsValid())
		{
			RaiseEvent(FAccessibleEventArgs(FSlateAccessibleWidgetCache::GetAccessibleWidget(ActiveTopLevelWindow.ToSharedRef()), EAccessibleEvent::Notification, FString(), AnnouncementString));
		}
	}
}

#endif
