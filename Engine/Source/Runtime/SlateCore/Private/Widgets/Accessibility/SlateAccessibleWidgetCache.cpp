// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY

#include "Widgets/Accessibility/SlateAccessibleWidgetCache.h"
#include "Application/SlateApplicationBase.h"
#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#include "Widgets/SWidget.h"
#include "HAL/IConsoleManager.h"

TMap<SWidget*, TSharedRef<FSlateAccessibleWidget>> FSlateAccessibleWidgetCache::AccessibleWidgetMap;
TMap<AccessibleWidgetId, TSharedRef<FSlateAccessibleWidget>> FSlateAccessibleWidgetCache::AccessibleIdMap;

void FSlateAccessibleWidgetCache::ClearAll()
{
	AccessibleWidgetMap.Empty();
	AccessibleIdMap.Empty();
}

TSharedPtr<FSlateAccessibleWidget> FSlateAccessibleWidgetCache::RemoveWidget(SWidget* Widget)
{
	TSharedRef<FSlateAccessibleWidget>* AccessibleWidgetPtr = AccessibleWidgetMap.Find(Widget);
	if (AccessibleWidgetPtr)
	{
		TSharedRef<FSlateAccessibleWidget> AccessibleWidget = *AccessibleWidgetPtr;
		AccessibleWidgetMap.Remove(Widget);
		AccessibleIdMap.Remove(AccessibleWidget->GetId());
		return AccessibleWidget;
	}
	return nullptr;
}

TSharedPtr<FSlateAccessibleWidget> FSlateAccessibleWidgetCache::GetAccessibleWidgetChecked(const TSharedPtr<SWidget>& Widget)
{
	if (!Widget.IsValid() || !Widget->IsAccessible())
	{
		return nullptr;
	}

	return GetAccessibleWidget(Widget.ToSharedRef());
}

TSharedRef<FSlateAccessibleWidget> FSlateAccessibleWidgetCache::GetAccessibleWidget(const TSharedRef<SWidget>& Widget)
{
	TSharedRef<FSlateAccessibleWidget>* AccessibleWidget = AccessibleWidgetMap.Find(&Widget.Get());
	if (AccessibleWidget)
	{
		return *AccessibleWidget;
	}
	else
	{
		TSharedRef<FSlateAccessibleWidget> NewWidget = AccessibleWidgetMap.Add(&Widget.Get(), Widget->CreateAccessibleWidget());
		AccessibleIdMap.Add(NewWidget->GetId(), NewWidget);
		return NewWidget;
	}
}

TSharedPtr<FSlateAccessibleWidget> FSlateAccessibleWidgetCache::GetAccessibleWidgetFromId(AccessibleWidgetId Id)
{
	TSharedRef<FSlateAccessibleWidget>* AccessibleWidget = AccessibleIdMap.Find(Id);
	if (AccessibleWidget)
	{
		return *AccessibleWidget;
	}
	return nullptr;
}

TMap<SWidget*, TSharedRef<FSlateAccessibleWidget>>::TConstIterator FSlateAccessibleWidgetCache::GetAllWidgets()
{
	return AccessibleWidgetMap.CreateConstIterator();
}

#if !UE_BUILD_SHIPPING
void FSlateAccessibleWidgetCache::DumpAccessibilityStats()
{
	const int32 NumWidgets = AccessibleWidgetMap.Num();
	const int32 NumIds = AccessibleIdMap.Num();
	const uint32 SizeOfWidget = sizeof(FSlateAccessibleWidget);
	const uint32 SizeOfWidgetMap = AccessibleWidgetMap.GetAllocatedSize();
	const uint32 SizeOfIdMap = AccessibleIdMap.GetAllocatedSize();
	const uint32 CacheSize = NumWidgets * SizeOfWidget + SizeOfWidgetMap + SizeOfIdMap;

	UE_LOGF(LogAccessibility, Log, "Dumping Slate accessibility stats:");
	UE_LOGF(LogAccessibility, Log, "Number of cached widgets and Ids: %i, %i", NumWidgets, NumIds);
	UE_LOGF(LogAccessibility, Log, "Size of FSlateAccessibleWidget: %u", SizeOfWidget);
	UE_LOGF(LogAccessibility, Log, "Size of id map: %u", SizeOfIdMap);
	UE_LOGF(LogAccessibility, Log, "Approximate memory stored in cache: %u kb", CacheSize / 1000);
}

static FAutoConsoleCommand DumpAccessibilityStatsSlateCommand
(
	TEXT("Accessibility.DumpStatsSlate"),
	TEXT("Writes memory stats for Slate's accessibility data stored to LogAccessibility."),
	FConsoleCommandDelegate::CreateStatic(&FSlateAccessibleWidgetCache::DumpAccessibilityStats)
);
#endif

#endif
