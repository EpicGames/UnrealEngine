// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_ACCESSIBILITY

#include "Widgets/Accessibility/SlateCoreAccessibleWidgets.h"
#include "Widgets/Accessibility/SlateAccessibleWidgetCache.h"
#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#include "Layout/WidgetPath.h"
#include "Widgets/IToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Input/HittestGrid.h"
#include "Application/SlateApplicationBase.h"
#include "Application/SlateWindowHelper.h"
#include "Math/NumericLimits.h"

DECLARE_CYCLE_STAT(TEXT("Slate Accessibility: Get Widget At Point"), STAT_AccessibilitySlateGetChildAtPosition, STATGROUP_Accessibility);

FSlateAccessibleWidget::FSlateAccessibleWidget(TWeakPtr<SWidget> InWidget, EAccessibleWidgetType InWidgetType)
	: Widget(InWidget)
	, WidgetType(InWidgetType)
	, SiblingIndex(INDEX_NONE)
{
	static AccessibleWidgetId RuntimeIdCounter = 0;
	if (RuntimeIdCounter == TNumericLimits<AccessibleWidgetId>::Max())
	{
		RuntimeIdCounter = TNumericLimits<AccessibleWidgetId>::Min();
	}
	if (RuntimeIdCounter == InvalidAccessibleWidgetId)
	{
		++RuntimeIdCounter;
	}
	Id = RuntimeIdCounter++;
}

FSlateAccessibleWidget::~FSlateAccessibleWidget()
{
}

AccessibleWidgetId FSlateAccessibleWidget::GetId() const
{
	return Id;
}

bool FSlateAccessibleWidget::IsValid() const
{
	return Widget.IsValid();
}

TSharedPtr<SWindow> FSlateAccessibleWidget::GetSlateWindow() const
{
	if (Widget.IsValid())
	{
		TSharedPtr<SWidget> WindowWidget = Widget.Pin();
		while (WindowWidget.IsValid())
		{
			if (WindowWidget->Advanced_IsWindow())
			{
				return StaticCastSharedPtr<SWindow>(WindowWidget);
			}
			WindowWidget = WindowWidget->GetParentWidget();
		}
	}
	return nullptr;
}

TSharedPtr<IAccessibleWidget> FSlateAccessibleWidget::GetWindow() const
{
	return FSlateAccessibleWidgetCache::GetAccessibleWidgetChecked(GetSlateWindow());
}

FBox2D FSlateAccessibleWidget::GetBounds() const
{
	if (Widget.IsValid())
	{
		const FGeometry& Geometry = Widget.Pin()->GetCachedGeometry();
		const FVector2f Min = Geometry.GetAbsolutePosition();
		const FVector2f Max = Min + Geometry.GetAbsoluteSize();

		return FBox2D(FVector2d(Min), FVector2d(Max));
	}
	return FBox2D();
}

FString FSlateAccessibleWidget::GetClassName() const
{
	if (Widget.IsValid())
	{
		// Note: this is technically debug code and not guaranteed to work
		return Widget.Pin()->GetTypeAsString();
	}
	return FString();
}

FString FSlateAccessibleWidget::GetWidgetName() const
{
	if (Widget.IsValid())
	{
		TSharedPtr<SWidget> SharedWidget = Widget.Pin();
		FText AccessibleText = SharedWidget->GetAccessibleText();
		if (AccessibleText.IsEmpty())
		{
			TSharedPtr<FTagMetaData> Tag = SharedWidget->GetMetaData<FTagMetaData>();
			if (Tag.IsValid())
			{
				return Tag->Tag.ToString();
			}
			else
			{
				return GetClassName();
			}
		}
		else
		{
			return AccessibleText.ToString();
		}
	}
	return FString();
}

FString FSlateAccessibleWidget::GetHelpText() const
{
	if (Widget.IsValid())
	{
		TSharedPtr<SWidget> SharedWidget = Widget.Pin();
		// If the accessible text is already the tooltip, don't duplicate it for the help text.
		if (SharedWidget->GetAccessibleBehavior() != EAccessibleBehavior::ToolTip)
		{
			TSharedPtr<IToolTip> ToolTip = SharedWidget->GetToolTip();
			if (ToolTip.IsValid())
			{
				return ToolTip->GetContentWidget()->GetAccessibleText().ToString();
			}
		}
	}
	return FString();
}

bool FSlateAccessibleWidget::IsEnabled() const
{
	if (Widget.IsValid())
	{
		return Widget.Pin()->IsEnabled();
	}
	return false;
}

bool FSlateAccessibleWidget::IsHidden() const
{
	if (Widget.IsValid())
	{
		return !Widget.Pin()->GetVisibility().IsVisible();
	}
	return true;
}

bool FSlateAccessibleWidget::SupportsFocus() const
{
	if (Widget.IsValid())
	{
		return Widget.Pin()->SupportsKeyboardFocus();
	}
	return false;
}

bool FSlateAccessibleWidget::SupportsAccessibleFocus() const
{
	// all widgets that support accessibility support accessible focus right now
	// This check is analogous to Widget.Pin()->IsAccessible()
	// By definition all FSlateAccessibleWidgets are accessible. So we just return true
	return true;
}

bool FSlateAccessibleWidget::CanCurrentlyAcceptAccessibleFocus() const
{
	return IsEnabled() && !IsHidden();
}

bool FSlateAccessibleWidget::HasUserFocus(const FAccessibleUserIndex UserIndex) const
{
	FGenericAccessibleUserRegistry& UserManager = FSlateApplicationBase::Get().GetAccessibleMessageHandler()->GetAccessibleUserRegistry();
	TSharedPtr<FGenericAccessibleUser> User = UserManager.GetUser(UserIndex);
	if (User)
	{
		return (User->GetFocusedAccessibleWidget()) == AsShared();
	}
	return false;
}

bool FSlateAccessibleWidget::SetUserFocus(const FAccessibleUserIndex UserIndex)
{
	// Most likely a mistake to set focus on a widget that cannot currently accept focus
	if (!CanCurrentlyAcceptAccessibleFocus())
	{
		UE_LOGF(LogAccessibility, Warning, "Accessible widget %ls cannot currently be focused. Focus not changed.", *ToString());
		return false;
	}
	if (SupportsFocus())
	{
		TSharedPtr<SWindow> WidgetWindow = GetSlateWindow();
		if (WidgetWindow.IsValid())
		{
			TArray<TSharedRef<SWindow>> WindowArray;
			WindowArray.Add(WidgetWindow.ToSharedRef());
			FWidgetPath WidgetPath;
			if (FSlateWindowHelper::FindPathToWidget(WindowArray, Widget.Pin().ToSharedRef(), WidgetPath))
			{
				FSlateApplicationBase& SlateApp = FSlateApplicationBase::Get();
				// Focus accessible events are already raised from this function call
				if (!SlateApp.SetUserFocus(UserIndex, WidgetPath, EFocusCause::SetDirectly))
				{
					// If the function returns false, it means the SWidget corresponding to the accessible widget is already focused by the application. We will need to
					// manually raise a focus change event to sync the accessible focus and Slate focus
					SlateApp.GetAccessibleMessageHandler()->OnWidgetEventRaised(FSlateAccessibleMessageHandler::FSlateWidgetAccessibleEventArgs(Widget.Pin().ToSharedRef(), EAccessibleEvent::FocusChange, false, true, UserIndex));
				}
				return true;
			}
		}
	}
	// The widget is not keyboard/gamepad focusable but supports accessible focus
	else if (SupportsAccessibleFocus())
	{
		// we manually raise an accessible focus event 
		FSlateApplicationBase::Get().GetAccessibleMessageHandler()->OnWidgetEventRaised(FSlateAccessibleMessageHandler::FSlateWidgetAccessibleEventArgs(Widget.Pin().ToSharedRef(), EAccessibleEvent::FocusChange, false, true, UserIndex));
		return true;
	}
	return false;
}

void FSlateAccessibleWidget::UpdateParent(TSharedPtr<IAccessibleWidget> NewParent)
{
	if (Parent != NewParent)
	{
		FSlateApplicationBase::Get().GetAccessibleMessageHandler()->RaiseEvent(
			FAccessibleEventArgs(AsShared(), EAccessibleEvent::ParentChanged,
			Parent.IsValid() ? Parent.Pin()->GetId() : IAccessibleWidget::InvalidAccessibleWidgetId,
			NewParent.IsValid() ? NewParent->GetId() : IAccessibleWidget::InvalidAccessibleWidgetId));
		Parent = StaticCastSharedPtr<FSlateAccessibleWidget>(NewParent);
	}
}

TSharedPtr<IAccessibleWidget> FSlateAccessibleWidget::GetParent()
{
	return Parent.Pin();
}

TSharedPtr<IAccessibleWidget> FSlateAccessibleWidget::GetNextSibling()
{
	if (Parent.IsValid())
	{
		TSharedPtr<FSlateAccessibleWidget> SharedParent = Parent.Pin();
		if (SiblingIndex >= 0 && SiblingIndex < SharedParent->Children.Num() - 1)
		{
			return SharedParent->Children[SiblingIndex + 1].Pin();
		}
	}
	return nullptr;
}

TSharedPtr<IAccessibleWidget> FSlateAccessibleWidget::GetPreviousSibling()
{
	if (Parent.IsValid())
	{
		TSharedPtr<FSlateAccessibleWidget> SharedParent = Parent.Pin();
		if (SiblingIndex >= 1 && SiblingIndex < SharedParent->Children.Num())
		{
			return SharedParent->Children[SiblingIndex - 1].Pin();
		}
	}
	return nullptr;
}

TSharedPtr<IAccessibleWidget> FSlateAccessibleWidget::GetNextWidgetInHierarchy()
{
	// 1. if the current widget has children, return the first child
	// 2. If the widget has no children, return the next sibling of the current widget.
	// 3. If the widget has no sibling, find the first ancestor with a next sibling and return that ancestor's next sibling.
	if (GetNumberOfChildren() > 0)
	{
		return GetChildAt(0);
	}
	else if (TSharedPtr<IAccessibleWidget> NextSibling = GetNextSibling())
	{
		return NextSibling;
	}
	else if (TSharedPtr<IAccessibleWidget> CurrentAncestor = GetParent())
	{
		TSharedPtr<IAccessibleWidget> FoundAncestorSibling;
		while (CurrentAncestor)
		{
			FoundAncestorSibling = CurrentAncestor->GetNextSibling();
			if (FoundAncestorSibling)
			{
				return FoundAncestorSibling;
			}
			// if we get here, the current ancestor has no next sibling, move up the tree and try again
			CurrentAncestor = CurrentAncestor->GetParent();
		}
		// if we get here, we are the right most leaf of the tree. There is no next accessible widget
	}
		return nullptr;
}

TSharedPtr<IAccessibleWidget> FSlateAccessibleWidget::GetPreviousWidgetInHierarchy()
{
	// 1. Find the previous sibling and see if it has children
	// 2. If the previous sibling has children, DFS down the right most child until you find the child that is a leaf
	// 3. If the sibling has no children, just return the previous sibling
	// 4. If there is no previous sibling, return the parent
	if (TSharedPtr<IAccessibleWidget> PreviousSibling = GetPreviousSibling())
	{
		TSharedPtr<IAccessibleWidget> ReturnWidget = PreviousSibling;
		while (ReturnWidget)
		{
			int32 ReturnWidgetChildrenCount = ReturnWidget->GetNumberOfChildren();
			if (ReturnWidgetChildrenCount > 0)
			{
				// keep taking the last child until we get to the leaf
				ReturnWidget = ReturnWidget->GetChildAt(ReturnWidgetChildrenCount - 1);
			}
			else
			{
				// we are at a leaf just return
				return ReturnWidget;
			}
		}
	}
	else
	{
		// note that it is possible for this widget to be the root of the tree and for this to return nullptr
		return GetParent();
	}
	return nullptr;
}

TSharedPtr<IAccessibleWidget> FSlateAccessibleWidget::GetChildAt(int32 Index)
{
	check(Index >= 0 && Index < Children.Num());
	if (Widget.IsValid())
	{
		return Children[Index].Pin();
	}
	return nullptr;
}

int32 FSlateAccessibleWidget::GetNumberOfChildren()
{
	if (Widget.IsValid())
	{
		return Children.Num();
	}
	return 0;
}

// SWindow
TSharedPtr<FGenericWindow> FSlateAccessibleWindow::GetNativeWindow() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SWindow>(Widget.Pin())->GetNativeWindow();
	}
	return nullptr;
}

TSharedPtr<IAccessibleWidget> FSlateAccessibleWindow::GetChildAtPosition(int32 X, int32 Y)
{
	TSharedPtr<IAccessibleWidget> HitWidget;
	if (Widget.IsValid())
	{
		static const bool UseHitTestGrid = false;

		SCOPE_CYCLE_COUNTER(STAT_AccessibilitySlateGetChildAtPosition);
		if (UseHitTestGrid)
		{
			TSharedPtr<SWindow> SlateWindow = StaticCastSharedPtr<SWindow>(Widget.Pin());
			TArray<FWidgetAndPointer> Hits = SlateWindow->GetHittestGrid().GetBubblePath(FVector2f((float)X, (float)Y), 0.0f, false, INDEX_NONE);
			TSharedPtr<SWidget> LastAccessibleWidget = nullptr;
			for (int32 i = 0; i < Hits.Num(); ++i)
			{
				if (Hits[i].Widget->GetAccessibleBehavior() != EAccessibleBehavior::NotAccessible)
				{
					LastAccessibleWidget = Hits[i].Widget;
				}
				if (!Hits[i].Widget->CanChildrenBeAccessible())
				{
					break;
				}
			}
			if (LastAccessibleWidget.IsValid())
			{
				HitWidget = FSlateAccessibleWidgetCache::GetAccessibleWidget(LastAccessibleWidget.ToSharedRef());
			}
		}
		else
		{
			TArray<TSharedPtr<IAccessibleWidget>> ToProcess;
			ToProcess.Reserve(10);
			ToProcess.Add(AsShared());

			while (ToProcess.Num() > 0)
			{
				const TSharedPtr<IAccessibleWidget> Current = ToProcess.Pop(EAllowShrinking::No);
				// Because children are weak pointers, Current could be invalid in the case where the SWidget and all
				// shared pointers were deleted while in the middle of FSlateAccessibleMessageHandler refreshing the data.
				if (Current.IsValid() && !Current->IsHidden() && Current->GetBounds().IsInside(FVector2D((float)X, (float)Y)))
				{
					// The widgets are being traversed in reverse render order, so usually if a widget is rendered
					// on top of another this will return the rendered one. But it's not 100% guarantee, and opacity
					// screws things up sometimes. ToProcess can safely be reset because once we go down a branch
					// we no longer care about any other branches.
					ToProcess.Reset();
					HitWidget = Current;
					const int32 NumChildren = Current->GetNumberOfChildren();
					for (int32 i = 0; i < NumChildren; ++i)
					{
						ToProcess.Add(Current->GetChildAt(i));
					}
				}
			}
		}
	}

	return HitWidget;
}

TSharedPtr<IAccessibleWidget> FSlateAccessibleWindow::GetUserFocusedWidget(const FAccessibleUserIndex UserIndex) const
{
	TSharedPtr<FGenericAccessibleUser> User = FSlateApplicationBase::Get().GetAccessibleMessageHandler()->GetAccessibleUserRegistry().GetUser(UserIndex);
	return User ? User->GetFocusedAccessibleWidget() : nullptr;
}

FString FSlateAccessibleWindow::GetWidgetName() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SWindow>(Widget.Pin())->GetTitle().ToString();
	}
	else
	{
		return FSlateAccessibleWidget::GetWidgetName();
	}
}

void FSlateAccessibleWindow::Close()
{
	if (Widget.IsValid())
	{
		StaticCastSharedPtr<SWindow>(Widget.Pin())->RequestDestroyWindow();
	}
}

bool FSlateAccessibleWindow::SupportsDisplayState(EWindowDisplayState State) const
{
	if (Widget.IsValid())
	{
		switch (State)
		{
		case IAccessibleWindow::EWindowDisplayState::Normal:
			return true;
		case IAccessibleWindow::EWindowDisplayState::Minimize:
			return StaticCastSharedPtr<SWindow>(Widget.Pin())->HasMinimizeBox();
		case IAccessibleWindow::EWindowDisplayState::Maximize:
			return StaticCastSharedPtr<SWindow>(Widget.Pin())->HasMaximizeBox();
		}
	}
	return false;
}

IAccessibleWindow::EWindowDisplayState FSlateAccessibleWindow::GetDisplayState() const
{
	if (Widget.IsValid())
	{
		TSharedPtr<SWindow> Window = StaticCastSharedPtr<SWindow>(Widget.Pin());
		if (Window->IsWindowMaximized())
		{
			return IAccessibleWindow::EWindowDisplayState::Maximize;
		}
		else if (Window->IsWindowMinimized())
		{
			return IAccessibleWindow::EWindowDisplayState::Minimize;
		}
		else
		{
			return IAccessibleWindow::EWindowDisplayState::Normal;
		}
	}
	return IAccessibleWindow::EWindowDisplayState::Normal;
}

void FSlateAccessibleWindow::SetDisplayState(EWindowDisplayState State)
{
	if (Widget.IsValid() && GetDisplayState() != State)
	{
		switch (State)
		{
		case IAccessibleWindow::EWindowDisplayState::Normal:
			StaticCastSharedPtr<SWindow>(Widget.Pin())->Restore();
			break;
		case IAccessibleWindow::EWindowDisplayState::Minimize:
			StaticCastSharedPtr<SWindow>(Widget.Pin())->Minimize();
			break;
		case IAccessibleWindow::EWindowDisplayState::Maximize:
			StaticCastSharedPtr<SWindow>(Widget.Pin())->Maximize();
			break;
		}
	}
}

bool FSlateAccessibleWindow::IsModal() const
{
	if (Widget.IsValid())
	{
		return StaticCastSharedPtr<SWindow>(Widget.Pin())->IsModalWindow();
	}
	return false;
}

// ~

// SImage
FString FSlateAccessibleImage::GetHelpText() const
{
	// todo: See UIA_HelpTextPropertyId on https://docs.microsoft.com/en-us/windows/desktop/winauto/uiauto-supportimagecontroltype
	return FString();
}
// ~

#endif
