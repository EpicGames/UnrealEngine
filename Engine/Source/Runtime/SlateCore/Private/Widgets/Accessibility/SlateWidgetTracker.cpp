// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Accessibility/SlateWidgetTracker.h"
#include "Types/ISlateMetaData.h"
#include "HAL/IConsoleManager.h"
#include "Widgets/SWidget.h"
#include "Misc/MemStack.h"

static TAutoConsoleVariable<int32> CVarEnableSlateWidgetTracker(
	TEXT("Slate.EnableSlateWidgetTracker"),
	0,
	TEXT("Whether or not we enable the tracking of widgets via the Slate Widget Tracker."),
	ECVF_Default);

FSlateWidgetTracker& FSlateWidgetTracker::Get()
{
	static FSlateWidgetTracker Singleton;
	return Singleton;
}

bool FSlateWidgetTracker::IsEnabled() const
{
	return CVarEnableSlateWidgetTracker->GetInt() == 1;
}

void FSlateWidgetTracker::AddTrackedWidget(const SWidget* WidgetToTrack, const TArray<FName>& Tags)
{
	if (IsEnabled() && WidgetToTrack != nullptr)
	{
		for (const FName& Tag : Tags)
		{
			AddTrackedWidget(WidgetToTrack, Tag);
		}
	}
}

void FSlateWidgetTracker::AddTrackedWidget(const SWidget* WidgetToTrack, FName Tag)
{
	if (IsEnabled() && WidgetToTrack != nullptr)
	{
		TrackedWidgets.FindOrAdd(Tag).Add(WidgetToTrack);
		if (FTrackedWidgetsChangedEvent* TrackedWidgetsChangedEvent = TrackedWidgetsChangedEvents.Find(Tag))
		{
			TrackedWidgetsChangedEvent->Broadcast(WidgetToTrack, Tag, ETrackedSlateWidgetOperations::AddedTrackedWidget);
		}
	}
}

void FSlateWidgetTracker::RemoveTrackedWidget(const SWidget* WidgetToStopTracking)
{
	if (IsEnabled() && WidgetToStopTracking != nullptr)
	{
		TArray<FName, FConcurrentLinearArrayAllocator> AllTags;
		TrackedWidgets.GenerateKeyArray(AllTags);
		for (const FName& Tag : AllTags)
		{
			if (TArray<const SWidget*>* TrackedWidgetsOfTag = TrackedWidgets.Find(Tag))
			{
				TrackedWidgetsOfTag->Remove(WidgetToStopTracking);

				if (TrackedWidgetsOfTag->Num() == 0)
				{
					TrackedWidgets.Remove(Tag);
				}

				if (FTrackedWidgetsChangedEvent* TrackedWidgetsChangedEvent = TrackedWidgetsChangedEvents.Find(Tag))
				{
					TrackedWidgetsChangedEvent->Broadcast(WidgetToStopTracking, Tag, ETrackedSlateWidgetOperations::RemovedTrackedWidget);
				}
			}
		}
	}
}

FTrackedWidgetsChangedEvent& FSlateWidgetTracker::OnTrackedWidgetsChanged(const FName& Tag)
{
	return TrackedWidgetsChangedEvents.FindOrAdd(Tag);
}
