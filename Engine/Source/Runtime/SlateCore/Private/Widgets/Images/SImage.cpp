// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Images/SImage.h"
#include "Rendering/DrawElements.h"
#include "Widgets/IToolTip.h"
#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateCoreAccessibleWidgets.h"
#endif
#include "Debugging/SlateCrashReporterHandler.h"

SLATE_IMPLEMENT_WIDGET(SImage)
void SImage::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "Image", ImageAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ColorAndOpacity", ColorAndOpacityAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "DesiredSizeOverride", DesiredSizeOverrideAttribute, EInvalidateWidgetReason::Layout);
}

SImage::SImage()
	: ImageAttribute(*this)
	, ColorAndOpacityAttribute(*this)
	, DesiredSizeOverrideAttribute(*this)
{
	SetCanTick(false);
	bCanSupportFocus = false;
}

SImage::~SImage() = default;

void SImage::Construct( const FArguments& InArgs )
{
	ImageAttribute.Assign(*this, InArgs._Image);
	ColorAndOpacityAttribute.Assign(*this, InArgs._ColorAndOpacity);
	bFlipForRightToLeftFlowDirection = InArgs._FlipForRightToLeftFlowDirection;

	DesiredSizeOverrideAttribute.Assign(*this, InArgs._DesiredSizeOverride);

	if (InArgs._OnMouseButtonDown.IsBound())
	{
		SetOnMouseButtonDown(InArgs._OnMouseButtonDown);
	}
}

int32 SImage::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
#if UE_WITH_SLATE_CRASHREPORTER
	UE_SLATE_CRASH_REPORTER_CONTEXT_SCOPE(*this, "ImagePaint");
#endif

	const FSlateBrush* ImageBrush = ImageAttribute.Get();

	if ((ImageBrush != nullptr) && (ImageBrush->DrawAs != ESlateBrushDrawType::NoDrawType))
	{
		const bool bIsEnabled = ShouldBeEnabled(bParentEnabled);
		const ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		const FLinearColor FinalColorAndOpacity( InWidgetStyle.GetColorAndOpacityTint() * ColorAndOpacityAttribute.Get().GetColor(InWidgetStyle) * ImageBrush->GetTint( InWidgetStyle ) );

		if (bFlipForRightToLeftFlowDirection && GSlateFlowDirection == EFlowDirection::RightToLeft)
		{
			const FGeometry FlippedGeometry = AllottedGeometry.MakeChild(FSlateRenderTransform(FScale2D(-1, 1)));
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId, FlippedGeometry.ToPaintGeometry(), ImageBrush, DrawEffects, FinalColorAndOpacity);
		}
		else
		{
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), ImageBrush, DrawEffects, FinalColorAndOpacity);
		}
	}

	return LayerId;
}

FVector2D SImage::ComputeDesiredSize( float ) const
{
	const FSlateBrush* ImageBrush = ImageAttribute.Get();
	if (ImageBrush != nullptr)
	{
		const TOptional<FVector2D>& CurrentSizeOverride = DesiredSizeOverrideAttribute.Get();

		return CurrentSizeOverride.IsSet() ? CurrentSizeOverride.GetValue() : FVector2D(ImageBrush->ImageSize.X, ImageBrush->ImageSize.Y);
	}
	return FVector2D::ZeroVector;
}

void SImage::SetColorAndOpacity(TAttribute<FSlateColor> InColorAndOpacity )
{
	ColorAndOpacityAttribute.Assign(*this, MoveTemp(InColorAndOpacity));
}

void SImage::SetColorAndOpacity(FLinearColor InColorAndOpacity)
{
	ColorAndOpacityAttribute.Set(*this, InColorAndOpacity);
}

void SImage::SetImage(TAttribute<const FSlateBrush*> InImage)
{
	ImageAttribute.Assign(*this, MoveTemp(InImage));
}


void SImage::InvalidateImage()
{
	Invalidate(EInvalidateWidgetReason::Layout);
}

void SImage::SetDesiredSizeOverride(TAttribute<TOptional<FVector2D>> InDesiredSizeOverride)
{
	DesiredSizeOverrideAttribute.Assign(*this, MoveTemp(InDesiredSizeOverride));
}

void SImage::FlipForRightToLeftFlowDirection(bool InbFlipForRightToLeftFlowDirection)
{
	if (InbFlipForRightToLeftFlowDirection != bFlipForRightToLeftFlowDirection)
	{
		bFlipForRightToLeftFlowDirection = InbFlipForRightToLeftFlowDirection;
		Invalidate(EInvalidateWidgetReason::Paint);
	}
}

#if WITH_ACCESSIBILITY
TSharedRef<FSlateAccessibleWidget> SImage::CreateAccessibleWidget()
{
	return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleImage(SharedThis(this)));
}
#endif
