// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Images/SLayeredImage.h"
#include "Textures/SlateIcon.h"

void SLayeredImage::Construct(const FArguments& InArgs, const TArray<ImageLayer>& InLayers)
{
	SImage::Construct(InArgs);

	Layers.Reserve(InLayers.Num());

	for(const ImageLayer& Layer : InLayers)
	{
		AddLayer(Layer.Get<0>(), Layer.Get<1>());
	}
}

void SLayeredImage::Construct(const SLayeredImage::FArguments& InArgs, TArray<ImageLayer>&& InLayers)
{
	Layers.Reserve(InLayers.Num());

	for(ImageLayer& Layer : InLayers)
	{
		BrushAttributeType TmpBrush = BrushAttributeType{ AsShared(), MoveTemp(Layer.Get<0>()), nullptr };
		ColorAttributeType TmpColor = ColorAttributeType{ AsShared(), MoveTemp(Layer.Get<1>()), FLinearColor::White };
		Layers.Emplace(MoveTemp(TmpBrush), MoveTemp(TmpColor));
	}
}

void SLayeredImage::Construct(const FArguments& InArgs, TAttribute<const FSlateBrush*> Brush, TAttribute<FSlateColor> Color)
{
	SImage::Construct(InArgs);

	AddLayer(MoveTemp(Brush), MoveTemp(Color));
}

void SLayeredImage::Construct(const FArguments& InArgs, int32 NumLayers)
{
	SImage::Construct(InArgs);

	if (NumLayers > 0)
	{
		Layers.Reset(NumLayers);
		for (int32 Index = 0; Index < NumLayers; ++Index)
		{
			BrushAttributeType TmpBrush = BrushAttributeType(AsShared(), nullptr);
			ColorAttributeType TmpColor = ColorAttributeType(AsShared(), FLinearColor::White);
			Layers.Emplace(MoveTemp(TmpBrush), MoveTemp(TmpColor));
		}
	}
}

void SLayeredImage::Construct(const FArguments& InArgs, const FSlateIcon& InIcon)
{
	SImage::Construct(InArgs);

	SetFromSlateIcon(InIcon);
}

int32 SLayeredImage::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// this will draw Image[0]:
	SImage::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	const bool bIsEnabled = ShouldBeEnabled(bParentEnabled);
	const ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	
	// draw rest of the images, we reuse the LayerId because images are assumed to note overlap:
	for (const InnerImageLayerType& Layer : Layers)
	{
		const FSlateBrush* LayerImageResolved = Layer.Key.Get();
		if (LayerImageResolved && LayerImageResolved->DrawAs != ESlateBrushDrawType::NoDrawType)
		{
			const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint() * Layer.Value.Get().GetColor(InWidgetStyle) * LayerImageResolved->GetTint(InWidgetStyle));
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), LayerImageResolved, DrawEffects, FinalColorAndOpacity);
		}
	}

	return LayerId;
}

void SLayeredImage::AddLayer(TAttribute<const FSlateBrush*> Brush)
{
	BrushAttributeType TmpBrush = BrushAttributeType(AsShared(), MoveTemp(Brush), nullptr);
	ColorAttributeType TmpColor = ColorAttributeType(AsShared(), FLinearColor::White);
	Layers.Emplace(MoveTemp(TmpBrush), MoveTemp(TmpColor));

	Invalidate(EInvalidateWidgetReason::Paint);
}

void SLayeredImage::AddLayer(TAttribute<const FSlateBrush*> Brush, TAttribute<FSlateColor> Color)
{
	BrushAttributeType TmpBrush = BrushAttributeType(AsShared(), MoveTemp(Brush), nullptr);
	ColorAttributeType TmpColor = ColorAttributeType(AsShared(), MoveTemp(Color), FLinearColor::White);
	Layers.Emplace(MoveTemp(TmpBrush), MoveTemp(TmpColor));

	Invalidate(EInvalidateWidgetReason::Paint);
}

void SLayeredImage::SetFromSlateIcon(const FSlateIcon& InIcon)
{
	RemoveAllLayers();

	SetImage(InIcon.GetIcon());

	const FSlateBrush* OverlayIcon = InIcon.GetOverlayIcon();
	if (OverlayIcon)
	{
		AddLayer(OverlayIcon);
	}

	Invalidate(EInvalidateWidgetReason::Paint);
}

void SLayeredImage::RemoveAllLayers()
{
	Layers.Empty();

	Invalidate(EInvalidateWidgetReason::Paint);
}

int32 SLayeredImage::GetNumLayers() const
{
	return Layers.Num() + 1;
}

bool SLayeredImage::IsValidIndex(int32 Index) const
{
	// Index 0 is our local SImage
	return Index == 0 || Layers.IsValidIndex(Index - 1);
}

void SLayeredImage::SetLayerBrush(int32 Index, TAttribute<const FSlateBrush*> Brush)
{
	if (Index == 0)
	{
		SetImage(MoveTemp(Brush));
	}
	else if (Layers.IsValidIndex(Index - 1))
	{
		Layers[Index - 1].Key.Assign(MoveTemp(Brush));
	}
	else
	{
		// That layer doesn't exist
		return;
	}

	Invalidate(EInvalidateWidgetReason::Paint);
}

void SLayeredImage::SetLayerColor(int32 Index, TAttribute<FSlateColor> Color)
{
	if (Index == 0)
	{
		SetColorAndOpacity(MoveTemp(Color));
	}
	else if (Layers.IsValidIndex(Index - 1))
	{
		Layers[Index - 1].Value.Assign(MoveTemp(Color));
	}
	else
	{
		// That layer doesn't exist!
		return;
	}

	Invalidate(EInvalidateWidgetReason::Paint);
}

