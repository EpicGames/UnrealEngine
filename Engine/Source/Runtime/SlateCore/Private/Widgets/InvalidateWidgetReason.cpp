// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/InvalidateWidgetReason.h"
#include "Containers/StringFwd.h"

bool LexTryParseString(EInvalidateWidgetReason& OutValue, const TCHAR* Buffer)
{
	bool bResult = false;
	EInvalidateWidgetReason Value = EInvalidateWidgetReason::None;
	auto ParseResult = [&bResult, &Value](FStringView& SubString)
	{
		SubString.TrimStartAndEndInline();

		if (SubString.Equals(TEXT("All"), ESearchCase::IgnoreCase)) { Value = (EInvalidateWidgetReason)0xFF; return; }
		if (SubString.Equals(TEXT("Any"), ESearchCase::IgnoreCase)) { Value = (EInvalidateWidgetReason)0xFF; return; }

#define ENUM_CASE_FROM_STRING(Enum) if (SubString.Equals(TEXT(#Enum), ESearchCase::IgnoreCase)) { Value |= EInvalidateWidgetReason::Enum; return; }
		ENUM_CASE_FROM_STRING(None)
		ENUM_CASE_FROM_STRING(Layout)
		ENUM_CASE_FROM_STRING(Paint)
		ENUM_CASE_FROM_STRING(Volatility)
		ENUM_CASE_FROM_STRING(ChildOrder)
		ENUM_CASE_FROM_STRING(RenderTransform)
		ENUM_CASE_FROM_STRING(Visibility)
		ENUM_CASE_FROM_STRING(AttributeRegistration)
		ENUM_CASE_FROM_STRING(Prepass)
		bResult = false;
#undef ENUM_CASE_FROM_STRING
	};

	if (Buffer && *Buffer)
	{
		bResult = true;
		while (const TCHAR* At = FCString::Strchr(Buffer, TEXT('|')))
		{
			FStringView SubString{ Buffer, UE_PTRDIFF_TO_INT32(At - Buffer) };
			ParseResult(SubString);
			Buffer = At + 1;
		}
		if (*Buffer)
		{
			FStringView SubString{ Buffer };
			ParseResult(SubString);
		}
	}

	if (bResult)
	{
		OutValue = Value;
	}
	return bResult;
}

void LexFromString(EInvalidateWidgetReason& OutValue, const TCHAR* Buffer)
{
	OutValue = EInvalidateWidgetReason::None;
	LexTryParseString(OutValue, Buffer);
}

FString LexToString(EInvalidateWidgetReason InValue)
{
	if (InValue == EInvalidateWidgetReason::None)
	{
		return TEXT("None");
	}
	if (InValue == (EInvalidateWidgetReason)0xFF)
	{
		return TEXT("All");
	}

	TStringBuilder<512> Result;
	#define ENUM_CASE_TO_STRING(Enum) if (EnumHasAnyFlags(InValue, EInvalidateWidgetReason::Enum)) { if (Result.Len() != 0) { Result.AppendChar(TEXT('|')); } Result.Append(TEXT(#Enum)); }
	ENUM_CASE_TO_STRING(Layout);
	ENUM_CASE_TO_STRING(Paint);
	ENUM_CASE_TO_STRING(Volatility);
	ENUM_CASE_TO_STRING(ChildOrder);
	ENUM_CASE_TO_STRING(RenderTransform);
	ENUM_CASE_TO_STRING(Visibility);
	ENUM_CASE_TO_STRING(AttributeRegistration);
	ENUM_CASE_TO_STRING(Prepass);
	#undef ENUM_CASE_TO_STRING

	return Result.ToString();
}
