// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SBoxPanel.h"
#include "Layout/LayoutUtils.h"

SLATE_IMPLEMENT_WIDGET(SBoxPanel)
void SBoxPanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer Initializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Children);
	FSlot::RegisterAttributes(Initializer);
}

SLATE_IMPLEMENT_WIDGET(SHorizontalBox)
void SHorizontalBox::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

SLATE_IMPLEMENT_WIDGET(SVerticalBox)
void SVerticalBox::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

SLATE_IMPLEMENT_WIDGET(SStackBox)
void SStackBox::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

void SHorizontalBox::Construct( const SHorizontalBox::FArguments& InArgs )
{
	Children.Reserve(InArgs._Slots.Num());
	for (const FSlot::FSlotArguments& Arg : InArgs._Slots)
	{
		// Because we want to override the AutoWidth, the base class doesn't exactly have the same parent.
		//We are casting from parent to child to a different parent to prevent a reinterpret_cast
		const FSlotBase::FSlotArguments& ChilSlotArgument = static_cast<const FSlotBase::FSlotArguments&>(Arg);
		const SBoxPanel::FSlot::FSlotArguments& BoxSlotArgument = static_cast<const SBoxPanel::FSlot::FSlotArguments&>(ChilSlotArgument);
		// Because InArgs is const&, we need to do some hacking here. That would need to changed in the future.
		//The Slot has a unique_ptr, it cannot be copied. Anyway, previously, the Children.Add(), was wrong if we added the same slot twice.
		//Because of that, it doesn't matter if we steal the slot from the FArguments.
		Children.AddSlot(MoveTemp(const_cast<SBoxPanel::FSlot::FSlotArguments&>(BoxSlotArgument)));
	}
}

void SVerticalBox::Construct( const SVerticalBox::FArguments& InArgs )
{
	Children.Reserve(InArgs._Slots.Num());
	for (const FSlot::FSlotArguments& Arg : InArgs._Slots)
	{
		const FSlotBase::FSlotArguments& ChilSlotArgument = static_cast<const FSlotBase::FSlotArguments&>(Arg);
		const SBoxPanel::FSlot::FSlotArguments& BoxSlotArgument = static_cast<const SBoxPanel::FSlot::FSlotArguments&>(ChilSlotArgument);
		Children.AddSlot(MoveTemp(const_cast<SBoxPanel::FSlot::FSlotArguments&>(BoxSlotArgument)));
	}
}

void SStackBox::Construct(const SStackBox::FArguments& InArgs)
{
	SetOrientation(InArgs._Orientation);
	Children.Reserve(InArgs._Slots.Num());
	for (const FSlot::FSlotArguments& Arg : InArgs._Slots)
	{
		const FSlotBase::FSlotArguments& ChilSlotArgument = static_cast<const FSlotBase::FSlotArguments&>(Arg);
		const SBoxPanel::FSlot::FSlotArguments& BoxSlotArgument = static_cast<const SBoxPanel::FSlot::FSlotArguments&>(ChilSlotArgument);
		Children.AddSlot(MoveTemp(const_cast<SBoxPanel::FSlot::FSlotArguments&>(BoxSlotArgument)));
	}
}

SHorizontalBox::FSlot& SHorizontalBox::GetSlot(int32 SlotIndex)
{
	check(this->IsValidSlotIndex(SlotIndex));
	FSlotBase& BaseSlot = static_cast<FSlotBase&>(Children[SlotIndex]);
	return static_cast<SHorizontalBox::FSlot&>(BaseSlot);
}

const SHorizontalBox::FSlot& SHorizontalBox::GetSlot(int32 SlotIndex) const
{
	check(this->IsValidSlotIndex(SlotIndex));
	const FSlotBase& BaseSlot = static_cast<const FSlotBase&>(Children[SlotIndex]);
	return static_cast<const SHorizontalBox::FSlot&>(BaseSlot);
}

SVerticalBox::FSlot& SVerticalBox::GetSlot(int32 SlotIndex)
{
	check(this->IsValidSlotIndex(SlotIndex));
	FSlotBase& BaseSlot = static_cast<FSlotBase&>(Children[SlotIndex]);
	return static_cast<SVerticalBox::FSlot&>(BaseSlot);
}

const SVerticalBox::FSlot& SVerticalBox::GetSlot(int32 SlotIndex) const
{
	check(this->IsValidSlotIndex(SlotIndex));
	const FSlotBase& BaseSlot = static_cast<const FSlotBase&>(Children[SlotIndex]);
	return static_cast<const SVerticalBox::FSlot&>(BaseSlot);
}

SStackBox::FSlot& SStackBox::GetSlot(int32 SlotIndex)
{
	check(this->IsValidSlotIndex(SlotIndex));
	FSlotBase& BaseSlot = static_cast<FSlotBase&>(Children[SlotIndex]);
	return static_cast<SStackBox::FSlot&>(BaseSlot);
}

const SStackBox::FSlot& SStackBox::GetSlot(int32 SlotIndex) const
{
	check(this->IsValidSlotIndex(SlotIndex));
	const FSlotBase& BaseSlot = static_cast<const FSlotBase&>(Children[SlotIndex]);
	return static_cast<const SStackBox::FSlot&>(BaseSlot);
}

/**
 * Panels arrange their children in a space described by the AllottedGeometry parameter. The results of the arrangement
 * should be returned by appending a FArrangedWidget pair for every child widget. See StackPanel for an example
 *
 * @param AllottedGeometry    The geometry allotted for this widget by its parent.
 * @param ArrangedChildren    The array to which to add the WidgetGeometries that represent the arranged children.
 */
void SBoxPanel::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	const float Offset = 0.0f;
	const bool AllowShrink = true;

	if ( this->Orientation == EOrientation::Orient_Horizontal )
	{
		ArrangeChildrenInStack<EOrientation::Orient_Horizontal>(GSlateFlowDirection, this->Children, AllottedGeometry, ArrangedChildren, Offset, AllowShrink );
	}
	else
	{
		ArrangeChildrenInStack<EOrientation::Orient_Vertical>(GSlateFlowDirection, this->Children, AllottedGeometry, ArrangedChildren, Offset, AllowShrink );
	}
}

/**
 * Helper to ComputeDesiredSize().
 *
 * @param Orientation   Template parameters that controls the orientation in which the children are layed out
 * @param Children      The children whose size we want to assess in a horizontal or vertical arrangement.
 *
 * @return The size desired by the children given an orientation.
 */
template<EOrientation Orientation, typename SlotType>
static FVector2D ComputeDesiredSizeForBox( const TPanelChildren<SlotType>& Children )
{
	// The desired size of this panel is the total size desired by its children plus any margins specified in this panel.
	// The layout along the panel's axis is describe dy the SizeParam, while the perpendicular layout is described by the
	// alignment property.
	FVector2D MyDesiredSize(0,0);
	for( int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex )
	{
		const SlotType& CurChild = Children[ChildIndex];
		
		if ( CurChild.GetWidget()->GetVisibility() != EVisibility::Collapsed )
		{
			FVector2f CurChildDesiredSize = CurChild.GetWidget()->GetDesiredSize();

			auto ClampMinMax = [&CurChild](float& DesiredSizeToClamp)
			{
				if (const float MinSize = CurChild.GetMinSize();
					MinSize > 0)
				{
					DesiredSizeToClamp = FMath::Max(MinSize, DesiredSizeToClamp);
				}

				if (const float MaxSize = CurChild.GetMaxSize();
					MaxSize > 0)
				{
					DesiredSizeToClamp = FMath::Min(MaxSize, DesiredSizeToClamp);
				}
			};

			if (Orientation == Orient_Vertical)
			{
				// For a vertical panel, we want to find the maximum desired width (including margin).
				// That will be the desired width of the whole panel.
				MyDesiredSize.X = FMath::Max(MyDesiredSize.X, CurChildDesiredSize.X + CurChild.GetPadding().template GetTotalSpaceAlong<Orient_Horizontal>());

				// Clamp to the max size if it was specified
				float FinalChildDesiredSize = CurChildDesiredSize.Y;
				ClampMinMax(FinalChildDesiredSize);

				MyDesiredSize.Y += FinalChildDesiredSize + CurChild.GetPadding().template GetTotalSpaceAlong<Orient_Vertical>();
			}
			else
			{
				// A horizontal panel is just a sideways vertical panel: the axes are swapped.

				MyDesiredSize.Y = FMath::Max(MyDesiredSize.Y, CurChildDesiredSize.Y + CurChild.GetPadding().template GetTotalSpaceAlong<Orient_Vertical>());

				// Clamp to the max size if it was specified
				float FinalChildDesiredSize = CurChildDesiredSize.X;
				ClampMinMax(FinalChildDesiredSize);

				MyDesiredSize.X += FinalChildDesiredSize + CurChild.GetPadding().template GetTotalSpaceAlong<Orient_Horizontal>();
			}
		}
	}

	return MyDesiredSize;
}

FVector2D SBoxPanel::ComputeDesiredSize( float ) const
{
	return (Orientation == Orient_Horizontal)
		? ComputeDesiredSizeForBox<Orient_Horizontal>(this->Children)
		: ComputeDesiredSizeForBox<Orient_Vertical>(this->Children);
}

FChildren* SBoxPanel::GetChildren()
{
	return &Children;
}

int32 SBoxPanel::RemoveSlot( const TSharedRef<SWidget>& SlotWidget )
{
	return Children.Remove(SlotWidget);
}

void SBoxPanel::ClearChildren()
{
	Children.Empty();
}

void SBoxPanel::SetOrientation(EOrientation InOrientation)
{
	if (Orientation != InOrientation)
	{
		Orientation = InOrientation;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

SBoxPanel::SBoxPanel()
	: Children(this, GET_MEMBER_NAME_CHECKED(SBoxPanel, Children))
	, Orientation(EOrientation::Orient_Horizontal)
{

}

SBoxPanel::SBoxPanel( EOrientation InOrientation )
	: Children(this, GET_MEMBER_NAME_CHECKED(SBoxPanel, Children))
	, Orientation(InOrientation)
{

}

SBoxPanel::~SBoxPanel() = default;

void SDragAndDropVerticalBox::Construct(const FArguments& InArgs)
{
	SVerticalBox::Construct(SVerticalBox::FArguments());

	OnCanAcceptDropAdvanced = InArgs._OnCanAcceptDropAdvanced;
	OnAcceptDrop = InArgs._OnAcceptDrop;
	OnDragDetected_Handler = InArgs._OnDragDetected;
	OnDragEnter_Handler = InArgs._OnDragEnter;
	OnDragLeave_Handler = InArgs._OnDragLeave;
	OnDrop_Handler = InArgs._OnDrop;

	CurrentDragOperationScreenSpaceLocation = FVector2f::ZeroVector;
	CurrentDragOverSlotIndex = INDEX_NONE;
}

FReply SDragAndDropVerticalBox::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	return FReply::Unhandled();
}

FReply SDragAndDropVerticalBox::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	ArrangeChildren(MyGeometry, ArrangedChildren);

	int32 NodeUnderPositionIndex = SWidget::FindChildUnderMouse(ArrangedChildren, MouseEvent);

	if (Children.IsValidIndex(NodeUnderPositionIndex))
	{
		SVerticalBox::FSlot* Slot = static_cast<SVerticalBox::FSlot*>(&static_cast<FSlotBase&>(Children[NodeUnderPositionIndex]));
		if (OnDragDetected_Handler.IsBound())
		{
			return OnDragDetected_Handler.Execute(MyGeometry, MouseEvent, NodeUnderPositionIndex, Slot);
		}
	}

	return FReply::Unhandled();
}

void SDragAndDropVerticalBox::OnDragEnter(FGeometry const& MyGeometry, FDragDropEvent const& DragDropEvent)
{
	if (OnDragEnter_Handler.IsBound())
	{
		OnDragEnter_Handler.Execute(DragDropEvent);
	}
}

void SDragAndDropVerticalBox::OnDragLeave(FDragDropEvent const& DragDropEvent)
{
	ItemDropZone = TOptional<EItemDropZone>();
	CurrentDragOperationScreenSpaceLocation = FVector2f::ZeroVector;
	CurrentDragOverSlotIndex = INDEX_NONE;

	if (OnDragLeave_Handler.IsBound())
	{
		OnDragLeave_Handler.Execute(DragDropEvent);
	}
}

SDragAndDropVerticalBox::EItemDropZone SDragAndDropVerticalBox::ZoneFromPointerPosition(FVector2f LocalPointerPos, const FGeometry& CurrentGeometry, const FGeometry& StartGeometry) const
{
	FSlateLayoutTransform StartGeometryLayoutTransform = StartGeometry.GetAccumulatedLayoutTransform();
	FSlateLayoutTransform CurrentGeometryLayoutTransform = CurrentGeometry.GetAccumulatedLayoutTransform();

	if (StartGeometryLayoutTransform.GetTranslation().Y > CurrentGeometryLayoutTransform.GetTranslation().Y) // going up
	{
		return EItemDropZone::AboveItem;
	}
	else if (StartGeometryLayoutTransform.GetTranslation().Y < CurrentGeometryLayoutTransform.GetTranslation().Y) // going down
	{
		return EItemDropZone::BelowItem;
	}
	else
	{
		if (LocalPointerPos.Y <= CurrentGeometry.GetLocalSize().Y / 2.0f)
		{
			return EItemDropZone::AboveItem;
		}
		else
		{
			return EItemDropZone::BelowItem;
		}
	}
}

FReply SDragAndDropVerticalBox::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if (OnCanAcceptDropAdvanced.IsBound())
	{
		FArrangedChildren ArrangedChildren(EVisibility::Visible);
		ArrangeChildren(MyGeometry, ArrangedChildren);

		TSharedPtr<FDragAndDropVerticalBoxOp> DragOp = DragDropEvent.GetOperationAs<FDragAndDropVerticalBoxOp>();

		if (DragOp.IsValid() && ArrangedChildren.IsValidIndex(DragOp->SlotIndexBeingDragged))
		{
			int32 DragOverSlotIndex = SWidget::FindChildUnderPosition(ArrangedChildren, DragDropEvent.GetScreenSpacePosition());

			if (ArrangedChildren.IsValidIndex(DragOverSlotIndex))
			{
				FVector2f LocalPointerPos = ArrangedChildren[DragOverSlotIndex].Geometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
				EItemDropZone ItemHoverZone = ZoneFromPointerPosition(LocalPointerPos, ArrangedChildren[DragOverSlotIndex].Geometry, ArrangedChildren[DragOp->SlotIndexBeingDragged].Geometry);

				if (Children.IsValidIndex(DragOverSlotIndex))
				{
					SVerticalBox::FSlot* Slot = static_cast<SVerticalBox::FSlot*>(&static_cast<FSlotBase&>(Children[DragOverSlotIndex]));

					ItemDropZone = OnCanAcceptDropAdvanced.Execute(DragDropEvent, ItemHoverZone, DragOverSlotIndex, Slot);
					CurrentDragOperationScreenSpaceLocation = DragDropEvent.GetScreenSpacePosition();
					CurrentDragOverSlotIndex = DragOverSlotIndex;

					return FReply::Handled();
				}
			}
		}
	}

	return FReply::Unhandled();
}

FReply SDragAndDropVerticalBox::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	FReply DropReply = FReply::Unhandled();

	if (OnAcceptDrop.IsBound())
	{
		FArrangedChildren ArrangedChildren(EVisibility::Visible);
		ArrangeChildren(MyGeometry, ArrangedChildren);

		if (DragDropEvent.GetOperationAs<FDragAndDropVerticalBoxOp>().IsValid())
		{
			int32 NodeUnderPositionIndex = SWidget::FindChildUnderPosition(ArrangedChildren, DragDropEvent.GetScreenSpacePosition());
			if (Children.IsValidIndex(NodeUnderPositionIndex))
			{
				SVerticalBox::FSlot* Slot = static_cast<SVerticalBox::FSlot*>(&static_cast<FSlotBase&>(Children[NodeUnderPositionIndex]));
				TOptional<EItemDropZone> ReportedZone = ItemDropZone;

				if (OnCanAcceptDropAdvanced.IsBound() && ItemDropZone.IsSet())
				{
					ReportedZone = OnCanAcceptDropAdvanced.Execute(DragDropEvent, ItemDropZone.GetValue(), NodeUnderPositionIndex, Slot);
				}

				if (ReportedZone.IsSet())
				{
					DropReply = OnAcceptDrop.Execute(DragDropEvent, ReportedZone.GetValue(), NodeUnderPositionIndex, Slot);

					if (DropReply.IsEventHandled())
					{
						TSharedPtr<FDragAndDropVerticalBoxOp> DragOp = DragDropEvent.GetOperationAs<FDragAndDropVerticalBoxOp>();

						// Perform the slot changes
						Children.Move(DragOp->SlotIndexBeingDragged, NodeUnderPositionIndex);
					}
				}
			}

			ItemDropZone = TOptional<EItemDropZone>();
			CurrentDragOperationScreenSpaceLocation = FVector2f::ZeroVector;
			CurrentDragOverSlotIndex = INDEX_NONE;
		}		
	}

	return DropReply;
}

int32 SDragAndDropVerticalBox::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	ArrangeChildren(AllottedGeometry, ArrangedChildren);

	LayerId = SPanel::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	if (ItemDropZone.IsSet())
	{
		// Draw feedback for user dropping an item above, below.
		const FSlateBrush* DropIndicatorBrush = nullptr;

		switch (ItemDropZone.GetValue())
		{
			default:
			case EItemDropZone::AboveItem: DropIndicatorBrush = &DropIndicator_Above; break;
			case EItemDropZone::BelowItem: DropIndicatorBrush = &DropIndicator_Below; break;
		};

		if (ArrangedChildren.IsValidIndex(CurrentDragOverSlotIndex))
		{
			const FArrangedWidget& CurWidget = ArrangedChildren[CurrentDragOverSlotIndex];

			FSlateDrawElement::MakeBox
			(
				OutDrawElements,
				LayerId++,
				CurWidget.Geometry.ToPaintGeometry(),
				DropIndicatorBrush,
				ESlateDrawEffect::None,
				DropIndicatorBrush->GetTint(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
			);
		}
	}

	return LayerId;
}