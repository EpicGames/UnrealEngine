// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SCompoundWidget.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/LayoutUtils.h"
#include "SlateGlobals.h"

DECLARE_CYCLE_STAT(TEXT("Child Paint"), STAT_ChildPaint, STATGROUP_SlateVeryVerbose);

SLATE_IMPLEMENT_WIDGET(SCompoundWidget)
void SCompoundWidget::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "SlotPadding", ChildSlot.SlotPaddingAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ContentScale", ContentScaleAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ColorAndOpacity", ColorAndOpacityAttribute, EInvalidateWidgetReason::Paint);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ForegroundColor", ForegroundColorAttribute, EInvalidateWidgetReason::Paint);
}

int32 SCompoundWidget::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{

	// A CompoundWidget just draws its children
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	{
		this->ArrangeChildren(AllottedGeometry, ArrangedChildren);
	}

	// There may be zero elements in this array if our child collapsed/hidden
	if( ArrangedChildren.Num() > 0 )
	{
		const bool bShouldBeEnabled = ShouldBeEnabled(bParentEnabled);

		check( ArrangedChildren.Num() == 1 );
		FArrangedWidget& TheChild = ArrangedChildren[0];

		FWidgetStyle CompoundedWidgetStyle = FWidgetStyle(InWidgetStyle)
			.BlendColorAndOpacityTint(GetColorAndOpacity())
			.SetForegroundColor(bShouldBeEnabled ? GetForegroundColor() : GetDisabledForegroundColor() );

		int32 Layer = 0;
		{
#if WITH_VERY_VERBOSE_SLATE_STATS
			SCOPE_CYCLE_COUNTER(STAT_ChildPaint);
#endif
			Layer = TheChild.Widget->Paint( Args.WithNewParent(this), TheChild.Geometry, MyCullingRect, OutDrawElements, LayerId + 1, CompoundedWidgetStyle, bShouldBeEnabled);
		}
		return Layer;
	}
	return LayerId;
}

FChildren* SCompoundWidget::GetChildren()
{
	return &ChildSlot;
}


FVector2D SCompoundWidget::ComputeDesiredSize( float ) const
{
	EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
	if ( ChildVisibility != EVisibility::Collapsed )
	{
		FVector2f LocalDesiredSize = ChildSlot.GetWidget()->GetDesiredSize() + ChildSlot.GetPadding().GetDesiredSize();
		return FVector2D(LocalDesiredSize);
	}
	
	return FVector2D::ZeroVector;
}

void SCompoundWidget::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	ArrangeSingleChild(GSlateFlowDirection, AllottedGeometry, ArrangedChildren, ChildSlot, GetContentScale());
}

FSlateColor SCompoundWidget::GetForegroundColor() const
{
	return ForegroundColorAttribute.Get();
}

SCompoundWidget::SCompoundWidget()
	: ChildSlot(this)
	, ContentScaleAttribute( *this, FVector2D(1.0f,1.0f) )
	, ColorAndOpacityAttribute( *this, FLinearColor::White )
	, ForegroundColorAttribute( *this, FSlateColor::UseForeground() )
{
}

SCompoundWidget::~SCompoundWidget() = default;

void SCompoundWidget::SetVisibility( TAttribute<EVisibility> InVisibility )
{
	SWidget::SetVisibility(MoveTemp(InVisibility));
}
