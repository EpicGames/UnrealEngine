// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SLeafWidget.h"


/* SLeafWidget interface
 *****************************************************************************/

SLeafWidget::SLeafWidget()
{
	bCanHaveChildren = false;
}

SLeafWidget::~SLeafWidget() = default;

void SLeafWidget::SetVisibility( TAttribute<EVisibility> InVisibility )
{
	SWidget::SetVisibility( InVisibility );
}

FChildren* SLeafWidget::GetChildren( )
{
	return &FNoChildren::NoChildrenInstance;
}


void SLeafWidget::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	// Nothing to arrange; Leaf Widgets do not have children.
}
