// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SNullWidget.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

template <>
struct TWidgetTypeTraits<class SNullWidgetContent>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SLATECORE_API SNullWidgetContent
	: public SWidget
{
public:

	SLATE_BEGIN_ARGS(SNullWidgetContent)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
	}

	SNullWidgetContent()
	{
		SetCanTick(false);
		bCanSupportFocus = false;
		bCanHaveChildren = false;
	}

private:
	virtual void SetVisibility( TAttribute<EVisibility> InVisibility ) override final
	{
		ensureMsgf(!IsConstructed(), TEXT("Attempting to SetVisibility() on SNullWidget. Mutating SNullWidget is not allowed.") );
	}
public:
	
	// SWidget interface

	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override
	{
		return LayerId;
	}

	virtual FChildren* GetChildren( ) override final
	{
		return &FNoChildren::NoChildrenInstance;
	}

	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override final
	{
		// Nothing to arrange; Null Widgets do not have children.
	}

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	virtual bool SupportsInvalidationRecursive(EInvalidationStrategy InInvalidationStrategy) const override
	{
		return true;
	}
#endif

protected:
	virtual FVector2D ComputeDesiredSize(float) const override final
	{
		return FVector2D(0.0f, 0.0f);
	}

	virtual bool ComputeVolatility() const override final
	{
		return false;
	}

	// End of SWidget interface
};

namespace NullWidgetPrivate
{
	TSharedRef<SWidget> Construct()
	{
		static TSharedRef<SWidget> Result = SNew(SNullWidgetContent).Visibility(EVisibility::Hidden);
		return Result;
	}
}
SLATECORE_API TSharedRef<SWidget> SNullWidget::NullWidget = NullWidgetPrivate::Construct();

FNoChildren FNoChildren::NoChildrenInstance(&NullWidgetPrivate::Construct().Get());
