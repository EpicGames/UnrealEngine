// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SOverlay.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/LayoutUtils.h"
#include "Rendering/DrawElements.h"

SLATE_IMPLEMENT_WIDGET(SOverlay)
void SOverlay::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	FSlateWidgetSlotAttributeInitializer Initializer = SLATE_ADD_PANELCHILDREN_DEFINITION(AttributeInitializer, Children);
	FOverlaySlot::RegisterAttributes(Initializer);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void SOverlay::FOverlaySlot::Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
{
	ZOrder = InArgs._ZOrder.Get(ZOrder);
	TBasicLayoutWidgetSlot<FOverlaySlot>::Construct(SlotOwner, MoveTemp(InArgs));
}

void SOverlay::FOverlaySlot::SetZOrder(int32 InZOrder)
{
	if (SWidget* OwnerWidget = FSlotBase::GetOwnerWidget())
	{
		if (ZOrder != InZOrder)
		{
			TPanelChildren<FOverlaySlot>& OwnerChildren = static_cast<SOverlay*>(OwnerWidget)->Children;

			// if bellow 0 or INDEX_NONE, then move it to the end.
			if (InZOrder <= INDEX_NONE)
			{
				// We have at least one child since GetOwnerWidget returned a valid Widget (parent/child relationship).
				check(OwnerChildren.Num() != 0);

				const FOverlaySlot& LastSlot = OwnerChildren[OwnerChildren.Num() - 1];
				if (&LastSlot == this)
				{
					return; // We are already the last FSlot
				}

				InZOrder = LastSlot.GetZOrder() + 1;
				if (ZOrder == InZOrder)
				{
					return; 
				}
			}

			int32 ChildIndexToMove = 0;
			{
				bool bFound = false;
				const int32 ChildrenNum = OwnerChildren.Num();
				for (; ChildIndexToMove < ChildrenNum; ++ChildIndexToMove)
				{
					const FOverlaySlot& CurSlot = OwnerChildren[ChildIndexToMove];
					if (&CurSlot == this)
					{
						bFound = true;
						break;
					}
				}
				// This slot has to be contained by the children's owner.
				check(OwnerChildren.IsValidIndex(ChildIndexToMove) && bFound);
			}

			int32 ChildIndexDestination = 0;
			{
				const int32 ChildrenNum = OwnerChildren.Num();
				for (; ChildIndexDestination < ChildrenNum; ++ChildIndexDestination)
				{
					const FOverlaySlot& CurSlot = OwnerChildren[ChildIndexDestination];
					if (InZOrder < CurSlot.GetZOrder() && &CurSlot != this)
					{
						// Insert before
						break;
					}
				}
				if (ChildIndexDestination >= ChildrenNum)
				{
					const FOverlaySlot& CurSlot = OwnerChildren[ChildrenNum-1];
					if (&CurSlot == this)
					{
						// No need to move, it's already at the end.
						ChildIndexToMove = ChildIndexDestination;
					}
				}
			}

			ZOrder = InZOrder;

			if (ChildIndexToMove != ChildIndexDestination)
			{
				// TPanelChildren::Move does a remove before the insert, that may change the index location
				if (ChildIndexDestination > ChildIndexToMove)
				{
					--ChildIndexDestination;
				}
				OwnerChildren.Move(ChildIndexToMove, ChildIndexDestination);
			}
		}
	}
	else
	{
		ZOrder = InZOrder;
		ensureMsgf(false, TEXT("The order of the OverlaySlot could not be set because it's not added to an existing Widget."));
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

SOverlay::SOverlay()
	: Children(this, GET_MEMBER_NAME_CHECKED(SOverlay, Children))
{
	SetCanTick(false);
	bCanSupportFocus = false;
}

SOverlay::~SOverlay() = default;

SOverlay::FOverlaySlot::FSlotArguments SOverlay::Slot( int32 ZOrder )
{
	return FOverlaySlot::FSlotArguments(MakeUnique<FOverlaySlot>(ZOrder));
}

void SOverlay::Construct( const SOverlay::FArguments& InArgs )
{
	// Because InArgs is const&, we need to do some bad cast here. That would need to change in the future.
	//The Slot has a unique_ptr. It can't be copied.
	//Previously, the Children.Add(), was wrong if we added the same slot twice (it would create a lot of issues).
	//Because of that, it doesn't matter if we steal the slot from the FArguments and it enforces that a slot cannot be added twice.
	TArray<FOverlaySlot::FSlotArguments>& SlotArguments = const_cast<TArray<FOverlaySlot::FSlotArguments>&>(InArgs._Slots);
	SlotArguments.StableSort([](const FOverlaySlot::FSlotArguments& A, const FOverlaySlot::FSlotArguments& B)
	{
		return A._ZOrder.Get(0) < B._ZOrder.Get(0);
	});

	Children.AddSlots(MoveTemp(SlotArguments));
}

void SOverlay::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	for ( int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex )
	{
		const FOverlaySlot& CurChild = Children[ChildIndex];
		const EVisibility ChildVisibility = CurChild.GetWidget()->GetVisibility();
		if ( ArrangedChildren.Accepts(ChildVisibility) )
		{
			const FMargin SlotPadding(LayoutPaddingWithFlow(GSlateFlowDirection, CurChild.GetPadding()));
			AlignmentArrangeResult XResult = AlignChild<Orient_Horizontal>(GSlateFlowDirection, AllottedGeometry.GetLocalSize().X, CurChild, SlotPadding);
			AlignmentArrangeResult YResult = AlignChild<Orient_Vertical>(AllottedGeometry.GetLocalSize().Y, CurChild, SlotPadding);

			ArrangedChildren.AddWidget( ChildVisibility, AllottedGeometry.MakeChild(
				CurChild.GetWidget(),
				FVector2f(XResult.Size, YResult.Size),
				FSlateLayoutTransform(FVector2f(XResult.Offset, YResult.Offset))
			) );
		}
	}
}

FVector2D SOverlay::ComputeDesiredSize( float ) const
{
	FVector2D MaxSize(0,0);
	for ( int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex )
	{
		const FOverlaySlot& CurSlot = Children[ChildIndex];
		const EVisibility ChildVisibilty = CurSlot.GetWidget()->GetVisibility();
		if ( ChildVisibilty != EVisibility::Collapsed )
		{
			FVector2f ChildDesiredSize = CurSlot.GetWidget()->GetDesiredSize() + CurSlot.GetPadding().GetDesiredSize();
			MaxSize.X = FMath::Max( MaxSize.X, ChildDesiredSize.X );
			MaxSize.Y = FMath::Max( MaxSize.Y, ChildDesiredSize.Y );
		}
	}

	return MaxSize;
}

FChildren* SOverlay::GetChildren()
{
	return &Children;
}

int32 SOverlay::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	{
		// The box panel has no visualization of its own; it just visualizes its children.
		ArrangeChildren(AllottedGeometry, ArrangedChildren);
	}

	// Because we paint multiple children, we must track the maximum layer id that they produced in case one of our parents
	// wants to an overlay for all of its contents.
	int32 MaxLayerId = LayerId;
	FPaintArgs NewArgs = Args.WithNewParent(this);
	const bool bChildrenEnabled = ShouldBeEnabled(bParentEnabled);


	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& CurArrangedWidget = ArrangedChildren[ChildIndex];

		// We don't increment the first layer.
		if (ChildIndex > 0)
		{
			MaxLayerId++;
		}

		const int32 CurWidgetsMaxLayerId =
			CurArrangedWidget.Widget->Paint(
				NewArgs,
				CurArrangedWidget.Geometry,
				MyCullingRect,
				OutDrawElements,
				MaxLayerId,
				InWidgetStyle,
				bChildrenEnabled);
		
		// This is a hack to account for widgets incrementing their layer id inside an overlay in global invalidation mode.  
		// Overlay slots that do not update will not know about the new layer id.  This padding adds buffering to avoid that being a problem for now
		// This is a temporary solution until we build a full rendering tree
		const int32 OverlaySlotPadding = 10;
		MaxLayerId = CurWidgetsMaxLayerId + FMath::Min(FMath::Max((CurWidgetsMaxLayerId - MaxLayerId) / OverlaySlotPadding, 1) * OverlaySlotPadding,100);

		// Non padding method
		//MaxLayerId = FMath::Max(CurWidgetsMaxLayerId, MaxLayerId);
	
	}



	return MaxLayerId;
}

SOverlay::FScopedWidgetSlotArguments SOverlay::AddSlot( int32 ZOrder )
{
	int32 Index = INDEX_NONE;
	if ( ZOrder == INDEX_NONE )
	{
		// No ZOrder was specified; just add to the end of the list.
		// Use a ZOrder index one after the last elements.
		ZOrder = (Children.Num() == 0)
			? 0
			: ( Children[ Children.Num()-1 ].GetZOrder() + 1 );
	}
	else
	{
		// Figure out where to add the widget based on ZOrder
		bool bFoundSlot = false;
		int32 CurSlotIndex = 0;
		for( ; CurSlotIndex < Children.Num(); ++CurSlotIndex )
		{
			const FOverlaySlot& CurSlot = Children[ CurSlotIndex ];
			if( ZOrder < CurSlot.GetZOrder() )
			{
				// Insert before
				bFoundSlot = true;
				break;
			}
		}

		// Add a slot at the desired location
		Index = CurSlotIndex;
	}

	FScopedWidgetSlotArguments Result {MakeUnique<FOverlaySlot>(ZOrder), this->Children, Index};
	return MoveTemp(Result);
}

bool SOverlay::HasSlotWithZOrder( int32 ZOrder ) const
{
	return GetChildIndexByZOrder( ZOrder ) != INDEX_NONE;
}

int32 SOverlay::GetChildIndexByZOrder( int32 ZOrder ) const
{
	for( int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex )
	{
		if ( Children[ChildIndex].GetZOrder() == ZOrder )
		{
			return ChildIndex;
		}
	}

	return INDEX_NONE;
}

void SOverlay::RemoveSlot( int32 ZOrder )
{
	if (ZOrder != INDEX_NONE)
	{
		const int32 ChildIndex = GetChildIndexByZOrder( ZOrder );
		if (ChildIndex != INDEX_NONE)
		{
			Children.RemoveAt( ChildIndex );
			return;
		}

		ensureMsgf(false, TEXT("Could not remove slot. There are no children with ZOrder %d."), ZOrder);
	}
	else if (Children.Num() > 0)
	{
		Children.RemoveAt( Children.Num() - 1 );
	}
	else
	{
		ensureMsgf(false, TEXT("Could not remove slot. There are no slots left."));
	}
}

void SOverlay::ClearChildren()
{
	Children.Empty();
}

int32 SOverlay::GetNumWidgets() const
{
	return Children.Num();
}

bool SOverlay::RemoveSlot( TSharedRef< SWidget > SlotWidget)
{
	return Children.Remove(SlotWidget) != INDEX_NONE;
}
