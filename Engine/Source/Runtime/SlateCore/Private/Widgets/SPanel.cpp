// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SPanel.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"

SPanel::SPanel() = default;
SPanel::~SPanel() = default;

int32 SPanel::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	ArrangeChildren(AllottedGeometry, ArrangedChildren);

	return PaintArrangedChildren(Args, ArrangedChildren, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

int32 SPanel::PaintArrangedChildren( const FPaintArgs& Args, const FArrangedChildren& ArrangedChildren, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled  ) const
{
	// Because we paint multiple children, we must track the maximum layer id that they produced in case one of our parents
	// wants to an overlay for all of its contents.
	int32 MaxLayerId = LayerId;

	const FPaintArgs NewArgs = Args.WithNewParent(this);
	const bool bShouldBeEnabled = ShouldBeEnabled(bParentEnabled);

	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		const FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];

		if (!IsChildWidgetCulled(MyCullingRect, CurWidget))
		{
			const int32 CurWidgetsMaxLayerId = CurWidget.Widget->Paint(NewArgs, CurWidget.Geometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bShouldBeEnabled);
			MaxLayerId = FMath::Max(MaxLayerId, CurWidgetsMaxLayerId);
		}
		else
		{
			//SlateGI - RemoveContent
		}
	}
	
	return MaxLayerId;
}

void SPanel::SetVisibility(TAttribute<EVisibility> InVisibility)
{
	SWidget::SetVisibility(InVisibility);
}
