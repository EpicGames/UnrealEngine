// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SWidget.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/Children.h"
#include "SlateGlobals.h"
#include "Rendering/DrawElements.h"
#include "Widgets/IToolTip.h"
#include "Misc/Paths.h"
#include "HAL/IConsoleManager.h"
#include "Types/NavigationMetaData.h"
#include "Application/SlateApplicationBase.h"
#include "Styling/CoreStyle.h"
#include "Application/ActiveTimerHandle.h"
#include "Input/HittestGrid.h"
#include "Debugging/SlateDebugging.h"
#include "Debugging/SlateCrashReporterHandler.h"
#include "Debugging/WidgetList.h"
#include "Widgets/SWindow.h"
#include "Trace/SlateTrace.h"
#include "Types/SlateAttributeMetaData.h"
#include "Types/SlateActiveTimersMetaData.h"
#include "Types/SlateCursorMetaData.h"
#include "Types/SlateMouseEventsMetaData.h"
#include "Types/ReflectionMetadata.h"
#include "Types/SlateToolTipForceFieldExpansionMetaData.h"
#include "Types/SlateToolTipMetaData.h"
#include "Stats/Stats.h"
#include "Containers/StringConv.h"
#include "Misc/ScopeRWLock.h"
#include "HAL/CriticalSection.h"
#include "Widgets/SWidgetUtils.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "HAL/LowLevelMemStats.h"
#include "UObject/Package.h"

#include <limits>

#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateCoreAccessibleWidgets.h"
#include "Widgets/Accessibility/SlateAccessibleMessageHandler.h"
#endif

// Enabled to assign FindWidgetMetaData::FoundWidget to the widget that has the matching reflection data 
#ifndef UE_WITH_SLATE_DEBUG_FIND_WIDGET_REFLECTION_METADATA
	#define UE_WITH_SLATE_DEBUG_FIND_WIDGET_REFLECTION_METADATA 0
#endif

#ifndef UE_SLATE_WITH_WIDGET_RENDERING_TRANSFORM_NAN_DIAGNOSTIC
	#define UE_SLATE_WITH_WIDGET_RENDERING_TRANSFORM_NAN_DIAGNOSTIC (!ENABLE_NAN_DIAGNOSTIC && 0) // NAN diagnostic is already activated for every engine systems
#endif

#if UE_WITH_SLATE_DEBUG_FIND_WIDGET_REFLECTION_METADATA
namespace FindWidgetMetaData
{
	SWidget* FoundWidget = nullptr;
	FName WidgeName = "ItemNameToFind";
	FName AssetName = "AssetNameToFind";
}
#endif

DEFINE_STAT(STAT_SlateTotalWidgetsPerFrame);
DEFINE_STAT(STAT_SlateNumPaintedWidgets);
DEFINE_STAT(STAT_SlateNumTickedWidgets);
DEFINE_STAT(STAT_SlateExecuteActiveTimers);
DEFINE_STAT(STAT_SlateTickWidgets);
DEFINE_STAT(STAT_SlatePrepass);
DEFINE_STAT(STAT_SlateTotalWidgets);
DEFINE_STAT(STAT_SlateSWidgetAllocSize);

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
DEFINE_STAT(STAT_SlateGetMetaData);
DECLARE_CYCLE_STAT(TEXT("SWidget::CreateStatID"), STAT_Slate_CreateStatID, STATGROUP_Slate);
#endif

template <typename AnnotationType>
class TWidgetSparseAnnotation
{
public:
	const AnnotationType* Find(const SWidget* Widget)
	{
		FRWScopeLock Lock(RWLock, SLT_ReadOnly);
		return AnnotationMap.Find(Widget);
	}

	AnnotationType& FindOrAdd(const SWidget* Widget)
	{
		FRWScopeLock Lock(RWLock, SLT_Write);
		return AnnotationMap.FindOrAdd(Widget);
	}

	void Add(const SWidget* Widget, const AnnotationType& Type)
	{
		FRWScopeLock Lock(RWLock, SLT_Write);
		AnnotationMap.Add(Widget, Type);
	}

	void Remove(const SWidget* Widget)
	{
		FRWScopeLock Lock(RWLock, SLT_Write);
		AnnotationMap.Remove(Widget);
	}
private:
	TMap<const SWidget*, AnnotationType> AnnotationMap;
	FRWLock RWLock;
};

#if WITH_ACCESSIBILITY
TWidgetSparseAnnotation<TAttribute<FText>> AccessibleText;
TWidgetSparseAnnotation<TAttribute<FText>> AccessibleSummaryText;
#endif

static void ClearSparseAnnotationsForWidget(const SWidget* Widget)
{
#if WITH_ACCESSIBILITY
	AccessibleText.Remove(Widget);
	AccessibleSummaryText.Remove(Widget);
#endif
}

#if SLATE_CULL_WIDGETS

float GCullingSlackFillPercent = 0.25f;
static FAutoConsoleVariableRef CVarCullingSlackFillPercent(TEXT("Slate.CullingSlackFillPercent"), GCullingSlackFillPercent, TEXT("Scales the culling rect by the amount to provide extra slack/wiggle room for widgets that have a true bounds larger than the root child widget in a container."), ECVF_Default);

#endif

#if WITH_SLATE_DEBUGGING

bool GShowClipping = false;
static FAutoConsoleVariableRef CVarSlateShowClipRects(TEXT("Slate.ShowClipping"), GShowClipping, TEXT("Controls whether we should render a clipping zone outline.  Yellow = Axis Scissor Rect Clipping (cheap).  Red = Stencil Clipping (expensive)."), ECVF_Default);

bool GDebugCulling = false;
static FAutoConsoleVariableRef CVarSlateDebugCulling(TEXT("Slate.DebugCulling"), GDebugCulling, TEXT("Controls whether we should ignore clip rects, and just use culling."), ECVF_Default);

bool GSlateEnsureAllVisibleWidgetsPaint = false;
static FAutoConsoleVariableRef CVarSlateEnsureAllVisibleWidgetsPaint(TEXT("Slate.EnsureAllVisibleWidgetsPaint"), GSlateEnsureAllVisibleWidgetsPaint, TEXT("Ensures that if a child widget is visible before OnPaint, that it was painted this frame after OnPaint, if still marked as visible.  Only works if we're on the FastPaintPath."), ECVF_Default);

bool GSlateEnsureOutgoingLayerId = false;
static FAutoConsoleVariableRef CVarSlateEnsureOutgoingLayerId(TEXT("Slate.EnsureOutgoingLayerId"), GSlateEnsureOutgoingLayerId, TEXT("Ensures that child widget returns the correct layer id with OnPaint."), ECVF_Default);

#endif

#if STATS || ENABLE_STATNAMEDEVENTS

void SWidget::CreateStatID() const
{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	SCOPE_CYCLE_COUNTER(STAT_Slate_CreateStatID);
#endif

	const FString LongName = FReflectionMetaData::GetWidgetDebugInfo(this);

#if STATS
	StatID = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_Slate>(LongName);
#else // ENABLE_STATNAMEDEVENTS
	const auto& ConversionData = StringCast<PROFILER_CHAR>(*LongName);
	const int32 NumStorageChars = (ConversionData.Length() + 1);	//length doesn't include null terminator

	PROFILER_CHAR* StoragePtr = new PROFILER_CHAR[NumStorageChars];
	FMemory::Memcpy(StoragePtr, ConversionData.Get(), NumStorageChars * sizeof(PROFILER_CHAR));

	if (FPlatformAtomics::InterlockedCompareExchangePointer((void**)&StatIDStringStorage, StoragePtr, nullptr) != nullptr)
	{
		delete[] StoragePtr;
	}
	
	StatID = TStatId(StatIDStringStorage);
#endif
}

#endif

#if UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
namespace SlateTraceMetaData
{
	uint64 UniqueIdGenerator = 0;
}
#endif

SLATE_IMPLEMENT_WIDGET(SWidget)
void SWidget::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	// Visibility should be the first Attribute in the list.
	//The order in which SlateAttribute are declared in the .h dictates of the order.
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "Visibility", VisibilityAttribute, EInvalidateWidgetReason::Visibility)
		.AffectVisibility();
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "EnabledState", EnabledStateAttribute, EInvalidateWidgetReason::None);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "Hovered", HoveredAttribute, EInvalidateWidgetReason::None);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "RenderTransform", RenderTransformAttribute, EInvalidateWidgetReason::Layout | EInvalidateWidgetReason::RenderTransform);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "RenderTransformPivot", RenderTransformPivotAttribute, EInvalidateWidgetReason::Layout | EInvalidateWidgetReason::RenderTransform);

#if UE_SLATE_WITH_WIDGET_RENDERING_TRANSFORM_NAN_DIAGNOSTIC
	AttributeInitializer.OverrideOnValueChanged("RenderTransform", FSlateAttributeDescriptor::ECallbackOverrideType::ReplacePrevious
		, FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
		{
				if (Widget.RenderTransformAttribute.Get().IsSet() && Widget.RenderTransformAttribute.Get().GetValue().ContainsNaN())
				{
					logOrEnsureNanError(TEXT("RenderTransform contains NaN"));
					Widget.RenderTransformAttribute.Set(Widget, TOptional<FSlateRenderTransform>());
				}
		}));
	AttributeInitializer.OverrideOnValueChanged("RenderTransformPivot", FSlateAttributeDescriptor::ECallbackOverrideType::ReplacePrevious
		, FSlateAttributeDescriptor::FAttributeValueChangedDelegate::CreateLambda([](SWidget& Widget)
			{
				if (Widget.RenderTransformPivotAttribute.Get().ContainsNaN())
				{
					logOrEnsureNanError(TEXT("RenderTransformPivot contains NaN"));
					Widget.RenderTransformPivotAttribute.Set(Widget, FVector2D::ZeroVector);
				}
			}));
#endif
}

SWidget::SWidget()
	: bCanSupportFocus(true)
	, bCanHaveChildren(true)
	, bClippingProxy(false)
	, bToolTipForceFieldEnabled(false)
	, bForceVolatile(false)
	, bCachedVolatile(false)
	, bInheritedVolatility(false)
	, bNeedsPrepass(true)
	, bHasRegisteredSlateAttribute(false)
	, bEnabledAttributesUpdate(true)
	, bIsDeclarativeSyntaxConstructionCompleted(false)
	, bIsHoveredAttributeSet(false)
	, bHasActiveTimers(false)
	, bDesiredSizeSet(false)
	, bPrepassLayoutScaleMultiplierSet(false)
	, bIsProjectContent(false)
	, bIsProjectContentParent(false)
	, bHasCustomPrepass(false)
	, bHasRelativeLayoutScale(false)
	, bVolatilityAlwaysInvalidatesPrepass(false)
#if WITH_ACCESSIBILITY
	, bCanChildrenBeAccessible(true)
	, AccessibleBehavior(EAccessibleBehavior::NotAccessible)
	, AccessibleSummaryBehavior(EAccessibleBehavior::Auto)
#endif
	, Clipping(EWidgetClipping::Inherit)
	, PixelSnappingMethod(EWidgetPixelSnapping::Inherit)
	, FlowDirectionPreference(EFlowDirectionPreference::Inherit)
	// Note we are defaulting to tick for backwards compatibility
	, UpdateFlags(EWidgetUpdateFlags::NeedsTick)
	, VisibilityAttribute(*this, EVisibility::Visible)
	, EnabledStateAttribute(*this, true)
	, HoveredAttribute(*this, false)
	, PrepassLayoutScaleMultiplierValue(1.0f)
	, DesiredSize(FVector2f(0.0f, 0.0f))
	, RenderTransformPivotAttribute(*this, FVector2D::ZeroVector)
	, RenderTransformAttribute(*this)
	, CullingBoundsExtension()
	, RenderOpacity(1.0f)
#if UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
	, UniqueIdentifier(++SlateTraceMetaData::UniqueIdGenerator)
#endif
#if STATS
	, AllocSize(0)
#endif
#if ENABLE_STATNAMEDEVENTS
	, StatIDStringStorage(nullptr)
#endif
{
	if (GIsRunning)
	{
		INC_DWORD_STAT(STAT_SlateTotalWidgets);
		INC_DWORD_STAT(STAT_SlateTotalWidgetsPerFrame);
	}

	UE_SLATE_DEBUG_WIDGETLIST_ADD_WIDGET(this);
	UE_TRACE_SLATE_WIDGET_ADDED(this);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

SWidget::~SWidget()
{
#if WITH_SLATE_DEBUGGING
	UE_CLOGF(Debug_DestroyedTag != 0xDC, LogSlate, Fatal, "The widget is already destroyed.");
	Debug_DestroyedTag = 0xA3;
#endif

#if UE_WITH_SLATE_DEBUG_FIND_WIDGET_REFLECTION_METADATA
	if (FindWidgetMetaData::FoundWidget == this)
	{
		FindWidgetMetaData::FoundWidget = nullptr;
	}
#endif

	bHasRegisteredSlateAttribute = false;

	// Unregister all ActiveTimers so they aren't left stranded in the Application's list.
	if (FSlateApplicationBase::IsInitialized())
	{
		FSlateApplicationBase& SlateApplication = FSlateApplicationBase::Get();
		if (bHasActiveTimers)
		{
			TSharedPtr<UE::Slate::FActiveTimersMetaData> ActiveTimersMetaData = GetMetaData<UE::Slate::FActiveTimersMetaData>();
			check(ActiveTimersMetaData);
			for (const TSharedRef<FActiveTimerHandle>& ActiveTimerHandle : ActiveTimersMetaData->ActiveTimers)
			{
				SlateApplication.UnRegisterActiveTimer(ActiveTimerHandle);
			}
		}

#if WITH_ACCESSIBILITY
		SlateApplication.GetAccessibleMessageHandler()->OnWidgetRemoved(this);
#endif
		// Only clear if initialized because SNullWidget's destructor may be called after annotations are deleted
		ClearSparseAnnotationsForWidget(this);
	}

	// Warn the invalidation root
	if (FSlateInvalidationRoot* InvalidationRoot = FastPathProxyHandle.GetInvalidationRootHandle().GetInvalidationRoot())
	{
		InvalidationRoot->OnWidgetDestroyed(this);
	}

	// Reset handle
	FastPathProxyHandle = FWidgetProxyHandle();

	// Note: this would still be valid if a widget was painted and then destroyed in the same frame.  
	// In that case invalidation hasn't taken place for added widgets so the invalidation panel doesn't know about their cached element data to clean it up
	PersistentState.CachedElementHandle.RemoveFromCache();

#if ENABLE_STATNAMEDEVENTS
	delete[] StatIDStringStorage;
	StatIDStringStorage = nullptr;
#endif

	UE_SLATE_DEBUG_WIDGETLIST_REMOVE_WIDGET(this);
	UE_TRACE_SLATE_WIDGET_REMOVED(this);
	DEC_DWORD_STAT(STAT_SlateTotalWidgets);
	DEC_MEMORY_STAT_BY(STAT_SlateSWidgetAllocSize, AllocSize);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void SWidget::SWidgetConstruct(const FSlateBaseNamedArgs& Args)
{
	SetEnabled(Args._IsEnabled);
	VisibilityAttribute.Assign(*this, Args._Visibility); // SetVisibility is virtual, assign directly to stay backward compatible
	RenderOpacity = Args._RenderOpacity;
	SetRenderTransform(Args._RenderTransform);
	SetRenderTransformPivot(Args._RenderTransformPivot);
	Tag = Args._Tag;
	bForceVolatile = Args._ForceVolatile;
	Clipping = Args._Clipping;
	FlowDirectionPreference = Args._FlowDirectionPreference;
	bEnabledAttributesUpdate = Args._EnabledAttributesUpdate;
	MetaData.Append(Args.MetaData);

	if (Args._ToolTip.IsSet())
	{
		// If someone specified a fancy widget tooltip, use it.
		SetToolTip(Args._ToolTip);
	}
	else if (Args._ToolTipText.IsSet())
	{
		// If someone specified a text binding, make a tooltip out of it
		SetToolTipText(Args._ToolTipText);
	}

	if (Args._Cursor.IsSet())
	{
		SetCursor(Args._Cursor);
	}

#if WITH_ACCESSIBILITY
	// If custom text is provided, force behavior to custom. Otherwise, use the passed-in behavior and set their default text.
	if (Args._AccessibleText.IsSet() || Args._AccessibleParams.IsSet())
	{
		auto SetAccessibleWidgetData = [this](const FAccessibleWidgetData& AccessibleParams)
		{
			SetCanChildrenBeAccessible(AccessibleParams.bCanChildrenBeAccessible);
			SetAccessibleBehavior(AccessibleParams.AccessibleText.IsSet() ? EAccessibleBehavior::Custom : AccessibleParams.AccessibleBehavior, AccessibleParams.AccessibleText, EAccessibleType::Main);
			SetAccessibleBehavior(AccessibleParams.AccessibleSummaryText.IsSet() ? EAccessibleBehavior::Custom : AccessibleParams.AccessibleSummaryBehavior, AccessibleParams.AccessibleSummaryText, EAccessibleType::Summary);
		};
		if (Args._AccessibleText.IsSet())
		{
			SetAccessibleWidgetData(FAccessibleWidgetData{ Args._AccessibleText });
		}
		else
		{
			SetAccessibleWidgetData(Args._AccessibleParams.GetValue());
		}
	}
#endif //WITH_ACCESSIBILITY

}

FReply SWidget::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	return FReply::Unhandled();
}

void SWidget::OnFocusLost(const FFocusEvent& InFocusEvent)
{
}

void SWidget::OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent)
{
}

FReply SWidget::OnKeyChar( const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent )
{
	return FReply::Unhandled();
}

FReply SWidget::OnPreviewKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	return FReply::Unhandled();
}

FReply SWidget::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	if (bCanSupportFocus && SupportsKeyboardFocus())
	{
		EUINavigation Direction = FSlateApplicationBase::Get().GetNavigationDirectionFromKey(InKeyEvent);
		// It's the left stick return a navigation request of the correct direction
		if (Direction != EUINavigation::Invalid)
		{
			const ENavigationGenesis Genesis = InKeyEvent.GetKey().IsGamepadKey() ? ENavigationGenesis::Controller : ENavigationGenesis::Keyboard;
			return FReply::Handled().SetNavigation(Direction, Genesis);
		}
	}
	return FReply::Unhandled();
}

FReply SWidget::OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	return FReply::Unhandled();
}

FReply SWidget::OnAnalogValueChanged( const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent )
{
	if (bCanSupportFocus && SupportsKeyboardFocus())
	{
		EUINavigation Direction = FSlateApplicationBase::Get().GetNavigationDirectionFromAnalog(InAnalogInputEvent);
		// It's the left stick return a navigation request of the correct direction
		if (Direction != EUINavigation::Invalid)
		{
			return FReply::Handled().SetNavigation(Direction, ENavigationGenesis::Controller);
		}
	}
	return FReply::Unhandled();
}

FReply SWidget::OnPreviewMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return FReply::Unhandled();
}

FReply SWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (TSharedPtr<FSlateMouseEventsMetaData> Data = GetMetaData<FSlateMouseEventsMetaData>())
	{
		if (Data->MouseButtonDownHandle.IsBound() )
		{
			return Data->MouseButtonDownHandle.Execute(MyGeometry, MouseEvent);
		}
	}
	return FReply::Unhandled();
}

FReply SWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (TSharedPtr<FSlateMouseEventsMetaData> Data = GetMetaData<FSlateMouseEventsMetaData>())
	{
		if (Data->MouseButtonUpHandle.IsBound())
		{
			return Data->MouseButtonUpHandle.Execute(MyGeometry, MouseEvent);
		}
	}
	return FReply::Unhandled();
}

FReply SWidget::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (TSharedPtr<FSlateMouseEventsMetaData> Data = GetMetaData<FSlateMouseEventsMetaData>())
	{
		if (Data->MouseMoveHandle.IsBound())
		{
			return Data->MouseMoveHandle.Execute(MyGeometry, MouseEvent);
		}
	}
	return FReply::Unhandled();
}

FReply SWidget::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (TSharedPtr<FSlateMouseEventsMetaData> Data = GetMetaData<FSlateMouseEventsMetaData>())
	{
		if (Data->MouseDoubleClickHandle.IsBound())
		{
			return Data->MouseDoubleClickHandle.Execute(MyGeometry, MouseEvent);
		}
	}
	return FReply::Unhandled();
}

void SWidget::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (!bIsHoveredAttributeSet)
	{
		HoveredAttribute.Set(*this, true);
	}

	if (TSharedPtr<FSlateMouseEventsMetaData> Data = GetMetaData<FSlateMouseEventsMetaData>())
	{
		if (Data->MouseEnterHandler.IsBound())
		{
			// A valid handler is assigned; let it handle the event.
			Data->MouseEnterHandler.Execute(MyGeometry, MouseEvent);
		}
	}
}

void SWidget::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	if (!bIsHoveredAttributeSet)
	{
		HoveredAttribute.Set(*this, false);
	}

	if (TSharedPtr<FSlateMouseEventsMetaData> Data = GetMetaData<FSlateMouseEventsMetaData>())
	{
		if (Data->MouseLeaveHandler.IsBound())
		{
			// A valid handler is assigned; let it handle the event.
			Data->MouseLeaveHandler.Execute(MouseEvent);
		}
	}
}

FReply SWidget::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return FReply::Unhandled();
}

FCursorReply SWidget::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	TOptional<EMouseCursor::Type> TheCursor = GetCursor();
	return ( TheCursor.IsSet() )
		? FCursorReply::Cursor( TheCursor.GetValue() )
		: FCursorReply::Unhandled();
}

TOptional<TSharedRef<SWidget>> SWidget::OnMapCursor(const FCursorReply& CursorReply) const
{
	return TOptional<TSharedRef<SWidget>>();
}

bool SWidget::OnVisualizeTooltip( const TSharedPtr<SWidget>& TooltipContent )
{
	return false;
}

TSharedPtr<FPopupLayer> SWidget::OnVisualizePopup(const TSharedRef<SWidget>& PopupContent)
{
	return TSharedPtr<FPopupLayer>();
}

FReply SWidget::OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return FReply::Unhandled();
}

void SWidget::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
}

void SWidget::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
}

FReply SWidget::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	return FReply::Unhandled();
}

FReply SWidget::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	return FReply::Unhandled();
}

FReply SWidget::OnTouchGesture( const FGeometry& MyGeometry, const FPointerEvent& GestureEvent )
{
	return FReply::Unhandled();
}

FReply SWidget::OnTouchStarted( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
{
	return FReply::Unhandled();
}

FReply SWidget::OnTouchMoved( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
{
	return FReply::Unhandled();
}

FReply SWidget::OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
{
	return FReply::Unhandled();
}

FReply SWidget::OnTouchForceChanged(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	return FReply::Unhandled();
}

FReply SWidget::OnTouchFirstMove(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent)
{
	return FReply::Unhandled();
}

FReply SWidget::OnMotionDetected( const FGeometry& MyGeometry, const FMotionEvent& InMotionEvent )
{
	return FReply::Unhandled();
}

TOptional<bool> SWidget::OnQueryShowFocus(const EFocusCause InFocusCause) const
{
	return TOptional<bool>();
}

FPopupMethodReply SWidget::OnQueryPopupMethod() const
{
	return FPopupMethodReply::Unhandled();
}

TOptional<FVirtualPointerPosition> SWidget::TranslateMouseCoordinateForCustomHitTestChild(const SWidget& ChildWidget, const FGeometry& MyGeometry, const FVector2D ScreenSpaceMouseCoordinate, const FVector2D LastScreenSpaceMouseCoordinate) const
{
	return TOptional<FVirtualPointerPosition>();
}

void SWidget::OnFinishedPointerInput()
{

}

void SWidget::OnFinishedKeyInput()
{

}

FNavigationReply SWidget::OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent)
{
	return OnNavigation(MyGeometry, InNavigationEvent.GetNavigationType());
}

FNavigationReply SWidget::OnNavigation(const FGeometry& MyGeometry, const EUINavigation Type)
{
	TSharedPtr<FNavigationMetaData> NavigationMetaData = GetMetaData<FNavigationMetaData>();
	if (NavigationMetaData.IsValid())
	{
		TSharedPtr<SWidget> Widget = NavigationMetaData->GetFocusRecipient(Type).Pin();
		return FNavigationReply(NavigationMetaData->GetBoundaryRule(Type), Widget, NavigationMetaData->GetFocusDelegate(Type));
	}
	return FNavigationReply::Escape();
}

EWindowZone::Type SWidget::GetWindowZoneOverride() const
{
	// No special behavior.  Override this in derived widgets, if needed.
	return EWindowZone::Unspecified;
}

void SWidget::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
}

void SWidget::SlatePrepass()
{
	SlatePrepass(FSlateApplicationBase::Get().GetApplicationScale());
}

void SWidget::SlatePrepass(float InLayoutScaleMultiplier)
{
	UE_SLATE_CRASH_REPORTER_PREPASS_SCOPE(*this);
	SCOPE_CYCLE_COUNTER(STAT_SlatePrepass);

	if (!GSlateIsOnFastUpdatePath || bNeedsPrepass)
	{
		LLM_SCOPE_BYNAME("UI/Slate/Prepass");
#if UE_TRACE_ASSET_METADATA_ENABLED
		FName AssetName = NAME_None;
		FName ClassName = NAME_None;
		FName PackageName = NAME_None;
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(AssetMetadataChannel))
		{
			TSharedPtr<FReflectionMetaData> AssetMetaData = FReflectionMetaData::GetWidgetOrParentMetaData(this);
			if (AssetMetaData.IsValid())
			{
				if (const UObject* AssetPtr = AssetMetaData->Asset.Get())
				{
					AssetName = AssetMetaData->Name;
					ClassName = AssetMetaData->Class.Get()->GetFName();
					PackageName = AssetPtr->GetPackage()->GetFName();
				}
			}
		}
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(PackageName, ELLMTagSet::Assets);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(ClassName, ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_SCOPE_ASSET_FNAME(AssetName, ClassName, PackageName);
#endif
		if (HasRegisteredSlateAttribute() && IsAttributesUpdatesEnabled() && !GSlateIsOnFastProcessInvalidation)
		{
			FSlateAttributeMetaData::UpdateAllAttributes(*this, FSlateAttributeMetaData::EInvalidationPermission::AllowInvalidationIfConstructed);
		}
		Prepass_Internal(InLayoutScaleMultiplier);
	}
}

void SWidget::InvalidateChildRemovedFromTree(SWidget& Child)
{
	// If the root is invalidated, we need to clear out its PersistentState regardless.
	if (FSlateInvalidationRoot* ChildInvalidationRoot = Child.FastPathProxyHandle.GetInvalidationRootHandle().GetInvalidationRoot())
	{
		SCOPED_NAMED_EVENT(SWidget_InvalidateChildRemovedFromTree, FColor::Red);
		Child.UpdateFastPathWidgetRemoved(ChildInvalidationRoot->GetHittestGrid());
	}
}

UE::Slate::FDeprecateVector2DResult SWidget::GetDesiredSize() const
{
	return bDesiredSizeSet ? UE::Slate::FDeprecateVector2DResult(DesiredSize) : FVector2f::ZeroVector;
}

void SWidget::AssignParentWidget(TSharedPtr<SWidget> InParent)
{
#if !UE_BUILD_SHIPPING
	ensureMsgf(InParent != SNullWidget::NullWidget, TEXT("The Null Widget can't be anyone's parent."));
	ensureMsgf(this != &SNullWidget::NullWidget.Get(), TEXT("The Null Widget can't have a parent, because a single instance is shared everywhere."));
	ensureMsgf(InParent.IsValid(), TEXT("Are you trying to detatch the parent of a widget?  Use ConditionallyDetatchParentWidget()."));
#endif

	//@todo We should update inherited visibility and volatility here but currently we are relying on ChildOrder invalidation to do that for us

	const TSharedPtr<SWidget> PreviousParent = ParentWidgetPtr.Pin();
	ParentWidgetPtr = InParent;
#if WITH_ACCESSIBILITY
	if (FSlateApplicationBase::IsInitialized())
	{
		FSlateApplicationBase::Get().GetAccessibleMessageHandler()->MarkDirty();
	}
#endif
	if (PreviousParent.IsValid())
	{
		// We want to be sure that the previous parent gets a ChildOrder invalidation for Dynamic Invalidation,
		// otherwise we might not be able to catch widgets having the wrong parents
		PreviousParent->Invalidate(EInvalidateWidgetReason::ChildOrder);
	}
	if (InParent.IsValid())
	{
		InParent->Invalidate(EInvalidateWidgetReason::ChildOrder);
	}
}

bool SWidget::ConditionallyDetatchParentWidget(SWidget* InExpectedParent)
{
#if !UE_BUILD_SHIPPING
	ensureMsgf(this != &SNullWidget::NullWidget.Get(), TEXT("The Null Widget can't have a parent, because a single instance is shared everywhere."));
#endif

	TSharedPtr<SWidget> Parent = ParentWidgetPtr.Pin();
	if (Parent.Get() == InExpectedParent)
	{
		ParentWidgetPtr.Reset();
#if WITH_ACCESSIBILITY
		if (FSlateApplicationBase::IsInitialized())
		{
			FSlateApplicationBase::Get().GetAccessibleMessageHandler()->MarkDirty();
		}
#endif

		if (Parent.IsValid())
		{
			Parent->Invalidate(EInvalidateWidgetReason::ChildOrder);
		}

		InvalidateChildRemovedFromTree(*this);
		return true;
	}

	return false;
}

void SWidget::UpdateWidgetProxy(int32 NewLayerId, FSlateCachedElementsHandle& CacheHandle)
{
#if WITH_SLATE_DEBUGGING
	check(!CacheHandle.IsValid() || CacheHandle.IsOwnedByWidget(this));
#endif

	// Account for the case when the widget gets a new handle for some reason.  This should really never happen
	if (PersistentState.CachedElementHandle.IsValid() && PersistentState.CachedElementHandle != CacheHandle)
	{
		ensureMsgf(!CacheHandle.IsValid()
			, TEXT("Widget '%s' was assigned a new cache handle. This is not expected to happen.")
			, *FReflectionMetaData::GetWidgetPath(this));
		PersistentState.CachedElementHandle.RemoveFromCache();
	}
	PersistentState.CachedElementHandle = CacheHandle;
	PersistentState.OutgoingLayerId = NewLayerId;

#if WITH_SLATE_DEBUGGING
	const FSlateInvalidationRoot* InvalidationRoot = FastPathProxyHandle.GetInvalidationRootHandle().GetInvalidationRoot();
	if (FastPathProxyHandle.IsValid(this) && InvalidationRoot && !InvalidationRoot->NeedsSlowPath())
	{
		FWidgetProxy& MyProxy = FastPathProxyHandle.GetProxy();
#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
		MyProxy.bDebug_Updated = true;
#endif
		ensureMsgf(MyProxy.Visibility.IsVisibleDirectly() == GetVisibility().IsVisible() || EnumHasAnyFlags(MyProxy.CurrentInvalidateReason, EInvalidateWidgetReason::Visibility)
			, TEXT("The visibility of the widget '%s' changed during Paint")
			, *FReflectionMetaData::GetWidgetPath(this));
		if (IsVolatile() && !IsVolatileIndirectly())
		{
			ensureMsgf(HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePaint)
				, TEXT("The volatility of the widget '%s' changed during Paint")
				, *FReflectionMetaData::GetWidgetPath(this));
		}
		else
		{
			ensureMsgf(!HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePaint)
				, TEXT("The volatility of the widget '%s' changed during Paint")
				, *FReflectionMetaData::GetWidgetPath(this));
		}
	}
#endif
}

void SWidget::SetFastPathSortOrder(const FSlateInvalidationWidgetSortOrder InSortOrder)
{
	if (InSortOrder != FastPathProxyHandle.GetWidgetSortOrder())
	{
		FastPathProxyHandle.WidgetSortOrder = InSortOrder;
		if (FSlateInvalidationRoot* Root = FastPathProxyHandle.GetInvalidationRootHandle().GetInvalidationRoot())
		{
			if (FHittestGrid* HittestGrid = Root->GetHittestGrid())
			{
				HittestGrid->UpdateWidget(this, InSortOrder);
			}
		}

		//TODO, update Cached LayerId
	}
}

void SWidget::SetFastPathProxyHandle(const FWidgetProxyHandle& Handle, FSlateInvalidationWidgetVisibility InvalidationVisibility, bool bParentVolatile)
{
	check(this != &SNullWidget::NullWidget.Get());

	FastPathProxyHandle = Handle;
	if (FSlateInvalidationRoot* Root = FastPathProxyHandle.GetInvalidationRootHandle().GetInvalidationRoot())
	{
		if (FHittestGrid* HittestGrid = Root->GetHittestGrid())
		{
			HittestGrid->UpdateWidget(this, Handle.GetWidgetSortOrder());
		}
	}

	bInheritedVolatility = bParentVolatile;

	if (!InvalidationVisibility.IsVisible())
	{
		PersistentState.CachedElementHandle.RemoveFromCache();

#if WITH_SLATE_DEBUGGING
		if (PersistentState.CachedElementHandle.IsValid())
		{
			check(PersistentState.CachedElementHandle.IsOwnedByWidget(this));
		}
#endif
	}

	if (IsVolatile() && !IsVolatileIndirectly())
	{
		// Always add to the list, since it may have been removed with a ChildOrder invalidation.
		AddUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePaint);
	}
	else
	{
		if (HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePaint))
		{
			RemoveUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePaint);
		}
	}
}

void SWidget::UpdateFastPathVisibility(FSlateInvalidationWidgetVisibility ParentVisibility, FHittestGrid* ParentHittestGrid)
{
	const EVisibility CurrentVisibility = GetVisibility();
	const FSlateInvalidationWidgetVisibility NewVisibility = FSlateInvalidationWidgetVisibility{ ParentVisibility, CurrentVisibility };

	FHittestGrid* HittestGridToRemoveFrom = ParentHittestGrid;
	if (FastPathProxyHandle.IsValid(this))
	{	
		// Try and remove this from the current handles hit test grid. If we are in a nested invalidation situation the hittest grid may have changed
		HittestGridToRemoveFrom = FastPathProxyHandle.GetInvalidationRoot_NoCheck()->GetHittestGrid();
		FWidgetProxy& Proxy = FastPathProxyHandle.GetProxy();
		Proxy.Visibility = NewVisibility;
	}

	if (HittestGridToRemoveFrom)
	{
		HittestGridToRemoveFrom->RemoveWidget(this);
	}

	PersistentState.CachedElementHandle.ClearCachedElements();

	// Loop through children
	GetAllChildren()->ForEachWidget([NewVisibility, HittestGridToRemoveFrom](SWidget& Child)
		{
			Child.UpdateFastPathVisibility(NewVisibility, HittestGridToRemoveFrom);
		});
}

void SWidget::UpdateFastPathWidgetRemoved(FHittestGrid* ParentHittestGrid)
{
	FHittestGrid* HittestGridToRemoveFrom = ParentHittestGrid;

	if (FSlateInvalidationRoot* InvalidationRoot = FastPathProxyHandle.GetInvalidationRootHandle().GetInvalidationRoot())
	{
		HittestGridToRemoveFrom = FastPathProxyHandle.GetInvalidationRoot_NoCheck()->GetHittestGrid();
		InvalidationRoot->OnWidgetDestroyed(this);
	}
	FastPathProxyHandle = FWidgetProxyHandle();

	if (HittestGridToRemoveFrom)
	{
		HittestGridToRemoveFrom->RemoveWidget(this);
	}

	PersistentState.CachedElementHandle.RemoveFromCache();

	// Loop through children
	GetAllChildren()->ForEachWidget([HittestGridToRemoveFrom](SWidget& Child)
		{
			Child.UpdateFastPathWidgetRemoved(HittestGridToRemoveFrom);
		});
}

void SWidget::UpdateFastPathVolatility(bool bParentVolatile)
{
	bInheritedVolatility = bParentVolatile;

	if (IsVolatile() && !IsVolatileIndirectly())
	{
		AddUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePaint);
	}
	else
	{
		AddUpdateFlags(EWidgetUpdateFlags::NeedsRepaint);
		RemoveUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePaint);
	}

	const bool bNewParentVolatility = bParentVolatile || IsVolatile();
	GetAllChildren()->ForEachWidget([bNewParentVolatility](SWidget& Child)
		{
			Child.UpdateFastPathVolatility(bNewParentVolatility);
		});
}

void SWidget::CacheDesiredSize(float InLayoutScaleMultiplier)
{
#if WITH_VERY_VERBOSE_SLATE_STATS
	SCOPED_NAMED_EVENT(SWidget_CacheDesiredSize, FColor::Red);
#endif

	// Cache this widget's desired size.
	SetDesiredSize(ComputeDesiredSize(InLayoutScaleMultiplier));
}


bool SWidget::SupportsKeyboardFocus() const
{
	return false;
}

bool SWidget::HasKeyboardFocus() const
{
	return (FSlateApplicationBase::Get().GetKeyboardFocusedWidget().Get() == this);
}

TOptional<EFocusCause> SWidget::HasUserFocus(int32 UserIndex) const
{
	return FSlateApplicationBase::Get().HasUserFocus(SharedThis(this), UserIndex);
}

TOptional<EFocusCause> SWidget::HasAnyUserFocus() const
{
	return FSlateApplicationBase::Get().HasAnyUserFocus(SharedThis(this));
}

bool SWidget::HasUserFocusedDescendants(int32 UserIndex) const
{
	return FSlateApplicationBase::Get().HasUserFocusedDescendants(SharedThis(this), UserIndex);
}

bool SWidget::HasFocusedDescendants() const
{
	return FSlateApplicationBase::Get().HasFocusedDescendants(SharedThis(this));
}

bool SWidget::HasAnyUserFocusOrFocusedDescendants() const
{
	return HasAnyUserFocus().IsSet() || HasFocusedDescendants();
}

const FSlateBrush* SWidget::GetFocusBrush() const
{
	return FAppStyle::Get().GetBrush("FocusRectangle");
}

bool SWidget::HasMouseCapture() const
{
	return FSlateApplicationBase::Get().DoesWidgetHaveMouseCapture(SharedThis(this));
}

bool SWidget::HasMouseCaptureByUser(int32 UserIndex, TOptional<int32> PointerIndex) const
{
	return FSlateApplicationBase::Get().DoesWidgetHaveMouseCaptureByUser(SharedThis(this), UserIndex, PointerIndex);
}

void SWidget::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
}

bool SWidget::FindChildGeometries( const FGeometry& MyGeometry, const TSet< TSharedRef<SWidget> >& WidgetsToFind, TMap<TSharedRef<SWidget>, FArrangedWidget>& OutResult ) const
{
	FindChildGeometries_Helper(MyGeometry, WidgetsToFind, OutResult);
	return OutResult.Num() == WidgetsToFind.Num();
}


void SWidget::FindChildGeometries_Helper( const FGeometry& MyGeometry, const TSet< TSharedRef<SWidget> >& WidgetsToFind, TMap<TSharedRef<SWidget>, FArrangedWidget>& OutResult ) const
{
	// Perform a breadth first search!

	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	this->ArrangeChildren(MyGeometry, ArrangedChildren);
	const int32 NumChildren = ArrangedChildren.Num();

	// See if we found any of the widgets on this level.
	for(int32 ChildIndex=0; ChildIndex < NumChildren; ++ChildIndex )
	{
		const FArrangedWidget& CurChild = ArrangedChildren[ ChildIndex ];
		
		if ( WidgetsToFind.Contains(CurChild.Widget) )
		{
			// We found one of the widgets for which we need geometry!
			OutResult.Add( CurChild.Widget, CurChild );
		}
	}

	// If we have not found all the widgets that we were looking for, descend.
	if ( OutResult.Num() != WidgetsToFind.Num() )
	{
		// Look for widgets among the children.
		for( int32 ChildIndex=0; ChildIndex < NumChildren; ++ChildIndex )
		{
			const FArrangedWidget& CurChild = ArrangedChildren[ ChildIndex ];
			CurChild.Widget->FindChildGeometries_Helper( CurChild.Geometry, WidgetsToFind, OutResult );
		}	
	}	
}

FGeometry SWidget::FindChildGeometry( const FGeometry& MyGeometry, TSharedRef<SWidget> WidgetToFind ) const
{
	// We just need to find the one WidgetToFind among our descendants.
	TSet< TSharedRef<SWidget> > WidgetsToFind;
	{
		WidgetsToFind.Add( WidgetToFind );
	}
	TMap<TSharedRef<SWidget>, FArrangedWidget> Result;

	FindChildGeometries( MyGeometry, WidgetsToFind, Result );

	return Result.FindChecked( WidgetToFind ).Geometry;
}

int32 SWidget::FindChildUnderMouse( const FArrangedChildren& Children, const FPointerEvent& MouseEvent )
{
	FVector2f AbsoluteCursorLocation = MouseEvent.GetScreenSpacePosition();
	return SWidget::FindChildUnderPosition( Children, AbsoluteCursorLocation );
}

int32 SWidget::FindChildUnderPosition( const FArrangedChildren& Children, const UE::Slate::FDeprecateVector2DParameter& ArrangedSpacePosition )
{
	const int32 NumChildren = Children.Num();
	for( int32 ChildIndex=NumChildren-1; ChildIndex >= 0; --ChildIndex )
	{
		const FArrangedWidget& Candidate = Children[ChildIndex];
		const bool bCandidateUnderCursor = 
			// Candidate is physically under the cursor
			Candidate.Geometry.IsUnderLocation( ArrangedSpacePosition );

		if (bCandidateUnderCursor)
		{
			return ChildIndex;
		}
	}

	return INDEX_NONE;
}

FString SWidget::ToString() const
{
	TStringBuilder<256> StringBuilder;
	StringBuilder << this->TypeOfWidget << " [" << *this->GetReadableLocation() << "]";
	return FString(StringBuilder);
}

FString SWidget::GetTypeAsString() const
{
	return this->TypeOfWidget.ToString();
}

FName SWidget::GetType() const
{
	return TypeOfWidget;
}

FString SWidget::GetReadableLocation() const
{
#if !UE_BUILD_SHIPPING
	TStringBuilder<256> StringBuilder;
	StringBuilder << *FPaths::GetCleanFilename(this->CreatedInLocation.GetPlainNameString()) << "(" << this->CreatedInLocation.GetNumber() << ")";
	return FString(StringBuilder);
#else
	return FString();
#endif
}

FName SWidget::GetCreatedInLocation() const
{
#if !UE_BUILD_SHIPPING
	return CreatedInLocation;
#else
	return NAME_None;
#endif
}

FName SWidget::GetTag() const
{
	return Tag;
}

FSlateColor SWidget::GetForegroundColor() const
{
	static FSlateColor NoColor = FSlateColor::UseForeground();
	return NoColor;
}

FSlateColor SWidget::GetDisabledForegroundColor() const
{
	// By default just return the same as the non-disabled foreground color
	return GetForegroundColor();
}

const FGeometry& SWidget::GetCachedGeometry() const
{
	return GetTickSpaceGeometry();
}

const FGeometry& SWidget::GetTickSpaceGeometry() const
{
	return PersistentState.DesktopGeometry;
}

const FGeometry& SWidget::GetPaintSpaceGeometry() const
{
	return PersistentState.AllottedGeometry;
}

namespace Private
{
	TSharedPtr<FSlateToolTipMetaData> FindOrAddToolTipMetaData(SWidget* Widget)
	{
		TSharedPtr<FSlateToolTipMetaData> Data = Widget->GetMetaData<FSlateToolTipMetaData>();
		if (!Data)
		{
			Data = MakeShared<FSlateToolTipMetaData>();
			Widget->AddMetadata(Data.ToSharedRef());
		}
		return Data;
	}
}

void SWidget::SetToolTipText(const TAttribute<FText>& ToolTipText)
{
	if (ToolTipText.IsSet())
	{
		Private::FindOrAddToolTipMetaData(this)->ToolTip = FSlateApplicationBase::Get().MakeToolTip(ToolTipText);
	}
	else
	{
		RemoveMetaData<FSlateToolTipMetaData>();
	}
}

void SWidget::SetToolTipText( const FText& ToolTipText )
{
	if (!ToolTipText.IsEmptyOrWhitespace())
	{
		Private::FindOrAddToolTipMetaData(this)->ToolTip = FSlateApplicationBase::Get().MakeToolTip(ToolTipText);
	}
	else
	{
		RemoveMetaData<FSlateToolTipMetaData>();
	}
}

void SWidget::SetToolTip(const TAttribute<TSharedPtr<IToolTip>>& InToolTip)
{
	if (InToolTip.IsSet())
	{
		Private::FindOrAddToolTipMetaData(this)->ToolTip = InToolTip;
	}
	else
	{
		RemoveMetaData<FSlateToolTipMetaData>();
	}
}

TSharedPtr<IToolTip> SWidget::GetToolTip()
{
	if (TSharedPtr<FSlateToolTipMetaData> Data = GetMetaData<FSlateToolTipMetaData>())
	{
		return Data->ToolTip.Get();
	}
	return TSharedPtr<IToolTip>();
}

void SWidget::OnToolTipClosing()
{
}

void SWidget::EnableToolTipForceField( const bool bEnableForceField )
{
	bToolTipForceFieldEnabled = bEnableForceField;
}

void SWidget::SetToolTipForceFieldExpansion(const TAttribute<TOptional<FSlateRect>>& InExpansionRect)
{
	const TSharedRef<FSlateToolTipForceFieldExpansionMetaData> DefaultMetaData = FindOrAddMetaData<FSlateToolTipForceFieldExpansionMetaData>();
	DefaultMetaData->Attribute = InExpansionRect;
}

TOptional<FSlateRect> SWidget::GetToolTipForceFieldExpansion() const
{
	TOptional<FSlateRect> Result;
	for (const TSharedRef<FSlateToolTipForceFieldExpansionMetaData>& ExpansionMetaData : GetAllMetaData<FSlateToolTipForceFieldExpansionMetaData>())
	{
		if (ExpansionMetaData->Attribute.IsSet())
		{
			TOptional<FSlateRect> SlateRect = ExpansionMetaData->Attribute.Get();
			if (SlateRect.IsSet())
			{
				if (!Result.IsSet())
				{
					Result = SlateRect;
				}
				else
				{
					Result = Result->Expand(SlateRect.GetValue());
				}
			}
		}
	}
	return Result;
}

bool SWidget::IsDirectlyHovered() const
{
	return FSlateApplicationBase::Get().IsWidgetDirectlyHovered(SharedThis(this));
}

void SWidget::SetVisibility(TAttribute<EVisibility> InVisibility)
{
	VisibilityAttribute.Assign(*this, MoveTemp(InVisibility));
}

void SWidget::SetClipping(EWidgetClipping InClipping)
{
	if (Clipping != InClipping)
	{
		Clipping = InClipping;
		OnClippingChanged();
		// @todo - Fast path should this be Paint?
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SWidget::SetPixelSnapping(EWidgetPixelSnapping InPixelSnappingMethod)
{
	if (PixelSnappingMethod != InPixelSnappingMethod)
	{
		PixelSnappingMethod = InPixelSnappingMethod;
		Invalidate(EInvalidateWidget::Paint);
	}
}

void SWidget::Invalidate(EInvalidateWidgetReason InvalidateReason)
{
	SLATE_CROSS_THREAD_CHECK();

	if (InvalidateReason == EInvalidateWidgetReason::None || !IsConstructed())
	{
		return;
	}

	SCOPED_NAMED_EVENT_TEXT("SWidget::Invalidate", FColor::Orange);

	// Backwards compatibility fix:  Its no longer valid to just invalidate volatility since we need to repaint to cache elements if a widget becomes non-volatile. So after volatility changes force repaint
	if (EnumHasAnyFlags(InvalidateReason, EInvalidateWidgetReason::Volatility))
	{
		InvalidateReason |= EInvalidateWidgetReason::PaintAndVolatility;
	}

	if (EnumHasAnyFlags(InvalidateReason, EInvalidateWidgetReason::Prepass))
	{
		MarkPrepassAsDirty();
		InvalidateReason |= EInvalidateWidgetReason::Layout;
	}

	if (EnumHasAnyFlags(InvalidateReason, EInvalidateWidgetReason::ChildOrder) || !bPrepassLayoutScaleMultiplierSet)
	{
		MarkPrepassAsDirty();
		InvalidateReason |= EInvalidateWidgetReason::Prepass;
		InvalidateReason |= EInvalidateWidgetReason::Layout;
	}

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	if (EnumHasAnyFlags(InvalidateReason, EInvalidateWidgetReason::ChildOrder))
	{
		// This is done here rather than in the FSlateInvalidationRoot to help catching all cases of widgets having wrong parents
		InvalidateCachedSupportsInvalidation();
	}
#endif

	// NB Advanced_InvalidateVolatility needs to be called to update the bCachedVolatility
	const bool bVolatilityChanged = EnumHasAnyFlags(InvalidateReason, EInvalidateWidgetReason::Volatility) ? Advanced_InvalidateVolatility() : false;
	if(FastPathProxyHandle.IsValid(this))
	{
		if (bVolatilityChanged)
		{
			SCOPED_NAMED_EVENT(SWidget_UpdateFastPathVolatility, FColor::Red);

			TSharedPtr<SWidget> ParentWidget = GetParentWidget();

			UpdateFastPathVolatility(ParentWidget.IsValid() ? ParentWidget->IsVolatile() || ParentWidget->IsVolatileIndirectly() : false);

			ensure(!IsVolatile() || IsVolatileIndirectly() || EnumHasAnyFlags(UpdateFlags, EWidgetUpdateFlags::NeedsVolatilePaint));
		}

		FastPathProxyHandle.MarkWidgetDirty_NoCheck(InvalidateReason);
	}
	else
	{
#if WITH_SLATE_DEBUGGING
		FSlateDebugging::BroadcastWidgetInvalidate(this, nullptr, InvalidateReason);
#endif
		UE_TRACE_SLATE_WIDGET_INVALIDATED(this, nullptr, InvalidateReason);
	}
}

void SWidget::SetCursor( const TAttribute< TOptional<EMouseCursor::Type> >& InCursor )
{
	// If bounded or has a valid optional value
	if (InCursor.IsBound() || InCursor.Get().IsSet())
	{
		TSharedPtr<FSlateCursorMetaData> Data = GetMetaData<FSlateCursorMetaData>();
		if (!Data)
		{
			Data = MakeShared<FSlateCursorMetaData>();
			AddMetadata(Data.ToSharedRef());
		}
		Data->Cursor = InCursor;
	}
	else
	{
		RemoveMetaData<FSlateCursorMetaData>();
	}
}

TOptional<EMouseCursor::Type> SWidget::GetCursor() const
{
	if (TSharedPtr<FSlateCursorMetaData> Data = GetMetaData<FSlateCursorMetaData>())
	{
		return Data->Cursor.Get();
	}
	return TOptional<EMouseCursor::Type>();
}

void SWidget::SetDebugInfo( const ANSICHAR* InType, const ANSICHAR* InFile, int32 OnLine, size_t InAllocSize )
{
	TypeOfWidget = InType;

#if STATS
	AllocSize = InAllocSize;
#endif
	INC_MEMORY_STAT_BY(STAT_SlateSWidgetAllocSize, AllocSize);

#if !UE_BUILD_SHIPPING
	CreatedInLocation = FName( InFile );
	CreatedInLocation.SetNumber(OnLine);
#endif

	UE_TRACE_SLATE_WIDGET_DEBUG_INFO(this);
}

void SWidget::OnClippingChanged()
{

}

FSlateRect SWidget::CalculateCullingAndClippingRules(const FGeometry& AllottedGeometry, const FSlateRect& IncomingCullingRect, bool& bClipToBounds, bool& bAlwaysClip, bool& bIntersectClipBounds) const
{
	bClipToBounds = false;
	bIntersectClipBounds = true;
	bAlwaysClip = false;

	if (!bClippingProxy)
	{
		switch (Clipping)
		{
		case EWidgetClipping::ClipToBounds:
			bClipToBounds = true;
			break;
		case EWidgetClipping::ClipToBoundsAlways:
			bClipToBounds = true;
			bAlwaysClip = true;
			break;
		case EWidgetClipping::ClipToBoundsWithoutIntersecting:
			bClipToBounds = true;
			bIntersectClipBounds = false;
			break;
		case EWidgetClipping::OnDemand:
			const float OverflowEpsilon = 1.0f;
			const FVector2f& CurrentSize = GetDesiredSize();
			const FVector2f& LocalSize = AllottedGeometry.GetLocalSize();
			bClipToBounds =
				(CurrentSize.X - OverflowEpsilon) > LocalSize.X ||
				(CurrentSize.Y - OverflowEpsilon) > LocalSize.Y;
			break;
		}
	}

	if (bClipToBounds)
	{
		FSlateRect MyCullingRect(AllottedGeometry.GetRenderBoundingRect(CullingBoundsExtension));

		if (bIntersectClipBounds)
		{
			bool bClipBoundsOverlapping;
			return IncomingCullingRect.IntersectionWith(MyCullingRect, bClipBoundsOverlapping);
		}
		
		return MyCullingRect;
	}

	return IncomingCullingRect;
}

FSlateRect SWidget::GetHitTestBoundingRect() const
{
	return GetPaintSpaceGeometry().GetRenderBoundingRect();
}

int32 SWidget::Paint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const EWidgetUpdateFlags PreviousUpdateFlag = UpdateFlags;

	// TODO, Maybe we should just make Paint non-const and keep OnPaint const.
	SWidget* MutableThis = const_cast<SWidget*>(this);

	INC_DWORD_STAT(STAT_SlateNumPaintedWidgets);
	UE_TRACE_SCOPED_SLATE_WIDGET_PAINT(this);

	// If this widget clips to its bounds, then generate a new clipping rect representing the intersection of the bounding
	// rectangle of the widget's geometry, and the current clipping rectangle.
	bool bClipToBounds, bAlwaysClip, bIntersectClipBounds;

	FSlateRect CullingBounds = CalculateCullingAndClippingRules(AllottedGeometry, MyCullingRect, bClipToBounds, bAlwaysClip, bIntersectClipBounds);

	FWidgetStyle ContentWidgetStyle = FWidgetStyle(InWidgetStyle)
		.BlendOpacity(RenderOpacity);

	// Cache the geometry for tick to allow external users to get the last geometry that was used,
	// or would have been used to tick the Widget.
	FGeometry DesktopSpaceGeometry = AllottedGeometry;
	DesktopSpaceGeometry.AppendTransform(FSlateLayoutTransform(Args.GetWindowToDesktopTransform()));

	{
		UE_TRACE_SCOPED_SLATE_WIDGET_UPDATE(this);
		if (HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsActiveTimerUpdate))
		{
			SCOPE_CYCLE_COUNTER(STAT_SlateExecuteActiveTimers);
			MutableThis->ExecuteActiveTimers(Args.GetCurrentTime(), Args.GetDeltaTime());
		}

		if (HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsTick))
		{
			INC_DWORD_STAT(STAT_SlateNumTickedWidgets);

			SCOPE_CYCLE_COUNTER(STAT_SlateTickWidgets);
			SCOPE_CYCLE_SWIDGET(this);
			MutableThis->Tick(DesktopSpaceGeometry, Args.GetCurrentTime(), Args.GetDeltaTime());
		}
	}

	// the rule our parent has set for us
	const bool bInheritedHittestability = Args.GetInheritedHittestability();
	const bool bOutgoingHittestability = bInheritedHittestability && GetVisibility().AreChildrenHitTestVisible();

#if WITH_SLATE_DEBUGGING
	if (GDebugCulling)
	{
		// When we're debugging culling, don't actually clip, we'll just pretend to, so we can see the effects of
		// any widget doing culling to know if it's doing the right thing.
		bClipToBounds = false;
	}
#endif

	SWidget* PaintParentPtr = const_cast<SWidget*>(Args.GetPaintParent());
	ensure(PaintParentPtr != this);
	if (PaintParentPtr)
	{
		PersistentState.PaintParent = PaintParentPtr->AsShared();
	}
	else
	{
		PaintParentPtr = nullptr;
	}
	
	// @todo This should not do this copy if the clipping state is unset
	PersistentState.InitialClipState = OutDrawElements.GetClippingState();
	PersistentState.LayerId = LayerId;
	PersistentState.bParentEnabled = bParentEnabled;
	PersistentState.bInheritedHittestability = bInheritedHittestability;
	PersistentState.bDeferredPainting = Args.GetDeferredPaint();
	PersistentState.AllottedGeometry = AllottedGeometry;
	PersistentState.DesktopGeometry = DesktopSpaceGeometry;
	PersistentState.WidgetStyle = InWidgetStyle;
	PersistentState.CullingBounds = MyCullingRect;
	PersistentState.InitialPixelSnappingMethod = OutDrawElements.GetPixelSnappingMethod();

	const int32 IncomingUserIndex = Args.GetHittestGrid().GetUserIndex();
	ensure(IncomingUserIndex <= std::numeric_limits<int8>::max()); // shorten to save memory
	PersistentState.IncomingUserIndex = (int8)IncomingUserIndex;

	const int32 IncomingSceneIndex = FSlateApplicationBase::Get().GetRenderer()->GetCurrentSceneIndex();
	ensure(IncomingSceneIndex <= TNumericLimits<int8>::Max());
	PersistentState.IncomingSceneIndex = static_cast<int8>(IncomingSceneIndex);

	PersistentState.IncomingFlowDirection = GSlateFlowDirection;
	PersistentState.SetIsProjectContent(IsProjectContent());

	FPaintArgs UpdatedArgs = Args.WithNewParent(this);
	UpdatedArgs.SetInheritedHittestability(bOutgoingHittestability);

#if WITH_SLATE_DEBUGGING
	if (FastPathProxyHandle.IsValid(this) && PersistentState.CachedElementHandle.HasCachedElements())
	{
		ensureMsgf(FastPathProxyHandle.GetProxy().Visibility.IsVisible()
			, TEXT("The widget '%s' is collapsed or not visible. It should not have Cached Element."), *FReflectionMetaData::GetWidgetDebugInfo(this));
	}
#endif

	OutDrawElements.PushPaintingWidget(*this, LayerId, PersistentState.CachedElementHandle);

	if (bOutgoingHittestability)
	{
		Args.GetHittestGrid().AddWidget(MutableThis, 0, LayerId, FastPathProxyHandle.GetWidgetSortOrder());
	}

	if (bClipToBounds)
	{
		// This sets up the clip state for any children NOT myself
		FSlateClippingZone ClippingZone(AllottedGeometry);
		ClippingZone.SetShouldIntersectParent(bIntersectClipBounds);
		ClippingZone.SetAlwaysClip(bAlwaysClip);
		OutDrawElements.PushClip(ClippingZone);
	}

	const bool bNewPixelSnappingMethod = PixelSnappingMethod != EWidgetPixelSnapping::Inherit;
	
	if (bNewPixelSnappingMethod)
	{
		OutDrawElements.PushPixelSnappingMethod(PixelSnappingMethod);
	}

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::BeginWidgetPaint.Broadcast(this, UpdatedArgs, AllottedGeometry, CullingBounds, OutDrawElements, LayerId);
#endif

	// Establish the flow direction if we're changing from inherit.
	// FOR RB mode, this should first set GSlateFlowDirection to the incoming state that was cached for the widget, then paint
	// will override it here to reflow is needed.
	TGuardValue<EFlowDirection> FlowGuard(GSlateFlowDirection, ComputeFlowDirection());

#if WITH_SLATE_DEBUGGING
	TArray<TWeakPtr<const SWidget>, TInlineAllocator<16>> DebugChildWidgetsToPaint;

	if ((GSlateIsOnFastUpdatePath && GSlateEnsureAllVisibleWidgetsPaint) || GSlateEnsureOutgoingLayerId)
	{
		// Don't check for invalidation roots a completely different set of rules apply to those widgets.
		if (!Advanced_IsInvalidationRoot())
		{
			MutableThis->GetAllChildren()->ForEachWidget([&DebugChildWidgetsToPaint](const SWidget& Child)
				{
					if (Child.GetVisibility().IsVisible())
					{
						DebugChildWidgetsToPaint.Add(Child.AsShared());
					}
				});
		}
	}
#endif
	
	int32 NewLayerId = 0;
	{
		LLM_SCOPE_BYNAME("UI/Slate/OnPaint");
#if UE_TRACE_ASSET_METADATA_ENABLED
		FName AssetName = NAME_None;
		FName ClassName = NAME_None;
		FName PackageName = NAME_None;
		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(AssetMetadataChannel))
		{
			TSharedPtr<FReflectionMetaData> AssetMetaData = FReflectionMetaData::GetWidgetOrParentMetaData(this);
			if (AssetMetaData.IsValid())
			{
				if (const UObject* AssetPtr = AssetMetaData->Asset.Get())
				{
					AssetName = AssetMetaData->Name;
					ClassName = AssetMetaData->Class.Get()->GetFName();
					PackageName = AssetPtr->GetPackage()->GetFName();
				}
			}
		}
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(PackageName, ELLMTagSet::Assets);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(ClassName, ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_SCOPE_ASSET_FNAME(AssetName, ClassName, PackageName);
#endif
		// Paint the geometry of this widget.
		OutDrawElements.SetIsInProjectContent(IsProjectContent());
		NewLayerId = OnPaint(UpdatedArgs, AllottedGeometry, CullingBounds, OutDrawElements, LayerId, ContentWidgetStyle, bParentEnabled);
		OutDrawElements.SetIsInProjectContent(false);
	}

	// Just repainted
	MutableThis->RemoveUpdateFlags(EWidgetUpdateFlags::NeedsRepaint);
#if WITH_SLATE_DEBUGGING
	MutableThis->Debug_UpdateLastPaintFrame();
#endif

	// Detect children that should have been painted, but were skipped during the paint process.
	// this will result in geometry being left on screen and not cleared, because it's visible, yet wasn't painted.
#if WITH_SLATE_DEBUGGING
	if (GSlateIsOnFastUpdatePath && GSlateEnsureAllVisibleWidgetsPaint && !GIntraFrameDebuggingGameThread)
	{
		for (TWeakPtr<const SWidget>& DebugChildThatShouldHaveBeenPaintedPtr : DebugChildWidgetsToPaint)
		{
			if (TSharedPtr<const SWidget> DebugChild = DebugChildThatShouldHaveBeenPaintedPtr.Pin())
			{
				if (DebugChild->GetVisibility().IsVisible())
				{
					if (DebugChild->Debug_GetLastPaintFrame() != GFrameNumber)
					{
						ensureAlwaysMsgf(false, TEXT("The widget '%s' was visible, but never painted. This means it was skipped during painting, without alerting the fast path."), *FReflectionMetaData::GetWidgetPath(DebugChild.Get()));
						CVarSlateEnsureAllVisibleWidgetsPaint->Set(false, CVarSlateEnsureAllVisibleWidgetsPaint->GetFlags());
					}
				}
			}
			else
			{
				CVarSlateEnsureAllVisibleWidgetsPaint->Set(false, CVarSlateEnsureAllVisibleWidgetsPaint->GetFlags());
				ensureAlwaysMsgf(false, TEXT("A widget was destroyed while painting it's parent. This is not supported by the Invalidation system."));
			}
		}
	}

	if (GSlateEnsureOutgoingLayerId && !GIntraFrameDebuggingGameThread)
	{
		for (TWeakPtr<const SWidget>& DebugChildWeakPtr : DebugChildWidgetsToPaint)
		{
			if (TSharedPtr<const SWidget> DebugChild = DebugChildWeakPtr.Pin())
			{
				const bool bIsChildDeferredPaint = DebugChild->GetPersistentState().bDeferredPainting;
				if (!bIsChildDeferredPaint && DebugChild->GetVisibility().IsVisible() && DebugChild->Debug_GetLastPaintFrame() == GFrameNumber)
				{
					if (NewLayerId < DebugChild->GetPersistentState().OutgoingLayerId)
					{
						ensureAlwaysMsgf(false, TEXT("The widget '%s' Outgoing Layer Id is bigger than its parent."), *FReflectionMetaData::GetWidgetPath(DebugChild.Get()));
						CVarSlateEnsureOutgoingLayerId->Set(false, CVarSlateEnsureOutgoingLayerId->GetFlags());
					}
				}
			}
		}
	}
#endif

	// Draw the clipping zone if we've got clipping enabled
#if WITH_SLATE_DEBUGGING
	FSlateDebugging::EndWidgetPaint.Broadcast(this, OutDrawElements, NewLayerId);

	if (GShowClipping && bClipToBounds)
	{
		FSlateClippingZone ClippingZone(AllottedGeometry);

		TArray<FVector2f> Points;
		Points.Add(FVector2f(ClippingZone.TopLeft));
		Points.Add(FVector2f(ClippingZone.TopRight));
		Points.Add(FVector2f(ClippingZone.BottomRight));
		Points.Add(FVector2f(ClippingZone.BottomLeft));
		Points.Add(FVector2f(ClippingZone.TopLeft));

		const bool bAntiAlias = true;
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			NewLayerId,
			FPaintGeometry(),
			MoveTemp(Points),
			ESlateDrawEffect::None,
			ClippingZone.IsAxisAligned() ? FLinearColor::Yellow : FLinearColor::Red,
			bAntiAlias,
			2.0f);
	}
#endif // WITH_SLATE_DEBUGGING

	if (bClipToBounds)
	{
		OutDrawElements.PopClip();
	}

	if (bNewPixelSnappingMethod)
	{
		OutDrawElements.PopPixelSnappingMethod();
	}

#if PLATFORM_UI_NEEDS_FOCUS_OUTLINES
	// Check if we need to show the keyboard focus ring, this is only necessary if the widget could be focused.
	if (bCanSupportFocus && SupportsKeyboardFocus())
	{
		bool bShowUserFocus = FSlateApplicationBase::Get().ShowUserFocus(SharedThis(this));
		if (bShowUserFocus)
		{
			const FSlateBrush* BrushResource = GetFocusBrush();

			if (BrushResource != nullptr)
			{
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					NewLayerId,
					AllottedGeometry.ToPaintGeometry(),
					BrushResource,
					ESlateDrawEffect::None,
					BrushResource->GetTint(InWidgetStyle)
				);
			}
		}
	}
#endif

	FSlateCachedElementsHandle NewCacheHandle = OutDrawElements.PopPaintingWidget(*this);
	if (OutDrawElements.ShouldResolveDeferred())
	{
		NewLayerId = OutDrawElements.PaintDeferred(NewLayerId, MyCullingRect);
	}

	MutableThis->UpdateWidgetProxy(NewLayerId, NewCacheHandle);

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::BroadcastWidgetUpdatedByPaint(this, PreviousUpdateFlag);
#endif

	return NewLayerId;
}

float SWidget::GetRelativeLayoutScale(int32 ChildIndex, float LayoutScaleMultiplier) const
{
	return 1.0f;
}

void SWidget::ArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, bool bUpdateAttributes) const
{
#if WITH_VERY_VERBOSE_SLATE_STATS
	SCOPED_NAMED_EVENT(SWidget_ArrangeChildren, FColor::Black);
#endif

	if (bUpdateAttributes)
	{
		// Update the Widgets visibility before getting the ArrangeChildren
		//const-casting for TSlateAttribute has the same behavior as previously with TAttribute. The const was hidden from the user.
		FSlateAttributeMetaData::UpdateChildrenOnlyVisibilityAttributes(const_cast<SWidget&>(*this), FSlateAttributeMetaData::EInvalidationPermission::DelayInvalidation, false);
	}

	OnArrangeChildren(AllottedGeometry, ArrangedChildren);
}

void SWidget::Prepass_Internal(float InLayoutScaleMultiplier)
{
#if WITH_SLATE_DEBUGGING
	FSlateDebugging::BeginWidgetPrepass.Broadcast(this);
#endif

	PrepassLayoutScaleMultiplierValue = InLayoutScaleMultiplier;
	bPrepassLayoutScaleMultiplierSet = true;

	bool bShouldPrepassChildren = true;
	if (bHasCustomPrepass)
	{
		bShouldPrepassChildren = CustomPrepass(InLayoutScaleMultiplier);
	}

	if (bCanHaveChildren && bShouldPrepassChildren)
	{
		// Cache child desired sizes first. This widget's desired size is
		// a function of its children's sizes.
		FChildren* MyChildren = this->GetChildren();
		const int32 NumChildren = MyChildren->Num();
		Prepass_ChildLoop(InLayoutScaleMultiplier, MyChildren);
		ensure(NumChildren == MyChildren->Num());
	}

	{
		// Cache this widget's desired size.
		CacheDesiredSize(GetPrepassLayoutScaleMultiplier());
		bNeedsPrepass = false;
#if WITH_SLATE_DEBUGGING
		Debug_UpdateLastPrepassFrame();
		FSlateDebugging::EndWidgetPrepass.Broadcast(this);
#endif
	}
}

void SWidget::Prepass_ChildLoop(float InLayoutScaleMultiplier, FChildren* MyChildren)
{
	int32 ChildIndex = 0;
	SWidget* Self = this;
	auto ForEachPred = [Self, &ChildIndex, InLayoutScaleMultiplier](SWidget& Child)
	{
		const bool bUpdateAttributes = Child.HasRegisteredSlateAttribute() && Child.IsAttributesUpdatesEnabled() && !GSlateIsOnFastProcessInvalidation;
		if (bUpdateAttributes)
		{
			FSlateAttributeMetaData::UpdateOnlyVisibilityAttributes(Child, FSlateAttributeMetaData::EInvalidationPermission::AllowInvalidationIfConstructed);
		}

		if (Child.GetVisibility() != EVisibility::Collapsed)
		{
			if (bUpdateAttributes)
			{
#if WITH_SLATE_DEBUGGING
				EVisibility PreviousVisibility = Self->GetVisibility();
				int32 PreviousAllChildrenNum = Child.GetAllChildren()->Num();
#endif

				FSlateAttributeMetaData::UpdateExceptVisibilityAttributes(Child, FSlateAttributeMetaData::EInvalidationPermission::AllowInvalidationIfConstructed);

#if WITH_SLATE_DEBUGGING
				ensureMsgf(PreviousVisibility == Self->GetVisibility(), TEXT("The visibility of widget '%s' doesn't match the previous visibility after the attribute update."), *FReflectionMetaData::GetWidgetDebugInfo(Self));
				ensureMsgf(PreviousAllChildrenNum == Child.GetAllChildren()->Num(), TEXT("The number of child of widget '%s' doesn't match the previous count after the attribute update."), *FReflectionMetaData::GetWidgetDebugInfo(Self));
#endif
			}

			const float ChildLayoutScaleMultiplier = Self->bHasRelativeLayoutScale
				? InLayoutScaleMultiplier * Self->GetRelativeLayoutScale(ChildIndex, InLayoutScaleMultiplier)
				: InLayoutScaleMultiplier;

			// Inherit project-content status by default.  Children can opt out (overwrite) in custom prepass.
			Child.bIsProjectContent = Self->bIsProjectContent || Self->bIsProjectContentParent;

			// Recur: Descend down the widget tree.
			Child.Prepass_Internal(ChildLayoutScaleMultiplier);
		}
		else
		{
			// If the child widget is collapsed, we need to store the new layout scale it will have when 
			// it is finally visible and invalidate it's prepass so that it gets that when its visibility
			// is finally invalidated.
			Child.MarkPrepassAsDirty();
			Child.PrepassLayoutScaleMultiplierValue = Self->bHasRelativeLayoutScale
				? InLayoutScaleMultiplier * Self->GetRelativeLayoutScale(ChildIndex, InLayoutScaleMultiplier)
				: InLayoutScaleMultiplier;
			Child.bPrepassLayoutScaleMultiplierSet = true;
		}
		++ChildIndex;
	};

	MyChildren->ForEachWidget(ForEachPred);
}

TSharedRef<FActiveTimerHandle> SWidget::RegisterActiveTimer(float TickPeriod, FWidgetActiveTimerDelegate TickFunction)
{
	TSharedRef<FActiveTimerHandle> ActiveTimerHandle = MakeShared<FActiveTimerHandle>(TickPeriod, TickFunction, FSlateApplicationBase::Get().GetCurrentTime() + TickPeriod);
	FSlateApplicationBase::Get().RegisterActiveTimer(ActiveTimerHandle);

	if (bHasActiveTimers)
	{
		GetMetaData<UE::Slate::FActiveTimersMetaData>()->ActiveTimers.Add(ActiveTimerHandle);
	}
	else
	{
		TSharedRef<UE::Slate::FActiveTimersMetaData> NewActiveTimers = MakeShared<UE::Slate::FActiveTimersMetaData>();
		NewActiveTimers->ActiveTimers.Add(ActiveTimerHandle);
		AddMetadata(NewActiveTimers);
		bHasActiveTimers = true;
	}	

	AddUpdateFlags(EWidgetUpdateFlags::NeedsActiveTimerUpdate);

	return ActiveTimerHandle;
}

void SWidget::UnRegisterActiveTimer(const TSharedRef<FActiveTimerHandle>& ActiveTimerHandle)
{
	if (bHasActiveTimers)
	{
		TSharedRef<UE::Slate::FActiveTimersMetaData> ActiveTimersMetaData = GetMetaData<UE::Slate::FActiveTimersMetaData>().ToSharedRef();
		ActiveTimersMetaData->ActiveTimers.RemoveSingle(ActiveTimerHandle);
		if (ActiveTimersMetaData->ActiveTimers.Num() == 0)
		{
			RemoveMetaData(ActiveTimersMetaData);
			bHasActiveTimers = false;
			RemoveUpdateFlags(EWidgetUpdateFlags::NeedsActiveTimerUpdate);
		}
	}

	if (FSlateApplicationBase::IsInitialized())
	{
		FSlateApplicationBase::Get().UnRegisterActiveTimer(ActiveTimerHandle);
	}
}

void SWidget::ExecuteActiveTimers(double CurrentTime, float DeltaTime)
{
	checkf(bHasActiveTimers, TEXT("The flag EWidgetUpdateFlags::NeedsActiveTimerUpdate should match with the bHasActiveTimers flag"));

	// loop over the registered tick handles and execute them, removing them if necessary.
	TSharedRef<UE::Slate::FActiveTimersMetaData> ActiveTimersMetaData = GetMetaData<UE::Slate::FActiveTimersMetaData>().ToSharedRef();
	for (int32 Index = 0; Index < ActiveTimersMetaData->ActiveTimers.Num();)
	{
		TWeakPtr<FActiveTimerHandle> WeakActiveTimerHandle = ActiveTimersMetaData->ActiveTimers[Index];
		EActiveTimerReturnType Result = ActiveTimersMetaData->ActiveTimers[Index]->ExecuteIfPending(CurrentTime, DeltaTime);
		if (Result == EActiveTimerReturnType::Continue)
		{
			++Index;
		}
		else
		{
			// Possible that execution unregistered the timer
			if (TSharedPtr<FActiveTimerHandle> ActiveTimerHandle = WeakActiveTimerHandle.Pin())
			{
				if (FSlateApplicationBase::IsInitialized())
				{
					FSlateApplicationBase::Get().UnRegisterActiveTimer(ActiveTimerHandle.ToSharedRef());
				}
				ActiveTimersMetaData->ActiveTimers.RemoveSingle(ActiveTimerHandle.ToSharedRef());
			}
		}
	}

	if (ActiveTimersMetaData->ActiveTimers.Num() == 0)
	{
		RemoveMetaData(ActiveTimersMetaData);
		bHasActiveTimers = false;
		RemoveUpdateFlags(EWidgetUpdateFlags::NeedsActiveTimerUpdate);
	}
}

namespace Private
{
	TSharedPtr<FSlateMouseEventsMetaData> FindOrAddMouseEventsMetaData(SWidget* Widget)
	{
		TSharedPtr<FSlateMouseEventsMetaData> Data = Widget->GetMetaData<FSlateMouseEventsMetaData>();
		if (!Data)
		{
			Data = MakeShared<FSlateMouseEventsMetaData>();
			Widget->AddMetadata(Data.ToSharedRef());
		}
		return Data;
	}
}

void SWidget::SetOnMouseButtonDown(FPointerEventHandler EventHandler)
{
	Private::FindOrAddMouseEventsMetaData(this)->MouseButtonDownHandle = EventHandler;
}

void SWidget::SetOnMouseButtonUp(FPointerEventHandler EventHandler)
{
	Private::FindOrAddMouseEventsMetaData(this)->MouseButtonUpHandle = EventHandler;
}

void SWidget::SetOnMouseMove(FPointerEventHandler EventHandler)
{
	Private::FindOrAddMouseEventsMetaData(this)->MouseMoveHandle = EventHandler;
}

void SWidget::SetOnMouseDoubleClick(FPointerEventHandler EventHandler)
{
	Private::FindOrAddMouseEventsMetaData(this)->MouseDoubleClickHandle = EventHandler;
}

void SWidget::SetOnMouseEnter(FNoReplyPointerEventHandler EventHandler)
{
	Private::FindOrAddMouseEventsMetaData(this)->MouseEnterHandler = EventHandler;
}

void SWidget::SetOnMouseLeave(FSimpleNoReplyPointerEventHandler EventHandler)
{
	Private::FindOrAddMouseEventsMetaData(this)->MouseLeaveHandler = EventHandler;
}

void SWidget::AddMetadataInternal(const TSharedRef<ISlateMetaData>& AddMe)
{
	int32 Index = MetaData.Add(AddMe);
	checkf(Index != 0 || !HasRegisteredSlateAttribute(), TEXT("The first slot is reserved for SlateAttribute"));

#if UE_WITH_SLATE_DEBUG_FIND_WIDGET_REFLECTION_METADATA || UE_SLATE_TRACE_ENABLED
	if (AddMe->IsOfType<FReflectionMetaData>())
	{
#if UE_WITH_SLATE_DEBUG_FIND_WIDGET_REFLECTION_METADATA
		FReflectionMetaData& Reflection = static_cast<FReflectionMetaData&>(AddMe.Get());
		if (Reflection.Name == FindWidgetMetaData::WidgeName && Reflection.Asset.Get() && Reflection.Asset.Get()->GetFName() == FindWidgetMetaData::AssetName)
		{
			FindWidgetMetaData::FoundWidget = this;
		}
#endif
#if UE_SLATE_TRACE_ENABLED
		UE_TRACE_SLATE_WIDGET_DEBUG_INFO(this);
#endif
	}
#endif

	OnAddedMetadata(AddMe);
}

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION

bool SWidget::SupportsInvalidation() const
{
	return bSupportsInvalidation;
}

bool SWidget::SupportsInvalidationRecursive(EInvalidationStrategy InInvalidationStrategy) const
{
	if ((InInvalidationStrategy == EInvalidationStrategy::UseCachedValue) &
		(CachedSupportsInvalidationRecursive != ECachedInvalidationResult::NeedsRefresh))
	{
		return CachedSupportsInvalidationRecursive == ECachedInvalidationResult::SupportsInvalidation;
	}

	if (!SupportsInvalidation())
	{
		check(CachedSupportsInvalidationRecursive == ECachedInvalidationResult::ClassNotSupportingInvalidation);
		return false;
	}
	check(CachedSupportsInvalidationRecursive != ECachedInvalidationResult::ClassNotSupportingInvalidation);


	const ECachedInvalidationResult PreviousCachedValue = CachedSupportsInvalidationRecursive;
	// Sets the value to Refreshing to avoid checking this widget multiple times
	CachedSupportsInvalidationRecursive = ECachedInvalidationResult::Refreshing;

	FChildren* Children = const_cast<SWidget*>(this)->GetAllChildren();
	for (int32 ChildIndex = 0; ChildIndex < Children->Num(); ++ChildIndex)
	{
		const TSharedRef<SWidget> ChildWidget = Children->GetChildAt(ChildIndex);
		if (ChildWidget == SNullWidget::NullWidget)
		{
			continue;
		}

		if (!ChildWidget->ParentWidgetPtr.HasSameObject(this))
		{
			// Here the child  widget has another widget as parent, so we need to mark them as not supporting invalidation
			CachedSupportsInvalidationRecursive = ECachedInvalidationResult::ChildrenNotSupportingInvalidation;
			if (ChildWidget->CachedSupportsInvalidationRecursive != ECachedInvalidationResult::ClassNotSupportingInvalidation)
			{
				ChildWidget->CachedSupportsInvalidationRecursive = ECachedInvalidationResult::InvalidParent;
			}
			break;
		}

		// If we had an invalid parent but we don't have one anymore, we recheck if we are invalidation compatible, otherwise we use the cached value
		const EInvalidationStrategy Strategy = ChildWidget->CachedSupportsInvalidationRecursive == ECachedInvalidationResult::InvalidParent ?
			EInvalidationStrategy::RecomputeSelf : EInvalidationStrategy::UseCachedValue;

		if (!ChildWidget->SupportsInvalidationRecursive(Strategy))
		{
			CachedSupportsInvalidationRecursive = ECachedInvalidationResult::ChildrenNotSupportingInvalidation;
			break;
		}
	}

	// If we have checked all the children and none of them were invalid, then we are supporting invalidation
	if (CachedSupportsInvalidationRecursive == ECachedInvalidationResult::Refreshing)
	{
		CachedSupportsInvalidationRecursive = ECachedInvalidationResult::SupportsInvalidation;
	}

	// If the value has changed, we invalidate the parent unless it is currently recomputing
	if (CachedSupportsInvalidationRecursive != PreviousCachedValue)
	{
		const TSharedPtr<SWidget> ParentWidget = GetParentWidget();
		if (ParentWidget && ParentWidget->CachedSupportsInvalidationRecursive != ECachedInvalidationResult::Refreshing)
		{
			if (CachedSupportsInvalidationRecursive != ECachedInvalidationResult::SupportsInvalidation)
			{
				// If we supported invalidation but we do not anymore, tell the parent they do not support invalidation anymore, no need for them to check
				ParentWidget->SetSelfAndParentsAsNotSupportingInvalidation();
			}
			else
			{
				// If we now support invalidation, ask the parent to recompute
				ParentWidget->SupportsInvalidationRecursive(EInvalidationStrategy::RecomputeSelf);
			}
		}
	}

	return CachedSupportsInvalidationRecursive == ECachedInvalidationResult::SupportsInvalidation;
}

void SWidget::SetSelfAndParentsAsNotSupportingInvalidation()
{
	TSharedPtr<SWidget> Widget = (this == &SNullWidget::NullWidget.Get()) ? GetParentWidget() : AsShared();
	while (Widget)
	{
		// If this widget was supporting invalidation and it does not anymore, it needs to tell its parent.
		// Otherwise, its parent already knew so no need to do anything.
		if (Widget->CachedSupportsInvalidationRecursive == ECachedInvalidationResult::SupportsInvalidation)
		{
			Widget->CachedSupportsInvalidationRecursive = ECachedInvalidationResult::ChildrenNotSupportingInvalidation;
			Widget = Widget->GetParentWidget();
		}
		else
		{
			return;
		}
	}
}

UE_TRACE_EVENT_BEGIN(Cpu, SWidget_InvalidateCachedSupportsInvalidation, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, CreatedInLocation)
UE_TRACE_EVENT_END()

void SWidget::InvalidateCachedSupportsInvalidation()
{
	if (!DoesSharedInstanceExist())
	{
		return;
	}

#if UE_SLATE_TRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, SWidget_InvalidateCachedSupportsInvalidation, SlateChannel)
		<< SWidget_InvalidateCachedSupportsInvalidation.CreatedInLocation(*GetCreatedInLocation().ToString());
#endif // UE_SLATE_TRACE_ENABLED

	TSharedPtr<SWidget> Widget = (this == &SNullWidget::NullWidget.Get()) ? GetParentWidget() : AsShared();
	while (Widget)
	{
		if ((Widget->CachedSupportsInvalidationRecursive == ECachedInvalidationResult::ClassNotSupportingInvalidation)
			| (Widget->CachedSupportsInvalidationRecursive == ECachedInvalidationResult::NeedsRefresh))
		{
			return;
		}

		if (GSlateIsOnFastUpdatePath && Widget->Advanced_IsInvalidationRoot())
		{
			if (!Widget->SupportsInvalidationRecursive(EInvalidationStrategy::RecomputeSelf))
			{
				// If we are currently in fast path, we want to stop and do a slow path as we might have invalid widgets that have been added
				const_cast<FSlateInvalidationRoot*>(Widget->Advanced_AsInvalidationRoot())->InvalidateRootLayout(this);
			}
			// We do not need to refresh the parents of this widget as `SupportsInvalidationRecursive` would have taken care of it if necessary.
			return;
		}

		Widget->CachedSupportsInvalidationRecursive = ECachedInvalidationResult::NeedsRefresh;
		Widget = Widget->GetParentWidget();
	}
}

void SWidget::SetClassSupportsInvalidation(bool bClassSupportsInvalidation)
{
	bSupportsInvalidation = bClassSupportsInvalidation;
	CachedSupportsInvalidationRecursive = bSupportsInvalidation ?
		ECachedInvalidationResult::NeedsRefresh : ECachedInvalidationResult::ClassNotSupportingInvalidation;
}

#endif // ~UE_SLATE_WITH_DYNAMIC_INVALIDATION

#if WITH_ACCESSIBILITY
TSharedRef<FSlateAccessibleWidget> SWidget::CreateAccessibleWidget()
{
	return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleWidget(AsShared()));
}

void SWidget::SetAccessibleBehavior(EAccessibleBehavior InBehavior, const TAttribute<FText>& InText, EAccessibleType AccessibleType)
{
	EAccessibleBehavior& Behavior = (AccessibleType == EAccessibleType::Main) ? AccessibleBehavior : AccessibleSummaryBehavior;

	if (InBehavior == EAccessibleBehavior::Custom)
	{
		TWidgetSparseAnnotation<TAttribute<FText>>& AccessibleTextAnnotation = (AccessibleType == EAccessibleType::Main) ? AccessibleText : AccessibleSummaryText;
		AccessibleTextAnnotation.FindOrAdd(this) = InText;
	}
	else if (Behavior == EAccessibleBehavior::Custom)
	{
		TWidgetSparseAnnotation<TAttribute<FText>>& AccessibleTextAnnotation = (AccessibleType == EAccessibleType::Main) ? AccessibleText : AccessibleSummaryText;
		AccessibleTextAnnotation.Remove(this);
	}

	if (Behavior != InBehavior)
	{
		const bool bWasAccessible = Behavior != EAccessibleBehavior::NotAccessible;
		Behavior = InBehavior;
		if (AccessibleType == EAccessibleType::Main && bWasAccessible != (Behavior != EAccessibleBehavior::NotAccessible))
		{
			FSlateApplicationBase::Get().GetAccessibleMessageHandler()->MarkDirty();
		}
	}
}

void SWidget::SetCanChildrenBeAccessible(bool InCanChildrenBeAccessible)
{
	if (bCanChildrenBeAccessible != InCanChildrenBeAccessible)
	{
		bCanChildrenBeAccessible = InCanChildrenBeAccessible;
		FSlateApplicationBase::Get().GetAccessibleMessageHandler()->MarkDirty();
	}
}

FText SWidget::GetAccessibleText(EAccessibleType AccessibleType) const
{
	const EAccessibleBehavior Behavior = (AccessibleType == EAccessibleType::Main) ? AccessibleBehavior : AccessibleSummaryBehavior;
	const EAccessibleBehavior OtherBehavior = (AccessibleType == EAccessibleType::Main) ? AccessibleSummaryBehavior : AccessibleBehavior;

	switch (Behavior)
	{
	case EAccessibleBehavior::Custom:
	{
		const TAttribute<FText>* Text = (AccessibleType == EAccessibleType::Main) ? AccessibleText.Find(this) : AccessibleSummaryText.Find(this);
		return Text->Get(FText::GetEmpty());
	}
	case EAccessibleBehavior::Summary:
		return GetAccessibleSummary();
	case EAccessibleBehavior::ToolTip:
	{
		//TODO should use GetToolTip
		if (TSharedPtr<FSlateToolTipMetaData> Data = GetMetaData<FSlateToolTipMetaData>())
		{
			if (TSharedPtr<IToolTip> ToolTip = Data->ToolTip.Get())
			{
				if (ToolTip && !ToolTip->IsEmpty())
				{
					return ToolTip->GetContentWidget()->GetAccessibleText(EAccessibleType::Main);
				}
			}
		}
		break;
	}
	case EAccessibleBehavior::Auto:
		// Auto first checks if custom text was set. This should never happen with user-defined values as custom should be
		// used instead in that case - however, this will be used for widgets with special default text such as TextBlocks.
		// If no text is found, then it will attempt to use the other variable's text, so that a developer can do things like
		// leave Summary on Auto, set Main to Custom, and have Summary automatically use Main's value without having to re-type it.
		TOptional<FText> DefaultText = GetDefaultAccessibleText(AccessibleType);
		if (DefaultText.IsSet())
		{
			return DefaultText.GetValue();
		}
		switch (OtherBehavior)
		{
		case EAccessibleBehavior::Custom:
		case EAccessibleBehavior::ToolTip:
			return GetAccessibleText(AccessibleType == EAccessibleType::Main ? EAccessibleType::Summary : EAccessibleType::Main);
		case EAccessibleBehavior::NotAccessible:
		case EAccessibleBehavior::Summary:
			return GetAccessibleSummary();
		}
		break;
	}
	return FText::GetEmpty();
}

TOptional<FText> SWidget::GetDefaultAccessibleText(EAccessibleType AccessibleType) const
{
	return TOptional<FText>();
}

FText SWidget::GetAccessibleSummary() const
{
	FTextBuilder Builder;
	FChildren* Children = const_cast<SWidget*>(this)->GetChildren();
	if (Children)
	{
		for (int32 i = 0; i < Children->Num(); ++i)
		{
			FText Text = Children->GetChildAt(i)->GetAccessibleText(EAccessibleType::Summary);
			if (!Text.IsEmpty())
			{
				Builder.AppendLine(Text);
			}
		}
	}
	return Builder.ToText();
}

bool SWidget::IsAccessible() const
{
	if (AccessibleBehavior == EAccessibleBehavior::NotAccessible)
	{
		return false;
	}

	TSharedPtr<SWidget> Parent = GetParentWidget();
	while (Parent.IsValid())
	{
		if (!Parent->CanChildrenBeAccessible())
		{
			return false;
		}
		Parent = Parent->GetParentWidget();
	}
	return true;
}

EAccessibleBehavior SWidget::GetAccessibleBehavior(EAccessibleType AccessibleType) const
{
	return AccessibleType == EAccessibleType::Main ? AccessibleBehavior : AccessibleSummaryBehavior;
}

bool SWidget::CanChildrenBeAccessible() const
{
	return bCanChildrenBeAccessible;
}

#endif

#if SLATE_CULL_WIDGETS

bool SWidget::IsChildWidgetCulled(const FSlateRect& MyCullingRect, const FArrangedWidget& ArrangedChild) const
{
	// If we've enabled global invalidation it's safe to run the culling logic and just 'stop' drawing
	// a widget, that widget has to be given an opportunity to paint, as well as all its children, the
	// only correct way is to remove the widget from the tree, or to change the visibility of it.
	if (GSlateIsOnFastUpdatePath)
	{
		return false;
	}

	// We add some slack fill to the culling rect to deal with the common occurrence
	// of widgets being larger than their root level widget is.  Happens when nested child widgets
	// inflate their rendering bounds to render beyond their parent (the child of this panel doing the culling), 
	// or using render transforms.  In either case, it introduces offsets to a bounding volume we don't 
	// actually know about or track in slate, so we have have two choices.
	//    1) Don't cull, set SLATE_CULL_WIDGETS to 0.
	//    2) Cull with a slack fill amount users can adjust.
	const FSlateRect CullingRectWithSlack = MyCullingRect.ScaleBy(GCullingSlackFillPercent);

	// 1) We check if the rendered bounding box overlaps with the culling rect.  Which is so that
	//    a render transformed element is never culled if it would have been visible to the user.
	if (FSlateRect::DoRectanglesIntersect(CullingRectWithSlack, ArrangedChild.Geometry.GetRenderBoundingRect()))
	{
		return false;
	}

	// 2) We also check the layout bounding box to see if it overlaps with the culling rect.  The
	//    reason for this is a bit more nuanced.  Suppose you dock a widget on the screen on the side
	//    and you want have it animate in and out of the screen.  Even though the layout transform 
	//    keeps the widget on the screen, the render transform alone would have caused it to be culled
	//    and therefore not ticked or painted.  The best way around this for now seems to be to simply
	//    check both rects to see if either one is overlapping the culling volume.
	if (FSlateRect::DoRectanglesIntersect(CullingRectWithSlack, ArrangedChild.Geometry.GetLayoutBoundingRect()))
	{
		return false;
	}

	// There's a special condition if the widget's clipping state is set does not intersect with clipping bounds, they in effect
	// will be setting a new culling rect, so let them pass being culling from this step.
	if (ArrangedChild.Widget->GetClipping() == EWidgetClipping::ClipToBoundsWithoutIntersecting)
	{
		return false;
	}

	return true;
}

#endif

#undef UE_WITH_SLATE_DEBUG_FIND_WIDGET_REFLECTION_METADATA
