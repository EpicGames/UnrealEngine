// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SWindow.h"
#include "Application/SlateWindowHelper.h"
#include "Application/SlateApplicationBase.h"
#include "Debugging/SlateCrashReporterHandler.h"
#include "Layout/WidgetPath.h"
#include "Input/HittestGrid.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Widgets/Images/SImage.h"

#if WITH_SLATE_DEBUGGING
#include "ProfilingDebugging/CsvProfiler.h"
#endif

#if WITH_ACCESSIBILITY
#include "Widgets/Accessibility/SlateCoreAccessibleWidgets.h"
#endif

FOverlayPopupLayer::FOverlayPopupLayer(const TSharedRef<SWindow>& InitHostWindow, const TSharedRef<SWidget>& InitPopupContent, TSharedPtr<SOverlay> InitOverlay)
	: FPopupLayer(InitHostWindow, InitPopupContent)
	, HostWindow(InitHostWindow)
	, Overlay(InitOverlay)
{
	Overlay->AddSlot()
	[
		InitPopupContent
	];
}

void FOverlayPopupLayer::Remove()
{
	Overlay->RemoveSlot(GetContent());
}

FSlateRect FOverlayPopupLayer::GetAbsoluteClientRect()
{
	return HostWindow->GetClientRectInScreen();
}


/**
 * An internal overlay used by Slate to support in-window pop ups and tooltips.
 * The overlay ignores DPI scaling when it does its own arrangement, but otherwise
 * passes all DPI scale values through.
 */
class SPopupLayer : public SPanel
{
public:
	SLATE_BEGIN_ARGS( SPopupLayer )
		{
			_Visibility = EVisibility::SelfHitTestInvisible;
		}
		SLATE_SLOT_ARGUMENT( FPopupLayerSlot, Slots )
	SLATE_END_ARGS()

	SPopupLayer()
		: Children(this)
	{}

	void Construct( const FArguments& InArgs, const TSharedRef<SWindow>& InWindow )
	{
		SetCanTick(false);

		OwnerWindow = InWindow;

		Children.AddSlots(MoveTemp(const_cast<TArray<FPopupLayerSlot::FSlotArguments>&>(InArgs._Slots)));
	}

	/** Make a new ListPanel::Slot  */
	static FPopupLayerSlot::FSlotArguments Slot()
	{
		return FPopupLayerSlot::FSlotArguments(MakeUnique<FPopupLayerSlot>());
	}

	using FScopedWidgetSlotArguments = TPanelChildren<FPopupLayerSlot>::FScopedWidgetSlotArguments;
	/** Add a slot to the ListPanel */
	FScopedWidgetSlotArguments AddSlot(int32 InsertAtIndex = INDEX_NONE)
	{
		return { MakeUnique<FPopupLayerSlot>(), Children, InsertAtIndex };
	}

	void RemoveSlot(const TSharedRef<SWidget>& WidgetToRemove)
	{
		for( int32 CurSlotIndex = 0; CurSlotIndex < Children.Num(); ++CurSlotIndex )
		{
			const FPopupLayerSlot& CurSlot = Children[ CurSlotIndex ];
			if( CurSlot.GetWidget() == WidgetToRemove )
			{
				Children.RemoveAt( CurSlotIndex );
				return;
			}
		}
	}

private:

	/**
	 * Each child slot essentially tries to place their contents at a specified position on the screen
	 * and scale as the widget initiating the popup, both of which are stored in the slot attributes.
	 * The tricky part is that the scale we are given is the fully accumulated layout scale of the widget, which already incorporates
	 * the DPI Scale of the window. The DPI Scale is also applied to the overlay since it is part of the window,
	 * so this scale needs to be factored out when determining the scale of the child geometry that will be created to hold the popup.
	 * We also optionally adjust the window position to keep it within the client bounds of the top-level window. This must be done in screenspace.
	 * This means some hairy transformation calculus goes on to ensure the computations are done in the proper space so scale is respected.
	 *
	 * There are 3 transformational spaces involved, each clearly specified in the variable names:
	 * Screen      - Basically desktop space. Contains desktop offset and DPI scale.
	 * WindowLocal - local space of the SWindow containing this popup. Screenspace == Concat(WindowLocal, DPI Scale, Desktop Offset)
	 * ChildLocal  - space of the child widget we want to display in the popup. The widget's LayoutTransform takes us from ChildLocal to WindowLocal space.
	 */
	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override
	{
		// skip all this work if there are no children to arrange.
		if (Children.Num() == 0) return;

		// create a transform from screen to local space.
		// This assumes that the PopupLayer is part of an Overlay that takes up the entire window space.
		// We should technically be using the AllottedGeometry to transform from AbsoluteToLocal space just in case it has an additional scale on it.
		// But we can't because the absolute space of the geometry is sometimes given in desktop space (picking, ticking)
		// and sometimes in window space (painting), and we can't necessarily tell by inspection so we have to just make an assumption here.
		FSlateLayoutTransform ScreenToWindowLocal = (ensure(OwnerWindow.IsValid())) ? Inverse(OwnerWindow.Pin()->GetLocalToScreenTransform()) : FSlateLayoutTransform();

		for ( int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex )
		{
			const FPopupLayerSlot& CurChild = Children[ChildIndex];
			const EVisibility ChildVisibility = CurChild.GetWidget()->GetVisibility();
			if ( ArrangedChildren.Accepts(ChildVisibility) )
			{
				// This scale+translate forms the ChildLocal to Screenspace transform.
				// The translation may be adjusted based on clamping, but the scale is accurate,
				// so we can transform vectors into screenspace using the scale alone.
				const float ChildLocalToScreenScale = CurChild.Scale_Attribute.Get();
				FVector2D ChildLocalToScreenOffset = CurChild.DesktopPosition_Attribute.Get();
				// The size of the child is either the desired size of the widget (computed in the child's local space) or the size override (specified in screen space)
				const FVector2D ChildSizeChildLocal = FVector2D(UE::Slate::CastToVector2f(CurChild.GetWidget()->GetDesiredSize()));
				// Convert the desired size to screen space. Here is were we convert a vector to screenspace
				// before we have the final position in screenspace (which would be needed to transform a point).
				FVector2D ChildSizeScreenspace = TransformVector(ChildLocalToScreenScale, ChildSizeChildLocal);
				// But then allow each size dimension to be overridden by the slot, which specifies the overrides in screen space.
				ChildSizeScreenspace = FVector2D(
						CurChild.WidthOverride_Attribute.IsSet() ? CurChild.WidthOverride_Attribute.Get() : ChildSizeScreenspace.X,
						CurChild.HeightOverride_Attribute.IsSet() ? CurChild.HeightOverride_Attribute.Get() : ChildSizeScreenspace.Y);

				// If clamping, move the screen space position to ensure the screen space size stays within the client rect of the top level window.
				if(CurChild.Clamp_Attribute.Get())
				{
					const FSlateRect WindowClientRectScreenspace = (ensure(OwnerWindow.IsValid())) ? OwnerWindow.Pin()->GetClientRectInScreen() : FSlateRect();
					const FVector2D ClampBufferScreenspace = CurChild.ClampBuffer_Attribute.Get();
					const FSlateRect ClampedWindowClientRectScreenspace = WindowClientRectScreenspace.InsetBy(FMargin(ClampBufferScreenspace.X, ClampBufferScreenspace.Y));
					// Find how much our child wants to extend beyond our client space and subtract that amount, but don't push it past the client edge.
					ChildLocalToScreenOffset.X = FMath::Max(WindowClientRectScreenspace.Left, ChildLocalToScreenOffset.X - FMath::Max(0.0f, (ChildLocalToScreenOffset.X  + ChildSizeScreenspace.X ) - ClampedWindowClientRectScreenspace.Right));
					ChildLocalToScreenOffset.Y = FMath::Max(WindowClientRectScreenspace.Top, ChildLocalToScreenOffset.Y - FMath::Max(0.0f, (ChildLocalToScreenOffset.Y  + ChildSizeScreenspace.Y ) - ClampedWindowClientRectScreenspace.Bottom));
				}

				// We now have the final position, so construct the transform from ChildLocal to Screenspace
				const FSlateLayoutTransform ChildLocalToScreen(ChildLocalToScreenScale, UE::Slate::CastToVector2f(ChildLocalToScreenOffset));
				// Using this we can compute the transform from ChildLocal to WindowLocal, which is effectively the LayoutTransform of the child widget.
				const FSlateLayoutTransform ChildLocalToWindowLocal = Concatenate(ChildLocalToScreen, ScreenToWindowLocal);
				// The ChildSize needs to be given in ChildLocal space when constructing a geometry.
				const FVector2f ChildSizeLocalspace = TransformVector(Inverse(ChildLocalToScreen), UE::Slate::CastToVector2f(ChildSizeScreenspace));

				// The position is explicitly in desktop pixels.
				// The size and DPI scale come from the widget that is using
				// this overlay to "punch" through the UI.
				ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(CurChild.GetWidget(), ChildSizeLocalspace, ChildLocalToWindowLocal));
			}
		}
	}

	virtual FVector2D ComputeDesiredSize(float) const override
	{
		return FVector2D(100,100);
	}

	/**
	 * All widgets must provide a way to access their children in a layout-agnostic way.
	 * Panels store their children in Slots, which creates a dilemma. Most panels
	 * can store their children in a TPanelChildren<Slot>, where the Slot class
	 * provides layout information about the child it stores. In that case
	 * GetChildren should simply return the TPanelChildren<Slot>. See StackPanel for an example.
	 */
	virtual FChildren* GetChildren() override
	{
		return &Children;
	}

	TPanelChildren<FPopupLayerSlot> Children;
	TWeakPtr<SWindow> OwnerWindow;
};


UE::Slate::FDeprecateVector2DResult SWindow::GetWindowSizeFromClientSize(UE::Slate::FDeprecateVector2DParameter InClientSize, TOptional<float> DPIScale)
{
	// If this is a regular non-OS window, we need to compensate for the border and title bar area that we will add
	// Note: Windows with an OS border do this in ReshapeWindow
	if (IsRegularWindow() && !HasOSWindowBorder() && !IsAutosized())
	{
		const FMargin BorderSize = GetWindowBorderSize();

		// Get DPIScale for border and title if not already supplied
		if (!DPIScale.IsSet())
		{
			if (NativeWindow.IsValid())
			{
				DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(ScreenPosition.X, ScreenPosition.Y);
			}
			else
			{
				DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(InitialDesiredScreenPosition.X, InitialDesiredScreenPosition.Y);
			}
		}

		// Apply DPIScale if not already applied by GetWindowBorderSize()
		if (!(NativeWindow.IsValid() && NativeWindow->IsMaximized()))
		{
			InClientSize.X += ((BorderSize.Left + BorderSize.Right) * DPIScale.GetValue());
			InClientSize.Y += ((BorderSize.Bottom + BorderSize.Top) * DPIScale.GetValue());
		}
		else
		{
			InClientSize.X += BorderSize.Left + BorderSize.Right;
			InClientSize.Y += BorderSize.Bottom + BorderSize.Top;
		}

		if (bCreateTitleBar)
		{
			InClientSize.Y += (SWindowDefs::DefaultTitleBarSize * DPIScale.GetValue());
		}
	}

	return InClientSize;
}

void SWindow::Construct(const FArguments& InArgs)
{
	check(InArgs._Style);
	this->Type = InArgs._Type;
	this->Style = InArgs._Style;
	this->WindowBackground = &InArgs._Style->BackgroundBrush;

	this->Title = InArgs._Title;
	this->CloseButtonToolTipText = InArgs._CloseButtonToolTipText;
	this->bDragAnywhere = InArgs._bDragAnywhere;
	this->TransparencySupport = InArgs._SupportsTransparency.Value;
	this->Opacity = InArgs._InitialOpacity;
	this->bInitiallyMaximized = InArgs._IsInitiallyMaximized;
	this->bInitiallyMinimized = InArgs._IsInitiallyMinimized;
	this->SizingRule = InArgs._SizingRule;
	this->bIsPopupWindow = InArgs._IsPopupWindow;
	this->bIsTopmostWindow = InArgs._IsTopmostWindow;
	this->bFocusWhenFirstShown = InArgs._FocusWhenFirstShown;
	this->FocusUserIndex = InArgs._FocusUserIndex;
	this->bHasOSWindowBorder = InArgs._UseOSWindowBorder;
	this->bHasCloseButton = InArgs._HasCloseButton;
	this->bHasMinimizeButton = InArgs._SupportsMinimize;
	this->bHasMaximizeButton = InArgs._SupportsMaximize;
	this->bHasSizingFrame = !InArgs._IsPopupWindow && InArgs._SizingRule == ESizingRule::UserSized;
	this->bShouldPreserveAspectRatio = InArgs._ShouldPreserveAspectRatio;
	this->WindowActivationPolicy = InArgs._ActivationPolicy;
	this->LayoutBorder = InArgs._LayoutBorder;
	this->UserResizeBorder = InArgs._UserResizeBorder;
	this->bVirtualWindow = false;
	this->SizeLimits = FWindowSizeLimits()
		.SetMinWidth(InArgs._MinWidth)
		.SetMinHeight(InArgs._MinHeight)
		.SetMaxWidth(InArgs._MaxWidth)
		.SetMaxHeight(InArgs._MaxHeight);
	this->bManualManageDPI = InArgs._bManualManageDPI;

	SetCanTick(false);

	// calculate window size from client size
	bCreateTitleBar = InArgs._CreateTitleBar && !bIsPopupWindow && Type != EWindowType::CursorDecorator && !bHasOSWindowBorder;

	// calculate initial window position
	FVector2f WindowPosition = InArgs._ScreenPosition;

	const bool bAnchorWindowWindowPositionTopLeft = FPlatformApplicationMisc::AnchorWindowWindowPositionTopLeft();
	if (bAnchorWindowWindowPositionTopLeft)
	{
		WindowPosition.X = WindowPosition.Y = 0;
	}
	else if (InArgs._AdjustInitialSizeAndPositionForDPIScale && !WindowPosition.IsZero())
	{
		// Will need to add additional logic to walk over multiple monitors at various DPIs to determine correct WindowPosition
		const float InitialDPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(WindowPosition.X, WindowPosition.Y);
		WindowPosition *= InitialDPIScale;
	}

	AutoCenterRule = InArgs._AutoCenter;

	// Get desktop metrics
	FDisplayMetrics DisplayMetrics;
	FSlateApplicationBase::Get().GetCachedDisplayMetrics( DisplayMetrics );
	const FPlatformRect& VirtualDisplayRect = DisplayMetrics.VirtualDisplayRect;
	FPlatformRect PrimaryDisplayRect = AutoCenterRule == EAutoCenter::PrimaryWorkArea ? DisplayMetrics.PrimaryDisplayWorkAreaRect : DisplayMetrics.GetMonitorWorkAreaFromPoint(FVector2d(WindowPosition));

	if (PrimaryDisplayRect == FPlatformRect(0, 0, 0, 0))
	{
		// If the primary display rect is empty we couldnt enumerate physical monitors (possibly remote desktop).  So assume virtual display rect is primary rect
		PrimaryDisplayRect = VirtualDisplayRect;
	}

	// If we're showing a pop-up window, to avoid creation of driver crashing sized
	// tooltips we limit the size a pop-up window can be if max size limit is unspecified.
	if ( bIsPopupWindow )
	{
		if ( !SizeLimits.GetMaxWidth().IsSet() )
		{
			SizeLimits.SetMaxWidth(static_cast<float>(PrimaryDisplayRect.Right - PrimaryDisplayRect.Left));
		}
		if ( !SizeLimits.GetMaxHeight().IsSet() )
		{
			SizeLimits.SetMaxHeight(static_cast<float>(PrimaryDisplayRect.Bottom - PrimaryDisplayRect.Top));
		}
	}

	// If we're manually positioning the window we need to check if it's outside
	// of the virtual bounds of the current displays or too large.
	if ( AutoCenterRule == EAutoCenter::None && InArgs._SaneWindowPlacement )
	{
		// Check to see if the upper left corner of the window is outside the virtual
		// bounds of the display, if so reset to preferred work area
		if (WindowPosition.X < VirtualDisplayRect.Left ||
			WindowPosition.X >= VirtualDisplayRect.Right ||
			WindowPosition.Y < VirtualDisplayRect.Top ||
			WindowPosition.Y >= VirtualDisplayRect.Bottom)
		{
			AutoCenterRule = EAutoCenter::PreferredWorkArea;
		}
	}

	FSlateRect AutoCenterRect(0, 0, 0, 0);
	float DPIScale = 1.0f;
	if (bAnchorWindowWindowPositionTopLeft)
	{
		WindowPosition.X = WindowPosition.Y = 0;
		DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(WindowPosition.X, WindowPosition.Y);
	}
	else if (AutoCenterRule != EAutoCenter::None)
	{
		switch( AutoCenterRule )
		{
		default:
		case EAutoCenter::PrimaryWorkArea:
			AutoCenterRect = FSlateRect(
				(float)PrimaryDisplayRect.Left,
				(float)PrimaryDisplayRect.Top,
				(float)PrimaryDisplayRect.Right,
				(float)PrimaryDisplayRect.Bottom );
			break;
		case EAutoCenter::PreferredWorkArea:
			AutoCenterRect = FSlateApplicationBase::Get().GetPreferredWorkArea();
			break;
		}
		DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(AutoCenterRect.Left, AutoCenterRect.Top);
	}
	else
	{
		DPIScale = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(WindowPosition.X, WindowPosition.Y);
	}

	// If the window has no OS border, simulate it ourselves, enlarging window by the size that OS border would have.
	const FVector2f DPIScaledClientSize = InArgs._AdjustInitialSizeAndPositionForDPIScale ? InArgs._ClientSize * DPIScale: FVector2f(InArgs._ClientSize);
	FVector2f WindowSize = GetWindowSizeFromClientSize(DPIScaledClientSize, DPIScale);

	// If we're manually positioning the window we need to check if it's outside
	// of the virtual bounds of the current displays or too large.
	if (AutoCenterRule == EAutoCenter::None || bAnchorWindowWindowPositionTopLeft)
	{
		if (InArgs._SaneWindowPlacement)
		{
			float PrimaryWidthPadding = static_cast<float>(DisplayMetrics.PrimaryDisplayWidth -
				(PrimaryDisplayRect.Right - PrimaryDisplayRect.Left));
			float PrimaryHeightPadding = static_cast<float>(DisplayMetrics.PrimaryDisplayHeight -
				(PrimaryDisplayRect.Bottom - PrimaryDisplayRect.Top));

			float VirtualWidth = static_cast<float>(VirtualDisplayRect.Right - VirtualDisplayRect.Left);
			float VirtualHeight = static_cast<float>(VirtualDisplayRect.Bottom - VirtualDisplayRect.Top);

			// Make sure that the window size is no larger than the virtual display area.
			WindowSize.X = FMath::Clamp(WindowSize.X, 0.0f, VirtualWidth - PrimaryWidthPadding);
			WindowSize.Y = FMath::Clamp(WindowSize.Y, 0.0f, VirtualHeight - PrimaryHeightPadding);
		}
	}
	else
	{
		if (InArgs._SaneWindowPlacement)
		{
			// Clamp window size to be no greater than the work area size
			WindowSize.X = FMath::Min(WindowSize.X, AutoCenterRect.GetSize().X);
			WindowSize.Y = FMath::Min(WindowSize.Y, AutoCenterRect.GetSize().Y);
		}

		// Setup a position and size for the main frame window that's centered in the desktop work area
		const FVector2f DisplayTopLeft( AutoCenterRect.Left, AutoCenterRect.Top );
		const FVector2f DisplaySize( AutoCenterRect.Right - AutoCenterRect.Left, AutoCenterRect.Bottom - AutoCenterRect.Top );

		WindowPosition = DisplayTopLeft + (DisplaySize - WindowSize) * 0.5f;

		// Don't allow the window to center to outside of the work area
		WindowPosition.X = FMath::Max(WindowPosition.X, AutoCenterRect.Left);
		WindowPosition.Y = FMath::Max(WindowPosition.Y, AutoCenterRect.Top);
	}

	this->InitialDesiredScreenPosition = WindowPosition;
	this->InitialDesiredSize = WindowSize;

	FSlateApplicationBase::Get().OnGlobalInvalidationToggled().AddSP(this, &SWindow::OnGlobalInvalidationToggled);

	// Resize without adding extra borders / title because they are already included in WindowSize
	ResizeWindowSize(WindowSize);

	this->ConstructWindowInternals();
	this->SetContent( InArgs._Content.Widget );
}


TSharedRef<SWindow> SWindow::MakeNotificationWindow()
{
	TSharedRef<SWindow> NewWindow =
		SNew(SWindow)
		.Style(FAppStyle::Get(), "NotificationWindow")
		.Type( EWindowType::Notification )
		.SupportsMaximize( false )
		.SupportsMinimize( false )
		.IsPopupWindow( true )
		.CreateTitleBar( false )
		.SizingRule( ESizingRule::Autosized )
		.SupportsTransparency( EWindowTransparency::PerWindow )
		.InitialOpacity( 0.0f )
		.FocusWhenFirstShown( false )
		.ActivationPolicy( EWindowActivationPolicy::Never );

	// Notification windows slide open so we'll mark them as resized frequently
	NewWindow->bSizeWillChangeOften = true;
	NewWindow->ExpectedMaxWidth = 1024;
	NewWindow->ExpectedMaxHeight = 256;

	return NewWindow;
}


TSharedRef<SWindow> SWindow::MakeToolTipWindow()
{
	TSharedRef<SWindow> NewWindow = SNew( SWindow )
		.Type( EWindowType::ToolTip )
		.IsPopupWindow( true )
		.IsTopmostWindow(true)
		.AdjustInitialSizeAndPositionForDPIScale(false)
		.SizingRule(ESizingRule::Autosized)
		.SupportsTransparency( EWindowTransparency::PerWindow )
		.FocusWhenFirstShown( false )
		.ActivationPolicy( EWindowActivationPolicy::Never );
	NewWindow->Opacity = 0.0f;

	// NOTE: These sizes are tweaked for SToolTip widgets (text wrap width of around 400 px)
	NewWindow->bSizeWillChangeOften = true;
	NewWindow->ExpectedMaxWidth = 512;
	NewWindow->ExpectedMaxHeight = 256;

	return NewWindow;
}


TSharedRef<SWindow> SWindow::MakeCursorDecorator()
{
	TSharedRef<SWindow> NewWindow = SNew( SWindow )
		.Type( EWindowType::CursorDecorator )
		.IsPopupWindow( true )
		.IsTopmostWindow(true)
		.SizingRule(ESizingRule::Autosized)
		.SupportsTransparency( EWindowTransparency::PerWindow )
		.FocusWhenFirstShown( false )
		.ActivationPolicy( EWindowActivationPolicy::Never );
	NewWindow->Opacity = 1.0f;

	return NewWindow;
}

TSharedRef<SWindow> SWindow::MakeStyledCursorDecorator(const FWindowStyle& InStyle)
{
	TSharedRef<SWindow> NewWindow = SNew(SWindow)
		.Style(&InStyle)
		.Type(EWindowType::CursorDecorator)
		.IsPopupWindow(true)
		.IsTopmostWindow(true)
		.SizingRule(ESizingRule::Autosized)
		.SupportsTransparency(EWindowTransparency::PerWindow)
		.FocusWhenFirstShown(false)
		.ActivationPolicy(EWindowActivationPolicy::Never);
	NewWindow->Opacity = 1.0f;

	return NewWindow;
}

UE::Slate::FDeprecateVector2DResult SWindow::ComputeWindowSizeForContent( UE::Slate::FDeprecateVector2DParameter ContentSize )
{
	// @todo mainframe: This code should be updated to handle the case where we're spawning a window that doesn't have
	//                  a traditional title bar, such as a window that contains a primary SDockingArea.  Currently, the
	//                  size reported here will be too large!
	return UE::Slate::FDeprecateVector2DResult(ContentSize + FVector2f(0, SWindowDefs::DefaultTitleBarSize));
}

TSharedRef<SWidget> SWindow::MakeWindowTitleBar(const TSharedRef<SWindow>& Window, const TSharedPtr<SWidget>& CenterContent, EHorizontalAlignment TitleContentAlignment)
{
	FWindowTitleBarArgs Args(Window);
	Args.CenterContent = CenterContent;
	Args.CenterContentAlignment = TitleContentAlignment;
	Args.CloseButtonToolTipText = CloseButtonToolTipText;

	return FSlateApplicationBase::Get().MakeWindowTitleBar(Args, TitleBar);
}


EHorizontalAlignment SWindow::GetTitleAlignment()
{
	EWindowTitleAlignment::Type TitleAlignment = FSlateApplicationBase::Get().GetPlatformApplication()->GetWindowTitleAlignment();
	EHorizontalAlignment TitleContentAlignment;

	if (TitleAlignment == EWindowTitleAlignment::Left)
	{
		TitleContentAlignment = HAlign_Left;
	}
	else if (TitleAlignment == EWindowTitleAlignment::Center)
	{
		TitleContentAlignment = HAlign_Center;
	}
	else
	{
		TitleContentAlignment = HAlign_Right;
	}
	return TitleContentAlignment;
}

void SWindow::ConstructWindowInternals()
{
	SetForegroundColor(FAppStyle::Get().GetSlateColor("DefaultForeground"));

	// Setup widget that represents the main area of the window.  That is, everything inside the window's border.
	TSharedRef< SVerticalBox > MainWindowArea =
		SNew( SVerticalBox )
		.Visibility( EVisibility::SelfHitTestInvisible );

	FWindowTitleBarArgs Args(SharedThis(this));
	Args.CenterContent = nullptr;
	Args.CenterContentAlignment = GetTitleAlignment();
	Args.CloseButtonToolTipText = CloseButtonToolTipText;

	TSharedRef<SWidget> TitleBarWidget = FSlateApplicationBase::Get().MakeWindowTitleBar(Args, TitleBar);

	if (bCreateTitleBar)
	{
		// @todo mainframe: Should be measured from actual title bar content widgets.  Don't use a hard-coded size!
		TitleBarSize = SWindowDefs::DefaultTitleBarSize;

		MainWindowArea->AddSlot()
		.AutoHeight()
		[
			TitleBarWidget
		];
	}
	else
	{
		TitleBarSize = 0;
	}

	UpdateWindowContentVisibility();

	// create window
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if ((Type == EWindowType::Normal || Type == EWindowType::GameWindow) && !bHasOSWindowBorder && !bVirtualWindow)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// create window content slot
		MainWindowArea->AddSlot()
			.FillHeight(1.0f)
			.Expose(ContentSlot)
			[
				SNullWidget::NullWidget
			];


		WindowBackgroundImage =
			FSlateApplicationBase::Get().MakeImage(
				WindowBackground,
				Style->BackgroundColor,
				WindowContentVisibility
			);

		WindowBorder =
			FSlateApplicationBase::Get().MakeImage(
				&Style->BorderBrush,
				Style->BorderColor,
				WindowContentVisibility
			);

		WindowOutline = FSlateApplicationBase::Get().MakeImage(
				&Style->OutlineBrush,
				Style->OutlineColor,
				WindowContentVisibility
			);

		this->ChildSlot
		[
			SAssignNew(WindowOverlay, SOverlay)
			// window background
			+ SOverlay::Slot()
			[
				WindowBackgroundImage.ToSharedRef()
			]

			// window border
			+ SOverlay::Slot()
			[
				WindowBorder.ToSharedRef()
			]

			// window outline
			+ SOverlay::Slot()
			.Padding(2.0f)
			[
				WindowOutline.ToSharedRef()
			]

			// main area
			+ SOverlay::Slot()
			.Padding(TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SWindow::GetWindowBorderSize, false)))
			[
				SAssignNew(ContentAreaVBox, SVerticalBox)
				.Visibility(WindowContentVisibility)
				+ SVerticalBox::Slot()	
				[
					MainWindowArea
				]
			]

			// pop-up layer
			+ SOverlay::Slot()
			[
				SAssignNew(PopupLayer, SPopupLayer, SharedThis(this))
			]
		];
	}
	else if ( bHasOSWindowBorder || bVirtualWindow )
	{
		// create window content slot
		MainWindowArea->AddSlot()
			.FillHeight(1.0f)
			.Expose(ContentSlot)
			[
				SNullWidget::NullWidget
			];


		this->ChildSlot
		[
			SAssignNew(WindowOverlay, SOverlay)
			+ SOverlay::Slot()
			[
				MainWindowArea
			]
 			+ SOverlay::Slot()
 			[
 				SAssignNew(PopupLayer, SPopupLayer, SharedThis(this))
 			]
		];
	}
}


/** Are any of our child windows active? */
bool SWindow::IsActive() const
{
	return FSlateApplicationBase::Get().GetActiveTopLevelWindow().Get() == this;
}

bool SWindow::HasActiveChildren() const
{
	for (int32 i = 0; i < ChildWindows.Num(); ++i)
	{
		if ( ChildWindows[i]->IsActive() || ChildWindows[i]->HasActiveChildren() )
		{
			return true;
		}
	}

	return false;
}

bool SWindow::HasActiveParent() const
{
	TSharedPtr<SWindow> ParentWindow = ParentWindowPtr.Pin();
	if ( ParentWindow.IsValid() )
	{
		if ( ParentWindow->IsActive() )
		{
			return true;
		}

		return ParentWindow->HasActiveParent();
	}

	return false;
}

FHittestGrid& SWindow::GetHittestGrid()
{
	return *HittestGrid;
}

FWindowSizeLimits SWindow::GetSizeLimits() const
{
	return SizeLimits;
}

void SWindow::SetSizeLimits(const FWindowSizeLimits& InSizeLimits)
{
	SizeLimits = InSizeLimits;
}

void SWindow::SetAllowFastUpdate(bool bInAllowFastUpdate)
{
	if (bAllowFastUpdate != bInAllowFastUpdate)
	{
		bAllowFastUpdate = bInAllowFastUpdate;
		if (bAllowFastUpdate)
		{
			InvalidateRootChildOrder();
		}
	}
}


void SWindow::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if( Morpher.bIsActive )
	{
		if ( Morpher.Sequence.IsPlaying() )
		{
			const float InterpAlpha = Morpher.Sequence.GetLerp();

			if( Morpher.bIsAnimatingWindowSize )
			{
				FSlateRect WindowRect = FMath::Lerp( Morpher.StartingMorphShape, Morpher.TargetMorphShape, InterpAlpha );
				if( WindowRect != GetRectInScreen() )
				{
					check( SizingRule != ESizingRule::Autosized );
					this->ReshapeWindow( WindowRect );
				}
			}
			else // if animating position
			{
				const FVector2f StartPosition( Morpher.StartingMorphShape.Left, Morpher.StartingMorphShape.Top );
				const FVector2f TargetPosition( Morpher.TargetMorphShape.Left, Morpher.TargetMorphShape.Top );
				const FVector2f NewPosition( FMath::Lerp( StartPosition, TargetPosition, InterpAlpha ) );
				if( NewPosition != this->GetPositionInScreen() )
				{
					this->MoveWindowTo( NewPosition );
				}
			}

			const float NewOpacity = FMath::Lerp( Morpher.StartingOpacity, Morpher.TargetOpacity, InterpAlpha );
			this->SetOpacity( NewOpacity );
		}
		else
		{
			// The animation is complete, so just make sure the target size/position and opacity are reached
			if( Morpher.bIsAnimatingWindowSize )
			{
				if( Morpher.TargetMorphShape != GetRectInScreen() )
				{
					check( SizingRule != ESizingRule::Autosized );
					this->ReshapeWindow( Morpher.TargetMorphShape );
				}
			}
			else // if animating position
			{
				const FVector2f TargetPosition( Morpher.TargetMorphShape.Left, Morpher.TargetMorphShape.Top );
				if( TargetPosition != this->GetPositionInScreen() )
				{
					this->MoveWindowTo( TargetPosition );
				}
			}

			this->SetOpacity( Morpher.TargetOpacity );
			Morpher.bIsActive = false;
			SetCanTick(false);
		}
	}
}

UE::Slate::FDeprecateVector2DResult SWindow::GetInitialDesiredSizeInScreen() const
{
	return InitialDesiredSize;
}

UE::Slate::FDeprecateVector2DResult SWindow::GetInitialDesiredPositionInScreen() const
{
	return InitialDesiredScreenPosition;
}

FGeometry SWindow::GetWindowGeometryInScreen() const
{
	// We are scaling children for layout, but our pixel bounds are not changing.
	// FGeometry expects Size in Local space, but our size is stored in screen space.
	// So we need to transform Size into the window's local space for FGeometry.
	FSlateLayoutTransform LocalToScreen = GetLocalToScreenTransform();
	return FGeometry::MakeRoot( TransformVector(Inverse(LocalToScreen), Size), LocalToScreen );
}

FGeometry SWindow::GetWindowGeometryInWindow() const
{
	// We are scaling children for layout, but our pixel bounds are not changing.
	// FGeometry expects Size in Local space, but our size is stored in screen space (same as window space + screen offset).
	// So we need to transform Size into the window's local space for FGeometry.
	FSlateLayoutTransform LocalToWindow = GetLocalToWindowTransform();
	FVector2f ViewSize = GetViewportSize();
	return FGeometry::MakeRoot(TransformVector(Inverse(LocalToWindow), ViewSize), LocalToWindow );
}

FSlateLayoutTransform SWindow::GetLocalToScreenTransform() const
{
	return FSlateLayoutTransform(FSlateApplicationBase::Get().GetApplicationScale() * GetDPIScaleFactor(), ScreenPosition);
}

FSlateLayoutTransform SWindow::GetLocalToWindowTransform() const
{
	return FSlateLayoutTransform(FSlateApplicationBase::Get().GetApplicationScale() * GetDPIScaleFactor());
}


UE::Slate::FDeprecateVector2DResult SWindow::GetPositionInScreen() const
{
	return ScreenPosition;
}

UE::Slate::FDeprecateVector2DResult SWindow::GetSizeInScreen() const
{
	return Size;
}

FSlateRect SWindow::GetNonMaximizedRectInScreen() const
{
	int32 X = 0;
	int32 Y = 0;
	int32 Width = 0;
	int32 Height = 0;

	if ( NativeWindow.IsValid() && NativeWindow->GetRestoredDimensions(X, Y, Width, Height) )
	{
		return FSlateRect( (float)X, (float)Y, static_cast<float>(X+Width), static_cast<float>(Y+Height) );
	}
	else
	{
		return GetRectInScreen();
	}
}

FSlateRect SWindow::GetRectInScreen() const
{
	if ( bVirtualWindow )
	{
		return FSlateRect(0, 0, Size.X, Size.Y);
	}

	return FSlateRect( ScreenPosition, ScreenPosition + Size );
}

FSlateRect SWindow::GetClientRectInScreen() const
{
	if ( bVirtualWindow )
	{
		return FSlateRect(0, 0, Size.X, Size.Y);
	}

	if (HasOSWindowBorder())
	{
		return GetRectInScreen();
	}

	return GetRectInScreen()
		.InsetBy(GetWindowBorderSize() * GetDPIScaleFactor())
		.InsetBy(FMargin(0.0f, TitleBarSize * GetDPIScaleFactor(), 0.0f, 0.0f));
}

UE::Slate::FDeprecateVector2DResult SWindow::GetClientSizeInScreen() const
{
	return GetClientRectInScreen().GetSize();
}

FSlateRect SWindow::GetClippingRectangleInWindow() const
{
	FVector2f ViewSize = GetViewportSize();
	return FSlateRect( 0, 0, ViewSize.X, ViewSize.Y );
}


FMargin SWindow::GetWindowBorderSize( bool bIncTitleBar ) const
{
// Mac didn't want a window border, and consoles don't either, so only do this in Windows

// @TODO This is not working for Linux. The window is not yet valid when this gets
// called from SWindow::Construct which is causing a default border to be retured even when the
// window is borderless. This causes problems for menu positioning.
	if (NativeWindow.IsValid() && NativeWindow->IsMaximized())
	{
		const float DesktopPixelsToSlateUnits = 1.0f / (FSlateApplicationBase::Get().GetApplicationScale() * GetDPIScaleFactor());
		FMargin BorderSize(NativeWindow->GetWindowBorderSize() * DesktopPixelsToSlateUnits);
		if(bIncTitleBar)
		{
			// Add title bar size (whether it's visible or not)
			BorderSize.Top += NativeWindow->GetWindowTitleBarSize()*DesktopPixelsToSlateUnits;
		}

		return BorderSize;
	}

	return GetNonMaximizedWindowBorderSize();
}


FMargin SWindow::GetNonMaximizedWindowBorderSize() const
{
#if PLATFORM_WINDOWS || PLATFORM_LINUX
	return LayoutBorder;
#else
	return FMargin();
#endif
}


void SWindow::MoveWindowTo( UE::Slate::FDeprecateVector2DParameter NewPosition )
{
	if (NativeWindow.IsValid())
	{
#if 1//PLATFORM_LINUX
		// Slate code often expects cached screen position to be accurate immediately after the move.
		// This expectation is generally invalid (see UE-1308) as there may be a delay before the OS reports it back.
		// This hack sets the position speculatively, keeping Slate happy while also giving the OS chance to report it
		// correctly after or even during the actual call.
		FVector2f SpeculativeScreenPosition(FMath::TruncToFloat(NewPosition.X), FMath::TruncToFloat(NewPosition.Y));
		SetCachedScreenPosition(SpeculativeScreenPosition);
#endif // PLATFORM_LINUX

		NativeWindow->MoveWindowTo( FMath::TruncToInt(NewPosition.X), FMath::TruncToInt(NewPosition.Y) );
	}
	else
	{
		InitialDesiredScreenPosition = NewPosition;
	}
}

void SWindow::ReshapeWindow( UE::Slate::FDeprecateVector2DParameter NewPosition, UE::Slate::FDeprecateVector2DParameter NewSize )
{
	const FVector2f CurrentPosition = GetPositionInScreen();
	const FVector2f CurrentSize = GetSizeInScreen();

	// Ceil (Minus a tad for float precision) to ensure contents are not a sub-pixel larger than the window, which will create unnecessary scroll bars 
	const FVector2f OldPositionTruncated = FVector2f(FMath::TruncToInt(CurrentPosition.X), FMath::TruncToInt(CurrentPosition.Y));
	const FVector2f OldSizeRounded = FVector2f(FMath::CeilToInt(CurrentSize.X - KINDA_SMALL_NUMBER), FMath::CeilToInt(CurrentSize.Y - KINDA_SMALL_NUMBER));
	const FVector2f NewPositionTruncated = FVector2f(FMath::TruncToInt(NewPosition.X), FMath::TruncToInt(NewPosition.Y));
	const FVector2f NewSizeRounded = FVector2f(FMath::CeilToInt(NewSize.X - KINDA_SMALL_NUMBER), FMath::CeilToInt(NewSize.Y - KINDA_SMALL_NUMBER));

	if (OldPositionTruncated != NewPositionTruncated || OldSizeRounded != NewSizeRounded )
	{
		if ( NativeWindow.IsValid() )
		{

// The following hack with speculative position is messing up with Mac window position with notched display
// The accurate position is calculated in FMacApplication::OnWindowDidMove
#if !PLATFORM_MAC
			// Slate code often expects cached screen position to be accurate immediately after the move.
			// This expectation is generally invalid (see UE-1308) as there may be a delay before the OS reports it back.
			// This hack sets the position speculatively, keeping Slate happy while also giving the OS chance to report it
			// correctly after or even during the actual call.
			SetCachedScreenPosition(NewPositionTruncated);
#endif
			NativeWindow->ReshapeWindow(NewPositionTruncated.X, NewPositionTruncated.Y, NewSizeRounded.X, NewSizeRounded.Y);
		}
		else
		{
			InitialDesiredScreenPosition = NewPositionTruncated;
			InitialDesiredSize = NewSizeRounded;
		}

		SetCachedSize(NewSizeRounded);
	}
}

void SWindow::ReshapeWindow( const FSlateRect& InNewShape )
{
	ReshapeWindow(FVector2f(InNewShape.Left, InNewShape.Top), FVector2f(InNewShape.Right - InNewShape.Left, InNewShape.Bottom - InNewShape.Top));
}

void SWindow::Resize( UE::Slate::FDeprecateVector2DParameter NewClientSize )
{
	ResizeWindowSize(GetWindowSizeFromClientSize(NewClientSize));
}

void SWindow::ResizeWindowSize( FVector2f NewWindowSize )
{
	NewWindowSize.X = FMath::Max(SizeLimits.GetMinWidth().Get(NewWindowSize.X), NewWindowSize.X);
	NewWindowSize.X = FMath::Min(SizeLimits.GetMaxWidth().Get(NewWindowSize.X), NewWindowSize.X);

	NewWindowSize.Y = FMath::Max(SizeLimits.GetMinHeight().Get(NewWindowSize.Y), NewWindowSize.Y);
	NewWindowSize.Y = FMath::Min(SizeLimits.GetMaxHeight().Get(NewWindowSize.Y), NewWindowSize.Y);

	// Ceil (Minus a tad for float precision) to ensure contents are not a sub-pixel larger than the window, which will create unnecessary scroll bars 
	FIntPoint CurrentIntSize = FIntPoint(FMath::CeilToInt(Size.X - KINDA_SMALL_NUMBER), FMath::CeilToInt(Size.Y - KINDA_SMALL_NUMBER));
	FIntPoint NewIntSize     = FIntPoint(FMath::CeilToInt(NewWindowSize.X - KINDA_SMALL_NUMBER), FMath::CeilToInt(NewWindowSize.Y - KINDA_SMALL_NUMBER));

	if (CurrentIntSize != NewIntSize)
	{
		Morpher.Sequence.JumpToEnd();

		if (NativeWindow.IsValid())
		{
			NativeWindow->ReshapeWindow(FMath::TruncToInt(ScreenPosition.X), FMath::TruncToInt(ScreenPosition.Y), NewIntSize.X, NewIntSize.Y);
		}
		else
		{
			InitialDesiredSize = FVector2f(NewIntSize);
		}
	}
	SetCachedSize(FVector2f(NewIntSize));
}

FSlateRect SWindow::GetFullScreenInfo() const
{
	if (NativeWindow.IsValid())
	{
		int32 X;
		int32 Y;
		int32 Width;
		int32 Height;

		if ( NativeWindow->GetFullScreenInfo( X, Y, Width, Height ) )
		{
			return FSlateRect( (float)X, (float)Y, (float)(X + Width), (float)(Y + Height) );
		}
	}

	return FSlateRect();
}

void SWindow::SetCachedScreenPosition(UE::Slate::FDeprecateVector2DParameter NewPosition)
{
	ScreenPosition = NewPosition;

	InvalidateScreenPosition();

	OnWindowMoved.ExecuteIfBound( SharedThis( this ) );
}

void SWindow::SetCachedSize( UE::Slate::FDeprecateVector2DParameter NewSize )
{
	if( NativeWindow.IsValid() )
	{
		FVector2d NewSize2d(NewSize);
		NativeWindow->AdjustCachedSize( NewSize2d );
		NewSize = UE::Slate::CastToVector2f(NewSize2d);
	}

	if(Size != NewSize)
	{
		Size = NewSize;
		InvalidateRootChildOrder();
	}
}

bool SWindow::IsMorphing() const
{
	return Morpher.bIsActive && Morpher.Sequence.IsPlaying();
}

bool SWindow::IsMorphingSize() const
{
	return IsMorphing() && Morpher.bIsAnimatingWindowSize;
}


void SWindow::MorphToPosition( const FCurveSequence& Sequence, const float TargetOpacity, const UE::Slate::FDeprecateVector2DParameter& TargetPosition )
{
	Morpher.bIsAnimatingWindowSize = false;
	Morpher.Sequence = Sequence;
	Morpher.TargetOpacity = TargetOpacity;
	UpdateMorphTargetPosition( TargetPosition );
	StartMorph();
}


void SWindow::MorphToShape( const FCurveSequence& Sequence, const float TargetOpacity, const FSlateRect& TargetShape )
{
	Morpher.bIsAnimatingWindowSize = true;
	Morpher.Sequence = Sequence;
	Morpher.TargetOpacity = TargetOpacity;
	UpdateMorphTargetShape(TargetShape);
	StartMorph();
}

void SWindow::StartMorph()
{
	Morpher.StartingOpacity = GetOpacity();
	Morpher.StartingMorphShape = FSlateRect( this->ScreenPosition.X, this->ScreenPosition.Y, this->ScreenPosition.X + this->Size.X, this->ScreenPosition.Y + this->Size.Y );
	Morpher.bIsActive = true;
	Morpher.Sequence.JumpToStart();
	SetCanTick(true);
	if ( !ActiveTimerHandle.IsValid() )
	{
		ActiveTimerHandle = RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SWindow::TriggerPlayMorphSequence ) );
	}
}

bool SWindow::Advanced_IsInvalidationRoot() const
{
	return bAllowFastUpdate && GSlateEnableGlobalInvalidation;
}

const FSlateInvalidationRoot* SWindow::Advanced_AsInvalidationRoot() const
{
	return Advanced_IsInvalidationRoot() ? this : nullptr;
}

void SWindow::ProcessWindowInvalidation()
{
	if (Advanced_IsInvalidationRoot())
	{
		ProcessInvalidation();
	}
}

bool SWindow::CustomPrepass(float LayoutScaleMultiplier)
{
	if (Advanced_IsInvalidationRoot())
	{
		return NeedsPrepass();
	}
	else
	{
		return true;
	}
}

EVisibility SWindow::GetWindowVisibility() const
{
	return ( AcceptsInput() || FSlateApplicationBase::Get().IsWindowHousingInteractiveTooltip(SharedThis(this)) )
		? EVisibility::Visible
		: EVisibility::HitTestInvisible;
}

TAttribute<FText> SWindow::GetWindowCloseButtonToolTipText() const
{
	return CloseButtonToolTipText;
}

void SWindow::UpdateMorphTargetShape( const FSlateRect& TargetShape )
{
	Morpher.TargetMorphShape = TargetShape;
}

void SWindow::UpdateMorphTargetPosition( const UE::Slate::FDeprecateVector2DParameter& TargetPosition )
{
	Morpher.TargetMorphShape.Left = Morpher.TargetMorphShape.Right = TargetPosition.X;
	Morpher.TargetMorphShape.Top = Morpher.TargetMorphShape.Bottom = TargetPosition.Y;
}

UE::Slate::FDeprecateVector2DResult SWindow::GetMorphTargetPosition() const
{
	return UE::Slate::FDeprecateVector2DResult(FVector2f( Morpher.TargetMorphShape.Left, Morpher.TargetMorphShape.Top ));
}


FSlateRect SWindow::GetMorphTargetShape() const
{
	return Morpher.TargetMorphShape;
}

void SWindow::FlashWindow()
{
	if (TitleBar.IsValid())
	{
		TitleBar->Flash();
	}
}

void SWindow::DrawAttention(const FWindowDrawAttentionParameters& Parameters)
{
	if (NativeWindow.IsValid())
	{
		NativeWindow->DrawAttention(Parameters);
	}
}

void SWindow::BringToFront( bool bForce )
{
	if (NativeWindow.IsValid())
	{
		NativeWindow->BringToFront( bForce );
	}
}

void SWindow::HACK_ForceToFront()
{
	if (NativeWindow.IsValid())
	{
		NativeWindow->HACK_ForceToFront();
	}
}

TSharedPtr<FGenericWindow> SWindow::GetNativeWindow()
{
	return NativeWindow;
}

TSharedPtr<const FGenericWindow> SWindow::GetNativeWindow() const
{
	return NativeWindow;
}

float SWindow::GetDPIScaleFactor() const
{
	if (NativeWindow.IsValid())
	{
		return NativeWindow->GetDPIScaleFactor();
	}

	return 1.0f;
}

void SWindow::SetDPIScaleFactor(const float Factor)
{
	if (NativeWindow.IsValid())
	{
		NativeWindow->SetDPIScaleFactor(Factor);
	}
}

void SWindow::SetManualManageDPIChanges(const bool bManualDPI)
{
	bManualManageDPI = bManualDPI;

	if (NativeWindow.IsValid())
	{
		NativeWindow->SetManualManageDPIChanges(bManualManageDPI);
	}
}

bool SWindow::IsDescendantOf( const TSharedPtr<SWindow>& ParentWindow ) const
{
	TSharedPtr<SWindow> CandidateToCheck = this->GetParentWindow();

	// Keep checking our parent until we get to the root of the tree or find the window we were looking for.
	while (CandidateToCheck.IsValid())
	{
		if (CandidateToCheck == ParentWindow)
		{
			// One of our ancestor windows is the ParentWindow we were looking for!
			return true;
		}

		// Consider the next ancestor
		CandidateToCheck = CandidateToCheck->GetParentWindow();
	}

	return false;
}

void SWindow::SetNativeWindow( TSharedRef<FGenericWindow> InNativeWindow )
{
	check( ! NativeWindow.IsValid() );
	NativeWindow = InNativeWindow;
}

void SWindow::SetContent( TSharedRef<SWidget> InContent )
{
	if (ContentSlot)
	{
		ContentSlot->operator[](InContent);
	}
	else
	{
		ChildSlot.operator[](InContent);
	}
	
	if (Advanced_IsInvalidationRoot())
	{
		InvalidateRootChildOrder();
	}

	Invalidate(EInvalidateWidgetReason::ChildOrder);
}

TSharedRef<SWidget> SWindow::GetContent()
{
	if (ContentSlot)
	{
		return ContentSlot->GetWidget();
	}
	else
	{
		return ChildSlot.GetChildAt(0);
	}
}

bool SWindow::HasOverlay() const
{
	return WindowOverlay.IsValid();
}

SOverlay::FScopedWidgetSlotArguments SWindow::AddOverlaySlot( const int32 ZOrder )
{
	if(!WindowOverlay.IsValid())
	{
		ensureMsgf( false, TEXT("This window does not support overlays. The added slot will not be visible!") );
		WindowOverlay = SNew(SOverlay).Visibility( EVisibility::HitTestInvisible );
	}

	return WindowOverlay->AddSlot(ZOrder);
}

bool SWindow::RemoveOverlaySlot(const TSharedRef<SWidget>& InContent)
{
	if(WindowOverlay.IsValid())
	{
		return WindowOverlay->RemoveSlot(InContent);
	}

	return false;
}

TSharedPtr<FPopupLayer> SWindow::OnVisualizePopup(const TSharedRef<SWidget>& PopupContent)
{
	if ( WindowOverlay.IsValid() )
	{
		return MakeShareable(new FOverlayPopupLayer(SharedThis(this), PopupContent, WindowOverlay));
	}

	return TSharedPtr<FPopupLayer>();
}

/** Return a new slot in the popup layer. Assumes that the window has a popup layer. */
SWindow::FScopedWidgetSlotArguments SWindow::AddPopupLayerSlot()
{
	ensure( PopupLayer.IsValid() );
	return PopupLayer->AddSlot();
}

/** Counterpart to AddPopupLayerSlot */
void SWindow::RemovePopupLayerSlot( const TSharedRef<SWidget>& WidgetToRemove )
{
	PopupLayer->RemoveSlot( WidgetToRemove );
}

/** @return should this window show up in the taskbar */
bool SWindow::AppearsInTaskbar() const
{
	return !bIsPopupWindow && Type != EWindowType::ToolTip && Type != EWindowType::CursorDecorator;
}

/** Sets the delegate to execute right before the window is closed */
void SWindow::SetOnWindowClosed( const FOnWindowClosed& InDelegate )
{
	OnWindowClosed = InDelegate;
}

/** Sets the delegate to execute right after the window has been moved */
void SWindow::SetOnWindowMoved( const FOnWindowMoved& InDelegate)
{
	OnWindowMoved = InDelegate;
}

/** Sets the delegate to override RequestDestroyWindow */
void SWindow::SetRequestDestroyWindowOverride( const FRequestDestroyWindowOverride& InDelegate )
{
	RequestDestroyWindowOverride = InDelegate;
}

/** Request that this window be destroyed. The window is not destroyed immediately. Instead it is placed in a queue for destruction on next Tick */
void SWindow::RequestDestroyWindow()
{
	if( RequestDestroyWindowOverride.IsBound() )
	{
		RequestDestroyWindowOverride.Execute( SharedThis(this) );
	}
	else
	{
		FSlateApplicationBase::Get().RequestDestroyWindow( SharedThis(this) );
	}
}

/** Warning: use Request Destroy Window whenever possible!  This method destroys the window immediately! */
void SWindow::DestroyWindowImmediately()
{
	if ( NativeWindow.IsValid() )
	{
		// Destroy the native window
		NativeWindow->Destroy();
	}
}

/** Calls OnWindowClosed delegate and WindowClosedEvent when this window is about to be closed */
void SWindow::NotifyWindowBeingDestroyed()
{
	OnWindowClosed.ExecuteIfBound( SharedThis( this ) );
	WindowClosedEvent.Broadcast( SharedThis( this ) );

#if WITH_EDITOR
	if(bIsModalWindow)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FCoreDelegates::PostSlateModal.Broadcast();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		FCoreDelegates::PostSlateModalWithContext.Broadcast({ .WindowIdentifier = (intptr_t)this });
	}
#endif

	// Logging to track down window shutdown issues
	if (IsRegularWindow())
	{
		UE_LOGF(LogSlate, Log, "Window '%ls' being destroyed", *GetTitle().ToString());
	}
}

/** Make the window visible */
void SWindow::ShowWindow()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SWindow::ShowWindow);

	// Make sure the viewport is setup for this window
	if( !bHasEverBeenShown )
	{
		if( ensure( NativeWindow.IsValid() ) )
		{
			// We can only create a viewport after the window has been shown (otherwise the swap chain creation may fail)
			FSlateApplicationBase::Get().GetRenderer()->CreateViewport( SharedThis( this ) );
		}

		// Auto sized windows don't know their size until after their position is set.
		// Repositioning the window on show with the new size solves this.
		if ( SizingRule == ESizingRule::Autosized && AutoCenterRule != EAutoCenter::None )
		{
			SlatePrepass( FSlateApplicationBase::Get().GetApplicationScale() * GetDPIScaleFactor() );
			const FVector2f WindowDesiredSizePixels = GetDesiredSizeDesktopPixels();
			ReshapeWindow(InitialDesiredScreenPosition - (WindowDesiredSizePixels * 0.5f), WindowDesiredSizePixels);
		}

		// Set the window to be maximized if we need to.  Note that this won't actually show the window if its not
		// already shown.
		InitialMaximize();

		// Set the window to be minimized if we need to.  Note that this won't actually show the window if its not
		// already shown.
		InitialMinimize();
	}

	bHasEverBeenShown = true;

	if (NativeWindow.IsValid())
	{
		NativeWindow->Show();

		// If this is a tompost window (like a tooltip), make sure that its always rendered top most
		if( IsTopmostWindow() )
		{
			NativeWindow->BringToFront();
		}
	}
}

/** Make the window invisible */
void SWindow::HideWindow()
{
	if (NativeWindow.IsValid())
	{
		NativeWindow->Hide();
	}
}

void SWindow::EnableWindow( bool bEnable )
{
	if (NativeWindow.IsValid())
	{
		NativeWindow->Enable( bEnable );
	}
	
	for( int32 ChildIndex = 0; ChildIndex < ChildWindows.Num(); ++ChildIndex )
	{
		ChildWindows[ChildIndex]->EnableWindow( bEnable );
	}
}


/** @return true if the window is visible, false otherwise*/
bool SWindow::IsVisible() const
{
	return NativeWindow.IsValid() && NativeWindow->IsVisible();
}

bool SWindow::IsWindowMaximized() const
{
	if ( NativeWindow.IsValid() )
	{
		return NativeWindow->IsMaximized();
	}

	return false;
}

bool SWindow::IsWindowMinimized() const
{
	if ( NativeWindow.IsValid() )
	{
		return NativeWindow->IsMinimized();
	}

	return false;
}


/** Maximize the window if bInitiallyMaximized is set */
void SWindow::InitialMaximize()
{
	if (NativeWindow.IsValid() && bInitiallyMaximized)
	{
		NativeWindow->Maximize();
	}
}

void SWindow::InitialMinimize()
{
	if (NativeWindow.IsValid() && bInitiallyMinimized)
	{
		NativeWindow->Minimize();
	}
}

/**
 * Sets the opacity of this window
 *
 * @param	InOpacity	The new window opacity represented as a floating point scalar
 */
void SWindow::SetOpacity( const float InOpacity )
{
	if( Opacity != InOpacity )
	{
		check( NativeWindow.IsValid() );
		Opacity = InOpacity;
		NativeWindow->SetOpacity( Opacity );
	}
}


/** @return the window's current opacity */
float SWindow::GetOpacity() const
{
	return Opacity;
}

EWindowTransparency SWindow::GetTransparencySupport() const
{
	return TransparencySupport;
}


/** @return A String representation of the widget */
FString SWindow::ToString() const
{
	return FText::Format(NSLOCTEXT("SWindow", "Window_TitleFmt", " Window : {0} "), GetTitle()).ToString();
}

/** @return the window activation policy used when showing the window */
EWindowActivationPolicy SWindow::ActivationPolicy() const
{
	return WindowActivationPolicy;
}

/** @return true if the window accepts input; false if the window is non-interactive */
bool SWindow::AcceptsInput() const
{
	return bAcceptsInput && Type != EWindowType::CursorDecorator
		&& (Type != EWindowType::ToolTip || !FSlateApplicationBase::Get().IsWindowHousingInteractiveTooltip(SharedThis(this)));
}

/** @return true if the user decides the size of the window; false if the content determines the size of the window */
bool SWindow::IsUserSized() const
{
	return SizingRule == ESizingRule::UserSized;
}

bool SWindow::IsAutosized() const
{
	return SizingRule == ESizingRule::Autosized;
}

void SWindow::SetSizingRule( ESizingRule InSizingRule )
{
	SizingRule = InSizingRule;
}

/** @return true if this is a vanilla window, or one being used for some special purpose: e.g. tooltip or menu */
bool SWindow::IsRegularWindow() const
{
	return !bIsPopupWindow && Type != EWindowType::ToolTip && Type != EWindowType::CursorDecorator;
}

/** @return true if the window should be on top of all other windows; false otherwise */
bool SWindow::IsTopmostWindow() const
{
	return bIsTopmostWindow;
}

/** @return true if mouse coordinates is within this window */
bool SWindow::IsScreenspaceMouseWithin(UE::Slate::FDeprecateVector2DParameter ScreenspaceMouseCoordinate) const
{
	const FVector2f LocalMouseCoordinate = ScreenspaceMouseCoordinate - ScreenPosition;
	return !LocalMouseCoordinate.ContainsNaN() && NativeWindow->IsPointInWindow(FMath::TruncToInt(LocalMouseCoordinate.X), FMath::TruncToInt(LocalMouseCoordinate.Y));
}

/** @return true if this is a user-sized window with a thick edge */
bool SWindow::HasSizingFrame() const
{
	return bHasSizingFrame;
}

/** @return true if this window has a close button/box on the titlebar area */
bool SWindow::HasCloseBox() const
{
	return bHasCloseButton;
}

/** @return true if this window has a maximize button/box on the titlebar area */
bool SWindow::HasMaximizeBox() const
{
	return bHasMaximizeButton;
}

/** @return true if this window has a minimize button/box on the titlebar area */
bool SWindow::HasMinimizeBox() const
{
	return bHasMinimizeButton;
}

FCursorReply SWindow::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	bool bUseOSSizingCursor = this->HasOSWindowBorder() && bHasSizingFrame;

#if PLATFORM_MAC // On Mac we depend on system's window resizing
	bUseOSSizingCursor = true;
#endif

	if (!bUseOSSizingCursor && bHasSizingFrame)
	{
		if (WindowZone == EWindowZone::TopLeftBorder || WindowZone == EWindowZone::BottomRightBorder)
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeSouthEast);
		}
		else if (WindowZone == EWindowZone::BottomLeftBorder || WindowZone == EWindowZone::TopRightBorder)
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeSouthWest);
		}
		else if (WindowZone == EWindowZone::TopBorder || WindowZone == EWindowZone::BottomBorder)
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);
		}
		else if (WindowZone == EWindowZone::LeftBorder || WindowZone == EWindowZone::RightBorder)
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
		}
	}
	return FCursorReply::Unhandled();
}

bool SWindow::OnIsActiveChanged( const FWindowActivateEvent& ActivateEvent )
{
	const bool bWasDeactivated = ActivateEvent.GetActivationType() == FWindowActivateEvent::EA_Deactivate;
	if (bWasDeactivated)
	{
		OnWindowDeactivated.ExecuteIfBound();	// deprecated
		WindowDeactivatedEvent.Broadcast();

		WidgetFocusedOnDeactivate.Reset();

		const EWindowMode::Type WindowMode = GetWindowMode();
		// If the window is not fullscreen, we do not want to automatically recapture the mouse unless an external UI such as Steam is open. Fullscreen windows we do.
		if (WindowMode != EWindowMode::Fullscreen && WidgetToFocusOnActivate.IsValid() && WidgetToFocusOnActivate.Pin()->HasMouseCapture() && !FSlateApplicationBase::Get().IsExternalUIOpened())
		{
			WidgetToFocusOnActivate.Reset();
		}
		else if (SupportsKeyboardFocus())
		{
			// If we have no specific widget to focus then cache the currently focused widget so we can restore its focus when we regain focus
			WidgetFocusedOnDeactivate = (FocusUserIndex == INDEX_NONE) ? FSlateApplicationBase::Get().GetKeyboardFocusedWidget() : FSlateApplicationBase::Get().GetUserFocusedWidget(FocusUserIndex);
			if (!WidgetFocusedOnDeactivate.IsValid())
			{
				WidgetFocusedOnDeactivate = FSlateApplicationBase::Get().GetUserFocusedWidget(0);
			}
		}
	}
	else
	{
		if (ActivateEvent.GetActivationType() == FWindowActivateEvent::EA_Activate)
		{
			TArray< TSharedRef<SWindow> > JustThisWindow;
			JustThisWindow.Add(SharedThis(this));

			// If we're becoming active and we were set to restore keyboard focus to a specific widget
			// after reactivating, then do so now
			TSharedPtr< SWidget > PinnedWidgetToFocus(WidgetToFocusOnActivate.Pin());
			if (PinnedWidgetToFocus.IsValid())
			{
				FWidgetPath WidgetToFocusPath;
				if (FSlateWindowHelper::FindPathToWidget(JustThisWindow, PinnedWidgetToFocus.ToSharedRef(), WidgetToFocusPath))
				{
					if (FocusUserIndex == INDEX_NONE)
					{
						FSlateApplicationBase::Get().SetAllUserFocus(WidgetToFocusPath, EFocusCause::WindowActivate);
					}
					else
					{
						FSlateApplicationBase::Get().SetUserFocus(FocusUserIndex, WidgetToFocusPath, EFocusCause::WindowActivate);
					}
				}
			}

			// If we didn't have a specified widget to focus (above)
			// We'll make sure all the users focus this window, however if they are already focusing something in the window we leave them be.
			else if (SupportsKeyboardFocus())
			{
				FWidgetPath WindowWidgetPath;
				TSharedRef<SWidget> WindowWidgetToFocus = WidgetFocusedOnDeactivate.IsValid() ? WidgetFocusedOnDeactivate.Pin().ToSharedRef() : AsShared();
				if (FSlateWindowHelper::FindPathToWidget(JustThisWindow, WindowWidgetToFocus, WindowWidgetPath))
				{
					if (FocusUserIndex == INDEX_NONE)
					{
						FSlateApplicationBase::Get().SetAllUserFocusAllowingDescendantFocus(WindowWidgetPath, EFocusCause::WindowActivate);
					}
					else
					{
						FSlateApplicationBase::Get().SetUserFocusAllowingDescendantFocus(FocusUserIndex, WindowWidgetPath, EFocusCause::WindowActivate);
					}
				}
			}
		}

		OnWindowActivated.ExecuteIfBound();	// deprecated
		WindowActivatedEvent.Broadcast();
	}

	return true;
}

void SWindow::Maximize()
{
	if (NativeWindow.IsValid())
	{
		NativeWindow->Maximize();
	}
}

void SWindow::Restore()
{
	if (NativeWindow.IsValid())
	{
		NativeWindow->Restore();
	}
}

void SWindow::Minimize()
{
	if (NativeWindow.IsValid())
	{
		NativeWindow->Minimize();
	}
}

int32 SWindow::GetCornerRadius()
{
	return Style->WindowCornerRadius;
}

bool SWindow::SupportsKeyboardFocus() const
{
	return Type != EWindowType::ToolTip && Type != EWindowType::CursorDecorator;
}




FReply SWindow::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	return FReply::Handled();
}

FReply SWindow::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (bDragAnywhere && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		MoveResizeZone = WindowZone;
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SWindow::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (bDragAnywhere &&  this->HasMouseCapture() && MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		MoveResizeZone =  EWindowZone::Unspecified;
		return FReply::Handled().ReleaseMouseCapture();
	}
	else
	{
		return FReply::Unhandled();
	}
}

FReply SWindow::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( bDragAnywhere && this->HasMouseCapture() && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && MoveResizeZone != EWindowZone::TitleBar )
	{
		this->MoveWindowTo( ScreenPosition + MouseEvent.GetCursorDelta() );
		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}

}

FVector2D SWindow::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return SCompoundWidget::ComputeDesiredSize(LayoutScaleMultiplier) * LayoutScaleMultiplier;
}

bool SWindow::ComputeVolatility() const
{
	// If the entire window is volatile in fast path that defeats the whole purpose.
	return Advanced_IsInvalidationRoot() ? false : SWidget::ComputeVolatility();
}

void SWindow::OnGlobalInvalidationToggled(bool bGlobalInvalidationEnabled)
{
	InvalidateRootChildOrder();
	UE_LOGF(LogSlate, Log, "Toggling fast path.  New State: %d", bGlobalInvalidationEnabled);
}

const TArray< TSharedRef<SWindow> >& SWindow::GetChildWindows() const
{
	return ChildWindows;
}

TArray< TSharedRef<SWindow> >& SWindow::GetChildWindows()
{
	return ChildWindows;
}

void SWindow::AddChildWindow( const TSharedRef<SWindow>& ChildWindow )
{
	TSharedPtr<SWindow> PreviousParent = ChildWindow->ParentWindowPtr.Pin();
	if (PreviousParent.IsValid())
	{
		// This child already had a parent, so we are actually re-parenting it
		const bool bRemovedSuccessfully = PreviousParent->RemoveDescendantWindow(ChildWindow);
		check(bRemovedSuccessfully);
	}

	ChildWindow->ParentWindowPtr = SharedThis(this);
	ChildWindow->SetWindowBackground(&Style->ChildBackgroundBrush);

	FSlateApplicationBase::Get().ArrangeWindowToFrontVirtual( ChildWindows, ChildWindow );
}

TSharedPtr<SWindow> SWindow::GetParentWindow() const
{
	return ParentWindowPtr.Pin();
}


TSharedPtr<SWindow> SWindow::GetTopmostAncestor()
{
	TSharedPtr<SWindow> TopmostParentSoFar = SharedThis(this);
	while ( TopmostParentSoFar->ParentWindowPtr.IsValid() )
	{
		TopmostParentSoFar = TopmostParentSoFar->ParentWindowPtr.Pin();
	}

	return TopmostParentSoFar;
}

bool SWindow::RemoveDescendantWindow( const TSharedRef<SWindow>& DescendantToRemove )
{
	const bool bRemoved = 0 != ChildWindows.Remove(DescendantToRemove);

	for ( int32 ChildIndex=0; ChildIndex < ChildWindows.Num(); ++ChildIndex )
	{
		TSharedRef<SWindow>& ChildWindow = ChildWindows[ChildIndex];
		if ( ChildWindow->RemoveDescendantWindow( DescendantToRemove ))
		{
			// Reset to the non-child background style
			ChildWindow->SetWindowBackground(&Style->BackgroundBrush);
			return true;
		}
	}

	return false;
}

void SWindow::SetOnWorldSwitchHack( FOnSwitchWorldHack& InOnSwitchWorldHack )
{
	OnWorldSwitchHack = InOnSwitchWorldHack;
}

int32 SWindow::SwitchWorlds( int32 WorldId ) const
{
	return OnWorldSwitchHack.IsBound() ? OnWorldSwitchHack.Execute( WorldId ) : false;
}

bool PointWithinSlateRect(const FVector2f& Point, const FSlateRect& Rect)
{
	return Point.X >= Rect.Left && Point.X < Rect.Right &&
		Point.Y >= Rect.Top && Point.Y < Rect.Bottom;
}

EWindowZone::Type SWindow::GetCurrentWindowZone(UE::Slate::FDeprecateVector2DParameter LocalMousePosition)
{
	const float WindowDPIScale = FSlateApplicationBase::Get().GetApplicationScale() * (NativeWindow.IsValid() ? GetDPIScaleFactor() : 1.0f);

	const FMargin DPIScaledResizeBorder = UserResizeBorder * WindowDPIScale;

	if (FSlateApplicationBase::Get().GetPlatformCursor()->GetType() == EMouseCursor::None)
	{
		return EWindowZone::ClientArea;
	}
	if (LocalMousePosition.X >= 0 && LocalMousePosition.X < Size.X && LocalMousePosition.Y >= 0 && LocalMousePosition.Y < Size.Y)
	{
		int32 Row = 1;
		int32 Col = 1;
		EWindowMode::Type Mode = GetWindowMode();
		const bool bFullscreen = Mode == EWindowMode::Fullscreen || Mode == EWindowMode::WindowedFullscreen;
		const bool bResizable = SizingRule == ESizingRule::UserSized && !bFullscreen && !NativeWindow->IsMaximized();
		if (bResizable)
		{
			if (LocalMousePosition.X < (DPIScaledResizeBorder.Left + 5))
			{
				Col = 0;
			}
			else if (LocalMousePosition.X >= Size.X - (DPIScaledResizeBorder.Right + 5))
			{
				Col = 2;
			}

			if (LocalMousePosition.Y < (DPIScaledResizeBorder.Top + 5))
			{
				Row = 0;
			}
			else if (LocalMousePosition.Y >= Size.Y - (DPIScaledResizeBorder.Bottom + 5))
			{
				Row = 2;
			}

			// The actual border is smaller than the hit result zones
			// This grants larger corner areas to grab onto
			bool bInBorder =	LocalMousePosition.X < DPIScaledResizeBorder.Left ||
								LocalMousePosition.X >= Size.X - DPIScaledResizeBorder.Right ||
								LocalMousePosition.Y < DPIScaledResizeBorder.Top ||
								LocalMousePosition.Y >= Size.Y - DPIScaledResizeBorder.Bottom;

			if (!bInBorder)
			{
				Row = 1;
				Col = 1;
			}
		}

		static const EWindowZone::Type TypeZones[3][3] =
		{
			{EWindowZone::TopLeftBorder,		EWindowZone::TopBorder,		EWindowZone::TopRightBorder},
			{EWindowZone::LeftBorder,			EWindowZone::ClientArea,	EWindowZone::RightBorder},
			{EWindowZone::BottomLeftBorder,		EWindowZone::BottomBorder,	EWindowZone::BottomRightBorder},
		};

		EWindowZone::Type InZone = TypeZones[Row][Col];
		if (InZone == EWindowZone::ClientArea)
		{
			// Hittest to see if the widget under the mouse should be treated as a title bar (i.e. should move the window)
			constexpr bool bIgnoreEnabledStatus = false;
			FWidgetPath HitTestResults = FSlateApplicationBase::Get().GetHitTesting()
				.LocateWidgetInWindow(GetPositionInScreen() + LocalMousePosition, SharedThis(this), bIgnoreEnabledStatus, INDEX_NONE);
			if( HitTestResults.Widgets.Num() > 0 )
			{
				const EWindowZone::Type ZoneOverride = HitTestResults.Widgets.Last().Widget->GetWindowZoneOverride();
				if (ZoneOverride != EWindowZone::Unspecified && (bResizable || !EWindowZone::IsBorder(ZoneOverride)))
				{
					// The widget overrode the window zone
					InZone = ZoneOverride;
				}
				else if( HitTestResults.Widgets.Last().Widget == AsShared() )
				{
					// The window itself was hit, so check for a traditional title bar
					if ((LocalMousePosition.Y - DPIScaledResizeBorder.Top) < TitleBarSize*WindowDPIScale)
					{
						InZone = EWindowZone::TitleBar;
					}
				}
			}

			WindowZone = InZone;
		}
		else if (FSlateApplicationBase::Get().AnyMenusVisible())
		{
			// Prevent resizing when a menu is open.  This is consistent with OS behavior and prevents a number of crashes when menus
			// stay open while resizing windows causing their parents to often be clipped (SClippingHorizontalBox)
			WindowZone = EWindowZone::ClientArea;
		}
		else
		{
			WindowZone = InZone;
		}
	}
	else
	{
		WindowZone = EWindowZone::NotInWindow;
	}
	return WindowZone;
}

/**
 * Default constructor. Protected because SWindows must always be used via TSharedPtr. Instead, use FSlateApplication::MakeWindow()
 */
SWindow::SWindow()
	: bDragAnywhere( false )
	, Opacity( 1.0f )
	, SizingRule( ESizingRule::UserSized )
	, TransparencySupport( EWindowTransparency::None )
	, bIsPopupWindow( false )
	, bIsTopmostWindow( false )
	, bSizeWillChangeOften( false )
	, bInitiallyMaximized( false )
	, bInitiallyMinimized(false)
	, bHasEverBeenShown( false )
	, bFocusWhenFirstShown( true )
	, bHasOSWindowBorder( false )
	, bHasCloseButton( false )
	, bHasMinimizeButton( false )
	, bHasMaximizeButton( false )
	, bHasSizingFrame( false )
	, bIsModalWindow( false )
	, bIsMirrorWindow( false )
	, bShouldPreserveAspectRatio( false )
	, bManualManageDPI( false )
	, bAllowFastUpdate( false )
	, bAcceptsInput( true )
	, WindowActivationPolicy( EWindowActivationPolicy::Always )
	, InitialDesiredScreenPosition( FVector2f::ZeroVector )
	, InitialDesiredSize( FVector2f::ZeroVector )
	, ScreenPosition( FVector2f::ZeroVector )
	, PreFullscreenPosition( FVector2f::ZeroVector )
	, Size( FVector2f::ZeroVector )
	, ViewportSize( FVector2f::ZeroVector )
	, ViewportScaleUIOverride( -1.f )
	, TitleBarSize( SWindowDefs::DefaultTitleBarSize )
	, ContentSlot(nullptr)
	, Style( &FCoreStyle::Get().GetWidgetStyle<FWindowStyle>("Window") )
	, WindowBackground( &Style->BackgroundBrush )
	, HittestGrid(MakeUnique<FHittestGrid>())
	, bShouldShowWindowContentDuringOverlay( false )
	, ExpectedMaxWidth( INDEX_NONE )
	, ExpectedMaxHeight( INDEX_NONE )
	, FocusUserIndex( INDEX_NONE )
	, TitleBar()
	, bIsDrawingEnabled( true )
	
{
	bHasCustomPrepass = true;
	SetInvalidationRootWidget(*this);
	SetInvalidationRootHittestGrid(*HittestGrid);

#if WITH_ACCESSIBILITY
	AccessibleBehavior = EAccessibleBehavior::Auto;
#endif
}

SWindow::~SWindow()
{
	check(IsInGameThread());
}

TSharedRef<SWidget> SWindow::GetRootWidget()
{
	return AsShared();
}

int32 SWindow::PaintSlowPath(const FSlateInvalidationContext& Context)
{
	HittestGrid->Clear();

	const FSlateRect WindowCullingBounds = GetClippingRectangleInWindow();
	const int32 LayerId = 0;
	const FGeometry WindowGeometry = GetWindowGeometryInWindow();

	int32 MaxLayerId = 0;

	//OutDrawElements.PushBatchPriortyGroup(*this);
	{
		
		MaxLayerId = Paint(*Context.PaintArgs, WindowGeometry, WindowCullingBounds, *Context.WindowElementList, LayerId, Context.WidgetStyle, Context.bParentEnabled);
	}

	//OutDrawElements.PopBatchPriortyGroup();



	return MaxLayerId;
}

int32 SWindow::PaintWindow( double CurrentTime, float DeltaTime, FSlateWindowElementList& OutDrawElements, const FWidgetStyle& InWidgetStyle, bool bParentEnabled )
{
	UE_SLATE_CRASH_REPORTER_PAINT_SCOPE(*this);

	OutDrawElements.BeginDeferredGroup();

	const bool HittestCleared = HittestGrid->SetHittestArea(GetPositionInScreen(), GetViewportSize());


	FPaintArgs PaintArgs(nullptr, GetHittestGrid(), GetPositionInScreen(), CurrentTime, DeltaTime);

	FSlateInvalidationContext Context(OutDrawElements, InWidgetStyle);
	Context.bParentEnabled = bParentEnabled;
	// Fast path at the window level should only be enabled if global invalidation is allowed
	Context.bAllowFastPathUpdate = Advanced_IsInvalidationRoot();
	Context.LayoutScaleMultiplier = FSlateApplicationBase::Get().GetApplicationScale() * GetDPIScaleFactor();
	Context.PaintArgs = &PaintArgs;
	Context.IncomingLayerId = 0;
	Context.CullingRect = GetClippingRectangleInWindow();

	// Always set the window geometry and visibility
	PersistentState.AllottedGeometry = GetWindowGeometryInWindow();
	PersistentState.CullingBounds = GetClippingRectangleInWindow();
	if (!GetVisibilityAttribute().IsBound())
	{
		SetVisibility(GetWindowVisibility());
	}


	FSlateInvalidationResult Result = PaintInvalidationRoot(Context);

#if WITH_SLATE_DEBUGGING
	if (GSlateHitTestGridDebugging)
	{
		const FGeometry& WindowGeometry = GetWindowGeometryInWindow();
		HittestGrid->DisplayGrid(INT_MAX, WindowGeometry, OutDrawElements);
	}
	if (GSlateNavigationDebugging)
	{
		const FGeometry& WindowGeometry = GetWindowGeometryInWindow();
		HittestGrid->DisplayNavigation(INT_MAX, WindowGeometry, OutDrawElements);
	}
#endif

	OutDrawElements.EndDeferredGroup();

	if (Context.bAllowFastPathUpdate)
	{
		OutDrawElements.PushCachedElementData(GetCachedElements());
	}

	if (OutDrawElements.ShouldResolveDeferred())
	{
		Result.MaxLayerIdPainted = OutDrawElements.PaintDeferred(Result.MaxLayerIdPainted, Context.CullingRect);
	}

	if (Context.bAllowFastPathUpdate)
	{
		OutDrawElements.PopCachedElementData();
	}

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::PaintDebugElements.Broadcast(PaintArgs, GetWindowGeometryInWindow(), OutDrawElements, Result.MaxLayerIdPainted);
#endif 

	return Result.MaxLayerIdPainted;

}

int32 SWindow::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 MaxLayer = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	return MaxLayer;
}

FOptionalSize SWindow::GetTitleBarSize() const
{
	return TitleBarSize;
}


UE::Slate::FDeprecateVector2DResult SWindow::GetDesiredSizeDesktopPixels() const
{
	// Note that the window already takes the layout multiplier
	// into account when computing its desired size.
	// @See SWindow::ComputeDesiredSize
	return this->GetDesiredSize();
}

void SWindow::SetFullWindowOverlayContent(TSharedPtr<SWidget> InContent)
{
	if( FullWindowOverlayWidget.IsValid() )
	{
		// Remove the last slot
		WindowOverlay->RemoveSlot( FullWindowOverlayWidget.ToSharedRef() );
		FullWindowOverlayWidget.Reset();
	}

	if( InContent.IsValid() )
	{
		FullWindowOverlayWidget = InContent;

		// Create a slot in our overlay to hold the content
		WindowOverlay->AddSlot( 1 )
		[
			InContent.ToSharedRef()
		];
	}

	UpdateWindowContentVisibility();
}

/** Toggle window between fullscreen and normal mode */
void SWindow::SetWindowMode( EWindowMode::Type NewWindowMode )
{
	EWindowMode::Type CurrentWindowMode = NativeWindow->GetWindowMode();

	if( CurrentWindowMode != NewWindowMode )
	{
		bool bFullscreen = NewWindowMode != EWindowMode::Windowed;

		bool bWasFullscreen = CurrentWindowMode != EWindowMode::Windowed;

		// We need to store off the screen position when entering fullscreen so that we can move the window back to its original position after leaving fullscreen
		if( bFullscreen )
		{
			PreFullscreenPosition = ScreenPosition;
		}

		bIsDrawingEnabled = false;

		NativeWindow->SetWindowMode( NewWindowMode );

		const FVector2f vp = IsMirrorWindow() ? GetSizeInScreen() : GetViewportSize();
		FSlateApplicationBase::Get().GetRenderer()->UpdateFullscreenState(SharedThis(this), (uint32)vp.X, (uint32)vp.Y);

		if( TitleArea.IsValid() )
		{
			// Collapse the Window title bar when switching to Fullscreen
			TitleArea->SetVisibility( (NewWindowMode == EWindowMode::Fullscreen || NewWindowMode == EWindowMode::WindowedFullscreen ) ? EVisibility::Collapsed : EVisibility::Visible );
		}

		if( bWasFullscreen )
		{
			// If we left fullscreen, reset the screen position;
			MoveWindowTo(PreFullscreenPosition);
		}

		bIsDrawingEnabled = true;
	}

}

bool SWindow::HasFullWindowOverlayContent() const
{
	return FullWindowOverlayWidget.IsValid();
}

void SWindow::BeginFullWindowOverlayTransition()
{
	bShouldShowWindowContentDuringOverlay = true;
	UpdateWindowContentVisibility();
}

void SWindow::EndFullWindowOverlayTransition()
{
	bShouldShowWindowContentDuringOverlay = false;
	UpdateWindowContentVisibility();
}

void SWindow::SetNativeWindowButtonsVisibility(bool bVisible)
{
	if (NativeWindow.IsValid())
	{
		NativeWindow->SetNativeWindowButtonsVisibility(bVisible);
	}
}

EActiveTimerReturnType SWindow::TriggerPlayMorphSequence( double InCurrentTime, float InDeltaTime )
{
	Morpher.Sequence.Play( this->AsShared() );
	return EActiveTimerReturnType::Stop;
}

void SWindow::SetWindowBackground(const FSlateBrush* InWindowBackground)
{
	WindowBackground = InWindowBackground;
	if (WindowBackgroundImage)
	{
		WindowBackgroundImage->SetImage(WindowBackground);
	}
}

void SWindow::UpdateWindowContentVisibility()
{
	// The content of the window should be visible unless we have a full window overlay content
	// in which case the full window overlay content is visible but nothing under it
	WindowContentVisibility = (bShouldShowWindowContentDuringOverlay == true || !FullWindowOverlayWidget.IsValid()) ? EVisibility::SelfHitTestInvisible : EVisibility::Hidden;

	if (WindowBackgroundImage.IsValid())
	{
		WindowBackgroundImage->SetVisibility(WindowContentVisibility);
	}

	if (WindowBorder.IsValid())
	{
		WindowBorder->SetVisibility(WindowContentVisibility);
	}

	if (WindowOutline.IsValid())
	{
		WindowOutline->SetVisibility(WindowContentVisibility);
	}

	if (ContentAreaVBox.IsValid())
	{
		ContentAreaVBox->SetVisibility(WindowContentVisibility);
	}
}

#if WITH_EDITOR
FScopedSwitchWorldHack::FScopedSwitchWorldHack( const FWidgetPath& WidgetPath )
	: Window( WidgetPath.TopLevelWindow )
	, WorldId( -1 )
{
	if( Window.IsValid() )
	{
		WorldId = Window->SwitchWorlds( WorldId );
	}
}
#endif

#if WITH_ACCESSIBILITY
TSharedRef<FSlateAccessibleWidget> SWindow::CreateAccessibleWidget()
{
	return MakeShareable<FSlateAccessibleWidget>(new FSlateAccessibleWindow(SharedThis(this)));
}

TOptional<FText> SWindow::GetDefaultAccessibleText(EAccessibleType AccessibleType) const
{
	return GetTitle();
}
#endif
