// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"

/**
 * Types of easing functions for Slate animation curves.  These are used to smooth out animations.
 */
enum class ECurveEaseFunction : uint8
{
	/** Linear interpolation, with no easing */
	Linear,

	/** Quadratic ease in */
	QuadIn,
		
	/** Quadratic ease out */
	QuadOut,

	/** Quadratic ease in, quadratic ease out */
	QuadInOut,

	/** Cubic ease in */
	CubicIn,
		
	/** Cubic ease out */
	CubicOut,

	/** Cubic ease in, cubic ease out */
	CubicInOut,
};


/**
 * A handle to curve within a curve sequence.
 */
struct FCurveHandle
{
	/**
	 * Creates and initializes a curve handle.
	 *
	 * @param InOwnerSequence The curve sequence that owns this handle.
	 * @param InCurveIndex The index of this handle.
	 */
	SLATECORE_API FCurveHandle( const struct FCurveSequence* InOwnerSequence = nullptr, int32 InCurveIndex = 0 );

public:

	/**
	 * Gets the linearly interpolated value between 0 and 1 for this curve.
	 *
	 * @return Lerp value.
	 * @see GetLerpLooping
	 */
	SLATECORE_API float GetLerp( ) const;

	/**
	 * Checks whether this handle is initialized.
	 *
	 * A curve handle is considered initialized if it has an owner sequence.
	 * @return true if initialized, false otherwise.
	 */
	bool IsInitialized( ) const
	{
		return (OwnerSequence != nullptr);
	}

public:

	/** Applies animation easing to lerp value */
	static SLATECORE_API float ApplyEasing( float Time, ECurveEaseFunction EaseType );

private:

	/** The sequence containing this curve */
	const struct FCurveSequence* OwnerSequence;

	/** The index of the curve in the Curves array */
	int32 CurveIndex;
};
