// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/CurveHandle.h"
#include "Containers/Ticker.h"

class FActiveTimerHandle;
class SWidget;
enum class EActiveTimerReturnType : uint8;

/**
 * A sequence of curves that can be used to drive animations for slate widgets.
 * Active timer registration is handled for the widget being animated when calling play.
 *
 * Each curve within the sequence has a time offset and a duration.
 * This makes FCurveSequence convenient for crating staggered animations.
 * e.g.
 *   // We want to zoom in a widget, and then fade in its contents.
 *   FCurveHandle ZoomCurve = Sequence.AddCurve( 0, 0.15f );
 *   FCurveHandle FadeCurve = Sequence.AddCurve( 0.15f, 0.1f );
 *	 Sequence.Play( this->AsShared() );
 */
struct FCurveSequence : public TSharedFromThis<FCurveSequence>
{
public:

	/** A curve has a time offset and duration.*/
	struct FSlateCurve
	{
		/** Constructor */
		FSlateCurve( float InStartTime, float InDurationSeconds, const ECurveEaseFunction InEaseFunction )
			: DurationSeconds(InDurationSeconds)
			, StartTime(InStartTime)
			, EaseFunction(InEaseFunction)
		{
		}

		/** Length of this animation in seconds */
		float DurationSeconds;

		/** Start time for this animation */
		float StartTime;

		/**
		 * Type of easing function to use for this curve.
		 * Could be passed it at call site.
		 */
		ECurveEaseFunction EaseFunction;
	};

	/** Default constructor */
	SLATECORE_API FCurveSequence( );

	/** Makes sure the active timer is unregistered */
	SLATECORE_API ~FCurveSequence();

	/**
	 * Construct by adding a single animation curve to this sequence.  Does not provide access to the curve though.
	 *
	 * @param InStartTimeSeconds   When to start this curve.
	 * @param InDurationSeconds    How long this curve lasts.
	 * @param InEaseFunction       Easing function to use for this curve.  Defaults to Linear.  Use this to smooth out your animation transitions.
	 * @return A FCurveHandle that can be used to get the value of this curve after the animation starts playing.
	 */
	SLATECORE_API FCurveSequence( const float InStartTimeSeconds, const float InDurationSeconds, const ECurveEaseFunction InEaseFunction = ECurveEaseFunction::Linear  );

	/**
	 * Add a new curve at a given time and offset.
	 *
	 * @param InStartTimeSeconds   When to start this curve.
	 * @param InDurationSeconds    How long this curve lasts.
	 * @param InEaseFunction       Easing function to use for this curve.  Defaults to Linear.  Use this to smooth out your animation transitions.
	 * @return A FCurveHandle that can be used to get the value of this curve after the animation starts playing.
	 */
	SLATECORE_API FCurveHandle AddCurve( const float InStartTimeSeconds, const float InDurationSeconds, const ECurveEaseFunction InEaseFunction = ECurveEaseFunction::Linear );

	/**
	 * Add a new curve relative to the current end of the sequence. Makes stacking easier.
	 * e.g. doing 
	 *     AddCurveRelative(0,5);
	 *     AddCurveRelative(0,3);
	 * Is equivalent to
	 *     AddCurve(0,5);
	 *     AddCurve(5,3)
	 *
	 * @param InOffset             Offset from the last curve in the sequence.
	 * @param InDurationSecond     How long this curve lasts.
	 * @param InEaseFunction       Easing function to use for this curve.  Defaults to Linear.  Use this to smooth out your animation transitions.
	 */
	SLATECORE_API FCurveHandle AddCurveRelative( const float InOffset, const float InDurationSecond, const ECurveEaseFunction InEaseFunction = ECurveEaseFunction::Linear );

	/**
	 * Start playing this curve sequence. Registers an active timer with the widget being animated.
	 *
	 * @param InOwnerWidget The widget that is being animated by this sequence.
	 * @param bPlayLooped True if the curve sequence should play continually on a loop. Note that the active timer will persist until this sequence is paused or jumped to the start/end.
	 * @param StartAtTime The relative time within the animation at which to begin playing (i.e. 0.0f is the beginning).
	 * @param bRequiresActiveTimer	Whether or not we need to register an active timer on the widget to keep slate ticking while the animation is playing.  If that is not necessary in your use case, you can set it to false for a small performance boost
	 */
	SLATECORE_API void Play( const TSharedRef<SWidget>& InOwnerWidget, bool bPlayLooped = false, const float StartAtTime = 0.0f, bool bRequiresActiveTimer = true);

	/**
	 * Plays the curve sequence for a ticker, rather than a widget.
	 */
	SLATECORE_API void Play(const FTickerDelegate& InDelegate, bool bPlayLooped = false, const float StartAtTime = 0.0f);

	/**
	 * Start playing this curve sequence in reverse. Registers an active timer for the widget using the sequence.
	 *
	 * @param InOwnerWidget The widget that is being animated by this sequence.
	 * @param bPlayLooped True if the curve sequence should play continually on a loop. Note that the active timer will persist until this sequence is paused or jumped to the start/end.
	 * @param StartAtTime The relative time within the animation at which to begin playing (i.e. 0.0f is the beginning).
	 * @param bRequiresActiveTimer	Whether or not we need to register an active timer on the widget to keep slate ticking while the animation is playing.  If that is not necessary in your use case, you can set it to false for a small performance boost
	 */
	SLATECORE_API void PlayReverse( const TSharedRef<SWidget>& InOwnerWidget, bool bPlayLooped = false, const float StartAtTime = 0.0f, bool bRequiresActiveTimer = true);

	/** Reverse the direction of an in-progress animation */
	SLATECORE_API void Reverse( );

	/** Pause this curve sequence. */
	SLATECORE_API void Pause();

	/** Unpause this curve sequence to resume play. */
	SLATECORE_API void Resume( );

	/** Plays forward if it can, otherwise holds it at the end of the animation, if we play in reverse it will reverse the animation if it can */
	SLATECORE_API void PlayRelative(const TSharedRef<SWidget>& InOwnerWidget, bool bForward);

	/**
	 * Checks whether the sequence is currently playing.
	 *
	 * @return true if playing, false otherwise.
	 */
	SLATECORE_API bool IsPlaying( ) const;

	/** @return the current time relative to the beginning of the sequence. */
	SLATECORE_API float GetSequenceTime( ) const;

	/** @return true if the animation is in reverse */
	SLATECORE_API bool IsInReverse( ) const;

	/** @return true if the animation is in forward gear */
	SLATECORE_API bool IsForward( ) const;

	/** Jumps immediately to the beginning of the animation sequence */
	SLATECORE_API void JumpToStart( );

	/** Jumps immediately to the end of the animation sequence */
	SLATECORE_API void JumpToEnd( );

	/** Is the sequence at the start? */
	SLATECORE_API bool IsAtStart( ) const;

	/** Is the sequence at the end? */
	SLATECORE_API bool IsAtEnd( ) const;

	/** Is the sequence looping? */
	SLATECORE_API bool IsLooping() const;

	/**
	 * For single-curve animations, returns the interpolation alpha for the animation.  If you call this function
	 * on a sequence with multiple curves, an assertion will trigger.
	 *
	 * @return A linearly interpolated value between 0 and 1 for this curve.
	 */
	SLATECORE_API float GetLerp() const;

	/**
	 * @param CurveIndex  Index of a curve in the curves array.
	 *
	 * @return A curve given the index into the curves array
	 */
	SLATECORE_API const FCurveSequence::FSlateCurve& GetCurve( int32 CurveIndex ) const;

protected:

	/** @param InStartTime  when this curve sequence started playing */
	SLATECORE_API void SetStartTime( double InStartTime );

private:
	/** Helper to take care of registering the active timer */
	void RegisterActiveTimerIfNeeded(TSharedRef<SWidget> InOwnerWidget);

	/** Tick callback from the Ticker based plays. */
	bool TickPlay(float InDeltaTime, FTickerDelegate InUserDelegate);

	/** Hollow active timer to ensure a Slate Tick/Paint pass while the sequence is playing */
	EActiveTimerReturnType EnsureSlateTickDuringAnimation( double InCurrentTime, float InDeltaTime );

private:

	/** 
	 * Weak reference to the owner widget that is being animated by this curve sequence.
	 * Necessary to ensure the active timer is unregistered if the sequence is destroyed before/by the owner.
	 */
	TWeakPtr<SWidget> OwnerWidget;

	/** The handle to the active timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	/** Handle to the ticker based player. */
	FTSTicker::FDelegateHandle TickerHandle;

	/** All the curves in this sequence. */
	TArray<FSlateCurve> Curves;

	/** When the curve started playing. */
	double StartTime;

	/** When the curve was paused */
	double PauseTime;

	/** How long the entire sequence lasts. */
	double TotalDuration;

	/** Are we playing the animation in reverse */
	uint8 bInReverse : 1;

	/** Is the sequence playing on a loop? */
	uint8 bIsLooping : 1;

	/** Is the sequence currently paused? */
	uint8 bIsPaused : 1;
};
