// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * One-dimensional spring simulation
 */
template< typename FloatType >
class TSpring1D
{

public:

	/**
	 * Spring configuration
	 */
	class FSpringConfig
	{

	public:

		/** Spring constant (how springy, lower values = more springy!) */
		FloatType SpringConstant;

		/** Length of the spring */
		FloatType SpringLength;

		/** Damp constant */
		FloatType DampConstant;

		/** Epsilon for snapping position and velocity */
		FloatType SnappingEpsilon;

		/** Whether to skip animation when a hitch occurs.  If enabled, the spring's position will be set
		    to the target position with large quantums, otherwise the spring will animate in slow motion. */
		bool bSkipAnimationOnHitches;


		/** Constructor */
		FSpringConfig()
			: SpringConstant( FloatType( 20.0 ) ),
			  SpringLength( FloatType( 0.0 ) ),
			  DampConstant( FloatType( 0.5 ) ),
			  SnappingEpsilon( FloatType( 0.01 ) ),
			  bSkipAnimationOnHitches( true )
		{
		}
	};



	/**
	 * Constructor
	 */
	TSpring1D()
		: Position( 0.0 ),
		  Target( 0.0 ),
		  PreviousTarget( 0.0 )
	{
	}


	/**
	 * Construct at specified position
	 */
	TSpring1D( FloatType InPosition )
		: Position( InPosition ),
		  Target( InPosition ),
		  PreviousTarget( InPosition )
	{
	}


	/**
	 * Sets the config for this spring
	 *
	 * @param  InConfig  New configuration
	 */
	void SetConfig( const FSpringConfig& InConfig )
	{
		Config = InConfig;
	}


	/**
	 * Sets the current position (and target position) for the spring
	 *
	 * @param  InTarget  New position
	 */
	void SetPosition( FloatType InPosition )
	{
		Position = InPosition;
		Target = InPosition;
		PreviousTarget = InPosition;
	}

	
	/**
	 * Gets the current position of the spring
	 *
	 * @return  Current position for this spring
	 */
	FloatType GetPosition() const
	{
		return Position;
	}


	/**
	 * Sets the target position for the spring
	 *
	 * @param  InTarget  New target position
	 */
	void SetTarget( FloatType InTarget )
	{
		Target = InTarget;
	}

	
	/**
	 * Gets the target position
	 *
	 * @return  Target position for this spring
	 */
	FloatType GetTarget() const
	{
		return Target;
	}

	/** @return True if the spring is at rest (i.e. at its target position) */
	bool IsAtRest()
	{
		return Target == Position;
	}

	/**
	 * Updates the simulation.  Should be called every tick!
	 *
	 * @param  InQuantum  Time passed since last update
	 */
	void Tick( float InQuantum )
	{
		const float MaxQuantum = 1.0f / 8.0f;
		if( InQuantum > MaxQuantum )
		{
			// If we are configured to reset the spring's state to the new position immediately upon
			// a hitch, then we'll do that now.
			if( Config.bSkipAnimationOnHitches )
			{
				Position = PreviousTarget = Target;
			}
			else
			{
				// Not asked to reset on large quantums, so we'll instead clamp the quantum so that
				// the spring does not behave erratically. (slow motion)
				InQuantum = MaxQuantum;
			}
		}

		const FloatType Disp = Target - Position;
		const FloatType DispLength = FMath::Abs( Disp );
		if( DispLength > Config.SnappingEpsilon )
		{
			const FloatType ForceDirection = Disp >= 0.0f ? 1.0f : -1.0f;
			const FloatType TargetDisp = Target - PreviousTarget;
			const FloatType VelocityOfTarget = TargetDisp * InQuantum;
			const FloatType DistBetweenDisplacements = FMath::Abs( Disp - VelocityOfTarget );

			const FloatType ForceAmount =
				Config.SpringConstant * FMath::Max< FloatType >( 0.0, DispLength - Config.SpringLength ) +
				Config.DampConstant * DistBetweenDisplacements;

			// We use Min here to prevent overshoots
			Position += ForceDirection * FMath::Min( DispLength, ForceAmount * InQuantum );
		}

		// Snap new position to target if we're close enough
		if( FMath::Abs( Position - Target ) < Config.SnappingEpsilon )
		{
			Position = Target;
		}

		PreviousTarget = Target;
	}

private:

	/** Configuration */
	FSpringConfig Config;

	/** Current position */
	FloatType Position;

	/** Target position */
	FloatType Target;

	/** Previous target position */
	FloatType PreviousTarget;

};

/** One-dimensional float spring */
typedef TSpring1D<float> FFloatSpring1D;

/** One-dimensional double spring */
typedef TSpring1D<double> FDoubleSpring1D;
