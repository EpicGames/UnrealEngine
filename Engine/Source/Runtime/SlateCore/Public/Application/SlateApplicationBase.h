// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "GenericPlatform/GenericApplication.h"
#include "Layout/Visibility.h"
#include "Layout/SlateRect.h"
#include "Rendering/SlateRenderer.h"
#include "Misc/CoreDelegates.h"
#include "Async/TaskGraphInterfaces.h"
#include "Types/SlateVector2.h"

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDebugSafeZoneChanged, const FMargin&, bool);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnConstrainedAspectRatioChanged, const float&);


class FActiveTimerHandle;
#if WITH_ACCESSIBILITY
class FSlateAccessibleMessageHandler;
#endif
class FSlateApplicationBase;
class FWidgetPath;
class IToolTip;
class SWidget;
class SWindow;
class SImage;
enum class EInvalidateWidgetReason : uint8;

template< typename ObjectType > class TAttribute;

struct FWindowTitleBarArgs
{
	FWindowTitleBarArgs(const TSharedRef<SWindow>& InWindow)
		: Window(InWindow)
		, CenterContentAlignment(HAlign_Fill)
	{}

	/** The window to create the title bar for */
	TSharedRef<SWindow> Window;
	
	/** Optional content for the title bar's center (will override window title) */
	TSharedPtr<SWidget> CenterContent;

	/** The horizontal alignment of the center content. */
	EHorizontalAlignment CenterContentAlignment;

	/** The text for the tooltip that shows when the user hovers over the Close button. */
	TAttribute<FText> CloseButtonToolTipText;
};

/**
 * Interface for window title bars.
 */
class IWindowTitleBar
{
public:
	virtual void Flash( ) = 0;
	virtual void UpdateWindowMenu(TSharedPtr<SWidget> MenuContent) = 0;
	virtual void UpdateBackgroundContent(TSharedPtr<SWidget> BackgroundContent) = 0;
	virtual void SetAllowMenuBar(bool bInAllowMenuBar) = 0;
};

class FSlateApplicationBase;


/**
 * Private interface to control which classes are allowed to perform hit-testing.
 */
class FHitTesting
{
public:
	FHitTesting(FSlateApplicationBase* InSlateApplication)
		: SlateApp(InSlateApplication)
	{
	}

private:
	// SWindow must be able to test which part of the window is being moused-over
	friend class SWindow;
	
private:	
	FSlateApplicationBase* SlateApp;
	// @see FSlateApplicationBase::LocateWidgetInWindow
	FWidgetPath LocateWidgetInWindow(FVector2f ScreenspaceMouseCoordinate, const TSharedRef<SWindow>& Window, bool bIgnoreEnabledStatus, int32 UserIndex) const;
};

/**
 * Design constraints for Slate applications
 */
namespace SlateApplicationDefs
{
	/** How many hardware users can we support at once? */
	static const int32 MaxHardwareUsers = 8;
}

/** Determines how popup window placement should treat the cursor. */
enum class EPopupCursorOverlapMode : uint8
{
	/** The popup window is not allowed to overlap the position of the cursor. */
	PreventOverlap = 0,
	/** The popup window may overlap the position of the cursor. */
	AllowOverlap
};

/**
 * Base class for Slate applications.
 *
 * This class currently serves a temporary workaround for solving SlateCore dependencies to FSlateApplication.
 * It should probably be removed once FSlateApplication has been refactored into SlateCore.
 */
class FSlateApplicationBase
{
	friend class SWidget;
public:

	SLATECORE_API FSlateApplicationBase();
	virtual ~FSlateApplicationBase() = default;

	/**
	 * Whether the application is active.
	 *
	 * @return application active or not
	 */
	virtual bool IsActive() const = 0;

	/**
	 * Gets the renderer being used to draw this application.
	 *
	 * @return The Slate renderer.
	 */
	inline FSlateRenderer* GetRenderer() const
	{
		return Renderer.Get();
	}

public:

	/**
	 * Associates a top level Slate Window with a native window and ensures that it is tracked properly by the application.
	 * Calling this method will cause the window to be displayed (unless specified otherwise), so be sure to associate
	 * content with the window object you're passing in first!
	 *
	 * @param InSlateWindow A SlateWindow to which to add a native window.
	 * @param bShowImmediately true to show the window, false if you're going to call ShowWindow() yourself later.
	 *
	 * @return a reference to the SWindow that was just added.
	 */
	virtual TSharedRef<SWindow> AddWindow( TSharedRef<SWindow> InSlateWindow, const bool bShowImmediately = true ) = 0;

	/**
	 * Reorders an array of windows so the specified window is "brought to the front"
	 */
	virtual void ArrangeWindowToFrontVirtual( TArray<TSharedRef<SWindow>>& Windows, const TSharedRef<SWindow>& WindowToBringToFront ) = 0;

	/**
	 * Searches for the specified widget and generates a full path to it.
	 *
	 * Note: this is a relatively slow operation!
	 * 
	 * @param  InWidget       Widget to generate a path to
	 * @param  OutWidgetPath  The generated widget path
	 * @param  VisibilityFilter	Widgets must have this type of visibility to be included the path
	 * @return	True if the widget path was found
	 */
	virtual bool FindPathToWidget( TSharedRef<const SWidget> InWidget, FWidgetPath& OutWidgetPath, EVisibility VisibilityFilter = EVisibility::Visible ) = 0;

	/**
	 * Returns the window the provided widget is contained in 

	 * @param  InWidget       Widget to find the window for
	 * @return	returns the window the provided widget is contained in or nullptr if it's not currently in a window
	 */
	virtual TSharedPtr<SWindow> FindWidgetWindow(TSharedRef<const SWidget> InWidget) const = 0;

	/**
	 * Gets the active top-level window.
	 *
	 * @return The top level window, or nullptr if no Slate windows are currently active.
	 */
	virtual TSharedPtr<SWindow> GetActiveTopLevelWindow() const = 0;

	/**
	 * Gets the active regular top-level window.
	 * A regular window is a non-menu, non-tooltip, non-cursor decorator window
	 * @return The top level window, or nullptr if no Slate windows are currently active.
	 */
	virtual TSharedPtr<SWindow> GetActiveTopLevelRegularWindow() const = 0;

	/**
	 * Get a list of all top-level windows in the application, excluding virtual windows.
	 *
	 * @return An array of all current top-level SWindows.
	 */
	virtual const TArray<TSharedRef<SWindow>> GetTopLevelWindows() const = 0;

	/**
	 * Gets the global application icon.
	 *
	 * @return The icon.
	 */
	virtual const FSlateBrush* GetAppIcon( ) const = 0;
	virtual const FSlateBrush* GetAppIconSmall() const = 0;

	/**
	 * Gets the ratio SlateUnit / ScreenPixel.
	 *
	 * @return Application scale.
	 */
	virtual float GetApplicationScale( ) const = 0;

	/**
	 * Gets Slate's current cached real time.
	 *
	 * This time value is updated every frame right before we tick widgets and is the recommended time value to use
	 * for UI animations and transitions, as opposed to calling FPlatformTime::Seconds() (which is generally slower)
	 *
	 * @return  The current Slate real time in seconds
	 */
	virtual const double GetCurrentTime( ) const = 0;

	/**
	 * Gets the current position of the cursor.
	 *
	 * @return Cursor position.
	 */
	virtual UE::Slate::FDeprecateVector2DResult GetCursorPos( ) const = 0;

	/**
	* Gets the last known position of the cursor.
	*
	* @return Cursor position.
	*/
	virtual UE::Slate::FDeprecateVector2DResult GetLastCursorPos( ) const = 0;

	/**
	 * Gets the size of the cursor..
	 *
	 * @return Cursor size.
	 */
	virtual UE::Slate::FDeprecateVector2DResult GetCursorSize( ) const = 0;

	/** 
	 * Whether the software cursor is enabled for this application.  
	 *
	 * @return software cursor available or not.
	 */
	virtual bool GetSoftwareCursorAvailable( ) const = 0;

	/** 
	 * Gets the desired visibility of the software cursor
	 *
	 * @return Software Cursor visibility
	 */
	virtual EVisibility GetSoftwareCursorVis( ) const = 0;

	/**
	 * Gets the application's display metrics.
	 *
	 * @param OutDisplayMetrics Will contain the display metrics.
	 */
	SLATECORE_API void GetDisplayMetrics(FDisplayMetrics& OutDisplayMetrics);

	/**
	 * Gets the application's cached display metrics.
	 *
	 * @param OutDisplayMetrics Will contain the display metrics.
	 */
	SLATECORE_API void GetCachedDisplayMetrics(FDisplayMetrics& OutDisplayMetrics) const;

	/**
	 * Gets the application's cached display metrics without copy.
	 *
	 * @return The cached display metrics.
	 */
	const FDisplayMetrics& GetCachedDisplayMetricsByRef() const { return CachedDisplayMetrics; }

	SLATECORE_API void GetSafeZoneSize(FMargin& SafeZone, const UE::Slate::FDeprecateVector2DParameter& OverrideSize);

	/**
	 * Get the highest level of window transparency support currently enabled by this application
	 *
	 * @return Enumeration value specifying the level of transparency currently supported
	 */
	virtual EWindowTransparency GetWindowTransparencySupport() const = 0;

	/**
	 * Gets the widget that currently has keyboard focus, if any.
	 *
	 * @return The focused widget, or nullptr if no widget has focus.
	 */
	virtual TSharedPtr< SWidget > GetKeyboardFocusedWidget( ) const = 0;

	virtual EUINavigation GetNavigationDirectionFromKey( const FKeyEvent& InKeyEvent ) const = 0;
	virtual EUINavigation GetNavigationDirectionFromAnalog(const FAnalogInputEvent& InAnalogEvent) = 0;
	virtual EUINavigationAction GetNavigationActionFromKey(const FKeyEvent& InKeyEvent) const = 0;
	virtual EUINavigationAction GetNavigationActionForKey(const FKey& InKey) const = 0;

	/** @return	Returns true if there are any pop-up menus summoned */
	virtual bool AnyMenusVisible() const = 0;

#if WITH_ACCESSIBILITY
	/** 
	 * Accessor for the accessible message handler. One must always exist, even if it's never activated.
	 *
	 * @return A reference to the assigned accessible message handler
	 */
	TSharedRef<FSlateAccessibleMessageHandler> GetAccessibleMessageHandler() const { return AccessibleMessageHandler; }
#endif
protected:
	/**
	 * Implementation of GetMouseCaptor which can be overridden without warnings.
	 * 
	 * @return Widget with the mouse capture
	 */
	virtual TSharedPtr< SWidget > GetMouseCaptorImpl() const = 0;

	SLATECORE_API void GetSafeZoneRatio(FMargin& SafeZoneRatio);

public:
	/**
	 * Gets whether or not a widget has captured the mouse.
	 *
	 * @return True if one or more widgets have capture, otherwise false.
	 */
	virtual bool HasAnyMouseCaptor( ) const = 0;

	/**
	 * Gets whether or not a widget has captured the mouse for a particular user.
	 *
	 * @return True if one or more widgets have capture, otherwise false.
	 */
	virtual bool HasUserMouseCapture(int32 UserIndex) const = 0;

	/**
	 * Gets the platform application.
	 *
	 * @return Platform application.
	 */
	virtual const TSharedPtr<GenericApplication> GetPlatformApplication ( ) const
	{
		return PlatformApplication;
	}

	/**
	 * Gets the rectangle of the current preferred work area.
	 *
	 * @return Area rectangle.
	 */
	virtual FSlateRect GetPreferredWorkArea( ) const = 0;

	/**
	 * Checks whether the specified widget has any descendants which are currently focused for the specified user user.
	 *
	 * @param Widget The widget to check.
	 * @param InUserIndex Index of the user that we want to check for.
	 * @return true if any descendants are focused, false otherwise.
	 */
	virtual bool HasUserFocusedDescendants(const TSharedRef< const SWidget >& Widget, int32 UserIndex) const = 0;

	/**
	 * Checks whether the specified widget has any descendants which are currently focused.
	 *
	 * @param Widget The widget to check.
	 * @return true if any descendants are focused, false otherwise.
	 */
	virtual bool HasFocusedDescendants( const TSharedRef< const SWidget >& Widget ) const = 0;

	/**
	 * Checks whether an UI for external services such as Steam is open.
	 *
	 * @return true if an external UI is open, false otherwise.
	 */
	virtual bool IsExternalUIOpened( ) = 0;

	/** @return a hittesting object that can perform hittests agains widgets. Only certain classes can make use of FHitTesting */
	friend class FHitTesting;
	SLATECORE_API const FHitTesting& GetHitTesting() const;

	/** 
	 * Given the screen-space coordinate of the mouse cursor, searches for a string of widgets that are under the mouse.
	 *
	 * The widgets will be returned with the associated geometry. The first item will always
	 * be the top-level window while the last item will be the leaf-most widget.
	 *
	 * @return The path to the widget.
	 */
	virtual FWidgetPath LocateWindowUnderMouse( UE::Slate::FDeprecateVector2DParameter ScreenspaceMouseCoordinate, const TArray< TSharedRef<SWindow > >& Windows, bool bIgnoreEnabledStatus = false, int32 UserIndex = INDEX_NONE) = 0;

	/**
	 * Calculates the tooltip window position.
	 *
	 * @param InAnchorRect The current(suggested) window position and size of an area which may not be covered by the popup.
	 * @param InSize The size of the tooltip window.
	 * @param bAutoAdjustForDPIScale Whether the method should attempt to correct for the DPI scale of the monitor.
	 * @param CursorOverlapMode Whether the tooltip window should be permitted to open under the position of the cursor. 
	 * @return The suggested position.
	 */
	virtual UE::Slate::FDeprecateVector2DResult CalculateTooltipWindowPosition(const FSlateRect& InAnchorRect, const UE::Slate::FDeprecateVector2DParameter& InSize, bool bAutoAdjustForDPIScale, EPopupCursorOverlapMode CursorOverlapMode = EPopupCursorOverlapMode::PreventOverlap) const = 0;

	/** @return true if 'WindowToTest' is being used to display the current tooltip and the tooltip is interactive. */
	virtual bool IsWindowHousingInteractiveTooltip(const TSharedRef<const SWindow>& WindowToTest) const = 0;

	/**
	 * Creates an image widget.
	 *
	 * @return The new image widget.
	 */
	virtual TSharedRef<SImage> MakeImage( const TAttribute<const FSlateBrush*>& Image, const TAttribute<FSlateColor>& Color, const TAttribute<EVisibility>& Visibility ) const = 0;

	/**
	 * Creates a tool tip with the specified text.
	 *
	 * @param ToolTipText The text attribute to assign to the tool tip.
	 *
	 * @return The tool tip.
	 */
	virtual TSharedRef<IToolTip> MakeToolTip(const TAttribute<FText>& ToolTipText) = 0;

	/**
	 * Creates a tool tip with the specified text.
	 *
	 * @param ToolTipText The text to assign to the tool tip.
	 * @return The tool tip.
	 */
	virtual TSharedRef<IToolTip> MakeToolTip( const FText& ToolTipText ) = 0;

	/**
	 * Creates a title bar for the specified window.
	 *
	 * @param InArgs	The creation arguments for the titlebar
	 * @param OutTitleBar Will hold a pointer to the title bar's interface.
	 * @return The new title bar widget.
	 */
	virtual TSharedRef<SWidget> MakeWindowTitleBar( const FWindowTitleBarArgs& InArgs, TSharedPtr<IWindowTitleBar>& OutTitleBar ) const = 0;

	/**
	 * Destroying windows has implications on some OSs (e.g. destroying Win32 HWNDs can cause events to be lost).
	 *
	 * Slate strictly controls when windows are destroyed.
	 *
	 * @param WindowToDestroy The window to queue for destruction.
	 */
	virtual void RequestDestroyWindow( TSharedRef<SWindow> WindowToDestroy ) = 0;

	/**
	 * Sets keyboard focus to the specified widget.  The widget must be allowed to receive keyboard focus.
	 *
	 * @param  InFocusPath WidgetPath to the Widget to being focused
	 * @param InCause The reason that keyboard focus is changing
	 * @return true if the widget is now focused, false otherwise.
	 */
	virtual bool SetKeyboardFocus( const FWidgetPath& InFocusPath, const EFocusCause InCause ) = 0;

	/**
	 * Sets user focus to the specified widget.  The widget must be allowed to receive focus.
	 *
	 * @param InUserIndex Index of the user that we want to change the focus of.
	 * @param InFocusPath WidgetPath to the Widget to being focused.
	 * @param InCause The reason that focus is changing.
	 * @return true if the widget is now focused, false otherwise.
	 */
	virtual bool SetUserFocus(const uint32 InUserIndex, const FWidgetPath& InFocusPath, const EFocusCause InCause) = 0;

	/**
	 * Sets the focus for the specified user to the specified widget unless that user is focused on a descendant.  The widget must be allowed to receive focus.
	 *
	 * @param InUserIndex Index of the user that we want to change the focus of.
	 * @param InFocusPath WidgetPath to the Widget to being focused.
	 * @param InCause The reason that focus is changing.
	 * @return true if the widget or a descendant is now focused, false otherwise.
	 */
	virtual bool SetUserFocusAllowingDescendantFocus(const uint32 InUserIndex, const FWidgetPath& InFocusPath, const EFocusCause InCause) = 0;

	/**
	 * Sets the focus for all users to the specified widget.  The widget must be allowed to receive focus.
	 *
	 * @param InFocusPath WidgetPath to the Widget to being focused.
	 * @param InCause The reason that focus is changing.
	 */
	virtual void SetAllUserFocus(const FWidgetPath& InFocusPath, const EFocusCause InCause) = 0;

	/**
	 * Sets the focus for all users to the specified widget unless that user is focused on a descendant.  The widget must be allowed to receive focus.
	 *
	 * @param InWidget WidgetPath to the Widget to being focused.
	 * @param InCause The reason that focus is changing.
	 */
	virtual void SetAllUserFocusAllowingDescendantFocus(const FWidgetPath& InFocusPath, const EFocusCause InCause) = 0;

	/**
	 * @return a pointer to the Widget that currently has the users focus; Empty pointer when the user has no focus. 
	 */
	virtual TSharedPtr<SWidget> GetUserFocusedWidget(uint32 UserIndex) const = 0;

	/**
	 * @return the Widget that started a tick, paint, prepass or an event.
	 */
	virtual TSharedPtr<SWidget> GetCurrentDebugContextWidget() const = 0;

	/**
	 * Gets a delegate that is invoked when a global invalidate of all widgets should occur
	 */
	DECLARE_EVENT_OneParam(FSlateApplicationBase, FOnInvalidateAllWidgets, bool);
	FOnInvalidateAllWidgets& OnInvalidateAllWidgets() { return OnInvalidateAllWidgetsEvent; }

	DECLARE_EVENT_OneParam(FSlateApplicationBase, FOnGlobalInvalidationToggled, bool);
	FOnGlobalInvalidationToggled& OnGlobalInvalidationToggled() { return OnGlobalInvalidationToggledEvent; }

	SLATECORE_API void ToggleGlobalInvalidation(bool bIsGlobalInvalidationEnabled);

	/**
	 * Notifies all invalidation panels that they should invalidate their contents
	 * Note: this is a very expensive call and should only be done in non-performance critical situations
	 */
	SLATECORE_API void InvalidateAllWidgets(bool bClearResourcesImmediately) const;
private:
	/**
	 * Implementation for active timer registration. See SWidget::RegisterActiveTimer.
	 */
	SLATECORE_API void RegisterActiveTimer( const TSharedRef<FActiveTimerHandle>& ActiveTimerHandle );

	/**
	 * Implementation for active timer registration. See SWidget::UnRegisterActiveTimer.
	 */
	SLATECORE_API void UnRegisterActiveTimer( const TSharedRef<FActiveTimerHandle>& ActiveTimerHandle );

	/** The set of active timer handles. */
	TSet<TWeakPtr<FActiveTimerHandle>> ActiveTimerHandles;

protected:
	enum class ECustomSafeZoneState : uint8
	{
		Unset,
		Set,
		Debug
	};

	/**
	 * Used to determine if any active timer handles are ready to fire.
	 * Means we need to tick slate even if no user interaction.
	 */
	SLATECORE_API void UpdateAnyActiveTimersArePending();

public:
	SLATECORE_API const static uint32 CursorPointerIndex;
	SLATECORE_API const static uint32 CursorUserIndex;

	/** The primary platform user for this application. They will own the mouse and cursor by default. Default ID is 0 */
	SLATECORE_API const static FPlatformUserId SlateAppPrimaryPlatformUser;

	/**
	 * Returns the current instance of the application. The application should have been initialized before
	 * this method is called
	 *
	 * @return  Reference to the application
	 */
	static FSlateApplicationBase& Get( )
	{
		check(CurrentBaseApplication.IsValid());
		checkSlow(IsThreadSafeForSlateRendering());
		return *CurrentBaseApplication;
	}

	/**
	 * Returns true if a Slate application instance is currently initialized and ready
	 *
	 * @return  True if Slate application is initialized
	 */
	static bool IsInitialized( )
	{
		return CurrentBaseApplication.IsValid();
	}

protected:

	/**
	 * Gets whether or not a particular widget has mouse capture.
	 *
	 * @return True if the widget has mouse capture, otherwise false.
	 */
	virtual bool DoesWidgetHaveMouseCapture(const TSharedPtr<const SWidget> Widget) const = 0;

	/**
	 * Gets whether or not a particular widget has mouse capture by a user.
	 *
	 * @return True if the widget has mouse capture, otherwise false.
	 */
	virtual bool DoesWidgetHaveMouseCaptureByUser(const TSharedPtr<const SWidget> Widget, int32 UserIndex, TOptional<int32> PointerIndex) const = 0;

	/**
	 * Gets whether or not a particular widget has the specified users focus, and if so the type of focus.
	 *
	 * @return The optional will be set with the focus cause, if unset this widget doesn't have focus.
	 */
	virtual TOptional<EFocusCause> HasUserFocus(const TSharedPtr<const SWidget> Widget, int32 UserIndex) const = 0;

	/**
	 * Gets whether or not a particular widget has any users focus, and if so the type of focus (first one found).
	 *
	 * @return The optional will be set with the focus cause, if unset this widget doesn't have focus.
	 */
	virtual TOptional<EFocusCause> HasAnyUserFocus(const TSharedPtr<const SWidget> Widget) const = 0;

	/**
	 * Gets whether or not a particular widget is directly hovered.
	 * Directly hovered means that the widget is directly under the pointer, is not true for ancestors tho they are Hovered.
	 *
	 * @return True if the widget is directly hovered, otherwise false.
	 */
	virtual bool IsWidgetDirectlyHovered(const TSharedPtr<const SWidget> Widget) const = 0;

	/**
	 * Gets whether or not a particular widget should show user focus.
	 *
	 * @return True if we should show user focus
	 */
	virtual bool ShowUserFocus(const TSharedPtr<const SWidget> Widget) const = 0;

	/** Given a window, locate a widget under the cursor in it; returns an invalid path if cursor is not over this window. */
	virtual FWidgetPath LocateWidgetInWindow(UE::Slate::FDeprecateVector2DParameter ScreenspaceMouseCoordinate, const TSharedRef<SWindow>& Window, bool bIgnoreEnabledStatus, int32 UserIndex) const = 0;
	
	SLATECORE_API void UpdateCustomSafeZone(const FMargin& NewSafeZoneRatio, bool bShouldRecacheMetrics);
#if WITH_EDITOR
	SLATECORE_API void SwapSafeZoneTypes();
#endif

protected:

	// Holds the Slate renderer used to render this application.
	TSharedPtr<FSlateRenderer> Renderer;

	// Private interface for select entities that are allowed to perform hittesting
	FHitTesting HitTesting;

protected:

	// Holds a pointer to the current application.
	static SLATECORE_API TSharedPtr<FSlateApplicationBase> CurrentBaseApplication;

	// Holds a pointer to the platform application.
	static SLATECORE_API TSharedPtr<class GenericApplication> PlatformApplication;

	// Caches the application's display metrics
	FDisplayMetrics CachedDisplayMetrics;

	// Caches the previous debug safe zone ratio
	float CachedDebugTitleSafeRatio;
public:

	/**
	 * Is Slate currently sleeping or not.
	 *
	 * @return True if Slate is sleeping.
	 */
	SLATECORE_API bool IsSlateAsleep();

	TSharedPtr<ICursor> GetPlatformCursor()
	{
		return PlatformApplication->Cursor;
	}

	TSharedPtr<class GenericApplication> GetPlatformApplication()
	{
		return PlatformApplication;
	}

	SLATECORE_API void ResetCustomSafeZone();
	SLATECORE_API bool IsCustomSafeZoneSet() const;
	SLATECORE_API void SetCustomSafeZone(const FMargin& InSafeZone);
	const FMargin& GetCustomSafeZone() const { return CustomSafeZoneRatio; }

#if WITH_EDITORONLY_DATA
	FOnDebugSafeZoneChanged OnDebugSafeZoneChanged;
#endif
#if WITH_EDITOR
	FOnConstrainedAspectRatioChanged OnConstrainedAspectRatioChanged;
#endif
protected:
#if WITH_ACCESSIBILITY
	/** Manager for widgets and application to interact with accessibility API */
	TSharedRef<FSlateAccessibleMessageHandler> AccessibleMessageHandler;
#endif

	/** multicast delegate to broadcast when a global invalidate is requested */
	FOnInvalidateAllWidgets OnInvalidateAllWidgetsEvent;
	FOnGlobalInvalidationToggled OnGlobalInvalidationToggledEvent;

	/** Critical section for active timer registration as it can be called from the movie thread and the game thread */
	FCriticalSection ActiveTimerCS;

	// Gets set when Slate goes to sleep and cleared when active.
	bool bIsSlateAsleep;

	/** If Safe Zone ratio has been manually set, unset, or set via debug */
	ECustomSafeZoneState CustomSafeZoneState;

	/** Safe Zone ratio to override platform settings */
	FMargin CustomSafeZoneRatio;

	/** Whether any active timers are pending for this Slate tick. Updated once a tick in AnyActiveTimersArePending(). */
	bool bAnyActiveTimersPending = false;
};
