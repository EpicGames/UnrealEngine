// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "GenericPlatform/GenericWindow.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

class FWidgetPath;

/**
 * Implements a manager for Slate windows.
 */
class FSlateWindowHelper
{
public:

	/**
	 * Reorders the given collection of windows so the specified window is brought to the front.
	 *
	 * @param Windows The collection of windows to reorder.
	 * @param WindowToBringToFront The window to bring to the front.
	 */
	static SLATECORE_API void ArrangeWindowToFront( TArray<TSharedRef<SWindow>>& Windows, const TSharedRef<SWindow>& WindowToBringToFront );

	/**
	 * Put 'BringMeToFront' at the font of the list of 'WindowsToReorder'.
	 *
	 * The top-most (front-most) window is last in Z-order and therefore is added to the end of the list.
	 *
	 * @param Windows An ordered list of windows.
	 * @param BingMeToFront The window to bring to front.
	 */
	static SLATECORE_API void BringWindowToFront( TArray<TSharedRef<SWindow>>& Windows, const TSharedRef<SWindow>& BringMeToFront );

	/**
	 * Checks whether any of the given windows overlap the specified work area.
	 *
	 * @param WindowsToSearch The collection of windows to check.
	 * @param WorkAreaRect The work area to check.
	 * @return true if at least one window overlaps with the work area, false otherwise.
	 */
	static SLATECORE_API bool CheckWorkAreaForWindows( const TArray< TSharedRef<SWindow> >& WindowsToSearch, const FSlateRect& WorkAreaRect );

	/**
	 * Checks whether the given collection of windows contains the specified window.
	 *
	 * @param WindowsToSearch The collection of windows to search.
	 * @param WindowToFind The window to find.
	 * @return true if the collection contains the window, false otherwise.
	 */
	static SLATECORE_API bool ContainsWindow( const TArray<TSharedRef<SWindow>>& WindowsToSearch, const TSharedRef<SWindow>& WindowToFind );

	/**
	 * Searches for the specified widget and generates a full path to it.
	 *
	 * Note: this is a relatively slow operation!
	 * 
	 * @param InWidget Widget to generate a path to.
	 * @param OutWidgetPath The generated widget path.
	 * @param VisibilityFilter Widgets must have this type of visibility to be included the path.
	 * @return True if the widget path was found.
	 */
	static SLATECORE_API bool FindPathToWidget( const TArray<TSharedRef<SWindow>>& WindowsToSearch, TSharedRef<const SWidget> InWidget, FWidgetPath& OutWidgetPath, EVisibility VisibilityFilter = EVisibility::Visible );

	/**
	 * Searches the given collection of windows to find the Slate window that corresponds to the specified platform window.
	 *
	 * @param WindowsToSearch The collection of windows to search.
	 * @param PlatformWindow The platform window to find the Slate window for.
	 * @return The corresponding Slate window, or nullptr if it was not found.
	 */
	static SLATECORE_API TSharedPtr<SWindow> FindWindowByPlatformWindow( const TArray<TSharedRef<SWindow>>& WindowsToSearch, const TSharedRef<FGenericWindow>& PlatformWindow );

	/**
	 * Removes the specified window from the given collection of windows.
	 *
	 * @param Windows The collection of windows.
	 * @param WindowToRemove The window to remove.
	 */
	static SLATECORE_API void RemoveWindowFromList( TArray<TSharedRef<SWindow>>& Windows, const TSharedRef<SWindow>& WindowToRemove );

protected:

	/**
	 * Make BringMeToFront first among its peers.
	 * i.e. make it the last window in its parent's list of child windows.
	 *
	 * @return The top-most window whose children were re-arranged
	 */
	static SLATECORE_API TSharedRef<SWindow> BringToFrontInParent( const TSharedRef<SWindow>& WindowToBringToFront );
};
