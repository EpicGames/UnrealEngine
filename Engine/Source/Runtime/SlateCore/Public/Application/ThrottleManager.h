// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/IConsoleManager.h"

/**
 * A handle to a throttle request made to the throttle manager
 * Throttles can only be ended by passing back a request
 */
class FThrottleRequest
{
	// Only the manager is allowed to change values
	friend class FSlateThrottleManager;

public:

	FThrottleRequest()
		: Index(INDEX_NONE)
	{ }

public:

	/**
	 * Whether or not the handle is valid
	 *
	 * @return true if the throttle is valid
	 */
	bool IsValid() const
	{
		return Index != INDEX_NONE;
	}

private:

	int32 Index;
};


/**
 * A class which manages requests to throttle parts of the engine to ensure Slate UI performance                   
 */
class FSlateThrottleManager
{
public:

	/**
	 * Constructor                   
	 */
	SLATECORE_API FSlateThrottleManager();

public:

	/**
	 * Requests that we enter responsive mode.  I.E throttle slow parts of the engine   
	 *
	 * @return A handle to the request to enter responsive mode.  Can only be ended with this request
	 */
	SLATECORE_API FThrottleRequest EnterResponsiveMode();

	/**
	 * Request to leave responsive mode.  
	 * Note that this may not end responsive mode in the case that multiple EnterResponsiveMode requests were made                    
	 *
	 * @param InHandle	The handle that was created with EnterResponsiveMode
	 */
	SLATECORE_API void LeaveResponsiveMode( FThrottleRequest& InHandle );

	/**
	 * Whether or not we allow expensive tasks which could hurt performance to occur
	 *
	 * @return true if we allow expensive tasks, false otherwise
	 */
	SLATECORE_API bool IsAllowingExpensiveTasks() const;

	/**
	 * Explicitly disable Slate throttling. This is intended to be used for code-driven exemptions
	 * to Slate throttling such as interactive actions that require multiple refreshes & world ticks.
	 * For single refresh scenarios, consider FEditorViewportClient.Invalidate(false, false).
	 *
	 * @param bState true if throttling should be disabled.
	 */
	SLATECORE_API void DisableThrottle(bool bState);

public:

	/**
	 * Gets the instance of this manager                   
	 */
	static SLATECORE_API FSlateThrottleManager& Get( );

private:
	/** Number of active throttle requests */
	uint32 ThrottleCount;

	/** If false, then throttling is explicitly disabled. */
	int32 DisableThrottleCount;
};
