// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateBrush.h"
#include "Types/SlateVector2.h"

/**
 * Similar to FSlateBoxBrush but has no middle and the sides tile instead of stretching.
 * The margin is applied exactly as in FSlateBoxBrush.
 */
struct FSlateBorderBrush
	: public FSlateBrush
{
	/**
	 * @param InImageName The name of the texture to draw
	 * @param InMargin Determines the sides and corner sizes; see FSlateBoxBrush.
	 * @param InColorAndOpacity	Color and opacity scale.
	 * @param InImageType The type of image this this is
	 */
	FORCENOINLINE FSlateBorderBrush( const FName& InImageName, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1,1,1,1), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Border, InImageName, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

	FORCENOINLINE FSlateBorderBrush( const FString& InImageName, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1,1,1,1), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Border, *InImageName, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

	FORCENOINLINE FSlateBorderBrush( const ANSICHAR* InImageName, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1,1,1,1), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Border, InImageName, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

	FORCENOINLINE FSlateBorderBrush( const WIDECHAR* InImageName, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1,1,1,1), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Border, InImageName, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

	FORCENOINLINE FSlateBorderBrush( const FName& InImageName, const FMargin& InMargin, const TSharedRef< FLinearColor >& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Border, InImageName, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

	FORCENOINLINE FSlateBorderBrush( const FString& InImageName, const FMargin& InMargin, const TSharedRef< FLinearColor >& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Border, *InImageName, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

	FORCENOINLINE FSlateBorderBrush( const ANSICHAR* InImageName, const FMargin& InMargin, const TSharedRef< FLinearColor >& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Border, InImageName, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

	FORCENOINLINE FSlateBorderBrush( const TCHAR* InImageName, const FMargin& InMargin, const TSharedRef< FLinearColor >& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Border, InImageName, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

	FORCENOINLINE FSlateBorderBrush( const FName& InImageName, const FMargin& InMargin, const FSlateColor& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Border, InImageName, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

	FORCENOINLINE FSlateBorderBrush( const FString& InImageName, const FMargin& InMargin, const FSlateColor& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Border, *InImageName, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

	FORCENOINLINE FSlateBorderBrush( const ANSICHAR* InImageName, const FMargin& InMargin, const FSlateColor& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Border, InImageName, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

	FORCENOINLINE FSlateBorderBrush( const TCHAR* InImageName, const FMargin& InMargin, const FSlateColor& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Border, InImageName, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

	/**
	 * @param InResourceObject	The image to render for this brush, can be a UTexture, UMaterialInterface, or AtlasedTextureInterface
	 * @param InMargin			Determines the sides and corner sizes; see FSlateBoxBrush.
	 * @param InColorAndOpacity	Color and opacity scale.
	 * @param InImageType		The type of image this this is
	 */
	FORCENOINLINE FSlateBorderBrush(UObject* InResourceObject, const FMargin& InMargin, const FSlateColor& InColorAndOpacity = FSlateColor(FLinearColor(1, 1, 1, 1)), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor)
		: FSlateBrush(ESlateBrushDrawType::Border, NAME_None, InMargin, ESlateBrushTileType::Both, InImageType, FVector2f::ZeroVector, InColorAndOpacity, InResourceObject)
	{
		// A border with no margin will not show up at all.
		check(InMargin.GetDesiredSize().SizeSquared() > 0);
	}

};

struct FSlateVectorBorderBrush
	: public FSlateBorderBrush
{
	FSlateVectorBorderBrush(const FString& InImageName, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1, 1, 1, 1))
		: FSlateBorderBrush(InImageName, InMargin, InColorAndOpacity, ESlateBrushImageType::Vector)
	{
	}
};
