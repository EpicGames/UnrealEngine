// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateBrush.h"
#include "Types/SlateVector2.h"

/**
 * A 3x3 box where the sides stretch horizontally and vertically and the middle stretches to fill.
 * The corners will not be stretched. The size of the sides and corners is determined by the
 * Margin as follows:
 *
 *                 _____________________
 *                |  | Margin.Top    |  |
 *                |__|_______________|__|   Margin.Right
 *                |  |               |  |  /
 *              +--> |               | <--+
 *             /  |__|_______________|__|
 *  Margin.Left   |  | Margin.Bottom |  |
 *                |__|_______________|__|
 *
 */
struct FSlateBoxBrush
	: public FSlateBrush
{
	/**
	 * Make a 3x3 box that stretches the texture.
	 *
	 * @param InImageName		The name of image to make into a box
	 * @param InMargin			The size of corners and sides in normalized texture UV space.
	 * @param InColorAndOpacity	Color and opacity scale.
	 */
	FORCENOINLINE FSlateBoxBrush( const FName& InImageName, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1,1,1,1), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const FString& InImageName, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1,1,1,1), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, *InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const ANSICHAR* InImageName, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1,1,1,1), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{ } 

	FORCENOINLINE FSlateBoxBrush( const WIDECHAR* InImageName, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1,1,1,1), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const FName& InImageName, const FMargin& InMargin, const TSharedRef< FLinearColor >& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const FString& InImageName, const FMargin& InMargin, const TSharedRef< FLinearColor >& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, *InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const ANSICHAR* InImageName, const FMargin& InMargin, const TSharedRef< FLinearColor >& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const TCHAR* InImageName, const FMargin& InMargin, const TSharedRef< FLinearColor >& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const FName& InImageName, const FMargin& InMargin, const FSlateColor& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const FString& InImageName, const FMargin& InMargin, const FSlateColor& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, *InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const ANSICHAR* InImageName, const FMargin& InMargin, const FSlateColor& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const TCHAR* InImageName, const FMargin& InMargin, const FSlateColor& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity)
	{ }

	/**
	 * Make a 3x3 box that stretches the texture.
	 *
	 * @param InImageName       The name of image to make into a box
	 * @param ImageSize         The size of the resource as we want it to appear in slate units.
	 * @param InMargin          The size of corners and sides in texture space.
	 * @param InColorAndOpacity	Color and opacity scale. Note of the image type is ImageType_TintMask, this value should be in HSV
	 */
	FORCENOINLINE FSlateBoxBrush( const FName& InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1,1,1,1), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const FString& InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1,1,1,1), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, *InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const ANSICHAR* InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1,1,1,1), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const WIDECHAR* InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1,1,1,1), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const FName& InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const TSharedRef< FLinearColor >& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const FString& InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const TSharedRef< FLinearColor >& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, *InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const ANSICHAR* InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const TSharedRef< FLinearColor >& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const TCHAR* InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const TSharedRef< FLinearColor >& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const FName& InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const FSlateColor& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const FString& InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const FSlateColor& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, *InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const ANSICHAR* InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const FSlateColor& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity)
	{ }

	FORCENOINLINE FSlateBoxBrush( const TCHAR* InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const FSlateColor& InColorAndOpacity, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Box, InImageName, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity)
	{ }

	/**
	 * Make a 3x3 box that stretches the texture.
	 *
	 * @param InResourceObject	The image to render for this brush, can be a UTexture, UMaterialInterface, or AtlasedTextureInterface
	 * @param InMargin			The size of corners and sides in normalized texture UV space.
	 * @param InColorAndOpacity	Color and opacity scale.
	 */

	FORCENOINLINE FSlateBoxBrush(UObject* InResourceObject, const FMargin& InMargin, const FSlateColor& InColorAndOpacity = FSlateColor(FLinearColor(1, 1, 1, 1)), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor)
		: FSlateBrush(ESlateBrushDrawType::Box, NAME_None, InMargin, ESlateBrushTileType::NoTile, InImageType, FVector2f::ZeroVector, InColorAndOpacity, InResourceObject)
	{ }

	/**
	 * Make a 3x3 box that stretches the texture.
	 *
	 * @param InImageName       The name of image to make into a box
	 * @param ImageSize         The size of the resource as we want it to appear in slate units.
	 * @param InMargin          The size of corners and sides in texture space.
	 * @param InColorAndOpacity	Color and opacity scale. Note of the image type is ImageType_TintMask, this value should be in HSV
	 */

	FORCENOINLINE FSlateBoxBrush(UObject* InResourceObject, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const FSlateColor& InColorAndOpacity = FSlateColor(FLinearColor(1, 1, 1, 1)), ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor)
		: FSlateBrush(ESlateBrushDrawType::Box, NAME_None, InMargin, ESlateBrushTileType::NoTile, InImageType, ImageSize, InColorAndOpacity, InResourceObject)
	{ }
};


struct FSlateVectorBoxBrush
	: public FSlateBoxBrush
{
	FSlateVectorBoxBrush(const FString& InImageName, const UE::Slate::FDeprecateVector2DParameter& ImageSize, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1, 1, 1, 1))
		: FSlateBoxBrush(InImageName, ImageSize, InMargin, InColorAndOpacity, ESlateBrushImageType::Vector)
	{ }


	FSlateVectorBoxBrush(const FString& InImageName, const FMargin& InMargin, const FLinearColor& InColorAndOpacity = FLinearColor(1, 1, 1, 1))
		: FSlateBoxBrush(InImageName, InMargin, InColorAndOpacity, ESlateBrushImageType::Vector)
	{ }
};

