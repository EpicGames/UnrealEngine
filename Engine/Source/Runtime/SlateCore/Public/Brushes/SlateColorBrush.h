// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Styling/SlateBrush.h"

/**
 * A color brush.  Draws a box with no margins that only has a color applied to it                   
 */
struct FSlateColorBrush
	: public FSlateBrush
{
	/** 
	 * Creates and initializes a new instance with the specified linear color.
	 *
	 * @param InColor The linear color to use
	 */
	FORCEINLINE FSlateColorBrush( const FLinearColor& InColor )
		: FSlateBrush(ESlateBrushDrawType::Image, NAME_None, FMargin(0.0f), ESlateBrushTileType::NoTile, ESlateBrushImageType::NoImage, FVector2f::ZeroVector, InColor)
	{ }

	/** 
	 * Creates and initializes a new instance with the specified color value.
	 *
	 * @param InColor The color value to use
	 */
	FORCEINLINE FSlateColorBrush( const FColor& InColor )
		: FSlateBrush(ESlateBrushDrawType::Image, NAME_None, FMargin(0.0f), ESlateBrushTileType::NoTile, ESlateBrushImageType::NoImage, FVector2f::ZeroVector, InColor.ReinterpretAsLinear())
	{ }

	/** 
	 * Creates and initializes a new instance with the specified shared color.
	 *
	 * @param InColor The shared color to use
	 */
	FORCEINLINE FSlateColorBrush( const TSharedRef< FLinearColor >& InColor )
		: FSlateBrush(ESlateBrushDrawType::Image, NAME_None, FMargin(0.0f), ESlateBrushTileType::NoTile, ESlateBrushImageType::NoImage, FVector2f::ZeroVector, InColor)
	{ }

	/**
	 * Creates and initializes a new instance with the specified shared color.
	 *
	 * @param InColor The shared color to use
	 */
	FORCEINLINE FSlateColorBrush(const FSlateColor& InColor)
		: FSlateBrush(ESlateBrushDrawType::Image, NAME_None, FMargin(0.0f), ESlateBrushTileType::NoTile, ESlateBrushImageType::NoImage, FVector2f::ZeroVector, InColor)
	{ }
};