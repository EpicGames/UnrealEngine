// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Styling/SlateBrush.h"
#include "Types/SlateVector2.h"

/**
 * Ignores the Margin. Just renders the image. Can tile the image instead of stretching.
 */
struct FSlateDynamicImageBrush
	: public FSlateBrush, public TSharedFromThis<FSlateDynamicImageBrush>
{
	/**
	 * @param InTexture		The UTexture2D being used for this brush.
	 * @param InImageSize		How large should the image be (not necessarily the image size on disk)
	 * @param InTint		The tint of the image
	 * @param InTiling		How do we tile if at all?
	 * @param InImageType		The type of image this this is
	 */
	FORCENOINLINE FSlateDynamicImageBrush( 
		class UTexture2D* InTexture, 
		const UE::Slate::FDeprecateVector2DParameter& InImageSize,
		const FName InTextureName,
		const FLinearColor& InTint = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), 
		ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, 
		ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor
	)
		: FSlateBrush(ESlateBrushDrawType::Image, FName("None"), FMargin(0.0f), InTiling, InImageType, InImageSize, InTint, (UObject*)InTexture)
		, bRemoveResourceFromRootSet(false)
		, bIsInitalized(true)
	{
		bIsDynamicallyLoaded = true;
		InitFromTextureObject(InTextureName);
	}

	/**
	 * @param InTexture		The UTexture2DDynamic being used for this brush.
	 * @param InImageSize	How large should the image be (not necessarily the image size on disk)
	 * @param InTint		The tint of the image
	 * @param InTiling		How do we tile if at all?
	 * @param InImageType	The type of image this this is
	 */
	FORCENOINLINE FSlateDynamicImageBrush( 
		class UTexture2DDynamic* InTexture, 
		const UE::Slate::FDeprecateVector2DParameter& InImageSize,
		const FName InTextureName,
		const FLinearColor& InTint = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), 
		ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, 
		ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor
	)
		: FSlateBrush(ESlateBrushDrawType::Image, FName("None"), FMargin(0.0f), InTiling, InImageType, InImageSize, InTint, (UObject*)InTexture)
		, bRemoveResourceFromRootSet(false)
		, bIsInitalized(true)
	{
		bIsDynamicallyLoaded = true;
		InitFromTextureObject(InTextureName);
	}
		
	/**
	 * @param InTextureName		The name of the texture to load.
	 * @param InImageSize		How large should the image be (not necessarily the image size on disk)
	 * @param InTint		The tint of the image.
	 * @param InTiling		How do we tile if at all?
	 * @param InImageType		The type of image this this is
	 */
	FORCENOINLINE FSlateDynamicImageBrush( 
		const FName InTextureName,
		const UE::Slate::FDeprecateVector2DParameter& InImageSize,
		const FLinearColor& InTint = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f), 
		ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, 
		ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, InTextureName, FMargin(0.0f), InTiling, InImageType, InImageSize, InTint, nullptr, true)
		, bRemoveResourceFromRootSet(false)
		, bIsInitalized(true)
	{
		bIsDynamicallyLoaded = true;
	}

	/**
	* @param InTextureName		The name to use when registering the image data as a texture.
	* @param InImageSize		How large should the image be (not necessarily the image size on disk)
	* @param InImageData		The raw image data formatted as BGRA
	* @param InTint				The tint of the image.
	* @param InTiling			How do we tile if at all?
	* @param InImageType		The type of image this this is
	*/
	static SLATECORE_API TSharedPtr<FSlateDynamicImageBrush> CreateWithImageData(
		const FName InTextureName,
		const UE::Slate::FDeprecateVector2DParameter& InImageSize,
		const TArray<uint8>& InImageData,
		const FLinearColor& InTint = FLinearColor(1.0f, 1.0f, 1.0f, 1.0f),
		ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile,
		ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor);

	/**
	 * Releases the resource when it is safe to do so
	 */
	SLATECORE_API void ReleaseResource();

	/** Destructor. */
	SLATECORE_API virtual ~FSlateDynamicImageBrush();

private:
	SLATECORE_API void ReleaseResourceInternal();

	void InitFromTextureObject(FName InTextureName)
	{
		UObject* Object = GetResourceObject();
		// if we have a texture, make a unique name
		if (Object != nullptr)
		{
			// @todo Slate - Hack:  This is to address an issue where the brush created and a GC occurs before the brush resource object becomes referenced
			// by the Slate resource manager. Don't add objects that are already in root set (and mark them as such) to avoid incorrect removing objects
			// from root set in destructor.
			if (!Object->IsRooted())
			{
				ensureMsgf(false, TEXT("This hack usually results in a crash during loading screens in slate.  Please change any code that arrives here to not use FSlateDynamicImageBrush.  In the case of loading screens, you can use FDeferredCleanupSlateBrush.  Which correctly accounts for both GC lifetime, and the lifetime of the object through the slate rendering pipeline which may be several frames after you stop using it."));

				Object->AddToRoot();
				bRemoveResourceFromRootSet = true;
			}

			ResourceName = InTextureName;
		}
	}

private:
	/** Tracks if Resource was in root set to avoid unnecessary removing it from there. */
	bool bRemoveResourceFromRootSet : 1;

	/** If the resource has been initialized */
	bool bIsInitalized : 1;
};
