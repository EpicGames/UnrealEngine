// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Styling/SlateBrush.h"
#include "Types/SlateVector2.h"

/**
 * Ignores the Margin. Just renders the image. Can tile the image instead of stretching.
 */
struct FSlateImageBrush
	: public FSlateBrush
{
	/**
	 * @param InImageName	The name of the texture to draw
	 * @param InImageSize	How large should the image be (not necessarily the image size on disk)
	 * @param InTint		The tint of the image.
	 * @param InTiling		How do we tile if at all?
	 * @param InImageType	The type of image this this is
	 */
	FORCENOINLINE FSlateImageBrush( const FName& InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FLinearColor& InTint = FLinearColor(1,1,1,1), ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, InImageName, FMargin(0), InTiling, InImageType, InImageSize, InTint)
	{ }

	FORCENOINLINE FSlateImageBrush( const FString& InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FLinearColor& InTint = FLinearColor(1,1,1,1), ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, *InImageName, FMargin(0), InTiling, InImageType, InImageSize, InTint)
	{ }

	FORCENOINLINE FSlateImageBrush( const ANSICHAR* InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FLinearColor& InTint = FLinearColor(1,1,1,1), ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, InImageName, FMargin(0), InTiling, InImageType, InImageSize, InTint)
	{ }

	FORCENOINLINE FSlateImageBrush( const WIDECHAR* InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FLinearColor& InTint = FLinearColor(1,1,1,1), ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, InImageName, FMargin(0), InTiling, InImageType, InImageSize, InTint)
	{ }

	FORCENOINLINE FSlateImageBrush( const FName& InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const TSharedRef< FLinearColor >& InTint, ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, InImageName, FMargin(0), InTiling, InImageType, InImageSize, InTint)
	{ }

	FORCENOINLINE FSlateImageBrush( const FString& InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const TSharedRef< FLinearColor >& InTint, ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, *InImageName, FMargin(0), InTiling, InImageType, InImageSize, InTint)
	{ }

	FORCENOINLINE FSlateImageBrush( const ANSICHAR* InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const TSharedRef< FLinearColor >& InTint, ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, InImageName, FMargin(0), InTiling, InImageType, InImageSize, InTint)
	{ }

	FORCENOINLINE FSlateImageBrush( const TCHAR* InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const TSharedRef< FLinearColor >& InTint, ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, InImageName, FMargin(0), InTiling, InImageType, InImageSize, InTint)
	{ }

	FORCENOINLINE FSlateImageBrush( const FName& InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FSlateColor& InTint, ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, InImageName, FMargin(0), InTiling, InImageType, InImageSize, InTint)
	{ }

	FORCENOINLINE FSlateImageBrush( const FString& InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FSlateColor& InTint, ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, *InImageName, FMargin(0), InTiling, InImageType, InImageSize, InTint)
	{ }

	FORCENOINLINE FSlateImageBrush( const ANSICHAR* InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FSlateColor& InTint, ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, InImageName, FMargin(0), InTiling, InImageType, InImageSize, InTint)
	{ }

	FORCENOINLINE FSlateImageBrush( const TCHAR* InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FSlateColor& InTint, ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor )
		: FSlateBrush(ESlateBrushDrawType::Image, InImageName, FMargin(0), InTiling, InImageType, InImageSize, InTint)
	{ }

	/**
	 * @param InResourceObject	The image to render for this brush, can be a UTexture, UMaterialInterface, or AtlasedTextureInterface
	 * @param InImageSize		How large should the image be (not necessarily the image size on disk)
	 * @param InTint			The tint of the image.
	 * @param InTiling			How do we tile if at all?
	 * @param InImageType		The type of image this this is
	 */

	FORCENOINLINE FSlateImageBrush(UObject* InResourceObject, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FSlateColor& InTint = FSlateColor(FLinearColor(1, 1, 1, 1)), ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile, ESlateBrushImageType::Type InImageType = ESlateBrushImageType::FullColor)
		: FSlateBrush(ESlateBrushDrawType::Image, NAME_None, FMargin(0), InTiling, InImageType, InImageSize, InTint, InResourceObject)
	{ }

};

struct FSlateVectorImageBrush
	: public FSlateImageBrush
{
	FSlateVectorImageBrush(const FString& InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FSlateColor& InTint, ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile)
		: FSlateImageBrush(InImageName, InImageSize, InTint, InTiling, ESlateBrushImageType::Vector)
	{ }

	FSlateVectorImageBrush(const FString& InImageName, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FLinearColor& InTint = FLinearColor(1, 1, 1, 1), ESlateBrushTileType::Type InTiling = ESlateBrushTileType::NoTile)
		: FSlateImageBrush(InImageName, InImageSize, InTint, InTiling, ESlateBrushImageType::Vector)
	{ }
};
