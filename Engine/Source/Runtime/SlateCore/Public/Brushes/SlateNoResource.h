// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Styling/SlateBrush.h"
#include "Types/SlateVector2.h"

/**
 * A resource that has no appearance
 */
struct FSlateNoResource
	: public FSlateBrush
{
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InImageSize An optional image size (default is zero).
	 */
	FORCENOINLINE FSlateNoResource( const UE::Slate::FDeprecateVector2DParameter& InImageSize = FVector2f::ZeroVector )
		: FSlateBrush(ESlateBrushDrawType::NoDrawType, FName(NAME_None), FMargin(0), ESlateBrushTileType::NoTile, ESlateBrushImageType::NoImage, InImageSize)
	{ }
};

/** 
 * This represents an undefined brush. FSlateNoResource is a valid brush which means don't render anything.  FSlateOptionalBrush means let the widget decide what to do if the brush is unset. 
 * Widgets that don't check for the brush being unspecified will draw nothing
 */
struct FSlateOptionalBrush
	: public FSlateBrush
{
	FSlateOptionalBrush(const UE::Slate::FDeprecateVector2DParameter& InImageSize = FVector2f::ZeroVector)
		: FSlateBrush(ESlateBrushDrawType::NoDrawType, FName(NAME_None), FMargin(0), ESlateBrushTileType::NoTile, ESlateBrushImageType::NoImage, InImageSize)
	{
		bIsSet = false;
	}
};
