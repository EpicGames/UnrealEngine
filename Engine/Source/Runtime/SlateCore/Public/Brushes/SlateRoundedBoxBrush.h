// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateBrush.h"
#include "Types/SlateVector2.h"

struct FSlateRoundedBoxBrush
	: public FSlateBrush
{

	/** 
	 * Creates and initializes a new instance with the specified color and rounds based on height
	 *
	 * @param InColor 		Linear Fill Color 
	 */
	template<typename FillColorType>
	FORCENOINLINE FSlateRoundedBoxBrush(const FillColorType& InColor, const UE::Slate::FDeprecateVector2DParameter& InImageSize = FVector2f::ZeroVector)
		: FSlateBrush(ESlateBrushDrawType::RoundedBox, 
					  NAME_None, 
					  FMargin(0.0f), 
					  ESlateBrushTileType::NoTile, 
					  ESlateBrushImageType::NoImage, 
					  InImageSize, 
					  InColor,
					  nullptr,
					  false
					 )
	{ 
	}

	template<typename FillColorType>
	FORCENOINLINE FSlateRoundedBoxBrush(const FillColorType& InColor, const UE::Slate::FDeprecateVector2DResult& InImageSize)
		: FSlateRoundedBoxBrush(InColor, UE::Slate::FDeprecateVector2DParameter(InImageSize))
	{ 
	}

#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS
	template<typename FillColorType>
	FORCENOINLINE FSlateRoundedBoxBrush(const FillColorType& InColor, const FVector2D& InImageSize)
		: FSlateRoundedBoxBrush(InColor, UE::Slate::FDeprecateVector2DParameter(UE::Slate::CastToVector2f(InImageSize)))
	{ 
	}
#endif
	template<typename FillColorType>
	FORCENOINLINE FSlateRoundedBoxBrush(const FillColorType& InColor, const FVector2f& InImageSize)
		: FSlateRoundedBoxBrush(InColor, UE::Slate::FDeprecateVector2DParameter(InImageSize))
	{ 
	}

	template<typename FillColorType>
	FORCENOINLINE FSlateRoundedBoxBrush(const FName InFillResourceName, const FillColorType& FillColorTint, const UE::Slate::FDeprecateVector2DParameter& InImageSize = FVector2f::ZeroVector, ESlateBrushTileType::Type InTileType = ESlateBrushTileType::NoTile)
		: FSlateBrush(ESlateBrushDrawType::RoundedBox,
			InFillResourceName,
			FMargin(0.0f),
			InTileType,
			ESlateBrushImageType::FullColor,
			InImageSize,
			FillColorTint,
			nullptr,
			false
		)
	{
	}


	/** 
	 * Creates and initializes a new instance with the specified color and corner radius
	 *
	 * @param InColor 		Linear Fill Color 
	 * @param InRadius      Corner Radius in Slate Units applied to the outline at each corner. X = Top Left, Y = Top Right, Z = Bottom Right, W = Bottom Left
	 */
	template<typename FillColorType, typename RadiusType>
	FORCENOINLINE FSlateRoundedBoxBrush(const FillColorType& InColor, RadiusType InRadius, const UE::Slate::FDeprecateVector2DParameter& InImageSize = FVector2f::ZeroVector)
		: FSlateBrush(ESlateBrushDrawType::RoundedBox, 
					  NAME_None, 
					  FMargin(0.0f), 
					  ESlateBrushTileType::NoTile, 
					  ESlateBrushImageType::NoImage, 
					  InImageSize,
					  InColor,
					  nullptr,
					  false
					 )
	{ 
		OutlineSettings = FSlateBrushOutlineSettings(InRadius);
	}


	template<typename FillColorType, typename RadiusType>
	FORCENOINLINE FSlateRoundedBoxBrush(const FName InFillResourceName, const FillColorType& FillColorTint, RadiusType InRadius, const UE::Slate::FDeprecateVector2DParameter& InImageSize = FVector2f::ZeroVector, ESlateBrushTileType::Type InTileType = ESlateBrushTileType::NoTile)
		: FSlateBrush(ESlateBrushDrawType::RoundedBox,
			InFillResourceName,
			FMargin(0.0f),
			InTileType,
			ESlateBrushImageType::FullColor,
			InImageSize,
			FillColorTint,
			nullptr,
			false
		)
	{
		OutlineSettings = FSlateBrushOutlineSettings(InRadius);
	}


	/** 
	 * Creates and initializes a new instance with the specified color and rounds based on height
	 *
	 * @param InColor 		 Linear Fill Color 
	 * @param InOutlineColor Outline Color 
	 * @param InOutlineWidth Outline Width or Thickness
	 */
	template<typename FillColorType, typename OutlineColorType>
	FORCENOINLINE FSlateRoundedBoxBrush(const FillColorType& InColor, const OutlineColorType& InOutlineColor, float InOutlineWidth, const UE::Slate::FDeprecateVector2DParameter& InImageSize = FVector2f::ZeroVector)
		: FSlateBrush(ESlateBrushDrawType::RoundedBox, 
					  NAME_None, 
					  FMargin(0.0f), 
					  ESlateBrushTileType::NoTile, 
					  ESlateBrushImageType::NoImage, 
					  InImageSize,
					  InColor,
					  nullptr,
					  false
					  )
	{ 
		OutlineSettings = FSlateBrushOutlineSettings(InOutlineColor, InOutlineWidth);
	}

	/**
	 * Creates and initializes a new instance with the specified color and rounds based on height
	 *
	 * @param InColor 		 Linear Fill Color
	 * @param InOutlineColor Outline Color
	 * @param InOutlineWidth Outline Width or Thickness
	 */
	template<typename FillColorType, typename OutlineColorType>
	FORCENOINLINE FSlateRoundedBoxBrush(const FName InFillResourceName, const FillColorType& FillColorTint, const OutlineColorType& InOutlineColor, float InOutlineWidth, const UE::Slate::FDeprecateVector2DParameter& InImageSize = FVector2f::ZeroVector, ESlateBrushTileType::Type InTileType = ESlateBrushTileType::NoTile)
		: FSlateBrush(ESlateBrushDrawType::RoundedBox,
			InFillResourceName,
			FMargin(0.0f),
			InTileType,
			ESlateBrushImageType::FullColor,
			InImageSize,
			FillColorTint,
			nullptr,
			false
		)
	{
		OutlineSettings = FSlateBrushOutlineSettings(InOutlineColor, InOutlineWidth);
	}



	/** 
	 * Creates and initializes a new instance with the specified color and corner radius
	 *
	 * @param InColor 		 Linear Fill Color 
	 * @param InRadius       Corner Radius in Slate Units applied to the outline at each corner. X = Top Left, Y = Top Right, Z = Bottom Right, W = Bottom Left
	 * @param InOutlineColor Outline Color 
	 * @param InOutlineWidth Outline Width or Thickness
	 */
	template<typename FillColorType, typename OutlineColorType, typename RadiusType>
	FORCENOINLINE FSlateRoundedBoxBrush(const FillColorType& InColor, RadiusType InRadius, const OutlineColorType& InOutlineColor, float InOutlineWidth, const UE::Slate::FDeprecateVector2DParameter& InImageSize = FVector2f::ZeroVector)
		: FSlateBrush(ESlateBrushDrawType::RoundedBox, 
					  NAME_None, 
					  FMargin(0.0f), 
					  ESlateBrushTileType::NoTile, 
					  ESlateBrushImageType::NoImage, 
					  InImageSize,
					  InColor,
					  nullptr,
					  false
					  )
	{ 
		OutlineSettings = FSlateBrushOutlineSettings(InRadius, InOutlineColor, InOutlineWidth);
	}

	/**
	 * Creates and initializes a new instance with the specified color and corner radius
	 *
	 * @param InColor 		 Linear Fill Color
	 * @param InRadius      Corner Radius in Slate Units applied to the outline at each corner. X = Top Left, Y = Top Right, Z = Bottom Right, W = Bottom Left
	 * @param InOutlineColor Outline Color
	 * @param InOutlineWidth Outline Width or Thickness
	 */
	template<typename FillColorType, typename OutlineColorType, typename RadiusType>
	FORCENOINLINE FSlateRoundedBoxBrush(const FName InFillResourceName, const FillColorType& FillColorTint, RadiusType InRadius, const OutlineColorType& InOutlineColor, float InOutlineWidth, const UE::Slate::FDeprecateVector2DParameter& InImageSize = FVector2f::ZeroVector, ESlateBrushTileType::Type InTileType = ESlateBrushTileType::NoTile)
		: FSlateBrush(ESlateBrushDrawType::RoundedBox,
					  InFillResourceName,
					  FMargin(0.0f),
					  InTileType,
					  ESlateBrushImageType::FullColor,
					  InImageSize,
					  FillColorTint,
					  nullptr,
					  false
		)
	{
		OutlineSettings = FSlateBrushOutlineSettings(InRadius, InOutlineColor, InOutlineWidth);
	}

};
