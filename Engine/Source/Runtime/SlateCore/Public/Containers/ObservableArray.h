// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/Integral.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Misc/MemStack.h"

namespace UE::Slate::Containers
{

template <typename InElementType>
struct TObservableArray;

/** Type of action of the FScriptObservableArrayChangedArgs */
enum class EObservableArrayChangedAction : uint8
{
	/** The array was reset. */
	Reset,
	/** Elements were added. */
	Add,
	/** Elements were removed. */
	Remove,
	/** Elements were removed and the same amount of elements moved from the end of the array to removed location. */
	RemoveSwap,
	/** 2 elements swapped location with each other. */
	Swap,
};

/**
 *
 */
template<typename InElementType>
struct TObservableArrayChangedArgs
{
private:
	using ArrayViewType = TArrayView<InElementType>;
	using SizeType = typename ArrayViewType::SizeType;
	using ElementType = typename ArrayViewType::ElementType;
	friend TObservableArray<InElementType>;
	
private:
	static TObservableArrayChangedArgs MakeResetAction()
	{
		TObservableArrayChangedArgs Result;
		Result.Action = EObservableArrayChangedAction::Reset;
		return Result;
	}

	static TObservableArrayChangedArgs MakeAddAction(const ArrayViewType InAddedItems, int32 InNewIndex)
	{
		TObservableArrayChangedArgs Result;
		Result.Array = InAddedItems;
		Result.StartIndex = InNewIndex;
		Result.Action = EObservableArrayChangedAction::Add;
		check(Result.Array.Num() > 0 && Result.StartIndex >= 0);
		return Result;
	}

	static TObservableArrayChangedArgs MakeRemoveAction(const ArrayViewType InRemovedItems, int32 InRemoveStartedIndex)
	{
		TObservableArrayChangedArgs Result;
		Result.Array = InRemovedItems;
		Result.StartIndex = InRemoveStartedIndex;
		Result.Action = EObservableArrayChangedAction::Remove;
		check(Result.Array.Num() > 0 && Result.StartIndex >= 0);
		return Result;
	}
	
	static TObservableArrayChangedArgs MakeRemoveSwapAction(const ArrayViewType InRemovedItems, int32 InRemoveStartedIndex, int32 InPreviousMovedItemLocation)
	{
		TObservableArrayChangedArgs Result;
		Result.Array = InRemovedItems;
		Result.StartIndex = InRemoveStartedIndex;
		Result.MoveIndex = InPreviousMovedItemLocation;
		Result.Action = EObservableArrayChangedAction::RemoveSwap;
		// The move index can be invalid if the array is now empty.
		check(Result.Array.Num() > 0 && (Result.MoveIndex > 0 || Result.MoveIndex == INDEX_NONE) && Result.StartIndex >= 0);
		return Result;
	}

	static TObservableArrayChangedArgs MakeSwapAction(int32 InFirstIndex, int32 InSecondIndex)
	{
		TObservableArrayChangedArgs Result;
		Result.StartIndex = InFirstIndex;
		Result.MoveIndex = InSecondIndex;
		Result.Action = EObservableArrayChangedAction::Swap;
		check(Result.Array.Num() == 0 && Result.MoveIndex >= 0 && Result.StartIndex >= 0 && Result.MoveIndex != Result.StartIndex);
		return Result;
	}

public:
	/** @return The action that caused the event. */
	EObservableArrayChangedAction GetAction() const
	{
		return Action;
	}

	/**
	 * Valid for the Add, Remove, RemoveSwap action.
	 * Use GetItems.Num() to know how many elements were added/removed.
	 * Add: The array index where we added the elements.
	 * Remove: The old array index before it removed the elements. The index is not valid anymore.
	 * RemoveSwap: The old array index before it removed the elements. The index is valid if the array is not empty.
	 * @return The index the action started.
	 */
	SizeType GetActionIndex() const
	{
		return StartIndex;
	}

	struct FRemoveSwapIndex
	{
		FRemoveSwapIndex(SizeType InRemoveIndex, SizeType InPreviousMovedElmenentIndex)
			: RemoveIndex(InRemoveIndex)
			, PreviousMovedElmenentIndex(InPreviousMovedElmenentIndex)
		{
			
		}
		/**
		 * The index of the removed elements before their removal.
		 * The index can now be out of bound.
		 * The swapped elements (if any) are now at that location.
		 */
		SizeType RemoveIndex;
		/**
		 * The previous location of the swapped elements (if any) before the swap.
		 * The index is not valid anymore.
		 * Set to INDEX_NONE, if no element was moved.
		 */
		SizeType PreviousMovedElmenentIndex;
	};

	/**
	 * Valid for the RemoveSwap action. 
	 * @return The index of the removed elements and the index of the moved elements.
	 */
	FRemoveSwapIndex GetRemovedSwapIndexes() const
	{
		return (Action == EObservableArrayChangedAction::RemoveSwap) ? FRemoveSwapIndex(StartIndex, MoveIndex) : FRemoveSwapIndex(INDEX_NONE, INDEX_NONE);
	}

	struct FSwapIndex
	{
		FSwapIndex(SizeType InFirstIndex, SizeType InSecondIndex)
			: FirstIndex(InFirstIndex)
			, SecondIndex(InSecondIndex)
		{
			
		}
		SizeType FirstIndex;
		SizeType SecondIndex;
	};
	
	/**
	 * Valid for the Swap action.
	 * @return The indexes of the 2 swapped elements. */
	FSwapIndex GetSwapIndex() const
	{
		return (Action == EObservableArrayChangedAction::Swap) ? FSwapIndex(MoveIndex, StartIndex): FSwapIndex(INDEX_NONE, INDEX_NONE);
	}

	/** @return The items added to the array or removed from the array. Valid for the Add, Remove and RemoveSwap action. */
	ArrayViewType GetItems() const
	{
		return Array;
	}
	
private:
	ArrayViewType Array;
	SizeType StartIndex = INDEX_NONE;
	SizeType MoveIndex = INDEX_NONE;
	EObservableArrayChangedAction Action = EObservableArrayChangedAction::Reset;
};


/**
 *
 */
template <typename InElementType>
struct TObservableArray
{
public:
	using ArrayType = TArray<InElementType>;
	using SizeType = typename ArrayType::SizeType;
	using ElementType = typename ArrayType::ElementType;
	using AllocatorType = typename ArrayType::AllocatorType;
	using RangedForIteratorType = typename ArrayType::RangedForIteratorType;
	using RangedForConstIteratorType = typename ArrayType::RangedForConstIteratorType;
	using ObservableArrayChangedArgsType = TObservableArrayChangedArgs<InElementType>;
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FArrayChangedDelegate, ObservableArrayChangedArgsType);
	
public:
	TObservableArray() = default;
	explicit TObservableArray(const ElementType* Ptr, SizeType Count)
		: Array(Ptr, Count)
	{
		
	}
	
	TObservableArray(std::initializer_list<ElementType> InitList)
		: Array(InitList)
	{
		
	}
	
	template<typename InOtherAllocatorType>
	explicit TObservableArray(const TArray<ElementType, InOtherAllocatorType>& Other)
		: Array(Other)
	{
		
	}
	
	template <typename InOtherAllocatorType>
	explicit TObservableArray(TArray<ElementType, InOtherAllocatorType>&& Other)
		: Array(MoveTemp(Other))
	{
		
	}

	// Non-copyable for now, but this could be made copyable in future if needed.
	TObservableArray(const TObservableArray&) = delete;
	TObservableArray& operator=(const TObservableArray&) = delete;
	TObservableArray(TObservableArray&& Other) = delete;
	TObservableArray& operator=(TObservableArray&& Other) = delete;

	~TObservableArray() = default;
	
public:
	FArrayChangedDelegate& OnArrayChanged()
	{
		return ArrayChangedDelegate;
	}

public:
	SizeType Add(const ElementType& Item)
	{
		SizeType NewIndex = Array.Add(Item);
		ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeAddAction({ GetData() + NewIndex, 1 }, NewIndex));
		return NewIndex;
	}

	SizeType Add(ElementType&& Item)
	{
		int32 NewIndex = Array.Add(MoveTempIfPossible(Item));
		ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeAddAction({ GetData() + NewIndex, 1 }, NewIndex));
		return NewIndex;
	}

	template <typename... InArgTypes>
	SizeType Emplace(InArgTypes&&... Args)
	{
		SizeType NewIndex = Array.Emplace(Forward<InArgTypes>(Args)...);
		ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeAddAction({ GetData() + NewIndex, 1 }, NewIndex));
		return NewIndex;
	}

	template <typename... InArgTypes>
	void EmplaceAt(SizeType Index, InArgTypes&&... Args)
	{
		Array.EmplaceAt(Index, Forward<InArgTypes>(Args)...);
		ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeAddAction({ GetData() + Index, 1 }, Index));
	}

	template <typename OtherElementType>
	void Append(const TObservableArray<OtherElementType>& Source)
	{
		Append(Source.Array);
	}

	template <typename OtherElementType, typename OtherAllocatorType>
	void Append(const TArray<OtherElementType, OtherAllocatorType>& Source)
	{
		if (Source.Num() > 0)
		{
			SizeType PreviousNum = Array.Num();
			Array.Append(Source);
			SizeType NewNum = Array.Num();
			ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeAddAction({ GetData() + PreviousNum, NewNum - PreviousNum }, PreviousNum));
		}
	}

	template <typename OtherElementType, typename OtherAllocator>
	void Append(TArray<OtherElementType, OtherAllocator>&& Source)
	{
		if (Source.Num() > 0)
		{
			SizeType PreviousNum = Array.Num();
			Array.Append(MoveTempIfPossible(Source));
			SizeType NewNum = Array.Num();
			ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeAddAction({ GetData() + PreviousNum, NewNum - PreviousNum }, PreviousNum));
		}
	}

	SizeType RemoveSingle(const ElementType& Item, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		SizeType Index = Array.Find(Item);
		if (Index == INDEX_NONE)
		{
			return 0;
		}
		RemoveAt(Index, 1, AllowShrinking);
		return 1;
	}

	SizeType RemoveSingleSwap(const ElementType& Item, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		SizeType Index = Array.Find(Item);
		if (Index == INDEX_NONE)
		{
			return 0;
		}
		RemoveAtSwap(Index, 1, AllowShrinking);
		return 1;
	}

	void RemoveAt(SizeType Index, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		check((Index >= 0) & (Index < Num()));

		ElementType RemovedElement = MoveTemp(Array[Index]);

		Array.RemoveAt(Index, AllowShrinking);
		ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeRemoveAction({ &RemovedElement, 1 }, Index));
	}
	template <UE::CIntegral CountType>
	void RemoveAt(SizeType Index, CountType NumToRemove, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		static_assert(!std::is_same_v<CountType, bool>, "TObservableArray::RemoveAt: unexpected bool passed as the Count argument");
		check((NumToRemove > 0) & (Index >= 0) & (Index + NumToRemove <= Num()));
		if (NumToRemove > 0)
		{		
			if (NumToRemove == 1)
			{
				ElementType RemovedElement = MoveTemp(Array[Index]);
				Array.RemoveAt(Index, NumToRemove, AllowShrinking);
				ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeRemoveAction({ &RemovedElement, 1 }, Index));
			}
			else
			{
				// Copy the items to a temporary array to call the delegate
				FMemMark Mark(FMemStack::Get());
				TArray<ElementType, TMemStackAllocator<>> RemovedElements;
				RemovedElements.Reserve(NumToRemove);
				for (SizeType RemoveIndex = 0; RemoveIndex < NumToRemove; ++RemoveIndex)
				{
					RemovedElements.Add(MoveTemp(Array[RemoveIndex + Index]));
				}
				
				Array.RemoveAt(Index, NumToRemove, AllowShrinking);
				ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeRemoveAction({ RemovedElements.GetData(), NumToRemove }, Index));
			}
		}
	}

	void RemoveAtSwap(SizeType Index, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		SizeType PreviousNum = Array.Num();
		SizeType SwapAmount = FPlatformMath::Min(1, PreviousNum - (Index + 1));

		ElementType RemovedElement = MoveTemp(Array[Index]);

		Array.RemoveAtSwap(Index, AllowShrinking);
		ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeRemoveSwapAction({ &RemovedElement, 1 }, Index, PreviousNum - SwapAmount));
	}
	template <UE::CIntegral CountType>
	void RemoveAtSwap(SizeType Index, CountType NumToRemove, EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		static_assert(!std::is_same_v<CountType, bool>, "TObservableArray::RemoveAtSwap: unexpected bool passed as the Count argument");
		check((NumToRemove > 0) & (Index >= 0) & (Index + NumToRemove <= Num()));
		if (NumToRemove > 0)
		{
			SizeType PreviousNum = Array.Num();
			SizeType SwapAmount = FPlatformMath::Min(NumToRemove, PreviousNum - (Index + NumToRemove));
			if (NumToRemove == 1)
			{
				ElementType RemovedElement = MoveTemp(Array[Index]);
				
				Array.RemoveAtSwap(Index, NumToRemove, AllowShrinking);
				ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeRemoveSwapAction({ &RemovedElement, 1 }, Index, PreviousNum - SwapAmount));
			}
			else
			{
				// Copy the items to a temporary array to call the delegate
				FMemMark Mark(FMemStack::Get());
				TArray<ElementType, TMemStackAllocator<>> RemovedElements;
				RemovedElements.Reserve(NumToRemove);
				for (SizeType RemoveIndex = 0; RemoveIndex < NumToRemove; ++RemoveIndex)
				{
					RemovedElements.Add(MoveTemp(Array[RemoveIndex + Index]));
				}

				Array.RemoveAtSwap(Index, NumToRemove, AllowShrinking);
				ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeRemoveSwapAction({ RemovedElements.GetData(), NumToRemove }, Index, PreviousNum - SwapAmount));
			}
		}
	}

	void Swap(SizeType FirstIndexToSwap, SizeType SecondIndexToSwap)
	{
		check(FirstIndexToSwap >= 0 && SecondIndexToSwap >= 0);
		if (FirstIndexToSwap != SecondIndexToSwap)
		{
			Array.SwapMemory(FirstIndexToSwap, SecondIndexToSwap);
			ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeSwapAction(FirstIndexToSwap, SecondIndexToSwap));
		}
	}

	void Reset(SizeType NewSize = 0)
	{
		SizeType PreviousNum = Array.Num();
		Array.Reset(NewSize);
		if (PreviousNum > 0)
		{
			ArrayChangedDelegate.Broadcast(ObservableArrayChangedArgsType::MakeResetAction());
		}
	}

	void Reserve(SizeType Number)
	{
		Array.Reserve(Number);
	}

	bool IsEmpty() const
	{
		return Array.IsEmpty();
	}

	int32 Num() const
	{
		return Array.Num();
	}

	bool IsValidIndex(SizeType Index) const
	{
		return Array.IsValidIndex(Index);
	}

	ElementType& operator[](SizeType Index)
	{
		return GetData()[Index];
	}

	const ElementType& operator[](SizeType Index) const
	{
		return GetData()[Index];
	}

	ElementType* GetData()
	{
		return Array.GetData();
	}

	const ElementType* GetData() const
	{
		return Array.GetData();
	}

	template <typename ComparisonType>
	bool Contains(const ComparisonType& Item) const
	{
		return Array.Contains(Item);
	}

	template <typename InPredicate>
	SizeType ContainsByPredicate(InPredicate Pred) const
	{
		return Array.ContainsByPredicate(Pred);
	}

	SizeType Find(const ElementType& Item) const
	{
		return Array.Find(Item);
	}

	template <typename InPredicate>
	ElementType* FindByPredicate(InPredicate Pred)
	{
		return Array.FindByPredicate(Pred);
	}

	template <typename InPredicate>
	const ElementType* FindByPredicate(InPredicate Pred) const
	{
		return Array.FindByPredicate(Pred);
	}

	template <typename InPredicate>
	UE_DEPRECATED(5.4, "IndexByPredicate is deprecated. Use IndexOfByPredicate instead")
	SizeType IndexByPredicate(InPredicate Pred) const
	{
		return Array.IndexOfByPredicate(Pred);
	}

	template <typename InPredicate>
	SizeType IndexOfByPredicate(InPredicate Pred) const
	{
		return Array.IndexOfByPredicate(Pred);
	}

public:
	template <typename InOtherAllocatorType>
	bool operator==(TArray<ElementType, InOtherAllocatorType>& OtherArray) const
	{
		return Array.operator==(OtherArray);
	}
	
	bool operator==(const TObservableArray& OtherArray) const
	{
		return Array.operator==(OtherArray.Array);
	}

	template <typename InOtherAllocatorType>
	friend bool operator==(TArray<ElementType, InOtherAllocatorType>& OtherArray, const TObservableArray& Self)
	{
		return Self.Array.operator==(OtherArray);
	}

public:
	RangedForIteratorType begin()
	{
		return Array.begin();
	}

	RangedForConstIteratorType begin() const
	{
		return Array.begin();
	}

	RangedForIteratorType end()
	{
		return Array.end();
	}

	RangedForConstIteratorType end() const
	{
		return Array.end();
	}
	
private:
	ArrayType Array;
	FArrayChangedDelegate ArrayChangedDelegate;
};
 
 } //namespace
