// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"

class SWidget;

#ifndef UE_WITH_SLATE_CRASHREPORTER
	#define UE_WITH_SLATE_CRASHREPORTER !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_ADDITIONAL_CRASH_CONTEXTS
#endif

namespace UE::Slate
{
	
#if UE_WITH_SLATE_CRASHREPORTER

/** Helper object allowing easy tracking of Slate code in crash reporter. */
class FSlateCrashReporterScope
{
public:
	static SLATECORE_API FName NAME_Paint;
	static SLATECORE_API FName NAME_Prepass;

	SLATECORE_API explicit FSlateCrashReporterScope(const SWidget& Widget, FName Context);
	SLATECORE_API ~FSlateCrashReporterScope();

private:
	uint8 ThreadId = 0;
	bool bWasEnabled = false;
};

#endif //UE_WITH_SLATE_CRASHREPORTER

} //namespace

#if UE_WITH_SLATE_CRASHREPORTER

#define UE_SLATE_CRASH_REPORTER_PAINT_SCOPE(Widget) UE::Slate::FSlateCrashReporterScope ANONYMOUS_VARIABLE(SlateAddCrashContext) {Widget, UE::Slate::FSlateCrashReporterScope::NAME_Paint}
#define UE_SLATE_CRASH_REPORTER_PREPASS_SCOPE(Widget) UE::Slate::FSlateCrashReporterScope ANONYMOUS_VARIABLE(SlateAddCrashContext) {Widget, UE::Slate::FSlateCrashReporterScope::NAME_Prepass}

#define UE_SLATE_CRASH_REPORTER_CONTEXT_SCOPE(Widget, Context) UE::Slate::FSlateCrashReporterScope ANONYMOUS_VARIABLE(SlateAddCrashContext) {Widget, Context}

#else //UE_WITH_SLATE_CRASHREPORTER

#define UE_SLATE_CRASH_REPORTER_PAINT_SCOPE(Widget)
#define UE_SLATE_CRASH_REPORTER_PREPASS_SCOPE(Widget)

#define UE_SLATE_CRASH_REPORTER_CONTEXT_SCOPE(Widget, Context)

#endif //UE_WITH_SLATE_CRASHREPORTER
