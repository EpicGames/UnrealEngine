// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "FastUpdate/WidgetUpdateFlags.h"
#include "Input/Reply.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Types/SlateAttribute.h"
#include "Widgets/InvalidateWidgetReason.h"

#include "SlateDebugging.generated.h"

#ifndef WITH_SLATE_DEBUGGING
	#define WITH_SLATE_DEBUGGING !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif

#ifndef WITH_SLATE_NAVIGATION_DEBUGGING
	#define WITH_SLATE_NAVIGATION_DEBUGGING 1
#endif

#ifndef SLATE_CSV_TRACKER
	#define SLATE_CSV_TRACKER CSV_PROFILER_STATS
#endif

// Enabled to build a list of all the SWidget currently constructed
#ifndef UE_WITH_SLATE_DEBUG_WIDGETLIST
	#define UE_WITH_SLATE_DEBUG_WIDGETLIST UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
#endif

class SWindow;
class SWidget;
struct FGeometry;
class FPaintArgs;
class FSlateWindowElementList;
class FSlateDrawElement;
class FSlateRect;
class FWeakWidgetPath;
class FWidgetPath;
class FNavigationReply;
class FSlateInvalidationRoot;
enum class EElementType : uint8;

UENUM()
enum class ESlateDebuggingInputEvent : uint8
{
	MouseMove = 0,
	MouseEnter,
	MouseLeave,
	PreviewMouseButtonDown,
	MouseButtonDown,
	MouseButtonUp,
	MouseButtonDoubleClick,
	MouseWheel,
	TouchStart,
	TouchEnd,
	TouchForceChanged,
	TouchFirstMove,
	TouchMoved,
	DragDetected,
	DragEnter,
	DragLeave,
	DragOver,
	DragDrop,
	DropMessage,
	PreviewKeyDown,
	KeyDown,
	KeyUp,
	KeyChar,
	AnalogInput,
	TouchGesture,
	MotionDetected,
	MAX,
};

UENUM()
enum class ESlateDebuggingStateChangeEvent : uint8
{
	MouseCaptureGained,
	MouseCaptureLost,
};

UENUM()
enum class ESlateDebuggingNavigationMethod : uint8
{
	Unknown,
	Explicit,
	CustomDelegateBound,
	CustomDelegateUnbound,
	NextOrPrevious,
	HitTestGrid
};

struct FSlateDebuggingInputEventArgs
{
public:
	SLATECORE_API FSlateDebuggingInputEventArgs(ESlateDebuggingInputEvent InInputEventType, const FInputEvent* InInputEvent, const FReply& InReply, const TSharedPtr<SWidget>& InHandlerWidget, const FString& InAdditionalContent);

	const ESlateDebuggingInputEvent InputEventType;
	const FInputEvent* InputEvent;
	const FReply& Reply;
	const TSharedPtr<SWidget>& HandlerWidget;
	const FString& AdditionalContent;

	SLATECORE_API FText ToText() const;
};

struct FSlateDebuggingCursorQueryEventArgs
{
public:
	SLATECORE_API FSlateDebuggingCursorQueryEventArgs(const TSharedPtr<const SWidget>& InWidgetOverridingCursor, const FCursorReply& InReply);

	const TSharedPtr<const SWidget>& WidgetOverridingCursor;
	const FCursorReply& Reply;

	SLATECORE_API FText ToText() const;
};

struct FSlateDebuggingElementTypeAddedEventArgs
{
public:
	SLATECORE_API FSlateDebuggingElementTypeAddedEventArgs(
		const FSlateWindowElementList& InElementList,
		int32 InElementIndex,
		EElementType InElementType
	);

	/** Element list containing the element */
	const FSlateWindowElementList& ElementList;

	/** Index into the element within the corresponding typed container */
	int32 ElementIndex;

	/** Type of the element, element will reside in a container dedicated for that type */
	EElementType ElementType;
};

UENUM()
enum class ESlateDebuggingFocusEvent : uint8
{
	FocusChanging = 0,
	FocusLost,
	FocusReceived,
	MAX
};


#if WITH_SLATE_DEBUGGING


struct FSlateDebuggingFocusEventArgs
{
public:
	SLATECORE_API FSlateDebuggingFocusEventArgs(
		ESlateDebuggingFocusEvent InFocusEventType,
		const FFocusEvent& InFocusEvent,
		const FWeakWidgetPath& InOldFocusedWidgetPath,
		const TSharedPtr<SWidget>& InOldFocusedWidget,
		const FWidgetPath& InNewFocusedWidgetPath,
		const TSharedPtr<SWidget>& InNewFocusedWidget
	);

	ESlateDebuggingFocusEvent FocusEventType;
	const FFocusEvent& FocusEvent;
	const FWeakWidgetPath& OldFocusedWidgetPath;
	const TSharedPtr<SWidget>& OldFocusedWidget;
	const FWidgetPath& NewFocusedWidgetPath;
	const TSharedPtr<SWidget>& NewFocusedWidget;

	SLATECORE_API FText ToText() const;
};

struct FSlateDebuggingNavigationEventArgs
{
public:
	SLATECORE_API FSlateDebuggingNavigationEventArgs(
		const FNavigationEvent& InNavigationEvent,
		const FNavigationReply& InNavigationReply,
		const FWidgetPath& InNavigationSource,
		const TSharedPtr<SWidget>& InDestinationWidget,
		const ESlateDebuggingNavigationMethod InNavigationMethod
	);

	const FNavigationEvent& NavigationEvent;
	const FNavigationReply& NavigationReply;
	const FWidgetPath& NavigationSource;
	const TSharedPtr<SWidget>& DestinationWidget;
	const ESlateDebuggingNavigationMethod NavigationMethod;

	SLATECORE_API FText ToText() const;
};

struct FSlateDebuggingExecuteNavigationEventArgs
{
public:
};

struct FSlateDebuggingWarningEventArgs
{
public:
	SLATECORE_API FSlateDebuggingWarningEventArgs(
		const FText& InWarning,
		const TSharedPtr<SWidget>& InOptionalContextWidget
	);

	const FText& Warning;
	const TSharedPtr<SWidget>& OptionalContextWidget;

	SLATECORE_API FText ToText() const;
};

struct FSlateDebuggingMouseCaptureEventArgs
{
public:
	SLATECORE_API FSlateDebuggingMouseCaptureEventArgs(
		bool InCaptured,
		uint32 InUserIndex,
		uint32 InPointerIndex,
		const TSharedPtr<const SWidget>& InCapturingWidget
	);

	bool Captured;
	uint32 UserIndex;
	uint32 PointerIndex;
	const TSharedPtr<const SWidget>& CaptureWidget;

	SLATECORE_API FText ToText() const;
};

enum class ESlateDebuggingInvalidateRootReason
{
	None = 0,
	ChildOrder = 1 << 0,
	Root = 1 << 1,
	ScreenPosition = 1 << 2,
};

ENUM_CLASS_FLAGS(ESlateDebuggingInvalidateRootReason)

SLATECORE_API FString LexToString(ESlateDebuggingInvalidateRootReason Reason);
SLATECORE_API bool LexTryParseString(ESlateDebuggingInvalidateRootReason& OutMode, const TCHAR* InBuffer);
SLATECORE_API void LexFromString(ESlateDebuggingInvalidateRootReason& OutMode, const TCHAR* InBuffer);

struct FSlateDebuggingInvalidateArgs
{
	SLATECORE_API FSlateDebuggingInvalidateArgs(
		const SWidget* WidgetInvalidated,
		const SWidget* WidgetInvalidateInvestigator,
		EInvalidateWidgetReason InvalidateReason);

	SLATECORE_API FSlateDebuggingInvalidateArgs(
		const SWidget* WidgetInvalidated,
		const SWidget* WidgetInvalidateInvestigator,
		ESlateDebuggingInvalidateRootReason InvalidateReason);

	const SWidget* WidgetInvalidated;
	const SWidget* WidgetInvalidateInvestigator;
	EInvalidateWidgetReason InvalidateWidgetReason;
	ESlateDebuggingInvalidateRootReason InvalidateInvalidationRootReason;
};

struct FSlateDebuggingWidgetUpdatedEventArgs
{
public:
	SLATECORE_API FSlateDebuggingWidgetUpdatedEventArgs(
		const SWidget* Widget,
		EWidgetUpdateFlags UpdateFlags,
		bool bFromPaint
	);

	const SWidget* Widget;
	/** Flag that was set by an invalidation or on the widget directly. */
	EWidgetUpdateFlags UpdateFlags;
	/** The widget got painted as a side effect of another widget that got painted */
	bool bFromPaint;

	SLATECORE_API FText ToText() const;
};

/**
 * 
 */
class FSlateDebugging
{
public:
	/** Called when a widget begins painting. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FBeginWindow, const FSlateWindowElementList& /*ElementList*/);
	static SLATECORE_API FBeginWindow BeginWindow;

	/** Called when a window finishes painting. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FEndWindow, const FSlateWindowElementList& /*ElementList*/);
	static SLATECORE_API FEndWindow EndWindow;

	/** Called just before a widget paints. */
	DECLARE_MULTICAST_DELEGATE_SixParams(FBeginWidgetPaint, const SWidget* /*Widget*/, const FPaintArgs& /*Args*/, const FGeometry& /*AllottedGeometry*/, const FSlateRect& /*MyCullingRect*/, const FSlateWindowElementList& /*OutDrawElements*/, int32 /*LayerId*/);
	static SLATECORE_API FBeginWidgetPaint BeginWidgetPaint;

	/** Called after a widget finishes painting. */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FEndWidgetPaint, const SWidget* /*Widget*/, const FSlateWindowElementList& /*OutDrawElements*/, int32 /*LayerId*/);
	static SLATECORE_API FEndWidgetPaint EndWidgetPaint;

	/** Called just before a widget starts its prepass. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FBeginWidgetPrepass, const SWidget* /*Widget*/);
	static SLATECORE_API FBeginWidgetPrepass BeginWidgetPrepass;

	/** Called after a widget finishes its prepass. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FEndWidgetPrepass, const SWidget* /*Widget*/);
	static SLATECORE_API FEndWidgetPrepass EndWidgetPrepass;

	/** Called once the window is drawn (including PaintDeferred). It can be used to draw additional debug info on top of the window. */
	DECLARE_MULTICAST_DELEGATE_FourParams(FPaintDebugElements, const FPaintArgs& /*InArgs*/, const FGeometry& /*InAllottedGeometry*/, FSlateWindowElementList& /*InOutDrawElements*/, int32& /*InOutLayerId*/);
	static SLATECORE_API FPaintDebugElements PaintDebugElements;

	/**
	 * Called as soon as the element is added to the element list.
	 * NOTE: These elements are not valid until the widget finishes painting, or you can resolve them all after the window finishes painting.
	 */
	UE_DEPRECATED(5.2, "FSlateDebugging::ElementAdded is deprecated, use FSlateDebugging::ElementTypeAdded instead")
	DECLARE_MULTICAST_DELEGATE_TwoParams(FDrawElement, const FSlateWindowElementList& /*ElementList*/, int32 /*ElementIndex*/);
SLATECORE_API PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static FDrawElement ElementAdded;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Called as soon as the element is added to the element list.
	 * NOTE: These elements are not valid until the widget finishes painting, or you can resolve them all after the window finishes painting.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FDrawElementType, const FSlateDebuggingElementTypeAddedEventArgs& /*ElementTypeAddedArgs*/);
	static SLATECORE_API FDrawElementType ElementTypeAdded;

public:
	/**  */
	DECLARE_MULTICAST_DELEGATE_OneParam(FWidgetWarningEvent, const FSlateDebuggingWarningEventArgs& /*EventArgs*/);
	static SLATECORE_API FWidgetWarningEvent Warning;
	static SLATECORE_API void BroadcastWarning(const FText& WarningText, const TSharedPtr<SWidget>& OptionalContextWidget);

public:
	/**  */
	DECLARE_MULTICAST_DELEGATE_OneParam(FWidgetInputEvent, const FSlateDebuggingInputEventArgs& /*EventArgs*/);
	static SLATECORE_API FWidgetInputEvent InputEvent;

	static SLATECORE_API void BroadcastInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const FReply& InReply);
	static SLATECORE_API void BroadcastInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const TSharedPtr<SWidget>& HandlerWidget);
	static SLATECORE_API void BroadcastInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const FReply& InReply, const TSharedPtr<SWidget>& HandlerWidget);
	static SLATECORE_API void BroadcastInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const FReply& InReply, const TSharedPtr<SWidget>& HandlerWidget, const FString& AdditionalContent);
	static SLATECORE_API void BroadcastInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const FReply& InReply, const TSharedPtr<SWidget>& HandlerWidget, const FName& AdditionalContent);
	static SLATECORE_API void BroadcastInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const FReply& InReply, const TSharedPtr<SWidget>& HandlerWidget, const TCHAR AdditionalContent);
	static SLATECORE_API void BroadcastNoReplyInputEvent(ESlateDebuggingInputEvent InputEventType, const FInputEvent* InInputEvent, const TSharedPtr<SWidget>& HandlerWidget);

public:
	/** */
	struct FScopeProcessInputEvent
	{
		ESlateDebuggingInputEvent InputEvent;
		SLATECORE_API FScopeProcessInputEvent(ESlateDebuggingInputEvent InputEvent, const FInputEvent& Event);
		SLATECORE_API ~FScopeProcessInputEvent();
	};

	/** */
	struct FScopeRouteInputEvent
	{
		ESlateDebuggingInputEvent InputEvent;
		SLATECORE_API FScopeRouteInputEvent(ESlateDebuggingInputEvent InputEvent, const FName& RoutingType);
		SLATECORE_API ~FScopeRouteInputEvent();
	};

	/** */
	static SLATECORE_API void BroadcastPreProcessInputEvent(ESlateDebuggingInputEvent InputEventType, const TCHAR* InputPrecessorName, bool bHandled);

	/** */
	struct IWidgetInputRoutingEvent
	{
		virtual void OnProcessInput(ESlateDebuggingInputEvent InputEventType, const FInputEvent& Event) = 0;
		virtual void OnPreProcessInput(ESlateDebuggingInputEvent InputEventType, const TCHAR* InputPrecessorName, bool bHandled) = 0;
		virtual void OnRouteInput(ESlateDebuggingInputEvent InputEventType, const FName& RoutedType) = 0;
		virtual void OnInputEvent(ESlateDebuggingInputEvent InputEventType, const FReply& InReply, const TSharedPtr<SWidget>& HandlerWidget) = 0;
		virtual void OnInputRouted(ESlateDebuggingInputEvent InputEventType) = 0;
		virtual void OnInputProcessed(ESlateDebuggingInputEvent InputEventType) = 0;
	};

	static SLATECORE_API void RegisterWidgetInputRoutingEvent(IWidgetInputRoutingEvent* Event);
	static SLATECORE_API void UnregisterWidgetInputRoutingEvent(IWidgetInputRoutingEvent* Event);

public:
	/**  */
	DECLARE_MULTICAST_DELEGATE_OneParam(FWidgetFocusEvent, const FSlateDebuggingFocusEventArgs& /*EventArgs*/);
	static SLATECORE_API FWidgetFocusEvent FocusEvent;

	static SLATECORE_API void BroadcastFocusChanging(const FFocusEvent& InFocusEvent, const FWeakWidgetPath& InOldFocusedWidgetPath, const TSharedPtr<SWidget>& InOldFocusedWidget, const FWidgetPath& InNewFocusedWidgetPath, const TSharedPtr<SWidget>& InNewFocusedWidget);
	static SLATECORE_API void BroadcastFocusLost(const FFocusEvent& InFocusEvent, const FWeakWidgetPath& InOldFocusedWidgetPath, const TSharedPtr<SWidget>& InOldFocusedWidget, const FWidgetPath& InNewFocusedWidgetPath, const TSharedPtr<SWidget>& InNewFocusedWidget);
	static SLATECORE_API void BroadcastFocusReceived(const FFocusEvent& InFocusEvent, const FWeakWidgetPath& InOldFocusedWidgetPath, const TSharedPtr<SWidget>& InOldFocusedWidget, const FWidgetPath& InNewFocusedWidgetPath, const TSharedPtr<SWidget>& InNewFocusedWidget);

public:
	/**  */
	DECLARE_MULTICAST_DELEGATE_OneParam(FWidgetAttemptNavigationEvent, const FSlateDebuggingNavigationEventArgs& /*EventArgs*/);
	static SLATECORE_API FWidgetAttemptNavigationEvent AttemptNavigationEvent;

	static SLATECORE_API void BroadcastAttemptNavigation(const FNavigationEvent& InNavigationEvent, const FNavigationReply& InNavigationReply, const FWidgetPath& InNavigationSource, const TSharedPtr<SWidget>& InDestinationWidget, ESlateDebuggingNavigationMethod InNavigationMethod);

	/**  */
	DECLARE_MULTICAST_DELEGATE_OneParam(FWidgetExecuteNavigationEvent, const FSlateDebuggingExecuteNavigationEventArgs& /*EventArgs*/);
	static SLATECORE_API FWidgetExecuteNavigationEvent ExecuteNavigationEvent;

	static SLATECORE_API void BroadcastExecuteNavigation();

public:
	/**  */
	DECLARE_MULTICAST_DELEGATE_OneParam(FWidgetMouseCaptureEvent, const FSlateDebuggingMouseCaptureEventArgs& /*EventArgs*/);
	static SLATECORE_API FWidgetMouseCaptureEvent MouseCaptureEvent;

	static SLATECORE_API void BroadcastMouseCapture(uint32 UserIndex, uint32 PointerIndex, TSharedPtr<const SWidget> InCapturingWidget);
	static SLATECORE_API void BroadcastMouseCaptureLost(uint32 UserIndex, uint32 PointerIndex, TSharedPtr<const SWidget> InWidgetLostCapture);

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FWidgetCursorQuery, const FSlateDebuggingCursorQueryEventArgs& /*EventArgs*/);
	static SLATECORE_API FWidgetCursorQuery CursorChangedEvent;

	static SLATECORE_API void BroadcastCursorQuery(TSharedPtr<const SWidget> InWidgetOverridingCursor, const FCursorReply& InReply);

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FWidgetInvalidate, const FSlateDebuggingInvalidateArgs& /*EventArgs*/);
	static SLATECORE_API FWidgetInvalidate WidgetInvalidateEvent;

	static SLATECORE_API void BroadcastWidgetInvalidate(const SWidget* WidgetInvalidated, const SWidget* WidgetInvalidateInvestigator, EInvalidateWidgetReason InvalidateReason);
	static SLATECORE_API void BroadcastInvalidationRootInvalidate(const SWidget* WidgetInvalidated, const SWidget* WidgetInvalidateInvestigator, ESlateDebuggingInvalidateRootReason InvalidateReason);

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FWidgetUpdatedEvent, const FSlateDebuggingWidgetUpdatedEventArgs& /*Args*/);
	static SLATECORE_API FWidgetUpdatedEvent WidgetUpdatedEvent;

	static SLATECORE_API void BroadcastWidgetUpdated(const SWidget* Invalidated, EWidgetUpdateFlags UpdateFlags);
	static SLATECORE_API void BroadcastWidgetUpdatedByPaint(const SWidget* Invalidated, EWidgetUpdateFlags UpdateFlags);

public:
	/**  */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FUICommandRun, const FName& /*CommandName*/, const FText& /*CommandLabel*/);
	static SLATECORE_API FUICommandRun CommandRun;

	static SLATECORE_API const TArray<const SWidget*>& GetAllWidgets();
	static SLATECORE_API void ExportWidgetList(FStringView Filename);

public:
	static SLATECORE_API void IncrementInvalidationPanelCount(bool bCanCache, bool bCanUseDynamic, bool bSupportsInvalidation);
	static SLATECORE_API void ResetInvalidationPanelCount();
	static uint32 GetNoInvalidationPanels() { return NoInvalidationPanels; }
	static uint32 GetNoDynamicInvalidationPanels() { return NoDynamicInvalidationPanels; }
	static uint32 GetNoDynamicInvalidationPanelsSupportingInvalidation() { return NoDynamicInvalidationPanelsSupportingInvalidation; }

private:

	// This class is only for namespace use
	FSlateDebugging() = delete;

	struct FLastCursorQuery
	{
		const SWidget* WidgetThatOverrideCursorLast_UnsafeToUseForAnythingButCompare = nullptr;
		TWeakPtr<SWidget> CursorWidget;
		EMouseCursor::Type MouseCursor;
	};
	static FLastCursorQuery LastCursorQuery;

	static TArray<IWidgetInputRoutingEvent*> RoutingEvents;

	static SLATECORE_API uint32 NoInvalidationPanels;
	static SLATECORE_API uint32 NoDynamicInvalidationPanels;
	static SLATECORE_API uint32 NoDynamicInvalidationPanelsSupportingInvalidation;
};

#endif
