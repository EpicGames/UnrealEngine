// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "WidgetProxy.h"
#include "FastUpdate/SlateInvalidationRootHandle.h"
#include "FastUpdate/SlateInvalidationWidgetIndex.h"
#include "FastUpdate/SlateInvalidationWidgetSortOrder.h"
#include "Rendering/DrawElements.h"

struct FSlateCachedElementData;
class FSlateInvalidationWidgetList;
class FSlateInvalidationWidgetPreHeap;
class FSlateInvalidationWidgetPrepassHeap;
class FSlateInvalidationWidgetPostHeap;
class FSlateWindowElementList;
class FWidgetStyle;

namespace UE::Slate::Private
{
	struct FSlateInvalidationPaintFastPathContext;
}

#define UE_SLATE_DEBUGGING_CLEAR_ALL_FAST_PATH_DATA 0

struct FSlateInvalidationContext
{
	FSlateInvalidationContext(FSlateWindowElementList& InWindowElementList, const FWidgetStyle& InWidgetStyle)
		: ViewOffset(0.0f, 0.0f)
		, PaintArgs(nullptr)
		, WidgetStyle(InWidgetStyle)
		, WindowElementList(&InWindowElementList)
		, LayoutScaleMultiplier(1.0f)
		, IncomingLayerId(0)
		, bParentEnabled(true)
		, bAllowFastPathUpdate(false)
	{
	} 

	FSlateRect CullingRect;
	UE::Slate::FDeprecateVector2DResult ViewOffset;
	const FPaintArgs* PaintArgs;
	const FWidgetStyle& WidgetStyle;
	FSlateWindowElementList* WindowElementList;
	float LayoutScaleMultiplier;
	int32 IncomingLayerId;
	bool bParentEnabled;
	bool bAllowFastPathUpdate;
};

enum class ESlateInvalidationPaintType
{
	None,
	Slow,
	Fast,
};

struct FSlateInvalidationResult
{
	FSlateInvalidationResult()
		: ViewOffset(0.0f, 0.0f)
		, MaxLayerIdPainted(0)
		, bRepaintedWidgets(false)
	{}

	/** The view offset to use with the draw buffer */
	UE::Slate::FDeprecateVector2DResult ViewOffset;
	/** The max layer id painted or cached */
	int32 MaxLayerIdPainted;
	/** If we had to repaint any widget */
	bool bRepaintedWidgets;
};

class FSlateInvalidationRoot : public FGCObject, public FNoncopyable
{
	friend class FSlateUpdateFastWidgetPathTask;
	friend class FSlateUpdateFastPathAndHitTestGridTask;
	friend class FWidgetProxyHandle;

public:
	SLATECORE_API FSlateInvalidationRoot();
	SLATECORE_API virtual ~FSlateInvalidationRoot();

	//~ Begin FGCObject interface
	SLATECORE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	SLATECORE_API virtual FString GetReferencerName() const override;
	//~ End FGCObject interface

	/** Rebuild the list and request a SlowPath. */
	SLATECORE_API void InvalidateRootChildOrder(const SWidget* Investigator = nullptr);
	/** Invalidate the layout, forcing the parent of the InvalidationRoot to be repainted. */
	SLATECORE_API void InvalidateRootLayout(const SWidget* Investigator = nullptr);
	/**
	 * Update the screen position of the SWidget owning the InvalidationRoot.
	 * This is faster then doing a SlowPath when only the DesktopGeometry changed.
	 */
	SLATECORE_API void InvalidateScreenPosition(const SWidget* Investigator = nullptr);

	/** @return if the InvalidationRoot will be rebuild, Prepass() and Paint will be called. */
	bool NeedsSlowPath() const { return bNeedsSlowPath; }

	/** @return the HittestGrid of the InvalidationRoot. */
	FHittestGrid* GetHittestGrid() const { return RootHittestGrid; }
	/** @return the cached draw elements for this window and its widget hierarchy. */
	FSlateCachedElementData& GetCachedElements() { return *CachedElementData; }
	/** @return the cached draw elements for this window and its widget hierarchy. */
	const FSlateCachedElementData& GetCachedElements() const { return *CachedElementData; }
	/** @return the invalidation root as a widget. */
	const SWidget* GetInvalidationRootWidget() const { return InvalidationRootWidget; }
	/** @return the Handle of the InvalidationRoot. */
	FSlateInvalidationRootHandle GetInvalidationRootHandle() const { return InvalidationRootHandle; }
	/** @return the list of widgets that are controlled by the InvalidationRoot. */
	const FSlateInvalidationWidgetList& GetFastPathWidgetList() const { return *FastWidgetPathList; }
	/** @return the widget that is the root of the InvalidationRoot. */
	SLATECORE_API const TSharedPtr<SWidget> GetFastPathWidgetListRoot() const;

	/** @return the cached draw elements for this window and its widget hierarchy */
	SLATECORE_API FSlateInvalidationResult PaintInvalidationRoot(const FSlateInvalidationContext& Context);

	/** @return if the InvalidationRoot is currently updating the attributes. */
	bool IsProcessingAttributeUpdate() const { return bProcessingAttributeUpdate; }

	void OnWidgetDestroyed(const SWidget* Widget);

	SLATECORE_API void Advanced_ResetInvalidation(bool bClearResourcesImmediately);

	/**
	 * @return if the InvalidationRoot can cache this frame. This can return false for a Dynamic Invalidation Panel
	 * not supporting invalidation, or a disabled Invalidation Panel.
	 */
	virtual bool CanCacheThisFrame() const { return true; };

#if WITH_SLATE_DEBUGGING
	/** @return the last paint type the invalidation root handle used. */
	ESlateInvalidationPaintType GetLastPaintType() const { return LastPaintType; }
	void SetLastPaintType(ESlateInvalidationPaintType Value) { LastPaintType = Value; }

	struct FPerformanceStat
	{
		double WidgetsPreUpdate = 0.0;
		double WidgetsAttribute = 0.0;
		double WidgetsPrepass = 0.0;
		double WidgetsUpdate = 0.0;
		/** Include the other stats + maintenance */
		double InvalidationProcessing = 0.0;
	};
	FPerformanceStat GetPerformanceStat() const { return PerformanceStat; }
#endif

protected:
	/** @return the children root widget of the Invalidation root. */
	virtual TSharedRef<SWidget> GetRootWidget() = 0;
	virtual int32 PaintSlowPath(const FSlateInvalidationContext& Context) = 0;

	void SetInvalidationRootWidget(SWidget& InInvalidationRootWidget) { InvalidationRootWidget = &InInvalidationRootWidget; }
	void SetInvalidationRootHittestGrid(FHittestGrid& InHittestGrid) { RootHittestGrid = &InHittestGrid; }
	int32 GetCachedMaxLayerId() const { return CachedMaxLayerId; }

	SLATECORE_API bool ProcessInvalidation();

	SLATECORE_API void ClearAllFastPathData(bool bClearResourcesImmediately);

	virtual void OnRootInvalidated() { }

	SLATECORE_API void SetNeedsSlowPath(bool InNeedsSlowPath);

private:
	FSlateInvalidationWidgetList& GetFastPathWidgetList() { return *FastWidgetPathList; }
	void HandleInvalidateAllWidgets(bool bClearResourcesImmediately);

	bool PaintFastPath(const FSlateInvalidationContext& Context);
	bool PaintFastPath_UpdateNextWidget(const FSlateInvalidationContext& Context, UE::Slate::Private::FSlateInvalidationPaintFastPathContext& FastPathContext);
	void PaintFastPath_FixupLayerId(UE::Slate::Private::FSlateInvalidationPaintFastPathContext& FastPathContext, const FWidgetProxy& InvalidationWidget, const int32 NewOutgoingLayerId);
	void PaintFastPath_FixupParentLayerId(UE::Slate::Private::FSlateInvalidationPaintFastPathContext& FastPathContext, const FWidgetProxy& InvalidationWidget, const int32 NewOutgoingLayerId);
	void PaintFastPath_AddUniqueSortedToFinalUpdateList(const FSlateInvalidationWidgetIndex InvalidationWidgetIndex);

	/** Call when an invalidation occurred. */
	void InvalidateWidget(FWidgetProxy& Proxy, EInvalidateWidgetReason InvalidateReason);

	void BuildFastPathWidgetList(const TSharedRef<SWidget>& RootWidget);
	void AdjustWidgetsDesktopGeometry(UE::Slate::FDeprecateVector2DParameter WindowToDesktopTransform);

	/** Update child order and slate attribute registration */
	void ProcessPreUpdate();
	/** Slate attribute update */
	void ProcessAttributeUpdate();
	/** Call Slate Prepass. */
	void ProcessPrepassUpdate();
	/** Update paint, tick, timers */
	bool ProcessPostUpdate();

private:
	/** List of all the Widget included by this SlateInvalidationRoot. */
	TUniquePtr<FSlateInvalidationWidgetList> FastWidgetPathList;

	/** Index of widgets that have the child order invalidated. They affect the widgets index/order. */
	TUniquePtr<FSlateInvalidationWidgetPreHeap> WidgetsNeedingPreUpdate;

	/**
	 * Index of widgets that have the Prepass invalidated.
	 * They will be updated before the the other layout invalidations to reduce the number of SlatePrepass call.
	 */
	TUniquePtr<FSlateInvalidationWidgetPrepassHeap> WidgetsNeedingPrepassUpdate;

	/**
	 * Index of widgets that have invalidation (volatile, or need some sort of per frame update such as a tick or timer).
	 * They will be added to the FinalUpdateList to be updated.
	 */
	TUniquePtr<FSlateInvalidationWidgetPostHeap> WidgetsNeedingPostUpdate;

	/** Widgets that will be updated. */
	TArray<FSlateInvalidationWidgetHeapElement> FinalUpdateList;

	FVector2f CachedViewOffset;

	FSlateCachedElementData* CachedElementData;

	SWidget* InvalidationRootWidget;

	FHittestGrid* RootHittestGrid;

	int32 CachedMaxLayerId;

	FSlateInvalidationRootHandle InvalidationRootHandle;

	bool bNeedsSlowPath;
	bool bNeedScreenPositionShift;
	bool bProcessingPreUpdate;
	bool bProcessingAttributeUpdate;
	bool bProcessingPrepassUpdate;
	bool bProcessingPostUpdate;
	bool bBuildingWidgetList;
	bool bProcessingChildOrderInvalidation;

	/**
	 * If set, HandleInvalidateAllWidgets will be called at the end of ProcessInvalidation when it is safe to do so.
	 * The value of this variable will be passed as the `bClearResourcesImmediately` parameter for HandleInvalidateAllWidgets
	 */
	TOptional<bool> bDeferInvalidateAllWidgetsAndClearResources;

#if WITH_SLATE_DEBUGGING
	ESlateInvalidationPaintType LastPaintType;
	FPerformanceStat PerformanceStat;
#endif
#if UE_SLATE_DEBUGGING_CLEAR_ALL_FAST_PATH_DATA
	TArray<const SWidget*> FastWidgetPathToClearedBecauseOfDelay;
#endif
};
