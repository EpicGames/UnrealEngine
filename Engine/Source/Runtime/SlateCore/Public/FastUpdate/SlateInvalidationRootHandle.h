// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FSlateInvalidationRoot;

struct FSlateInvalidationRootHandle
{
public:
	FSlateInvalidationRootHandle();
	FSlateInvalidationRootHandle(int32 UniqueId);

	int32 GetUniqueId() const
	{
		return UniqueId;
	}

	/** @returns true if this used to point to a InvalidatationRoot, but doesn't any more and has not been assigned or reset in the mean time. */
	bool IsStale() const
	{
		return InvalidationRoot != nullptr && GetInvalidationRoot() == nullptr;
	}

	SLATECORE_API FSlateInvalidationRoot* GetInvalidationRoot() const;

	FSlateInvalidationRoot* Advanced_GetInvalidationRootNoCheck() const
	{
		return InvalidationRoot;
	}

	bool operator==(const FSlateInvalidationRootHandle& Other) const
	{
		return UniqueId == Other.UniqueId;
	}

private:
	FSlateInvalidationRoot* InvalidationRoot;
	int32 UniqueId;
};