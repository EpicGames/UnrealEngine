// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"

class SWidget;

/** Index of the special container to order widget in InvalidateRoot. See SlateInvalidationWidgetlist. */
struct FSlateInvalidationWidgetIndex
{
	friend class FSlateInvalidationWidgetList;
	friend struct FSlateInvalidationWidgetSortOrder;
	
public:
	static const FSlateInvalidationWidgetIndex Invalid;
	bool operator== (FSlateInvalidationWidgetIndex Other) const { return ArrayIndex == Other.ArrayIndex && ElementIndex == Other.ElementIndex; }
	bool operator!= (FSlateInvalidationWidgetIndex Other) const { return ArrayIndex != Other.ArrayIndex || ElementIndex != Other.ElementIndex; }
	FString ToString() const { return FString::Printf(TEXT("{%d, %d}"), ArrayIndex, ElementIndex); }
	
private:
	using IndexType = uint16;

	FSlateInvalidationWidgetIndex(IndexType InArray, IndexType InElement) : ArrayIndex(InArray), ElementIndex(InElement) {}

	/** Index of the array node in FSlateInvalidationWidgetList */
	IndexType ArrayIndex;
	/** Index of the element node in FSlateInvalidationWidgetList */
	IndexType ElementIndex;
};
