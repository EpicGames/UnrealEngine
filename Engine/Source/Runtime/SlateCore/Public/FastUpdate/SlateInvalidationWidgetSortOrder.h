// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FastUpdate/SlateInvalidationWidgetIndex.h"

class FSlateInvalidationWidgetList;

/**
 * SlateInvalidationWidgetIndex cannot be used to sort the widget since the ArrayIndex may not be in the expected order. (See the array as a double linked list).
 * SlateInvalidationWidgetSortOrder builds a unique number that represents the order of the widget.
 * The number is padded in a way to keep the order but not necessarily sequential.
 * It is valid until the next SlateInvalidationRoot::ProcessInvalidation()
 */
struct FSlateInvalidationWidgetSortOrder
{
private:
	uint32 Order = 0;
	FSlateInvalidationWidgetSortOrder(uint32 InOrder) : Order(InOrder) {}

public:
	FSlateInvalidationWidgetSortOrder() = default;
	FSlateInvalidationWidgetSortOrder(const FSlateInvalidationWidgetList& List, FSlateInvalidationWidgetIndex Index);

	static FSlateInvalidationWidgetSortOrder LimitMax();
	static FSlateInvalidationWidgetSortOrder LimitMin();

	bool operator< (const FSlateInvalidationWidgetSortOrder Other) const { return Order < Other.Order; }
	bool operator<= (const FSlateInvalidationWidgetSortOrder Other) const { return Order <= Other.Order; }
	bool operator> (const FSlateInvalidationWidgetSortOrder Other) const { return Order > Other.Order; }
	bool operator>= (const FSlateInvalidationWidgetSortOrder Other) const { return Order >= Other.Order; }
	bool operator== (const FSlateInvalidationWidgetSortOrder Other) const { return Order == Other.Order; }
	bool operator!= (const FSlateInvalidationWidgetSortOrder Other) const { return Order != Other.Order; }
};


/**
 * Pair of WidgetIndex and WidgetSortIndex. Can be used to sort the widget.
 */
struct FSlateInvalidationWidgetHeapElement
{
public:
	FSlateInvalidationWidgetHeapElement(FSlateInvalidationWidgetIndex InIndex, FSlateInvalidationWidgetSortOrder InSortOrder)
		: WidgetIndex(InIndex), WidgetSortOrder(InSortOrder)
	{
	}

	inline FSlateInvalidationWidgetIndex GetWidgetIndex() const
	{
		return WidgetIndex;
	}

	inline FSlateInvalidationWidgetSortOrder GetWidgetSortOrder() const
	{
		return WidgetSortOrder;
	}

private:
	FSlateInvalidationWidgetIndex WidgetIndex;
	FSlateInvalidationWidgetSortOrder WidgetSortOrder;
};