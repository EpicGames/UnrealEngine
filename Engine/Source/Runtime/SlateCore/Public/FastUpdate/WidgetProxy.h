// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/PaintArgs.h"
#include "Styling/WidgetStyle.h"
#include "Misc/MemStack.h"
#include "FastUpdate/SlateInvalidationRootHandle.h"
#include "FastUpdate/SlateInvalidationWidgetIndex.h"
#include "FastUpdate/SlateInvalidationWidgetSortOrder.h"
#include "FastUpdate/WidgetUpdateFlags.h"
#include "Layout/Clipping.h"
#include "Layout/FlowDirection.h"
#include "Rendering/DrawElements.h"
#include <type_traits>

class SWidget;
class FPaintArgs;
struct FFastPathPerFrameData;
class FSlateInvalidationWidgetPostHeap;
class FSlateInvalidationWidgetList;
struct FSlateWidgetPersistentState;
class FSlateInvalidationRoot;

enum class EInvalidateWidgetReason : uint8;

#define UE_SLATE_WITH_WIDGETPROXY_WEAKPTR 0
#define UE_SLATE_VERIFY_WIDGETPROXY_WEAKPTR_STALE 0
#define UE_SLATE_WITH_WIDGETPROXY_WIDGETTYPE 0
#define UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && WITH_SLATE_DEBUGGING


struct FSlateInvalidationWidgetVisibility
{
public:
	FSlateInvalidationWidgetVisibility()
		: Flags(0)
	{ }
	FSlateInvalidationWidgetVisibility(EVisibility InVisibility)
		: bAncestorsVisible(true)
		, bVisible(InVisibility.IsVisible())
		, bAncestorCollapse(false)
		, bCollapse(InVisibility == EVisibility::Collapsed)
		, FlagPadding(0)
	{ }
	FSlateInvalidationWidgetVisibility(FSlateInvalidationWidgetVisibility ParentFlags, EVisibility InVisibility)
		: bAncestorsVisible(ParentFlags.IsVisible())
		, bVisible(InVisibility.IsVisible())
		, bAncestorCollapse(ParentFlags.IsCollapsed())
		, bCollapse(InVisibility == EVisibility::Collapsed)
		, FlagPadding(0)
	{ }
	FSlateInvalidationWidgetVisibility(const FSlateInvalidationWidgetVisibility& Other) : Flags(Other.Flags) {  }
	FSlateInvalidationWidgetVisibility& operator=(const FSlateInvalidationWidgetVisibility& Other) { Flags = Other.Flags; return *this; }

	/** @returns true when all the widget ancestors are visible and the widget itself is visible. */
	bool IsVisible() const { return bAncestorsVisible && bVisible; }
	/** @returns true when all the widget ancestors are visible but the widget itself may not be visible. */
	bool AreAncestorsVisible() const { return bAncestorsVisible; }
	/** @returns true when the widget itself is visible (but may have invisible ancestors). */
	bool IsVisibleDirectly() const { return bVisible; }
	/** @returns true when at least one of the widget's ancestor is collapse or the widget itself is collapse. */
	bool IsCollapsed() const { return bAncestorCollapse || bCollapse; }
	/** @returns true when at least one of the widget's ancestor is collapse but the widget itself may not be collapse. */
	bool IsCollapseIndirectly() const { return bAncestorCollapse; }
	/** @returns true the widget itself is collapse. */
	bool IsCollapseDirectly() const { return bCollapse; }

	void SetVisibility(FSlateInvalidationWidgetVisibility ParentFlags, EVisibility InVisibility)
	{
		*this = FSlateInvalidationWidgetVisibility(ParentFlags, InVisibility);
	}

	void SetAncestorsVisibility(FSlateInvalidationWidgetVisibility ParentFlags)
	{
		bAncestorsVisible = ParentFlags.IsVisible();
		bAncestorCollapse = ParentFlags.IsCollapsed();
	}

	/** Assign the ancestors value to the widget values. Mimicking as it would be the parent. */
	FSlateInvalidationWidgetVisibility MimicAsParent() const
	{
		FSlateInvalidationWidgetVisibility Result;
		Result.bAncestorsVisible = bAncestorsVisible;
		Result.bVisible = bAncestorsVisible;
		Result.bCollapse = bAncestorCollapse;
		Result.bAncestorCollapse = bAncestorCollapse;
		return Result;
	}

	bool operator==(FSlateInvalidationWidgetVisibility Other) const { return Other.Flags == Flags; }
	bool operator!=(FSlateInvalidationWidgetVisibility Other) const { return Other.Flags != Flags; }

private:
	union
	{
		struct 
		{
			uint8 bAncestorsVisible : 1;	// all ancestors are visible
			uint8 bVisible : 1;
			uint8 bAncestorCollapse : 1;	// at least one ancestor is collapse
			uint8 bCollapse : 1;
			uint8 FlagPadding : 4;
		};
		uint8 Flags;
	};
};
static_assert(sizeof(FSlateInvalidationWidgetVisibility) == sizeof(uint8), "FSlateInvalidationWidgetVisibility should be size of uint8");


class FWidgetProxy
{
public:
	FWidgetProxy(SWidget& InWidget);

	struct FUpdateResult
	{
		FUpdateResult() = default;
		FUpdateResult(int32 InPreviousOutgoingLayerId, int32 InNewOutgoingLayerId)
			: PreviousOutgoingLayerId(InPreviousOutgoingLayerId)
			, NewOutgoingLayerId(InNewOutgoingLayerId)
			, bPainted(true)
		{
		}
		int32 PreviousOutgoingLayerId = INDEX_NONE;
		int32 NewOutgoingLayerId = INDEX_NONE;
		bool bPainted = false;
	};

	/**
	 * Similar to SWidget::Paint but use the saved PersistentState. Only paint/tick if needed.
	 * @return the new and previous LayerId is the widget was painted.
	 */
	FUpdateResult Update(const FPaintArgs& PaintArgs, FSlateWindowElementList& OutDrawElements);

	void ProcessLayoutInvalidation(FSlateInvalidationWidgetPostHeap& UpdateList, FSlateInvalidationWidgetList& FastPathWidgetList, FSlateInvalidationRoot& Root);
	bool ProcessPostInvalidation(FSlateInvalidationWidgetPostHeap& UpdateList, FSlateInvalidationWidgetList& FastPathWidgetList, FSlateInvalidationRoot& Root);

	void MarkProxyUpdatedThisFrame(FSlateInvalidationWidgetPostHeap& UpdateList);

#if UE_SLATE_WITH_WIDGETPROXY_WEAKPTR
	SWidget* GetWidget() const
	{
#if UE_SLATE_VERIFY_WIDGETPROXY_WEAKPTR_STALE
		ensureAlways(!Widget.IsStale()); // Widget.Object != nullptr && !Widget.WeakRerenceCount.IsValid()
#endif
		return Widget.Pin().Get();
	}
	TSharedPtr<SWidget> GetWidgetAsShared() const;
	void ResetWidget() { Widget.Reset(); }
	bool IsSameWidget(const SWidget* InWidget) const
	{
#if UE_SLATE_VERIFY_WIDGETPROXY_WEAKPTR_STALE
		return (InWidget == Widget.Pin().Get()) || (Widget.IsStale() && GetTypeHash(Widget) == GetTypeHash(InWidget));
#else
		return InWidget == Widget.Pin().Get();
#endif
	}
#else
	SWidget* GetWidget() const { return Widget; }
	// ForceNoInline to workaround PGO FastGen issue
	FORCENOINLINE TSharedPtr<SWidget> GetWidgetAsShared() const;
	void ResetWidget() { Widget = nullptr; }
	bool IsSameWidget(const SWidget* InWidget) const { return InWidget == Widget; }
#endif

private:
	FUpdateResult Repaint(const FPaintArgs& PaintArgs, FSlateWindowElementList& OutDrawElements) const;

private:
#if UE_SLATE_WITH_WIDGETPROXY_WEAKPTR
	TWeakPtr<SWidget> Widget;
#else
	SWidget* Widget;
#endif

#if UE_SLATE_WITH_WIDGETPROXY_WIDGETTYPE
	FName WidgetType;
#endif

public:
	FSlateInvalidationWidgetIndex Index;
	FSlateInvalidationWidgetIndex ParentIndex;
	FSlateInvalidationWidgetIndex LeafMostChildIndex;
	EInvalidateWidgetReason CurrentInvalidateReason;
	FSlateInvalidationWidgetVisibility Visibility;

	union
	{
		struct
		{
		public:
			/** Is the widget in the pre update list. */
			uint8 bContainedByWidgetPreHeap : 1;
			/** Is the widget in the post update list. */
			uint8 bContainedByWidgetPostHeap : 1;
			/** Is the widget in a pending prepass list. */
			uint8 bContainedByWidgetPrepassList : 1;
			/** Is the widget an Invalidation Root. Cached value of SWidget::Advanced_IsInvalidationRoot */
			uint8 bIsInvalidationRoot : 1;
			/** Is the widget has volatile prepass flag. */
			uint8 bIsVolatilePrepass : 1;
		};
		uint8 PrivateFlags;
	};

#if UE_SLATE_WITH_INVALIDATIONWIDGETLIST_DEBUGGING
	union
	{
		struct
		{
		public:
			/** Use with "Slate.InvalidationRoot.VerifyWidgetVisibility". Cached the last FastPathVisible value to find widgets that do not call Invalidate properly. */
			uint8 bDebug_LastFrameVisible : 1;
			uint8 bDebug_LastFrameVisibleSet : 1;
			/** Use with "Slate.InvalidationRoot.VerifyWidgetAttribute". */
			uint8 bDebug_AttributeUpdated : 1;
			/** The widget was updated (paint or ticked/activetimer). */
			uint8 bDebug_Updated : 1;
		};
		uint8 PrivateDebugFlags;
	};
#endif
};

#if !UE_SLATE_WITH_WIDGETPROXY_WIDGETTYPE
static_assert(sizeof(FWidgetProxy) <= 32, "FWidgetProxy should be 32 bytes");
#endif

#if !UE_SLATE_WITH_WIDGETPROXY_WEAKPTR
static_assert(std::is_trivially_destructible_v<FWidgetProxy>, "FWidgetProxy must be trivially destructible");
template <> struct TIsPODType<FWidgetProxy> { enum { Value = true }; };
#endif

/**
 * Represents the state of a widget from when it last had SWidget::Paint called on it. 
 * This should contain everything needed to directly call Paint on a widget
 */
struct FSlateWidgetPersistentState
{
	FSlateWidgetPersistentState()
		: CachedElementHandle()
		, LayerId(0)
		, OutgoingLayerId(0)
		, IncomingUserIndex(INDEX_NONE)
		, IncomingSceneIndex(INDEX_NONE)
		, IncomingFlowDirection(EFlowDirection::LeftToRight)
		, InitialPixelSnappingMethod(EWidgetPixelSnapping::Inherit)
		, bParentEnabled(true)
		, bInheritedHittestability(false)
		, bDeferredPainting(false)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, bIsInGameLayer(false)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{}

	TWeakPtr<SWidget> PaintParent;
	TOptional<FSlateClippingState> InitialClipState;
	FGeometry AllottedGeometry;
	FGeometry DesktopGeometry;
	FSlateRect CullingBounds;
	FWidgetStyle WidgetStyle;
	FSlateCachedElementsHandle CachedElementHandle;
	/** Starting layer id for drawing children **/
	int32 LayerId;
	int32 OutgoingLayerId;
	int8 IncomingUserIndex;
	int8 IncomingSceneIndex;
	EFlowDirection IncomingFlowDirection;
	EWidgetPixelSnapping InitialPixelSnappingMethod;
	uint8 bParentEnabled : 1;
	uint8 bInheritedHittestability : 1;
	uint8 bDeferredPainting : 1;

	UE_DEPRECATED(5.8, "Use methods instead of direct field access")
	uint8 bIsInGameLayer : 1;

	/** The value of SWidget::IsContent() */
	bool IsProjectContent() const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return bIsInGameLayer;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/** Only to be called by SWidget::Paint. */
	void SetIsProjectContent(bool bInIsContent)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bIsInGameLayer = bInIsContent;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	static const FSlateWidgetPersistentState NoState;
};

class FWidgetProxyHandle
{
	friend class SWidget;
	friend class FSlateInvalidationRoot;
	friend class FSlateInvalidationWidgetList;

public:
	FWidgetProxyHandle()
		: WidgetIndex(FSlateInvalidationWidgetIndex::Invalid)
	{}

	/** @returns true if it has a valid InvalidationRoot and Index. */
	SLATECORE_API bool IsValid(const SWidget& Widget) const;
	SLATECORE_API bool IsValid(const SWidget* Widget) const;

	FSlateInvalidationRootHandle GetInvalidationRootHandle() const { return InvalidationRootHandle; }

	FSlateInvalidationWidgetIndex GetWidgetIndex() const { return WidgetIndex; }
	FSlateInvalidationWidgetSortOrder GetWidgetSortOrder() const { return WidgetSortOrder; }

	SLATECORE_API FSlateInvalidationWidgetVisibility GetWidgetVisibility(const SWidget* Widget) const;
	SLATECORE_API bool HasAllInvalidationReason(const SWidget* Widget, EInvalidateWidgetReason Reason) const;
	SLATECORE_API bool HasAnyInvalidationReason(const SWidget* Widget, EInvalidateWidgetReason Reason) const;

	FWidgetProxy& GetProxy();
	const FWidgetProxy& GetProxy() const;

private:
	FSlateInvalidationRoot* GetInvalidationRoot_NoCheck() const { return InvalidationRootHandle.Advanced_GetInvalidationRootNoCheck(); }
	
	void MarkWidgetDirty_NoCheck(FWidgetProxy& Proxy);
	void MarkWidgetDirty_NoCheck(EInvalidateWidgetReason InvalidateReason);
	SLATECORE_API void UpdateWidgetFlags(const SWidget* Widget, EWidgetUpdateFlags Previous, EWidgetUpdateFlags NewFlags);

private:
	FWidgetProxyHandle(const FSlateInvalidationRootHandle& InInvalidationRoot, FSlateInvalidationWidgetIndex InIndex, FSlateInvalidationWidgetSortOrder InSortIndex);
	FWidgetProxyHandle(FSlateInvalidationWidgetIndex InIndex);

private:
	/** The root of invalidation tree this proxy belongs to. */
	FSlateInvalidationRootHandle InvalidationRootHandle;
	/** Index to myself in the fast path list. */
	FSlateInvalidationWidgetIndex WidgetIndex;
	/** Order of the widget in the fast path list. */
	FSlateInvalidationWidgetSortOrder WidgetSortOrder;
};
