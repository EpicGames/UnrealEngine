// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Widget update flags for fast path. */
enum class EWidgetUpdateFlags : uint8
{
	None = 0,

	/** Widget has a tick function */
	NeedsTick = 1 << 2,

	/** Widget has an active timer that needs to update */
	NeedsActiveTimerUpdate = 1 << 3,

	/** Needs repaint because the widget is dirty */
	NeedsRepaint = 1 << 4,

	/** Needs repaint because the widget is volatile */
	NeedsVolatilePaint = 1 << 6,

	/** Needs slate prepass because the widget is volatile */
	NeedsVolatilePrepass = 1 << 7,
	
	AnyUpdate = 0xff,
};

ENUM_CLASS_FLAGS(EWidgetUpdateFlags)
