// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Fonts/FontRasterizationMode.h"
#include "UObject/ObjectMacros.h"
#include "UObject/GCObject.h"
#include "UObject/Class.h"
#include "CompositeFont.generated.h"

class UFontBulkData;
struct FFontFaceData;

namespace UE
{
namespace Slate
{
class FPreprocessedFontGeometry;
}
}

UENUM()
enum class EFontHinting : uint8
{
	/** Use the default hinting specified in the font. */
	Default,
	/** Force the use of an automatic hinting algorithm. */
	Auto,
	/** Force the use of an automatic light hinting algorithm, optimized for non-monochrome displays. */
	AutoLight,
	/** Force the use of an automatic hinting algorithm optimized for monochrome displays. */
	Monochrome,
	/** Do not use hinting. */
	None,
};

UENUM()
enum class EFontLoadingPolicy : uint8
{
	/** Lazy load the entire font into memory. This will consume more memory than Streaming, however there will be zero file-IO when rendering glyphs within the font, although the initial load may cause a hitch. */
	LazyLoad,
	/** Stream the font from disk. This will consume less memory than LazyLoad or Inline, however there will be file-IO when rendering glyphs, which may cause hitches under certain circumstances or on certain platforms. */
	Stream,
	/** Embed the font data within the asset. This will consume more memory than Streaming, however it is guaranteed to be hitch free (only valid for font data within a Font Face asset). */
	Inline,
};

UENUM()
enum class EFontLayoutMethod : uint8
{
	/** Layout the font using the metrics data available in the font. This is typically the desired option, however some fonts have broken or incorrect metrics so may yield better results when using the bounding box values to layout the font. */
	Metrics,
	/** Layout the font using the values from its bounding box. This typically yields a larger line height for fonts that have valid metrics, however it can also produce much better results for fonts that have broken or incorrect metrics. */
	BoundingBox,
};

typedef TSharedPtr<FFontFaceData, ESPMode::ThreadSafe> FFontFaceDataPtr;
typedef TSharedRef<FFontFaceData, ESPMode::ThreadSafe> FFontFaceDataRef;
typedef TSharedPtr<const FFontFaceData, ESPMode::ThreadSafe> FFontFaceDataConstPtr;
typedef TSharedRef<const FFontFaceData, ESPMode::ThreadSafe> FFontFaceDataConstRef;

/** 
 * Raw font data for a Font Face asset.
 * @note Exists as a struct so it can be shared between its owner asset and the font cache without worrying about UObject lifetimes. 
 */
struct FFontFaceData
{
public:
	/** Default constructor */
	SLATECORE_API FFontFaceData();

	/** Construct from an existing block of font data */
	SLATECORE_API FFontFaceData(TArray<uint8>&& InData);

	/** Destructor */
	SLATECORE_API ~FFontFaceData();

	/** @return true if this instance has data set on it */
	bool HasData() const
	{
		return Data.Num() > 0;
	}

	/** Set from an existing block of font data */
	SLATECORE_API void SetData(TArray<uint8>&& InData);
	
	/** Set the preprocessed font geometry data per sub-face for the given font data */
	SLATECORE_API void SetPreprocessedFontGeometries(TArray<UE::Slate::FPreprocessedFontGeometry>&& InPreprocessedFontGeometries);

	/** @return The data set on this instance */
	const TArray<uint8>& GetData() const
	{
		return Data;
	}

	/** @return The preprocessed font geometry per sub-face of the font represented by Data */
	TArrayView<const UE::Slate::FPreprocessedFontGeometry> GetPreprocessedFontGeometries() const
	{
		return PreprocessedFontGeometries;
	}

	/** @return the memory size of the raw font data and the associated preprocessed data */
	SLATECORE_API SIZE_T GetAllocatedSize() const;

	/** Serialize the data on this instance to/from an archive */
	SLATECORE_API void Serialize(FArchive& Ar);

	/** Helper function to construct a default font face data */
	static FFontFaceDataRef MakeFontFaceData()
	{
		return MakeShared<FFontFaceData, ESPMode::ThreadSafe>();
	}

	/** Helper function to construct a font face data from an existing block of font data */
	static FFontFaceDataRef MakeFontFaceData(TArray<uint8>&& InData)
	{
		return MakeShared<FFontFaceData, ESPMode::ThreadSafe>(MoveTemp(InData));
	}

private:
	/** Memory stat tracking */
	SLATECORE_API void TrackMemoryUsage() const;
	SLATECORE_API void UntrackMemoryUsage() const;

	/** Internal data */
	TArray<uint8> Data;

	/** Preprocessed font geometry data per font sub-face */
	TArray<UE::Slate::FPreprocessedFontGeometry> PreprocessedFontGeometries;
};

/** Font face rasterization settings */
struct FFontRasterizationSettings
{
	/** Rasterization mode (bitmap / distance field) */
	EFontRasterizationMode Mode = EFontRasterizationMode::Bitmap;
	/** Distance field px/em resolution, not used if Mode is Bitmap */
	int32 DistanceFieldPpem = 0;
};

/** Payload data describing an individual font in a typeface. Keep this lean as it's also used as a key! */
USTRUCT()
struct FFontData
{
	GENERATED_USTRUCT_BODY()

	/** Default constructor */
	SLATECORE_API FFontData();

	/** Construct the raw data from a font face asset */
	SLATECORE_API explicit FFontData(const UObject* const InFontFaceAsset, const int32 InSubFaceIndex = 0);

	/** Construct the raw data from a filename and the font data attributes */
	SLATECORE_API FFontData(FString InFontFilename, const EFontHinting InHinting, const EFontLoadingPolicy InLoadingPolicy, const int32 InSubFaceIndex = 0);

	/** Is this font data set to a font? (either by filename or by inline data) */
	SLATECORE_API bool HasFont() const;

	/** Get the filename of the font to use. This may not actually exist on disk in editor builds and we should load the face buffer instead. */
	SLATECORE_API const FString& GetFontFilename() const;

	/** Get the hinting algorithm to use with the font. */
	SLATECORE_API EFontHinting GetHinting() const;

	/** Get the enum controlling how this font should be loaded at runtime. */
	SLATECORE_API EFontLoadingPolicy GetLoadingPolicy() const;

	/** Get the index of the sub-face that should be used. */
	SLATECORE_API int32 GetSubFaceIndex() const;

	/** Set the index of the sub-face that should be used. */
	SLATECORE_API void SetSubFaceIndex(const int32 InSubFaceIndex);

	/** Get the method to use when laying out the font? */
	SLATECORE_API EFontLayoutMethod GetLayoutMethod() const;

	/** Returns true if the ascend is overridden. */
	SLATECORE_API bool IsAscendOverridden() const;

	/** Returns the overridden value of the ascend. This value will be used only if IsAscendOverridden returns true. */
	SLATECORE_API int32 GetAscendOverriddenValue() const;

	/** Returns true if the descend is overridden. */
	SLATECORE_API bool IsDescendOverridden() const;

	/** Returns the overridden value of the descend. This value will be used only if IsDescendOverridden returns true. */
	SLATECORE_API int32 GetDescendOverriddenValue() const;

	/** Gets the percentage of the font height that's used as the height of the strike brush. */
	SLATECORE_API int32 GetStrikeBrushHeightPercentage() const;

	/** Get the data buffer containing the data for the current font face. */
	SLATECORE_API FFontFaceDataConstPtr GetFontFaceData() const;

	/** Get the font face asset used by this data (if any). */
	SLATECORE_API const UObject* GetFontFaceAsset() const;

	/** Get the font face rasterization settings. */
	SLATECORE_API FFontRasterizationSettings GetFontRasterizationSettings() const;

#if WITH_EDITORONLY_DATA
	/** True if this object contains any legacy data that needs to be upgraded PostLoad by calling the functions below (in order). */
	SLATECORE_API bool HasLegacyData() const;

	/** Upgrade v1 font data to v2 bulk data. */
	SLATECORE_API void ConditionalUpgradeFontDataToBulkData(UObject* InOuter);

	/** Upgrade v2 bulk data to v3 font face. */
	SLATECORE_API void ConditionalUpgradeBulkDataToFontFace(UObject* InOuter, UClass* InFontFaceClass, const FName InFontFaceName);
#endif // WITH_EDITORONLY_DATA

	/** Check to see whether this font data is equal to the other font data */
	SLATECORE_API bool operator==(const FFontData& Other) const;

	/** Check to see whether this font data is not equal to the other font data */
	SLATECORE_API bool operator!=(const FFontData& Other) const;

	/** Get the type hash for this font data */
	friend inline uint32 GetTypeHash(const FFontData& Key)
	{
		uint32 KeyHash = 0;

		if (Key.FontFaceAsset)
		{
			KeyHash = HashCombine(KeyHash, GetTypeHash(Key.FontFaceAsset));
		}
		else
		{
			KeyHash = HashCombine(KeyHash, Key.FontFilenameHash);
			KeyHash = HashCombine(KeyHash, GetTypeHash(Key.Hinting));
			KeyHash = HashCombine(KeyHash, GetTypeHash(Key.LoadingPolicy));
		}
		KeyHash = HashCombine(KeyHash, GetTypeHash(Key.SubFaceIndex));

		return KeyHash;
	}

	/** Handle serialization for this struct. */
	SLATECORE_API bool Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FFontData& InFontData)
	{
		InFontData.Serialize(Ar);
		return Ar;
	}

	/** Called by FStandaloneCompositeFont to prevent our objects from being GCd */
	SLATECORE_API void AddReferencedObjects(FReferenceCollector& Collector);

private:
	/**
	 * The filename of the font to use.
	 * This variable is ignored if we have a font face asset, and is set to the .ufont file in a cooked build.
	 */
	UPROPERTY()
	FString FontFilename;

	/**
	 * Cached hash value of FontFilename.
	 * Must be updated everytime FontFilename changes.
	 */
	uint32  FontFilenameHash;

	/**
	 * The hinting algorithm to use with the font.
	 * This variable is ignored if we have a font face asset, and is synchronized with the font face asset on load in a cooked build.
	 */
	UPROPERTY()
	EFontHinting Hinting;

	/**
	 * Enum controlling how this font should be loaded at runtime. See the enum for more explanations of the options.
	 * This variable is ignored if we have a font face asset, and is synchronized with the font face asset on load in a cooked build.
	 */
	UPROPERTY()
	EFontLoadingPolicy LoadingPolicy;

	/**
	 * The index of the sub-face that should be used.
	 * This is typically zero unless using a TTC/OTC font.
	 */
	UPROPERTY()
	int32 SubFaceIndex;

	/**
	 * Font data v3. This points to a font face asset.
	 */
	UPROPERTY()
	TObjectPtr<const UObject> FontFaceAsset;

#if WITH_EDITORONLY_DATA
	/**
	 * Legacy font data v2. This used to be where font data was stored prior to font face assets. 
	 * This can be removed once we no longer support loading packages older than FEditorObjectVersion::AddedFontFaceAssets (as can UFontBulkData itself).
	 */
	UPROPERTY()
	TObjectPtr<UFontBulkData> BulkDataPtr_DEPRECATED;

	/**
	 * Legacy font data v1. This used to be where font data was stored prior to font bulk data.
	 * This can be removed once we no longer support loading packages older than VER_UE4_SLATE_BULK_FONT_DATA.
	 */
	UPROPERTY()
	TArray<uint8> FontData_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};

template<>
struct TStructOpsTypeTraits<FFontData> : public TStructOpsTypeTraitsBase2<FFontData>
{
	enum
	{
		WithSerializer = true,
		WithIdenticalViaEquality = true,
	};
};

/** A single entry in a typeface */
USTRUCT()
struct FTypefaceEntry
{
	GENERATED_USTRUCT_BODY()

	/** Default constructor */
	FTypefaceEntry()
	{
	}

	/** Construct the entry from a name */
	FTypefaceEntry(const FName& InFontName)
		: Name(InFontName)
	{
	}

	/** Construct the entry from a filename and the font data attributes */
	FTypefaceEntry(const FName& InFontName, FString InFontFilename, const EFontHinting InHinting, const EFontLoadingPolicy InLoadingPolicy)
		: Name(InFontName)
		, Font(MoveTemp(InFontFilename), InHinting, InLoadingPolicy)
	{
	}

	/** Name used to identify this font within its typeface */
	UPROPERTY()
	FName Name;

	/** Raw font data for this font */
	UPROPERTY()
	FFontData Font;
};

/** Definition for a typeface (a family of fonts) */
USTRUCT()
struct FTypeface
{
	GENERATED_USTRUCT_BODY()

	/** Default constructor */
	FTypeface()
	{
	}

	/** Convenience constructor for when your font family only contains a single font */
	FTypeface(const FName& InFontName, FString InFontFilename, const EFontHinting InHinting, const EFontLoadingPolicy InLoadingPolicy)
	{
		Fonts.Emplace(InFontName, MoveTemp(InFontFilename), InHinting, InLoadingPolicy);
	}

	/** Append a new font into this family */
	FTypeface& AppendFont(const FName& InFontName, FString InFontFilename, const EFontHinting InHinting, const EFontLoadingPolicy InLoadingPolicy)
	{
		Fonts.Emplace(InFontName, MoveTemp(InFontFilename), InHinting, InLoadingPolicy);
		return *this;
	}

	/** The fonts contained within this family */
	UPROPERTY()
	TArray<FTypefaceEntry> Fonts;
};

USTRUCT()
struct FCompositeFallbackFont
{
	GENERATED_USTRUCT_BODY()

	/** Default constructor */
	FCompositeFallbackFont()
		: Typeface()
		, ScalingFactor(1.0f)
	{
	}

	/** Typeface data for this sub-font */
	UPROPERTY()
	FTypeface Typeface;

	/** Amount to scale this sub-font so that it better matches the size of the default font */
	UPROPERTY()
	float ScalingFactor;
};

USTRUCT()
struct FCompositeSubFont : public FCompositeFallbackFont
{
	GENERATED_USTRUCT_BODY()

	/** Default constructor */
	FCompositeSubFont()
	{
	}

	/** Array of character ranges for which this sub-font should be used */
	UPROPERTY()
	TArray<FInt32Range> CharacterRanges;

	/** Optional semi-colon separated list of cultures that this sub-font should be used with (if specified, this sub-font will be favored by those cultures and ignored by others) */
	UPROPERTY()
	FString Cultures;

#if WITH_EDITORONLY_DATA
	/** Name of this sub-font. Only used by the editor UI as a convenience to let you state the purpose of the font family. */
	UPROPERTY()
	FName EditorName;
#endif
};

USTRUCT()
struct FCompositeFont
{
	GENERATED_USTRUCT_BODY()

	/** Default constructor */
	FCompositeFont()
		: DefaultTypeface()
		, SubTypefaces()
#if WITH_EDITORONLY_DATA
		, HistoryRevision(0)
#endif	// WITH_EDITORONLY_DATA
	{
	}

	/** Convenience constructor for when your composite font only contains a single font */
	FCompositeFont(const FName& InFontName, FString InFontFilename, const EFontHinting InHinting, const EFontLoadingPolicy InLoadingPolicy)
		: DefaultTypeface(InFontName, MoveTemp(InFontFilename), InHinting, InLoadingPolicy)
		, SubTypefaces()
#if WITH_EDITORONLY_DATA
		, HistoryRevision(0)
#endif	// WITH_EDITORONLY_DATA
	{
	}

#if WITH_EDITORONLY_DATA
	/** Call this when the composite font is changed after its initial setup - this allows various caches to update as required */
	void MakeDirty()
	{
		++HistoryRevision;
	}
#endif	// WITH_EDITORONLY_DATA

	/** The default typeface that will be used when not overridden by a sub-typeface */
	UPROPERTY()
	FTypeface DefaultTypeface;

	/** The fallback typeface that will be used as a last resort when no other typeface provides a match */
	UPROPERTY()
	FCompositeFallbackFont FallbackTypeface;

	/** Sub-typefaces to use for a specific set of characters */
	UPROPERTY()
	TArray<FCompositeSubFont> SubTypefaces;

	/** If set to false, the ascent and descent override specified in a Font face will be ignored, and the value from the font source file will be used instead. */
	UPROPERTY(EditAnywhere, Category = RuntimeFont)
	bool bEnableAscentDescentOverride = true;

#if WITH_EDITORONLY_DATA
	/** 
	 * Transient value containing the current history ID of this composite font
	 * This should be updated when the composite font is changed (which should happen infrequently as composite fonts are assumed to be mostly immutable) once they've been setup
	 */
	int32 HistoryRevision;
#endif	// WITH_EDITORONLY_DATA

	/** Returns the enabled state of ascent and descent override. */
	bool IsAscentDescentOverrideEnabled() const
	{
		return bEnableAscentDescentOverride;
	}
};

/**
 * A version of FCompositeFont that should be used when it's not being embedded within another UObject
 * This derives from FGCObject to ensure that the bulk data objects are referenced correctly 
 */
struct FStandaloneCompositeFont : public FCompositeFont, public FGCObject
{
	/** Default constructor */
	FStandaloneCompositeFont()
	{
	}

	/** Convenience constructor for when your composite font only contains a single font */
	FStandaloneCompositeFont(const FName& InFontName, FString InFontFilename, const EFontHinting InHinting, const EFontLoadingPolicy InLoadingPolicy)
		: FCompositeFont(InFontName, MoveTemp(InFontFilename), InHinting, InLoadingPolicy)
	{
	}

	// FGCObject interface
	SLATECORE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	SLATECORE_API virtual FString GetReferencerName() const override;
};
