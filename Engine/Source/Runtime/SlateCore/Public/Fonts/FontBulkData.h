// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Serialization/BulkData.h"
#include "FontBulkData.generated.h"

UCLASS(MinimalAPI)
class UFontBulkData : public UObject
{
	GENERATED_BODY()

public:
	/** Default constructor */
	SLATECORE_API UFontBulkData();

	/** Construct the bulk font data from the given file */
	SLATECORE_API void Initialize(const FString& InFontFilename);

	/** Construct the bulk font data from the given data */
	SLATECORE_API void Initialize(const void* const InFontData, const int64 InFontDataSizeBytes);

	/** Locks the bulk font data and returns a read-only pointer to it */
	SLATECORE_API const void* Lock(int64& OutFontDataSizeBytes) const;

	/** Unlock the bulk font data, after which point the pointer returned by Lock no longer is valid */
	SLATECORE_API void Unlock();

	/** Returns the size of the bulk data in bytes */
	SLATECORE_API int64 GetBulkDataSize() const;

	// UObject interface
	SLATECORE_API virtual void Serialize(FArchive& Ar) override;

private:
	/** Internal bulk data */
	FByteBulkData BulkData;

	/** Critical section to prevent concurrent access when locking the internal bulk data */
	mutable FCriticalSection CriticalSection;
};
