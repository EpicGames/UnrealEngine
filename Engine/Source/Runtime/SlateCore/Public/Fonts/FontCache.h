// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateGlobals.h"
#include "Fonts/ShapedTextFwd.h"
#include "UObject/ObjectMacros.h"
#include "Fonts/SlateFontInfo.h"
#include "Fonts/FontRasterizationMode.h"
#include "Fonts/FontSdfSettings.h"
#include "Textures/TextureAtlas.h"
#include "Fonts/FontTypes.h"
#include "FontCache.generated.h"

class FCompositeFontCache;
class FFreeTypeAdvanceCache;
class FFreeTypeCacheDirectory;
class FFreeTypeFace;
class FFreeTypeGlyphCache;
class FFreeTypeKerningCache;
class FFreeTypeLibrary;
class FShapedGlyphFaceData;
class FSlateFontCache;
class FSlateFontRenderer;
class FSlateShaderResource;
class FSlateTextShaper;
class FSlateSdfGenerator;

/** Controls whether the signed distance field rendering mode feature for text is enabled. */
SLATECORE_API bool IsSlateSdfTextFeatureEnabled();

enum class EFontCacheAtlasDataType : uint8
{
	/** Data was cached for a regular non-outline font */
	Regular = 0,

	/** Data was cached for a outline (stroked) font */
	Outline,

	/** Must be last */
	Num,
};


/** 
 * Methods that can be used to shape text.
 * @note If you change this enum, make sure and update CVarDefaultTextShapingMethod and GetDefaultTextShapingMethod.
 */
UENUM(BlueprintType)
enum class ETextShapingMethod : uint8
{
	/**
	 * Automatically picks the fastest possible shaping method (either KerningOnly or FullShaping) based on the reading direction of the text.
	 * Left-to-right text uses the KerningOnly method, and right-to-left text uses the FullShaping method.
	 */
	 Auto = 0,

	/** 
	 * Provides fake shaping using only kerning data.
	 * This can be faster than full shaping, but won't render complex right-to-left or bi-directional glyphs (such as Arabic) correctly.
	 * This can be useful as an optimization when you know your text block will only show simple glyphs (such as numbers).
	 */
	KerningOnly,

	/**
	 * Provides full text shaping, allowing accurate rendering of complex right-to-left or bi-directional glyphs (such as Arabic).
	 * This mode will perform ligature replacement for all languages (such as the combined "fi" glyph in English).
	 */
	FullShaping,
};

/** Get the default shaping method (from the "Slate.DefaultTextShapingMethod" CVar) */
SLATECORE_API ETextShapingMethod GetDefaultTextShapingMethod();

/** The font atlas data for a single glyph in a shaped text sequence */
struct FShapedGlyphFontAtlasData
{
	/** The vertical distance from the baseline to the topmost border of the glyph bitmap */
	int16 VerticalOffset = 0;
	/** The horizontal distance from the origin to the leftmost border of the glyph bitmap */
	int16 HorizontalOffset = 0;
	/** Start X location of the glyph in the texture */
	uint16 StartU = 0;
	/** Start Y location of the glyph in the texture */
	uint16 StartV = 0;
	/** X Size of the glyph in the texture */
	uint16 USize = 0;
	/** Y Size of the glyph in the texture */
	uint16 VSize = 0;
	/** Index to a specific texture in the font cache. */
	uint8 TextureIndex = 0;
	/** True if this entry supports outline rendering, false otherwise. */
	bool SupportsOutline = false;
	/** True if this entry is valid, false otherwise. */
	bool Valid = false;
};

/** The font atlas data for a single distance field-based glyph */
struct FSdfGlyphFontAtlasData
{
	/** The vertical distance from the baseline to the topmost border of the glyph sdf raster */
	int16 VerticalOffset = 0;
	/** The horizontal distance from the origin to the leftmost border of the glyph sdf raster */
	int16 HorizontalOffset = 0;
	/** Start X location of the sdf raster in the texture */
	uint16 StartU = 0;
	/** Start Y location of the sdf raster in the texture */
	uint16 StartV = 0;
	/** Width of the sdf raster in the texture */
	uint16 USize = 0;
	/** Height of the sdf raster in the texture */
	uint16 VSize = 0;
	/** Outer portion of the spread of representable distances, in em */
	float EmOuterSpread = 0.f;
	/** Inner portion of the spread of representable distances, in em */
	float EmInnerSpread = 0.f;

	struct FMetrics
	{
		/** Horizontal distance from the current cursor to the leftmost border of the glyph image bounding box */
		float BearingX = 0.f;
		/** Horizontal distance from the current cursor y position (baseline) to the topmost border of the glyph image bounding box */
		float BearingY = 0.f;
		/** Width of the glyph image bounding box */
		float Width = 0.f;
		/** Height of the glyph image bounding box */
		float Height = 0.f;
	};
	FMetrics Metrics;
	/** Index to a specific texture in the font cache. */
	uint8 TextureIndex = 0;
	/** True if the glyph is available in the face but sdf generation was not possible/successful */
	bool bSupportsSdf = false;
	/** True if the SDF is a placeholder and the task for its final version hasn't been spawned yet (due to too many running tasks) */
	bool bPendingRespawn = false;
	/** True if this entry is valid, false otherwise. */
	bool Valid = false;
};

/** Information for rendering one glyph in a shaped text sequence */
struct FShapedGlyphEntry
{
	friend class FSlateFontCache;

	/** Provides access to the FreeType face for this glyph (not available publicly) */
	TSharedPtr<FShapedGlyphFaceData> FontFaceData;
	/** The index of this glyph in the FreeType face */
	uint32 GlyphIndex = 0;
	/** The index of this glyph from the source text. The source indices may skip characters if the sequence contains ligatures, additionally, some characters produce multiple glyphs leading to duplicate source indices */
	int32 SourceIndex = 0;
	/** The amount to advance in X before drawing the next glyph in the sequence */
	int16 XAdvance = 0;
	/** The amount to advance in Y before drawing the next glyph in the sequence */
	int16 YAdvance = 0;
	/** The offset to apply in X when drawing this glyph */
	int16 XOffset = 0;
	/** The offset to apply in Y when drawing this glyph */
	int16 YOffset = 0;
	/** 
	 * The "kerning" between this glyph and the next one in the sequence
	 * @note This value is included in the XAdvance so you never usually need it unless you're manually combining two sets of glyphs together.
	 * @note This value isn't strictly the kerning value - it's simply the difference between the glyphs horizontal advance, and the shaped horizontal advance (so will contain any accumulated advance added by the shaper)
	 */
	int8 Kerning = 0;
	/**
	 * The number of source characters represented by this glyph
	 * This is typically 1, however will be greater for ligatures, or may be 0 if a single character produces multiple glyphs
	 */
	uint8 NumCharactersInGlyph = 0;
	/**
	 * The number of source grapheme clusters represented by this glyph
	 * This is typically 1, however will be greater for ligatures, or may be 0 if a single character produces multiple glyphs
	 */
	uint8 NumGraphemeClustersInGlyph = 0;
	/**
	 * The reading direction of the text this glyph was shaped from
	 */
	TextBiDi::ETextDirection TextDirection = TextBiDi::ETextDirection::LeftToRight;
	/**
	 * True if this is a visible glyph that should be drawn.
	 * False if the glyph is invisible (eg, whitespace or a control code) and should skip drawing, but still include its advance amount.
	 */
	bool bIsVisible = false;
	
	/** Check whether this entry contains a valid glyph (non-zero, and not the SlateFontRendererUtils::InvalidSubChar glyph) */
	SLATECORE_API bool HasValidGlyph() const;

	/** Get any additional scale that should be applied when rendering this glyph */
	SLATECORE_API float GetBitmapRenderScale() const;

private:
	/** 
	 * Pointer to the cached atlas data for this glyph entry.
	 * This is cached on the glyph by FSlateFontCache::GetShapedGlyphFontAtlasData to avoid repeated map look ups.
	 * First index is to determine if this is a cached outline glyph or a regular glyph
	 * Second index is the index of the thread dependent font cache. Index 0 is the cached value for the game thread font cache. Index 1 is the cached value for the render thread font cache.
	 */
	mutable TWeakPtr<FShapedGlyphFontAtlasData> CachedAtlasData[(uint8)EFontCacheAtlasDataType::Num][2];
	mutable TWeakPtr<FSdfGlyphFontAtlasData> CachedSdfFontAtlasData[2];
	mutable FSdfGlyphFontAtlasData::FMetrics CachedSdfMetrics[2];
};

/** Minimal FShapedGlyphEntry key information used for map lookups */
struct FShapedGlyphEntryKey
{
	friend struct FFontCacheStatsKey;
	friend class FSlateFontCache;

public:
	FShapedGlyphEntryKey(const FShapedGlyphFaceData& InFontFaceData, uint32 InGlyphIndex, const FFontOutlineSettings& InOutlineSettings);

	inline bool operator==(const FShapedGlyphEntryKey& Other) const
	{
		return FontFace == Other.FontFace 
			&& FontRenderSize == Other.FontRenderSize
			&& OutlineRenderSize == Other.OutlineRenderSize
			&& OutlineMiteredCorners == Other.OutlineMiteredCorners
			&& OutlineSeparateFillAlpha == Other.OutlineSeparateFillAlpha
			&& GlyphIndex == Other.GlyphIndex
			&& FontSkew == Other.FontSkew;
	}

	inline bool operator!=(const FShapedGlyphEntryKey& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FShapedGlyphEntryKey& Key)
	{
		return Key.KeyHash;
	}

private:
	/** Weak pointer to the FreeType face to render with */
	TWeakPtr<FFreeTypeFace> FontFace;
	/** The size in pixel that Freetype will actually use to render the font, scale included */
	uint32 FontRenderSize;
	/** The size in pixels of the outline to render, scale included */
	int32 OutlineRenderSize;
	/** If checked, the outline will have mitered corners, otherwise they will be rounded. @see FFontOutlineSettings */
	bool OutlineMiteredCorners;
	/** If checked, the outline will be completely translucent where the filled area will be. @see FFontOutlineSettings */
	bool OutlineSeparateFillAlpha;
	/** The index of this glyph in the FreeType face */
	uint32 GlyphIndex;
	/** Cached hash value used for map lookups */
	uint32 KeyHash;
	/** The skew transform amount for the rendered font */
	float FontSkew;
};

/** Minimal FSdfGlyphEntryKey key information used for map lookups */
struct FSdfGlyphEntryKey
{
public:
	FSdfGlyphEntryKey(const TWeakPtr<FFreeTypeFace> InFontFace, uint32 InGlyphIndex, ESlateFontAtlasContentType InAtlasContentType, int32 InPpem, float InEmOuterSpread, float InEmInnerSpread);

	inline bool operator==(const FSdfGlyphEntryKey& Other) const
	{
		return FontFace == Other.FontFace
			&& GlyphIndex == Other.GlyphIndex
			&& AtlasContentType == Other.AtlasContentType
			&& Ppem == Other.Ppem
			&& SpreadCategory == Other.SpreadCategory;
	}

	inline bool operator!=(const FSdfGlyphEntryKey& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FSdfGlyphEntryKey& Key)
	{
		return Key.KeyHash;
	}

	/** Weak pointer to the FreeType face to render with */
	const TWeakPtr<FFreeTypeFace> FontFace;
	/** The index of this glyph in the FreeType face */
	const uint32 GlyphIndex;
	/** Type of glyph pixel content */
	const ESlateFontAtlasContentType AtlasContentType;
	/** The pixel size at which the sdf glyph is generated */
	const int32 Ppem;
	/** The spread category. The spreads of a glyph entry can be arbitrary but similar values will share the same category and therefore glyph entry */
	const int32 SpreadCategory;

private:
	/** Cached hash value used for map lookups */
	uint32 KeyHash;

	/** Computes the discrete spread category from real spread values specified in em */
	static int32 GetSpreadCategory(float InEmOuterSpread, float InEmInnerSpread);
};

/** Used to lookup information about specific SDF generation tasks - unlike FSdfGlyphEntryKey is also identified by em spread */
struct FSdfGlyphTaskKey
{
public:
	inline FSdfGlyphTaskKey(const FSdfGlyphEntryKey& InSdfGlyphEntryKey, float InEmOuterSpread, float InEmInnerSpread);

	inline bool operator==(const FSdfGlyphTaskKey& Other) const
	{
		return SdfGlyphEntryKey == Other.SdfGlyphEntryKey
			&& EmOuterSpread == Other.EmOuterSpread
			&& EmInnerSpread == Other.EmInnerSpread;
	}

	inline bool operator!=(const FSdfGlyphTaskKey& Other) const
	{
		return !(*this == Other);
	}

	friend inline uint32 GetTypeHash(const FSdfGlyphTaskKey& Key)
	{
		return Key.KeyHash;
	}

	/** Glyph entry key */
	const FSdfGlyphEntryKey SdfGlyphEntryKey;
	/** Outer portion of distance field spread in em */
	float EmOuterSpread;
	/** Inner portion of distance field spread in em */
	float EmInnerSpread;

private:
	/** Cached hash value used for map lookups */
	uint32 KeyHash;
};

/** Information for rendering a shaped text sequence */
class FShapedGlyphSequence
{
public:
	struct FSourceTextRange
	{
		FSourceTextRange(const int32 InTextStart, const int32 InTextLen)
			: TextStart(InTextStart)
			, TextLen(InTextLen)
		{
		}

		int32 TextStart;
		int32 TextLen;
	};

	explicit FShapedGlyphSequence()
		: GlyphsToRender()
		, TextBaseline(0)
		, MaxTextHeight(0)
		, bMaterialIsStencil(false)
		, FontMaterial(nullptr)
		, OutlineSettings()
		, SequenceWidth(0)
		, GlyphFontFaces()
		, SourceIndicesToGlyphData(FSourceTextRange(0, 0))
		, CachedFontSkew(0.f)
	{ }

	SLATECORE_API FShapedGlyphSequence(TArray<FShapedGlyphEntry> InGlyphsToRender,
						 const int16 InTextBaseline,
						 const uint16 InMaxTextHeight,
						 const UObject* InFontMaterial,
						 const bool InMaterialIsStencil,
						 const FFontOutlineSettings& InOutlineSettings,
						 const FSourceTextRange& InSourceTextRange);
	SLATECORE_API ~FShapedGlyphSequence();

	/** Get the amount of memory allocated to this sequence */
	SLATECORE_API SIZE_T GetAllocatedSize() const;

	/** Get the array of glyphs in this sequence. This data will be ordered so that you can iterate and draw left-to-right, which means it will be backwards for right-to-left languages */
	const TArray<FShapedGlyphEntry>& GetGlyphsToRender() const
	{
		return GlyphsToRender;
	}

	/** Get the baseline to use when drawing the glyphs in this sequence */
	int16 GetTextBaseline() const
	{
		return TextBaseline;
	}

	/** Get the maximum height of any glyph in the font we're using */
	uint16 GetMaxTextHeight() const
	{
		return MaxTextHeight;
	}

	/** Get the material to use when rendering these glyphs */
	const UObject* GetFontMaterial() const
	{
		return FontMaterial;
	}

	/** Get "Material is stencil" flag */
	bool IsMaterialStencil() const
	{
		return bMaterialIsStencil;
	}

	/** Get the font outline settings to use when rendering these glyphs */
	const FFontOutlineSettings& GetFontOutlineSettings() const
	{
		return OutlineSettings;
	}

	/** Check to see whether this glyph sequence is dirty (ie, contains glyphs with invalid font pointers) */
	SLATECORE_API bool IsDirty() const;

	/** Get the font skew applied when this glyph sequence was shaped */
	float GetFontSkew() const;

	/**
	 * Get the measured width of the entire shaped text
	 * @return The measured width
	 */
	SLATECORE_API int32 GetMeasuredWidth() const;

	/**
	 * Get the measured width of the specified range of this shaped text
	 * @note The indices used here are relative to the start of the text we were shaped from, even if we were only shaped from a sub-section of that text
	 * @return The measured width, or an unset value if the text couldn't be measured (eg, because you started or ended on a merged ligature, or because the range is out-of-bounds)
	 */
	SLATECORE_API TOptional<int32> GetMeasuredWidth(const int32 InStartIndex, const int32 InEndIndex, const bool InIncludeKerningWithPrecedingGlyph = true) const;

	/** Return data used by GetGlyphAtOffset */
	struct FGlyphOffsetResult
	{
		FGlyphOffsetResult()
			: Glyph(nullptr)
			, GlyphOffset(0)
			, CharacterIndex(0)
		{
		}

		explicit FGlyphOffsetResult(const int32 InCharacterIndex)
			: Glyph(nullptr)
			, GlyphOffset(0)
			, CharacterIndex(InCharacterIndex)
		{
		}

		FGlyphOffsetResult(const FShapedGlyphEntry* InGlyph, const int32 InGlyphOffset)
			: Glyph(InGlyph)
			, GlyphOffset(InGlyphOffset)
			, CharacterIndex(InGlyph->SourceIndex)
		{
		}

		/** The glyph that was hit. May be null if we hit outside the range of any glyph */
		const FShapedGlyphEntry* Glyph;
		/** The offset to the left edge of the hit glyph */
		int32 GlyphOffset;
		/** The character index that was hit (set to the start or end index if we fail to hit a glyph) */
		int32 CharacterIndex;
	};

	/**
	 * Get the information for the glyph at the specified position in pixels along the string horizontally
	 * @return The result data (see FGlyphOffsetResult)
	 */
	SLATECORE_API FGlyphOffsetResult GetGlyphAtOffset(FSlateFontCache& InFontCache, const int32 InHorizontalOffset, const int32 InStartOffset = 0) const;

	/**
	 * Get the information for the glyph at the specified position in pixels along the string horizontally
	 * @note The indices used here are relative to the start of the text we were shaped from, even if we were only shaped from a sub-section of that text
	 * @return The result data (see FGlyphOffsetResult), or an unset value if we couldn't find the character (eg, because you started or ended on a merged ligature, or because the range is out-of-bounds)
	 */
	SLATECORE_API TOptional<FGlyphOffsetResult> GetGlyphAtOffset(FSlateFontCache& InFontCache, const int32 InStartIndex, const int32 InEndIndex, const int32 InHorizontalOffset, const int32 InStartOffset = 0, const bool InIncludeKerningWithPrecedingGlyph = true) const;

	/**
	 * Get the kerning value between the given entry and the next entry in the sequence
	 * @note The index used here is relative to the start of the text we were shaped from, even if we were only shaped from a sub-section of that text
	 * @return The kerning, or an unset value if we couldn't get the kerning (eg, because you specified a merged ligature, or because the index is out-of-bounds)
	 */
	SLATECORE_API TOptional<int8> GetKerning(const int32 InIndex) const;

	/**
	 * Get a sub-sequence of the specified range
	 * @note The indices used here are relative to the start of the text we were shaped from, even if we were only shaped from a sub-section of that text
	 * @return The sub-sequence, or an null if the sub-sequence couldn't be created (eg, because you started or ended on a merged ligature, or because the range is out-of-bounds)
	 */
	SLATECORE_API FShapedGlyphSequencePtr GetSubSequence(const int32 InStartIndex, const int32 InEndIndex) const;

	SLATECORE_API void AddReferencedObjects(FReferenceCollector& Collector);

private:
	/** Non-copyable */
	SLATECORE_API FShapedGlyphSequence(const FShapedGlyphSequence&);
	SLATECORE_API FShapedGlyphSequence& operator=(const FShapedGlyphSequence&);

	/** Helper function to share some common logic between the bound and unbound GetGlyphAtOffset functions */
	SLATECORE_API bool HasFoundGlyphAtOffset(FSlateFontCache& InFontCache, const int32 InHorizontalOffset, const FShapedGlyphEntry& InCurrentGlyph, const int32 InCurrentGlyphIndex, int32& InOutCurrentOffset, const FShapedGlyphEntry*& OutMatchedGlyph) const;

	/**
	 * Enumerate all of the glyphs within the given source index range (enumerates either visually or logically)
	 * @note The indices used here are relative to the start of the text we were shaped from, even if we were only shaped from a sub-section of that text
	 * @return EnumerationComplete if we found the start and end point and enumerated the glyphs, EnumerationAborted if the callback returned false, or EnumerationFailed (eg, because you started or ended on a merged ligature, or because the range is out-of-bounds)
	 */
	enum class EEnumerateGlyphsResult : uint8 { EnumerationFailed, EnumerationAborted, EnumerationComplete };
	typedef TFunctionRef<bool(const FShapedGlyphEntry&, int32)> FForEachShapedGlyphEntryCallback;
	SLATECORE_API EEnumerateGlyphsResult EnumerateLogicalGlyphsInSourceRange(const int32 InStartIndex, const int32 InEndIndex, const FForEachShapedGlyphEntryCallback& InGlyphCallback) const;
	SLATECORE_API EEnumerateGlyphsResult EnumerateVisualGlyphsInSourceRange(const int32 InStartIndex, const int32 InEndIndex, const FForEachShapedGlyphEntryCallback& InGlyphCallback) const;

	/** Contains the information needed when performing a reverse look-up from a source index to the corresponding shaped glyph */
	struct FSourceIndexToGlyphData
	{
		FSourceIndexToGlyphData()
			: GlyphIndex(INDEX_NONE)
			, AdditionalGlyphIndices()
		{
		}

		explicit FSourceIndexToGlyphData(const int32 InGlyphIndex)
			: GlyphIndex(InGlyphIndex)
			, AdditionalGlyphIndices()
		{
		}

		bool IsValid() const
		{
			return GlyphIndex != INDEX_NONE;
		}

		int32 GetLowestGlyphIndex() const
		{
			return GlyphIndex;
		};

		int32 GetHighestGlyphIndex() const
		{
			return (AdditionalGlyphIndices.Num() > 0) ? AdditionalGlyphIndices.Last() : GlyphIndex;
		}

		int32 GlyphIndex;
		TArray<int32> AdditionalGlyphIndices;
	};

	/** A map of source indices to their shaped glyph data indices. Stored internally as an array so we can perform a single allocation */
	struct FSourceIndicesToGlyphData
	{
	public:
		explicit FSourceIndicesToGlyphData(const FSourceTextRange& InSourceTextRange)
			: SourceTextRange(InSourceTextRange)
			, GlyphDataArray()
		{
			GlyphDataArray.SetNum(InSourceTextRange.TextLen);
		}

		inline int32 GetSourceTextStartIndex() const
		{
			return SourceTextRange.TextStart;
		}

		inline int32 GetSourceTextEndIndex() const
		{
			return SourceTextRange.TextStart + SourceTextRange.TextLen;
		}

		inline FSourceIndexToGlyphData* GetGlyphData(const int32 InSourceTextIndex)
		{
			const int32 InternalIndex = InSourceTextIndex - SourceTextRange.TextStart;
			return (GlyphDataArray.IsValidIndex(InternalIndex)) ? &GlyphDataArray[InternalIndex] : nullptr;
		}

		inline const FSourceIndexToGlyphData* GetGlyphData(const int32 InSourceTextIndex) const
		{
			const int32 InternalIndex = InSourceTextIndex - SourceTextRange.TextStart;
			return (GlyphDataArray.IsValidIndex(InternalIndex)) ? &GlyphDataArray[InternalIndex] : nullptr;
		}

		inline SIZE_T GetAllocatedSize() const
		{
			return GlyphDataArray.GetAllocatedSize();
		}

	private:
		FSourceTextRange SourceTextRange;
		TArray<FSourceIndexToGlyphData> GlyphDataArray;
	};

	/** Array of glyphs in this sequence. This data will be ordered so that you can iterate and draw left-to-right, which means it will be backwards for right-to-left languages */
	TArray<FShapedGlyphEntry> GlyphsToRender;
	/** The baseline to use when drawing the glyphs in this sequence */
	int16 TextBaseline;
	/** The maximum height of any glyph in the font we're using */
	uint16 MaxTextHeight;
	/** Toggles between stenciling text onto material output (false) or filling entire quads with material (true) */
	bool bMaterialIsStencil;
	/** The material to use when rendering these glyphs */
	TObjectPtr<const UObject> FontMaterial;
	/** Outline settings to use when rendering these glyphs */
	FFontOutlineSettings OutlineSettings;
	/** The cached width of the entire sequence */
	int32 SequenceWidth;
	/** The set of fonts being used by the glyphs within this sequence */
	TArray<TWeakPtr<FFreeTypeFace>> GlyphFontFaces;

	/** A map of source indices to their shaped glyph data indices - used to perform efficient reverse look-up */
	FSourceIndicesToGlyphData SourceIndicesToGlyphData;
	/** The Font Skew parameter of the FontInfo applied to the this Shaped glype sequence */
	float CachedFontSkew;

#if SLATE_CHECK_UOBJECT_SHAPED_GLYPH_SEQUENCE
	// Used to guard against crashes when the material object is deleted. This is expensive so we do not do it in shipping
	TWeakObjectPtr<const UObject> FontMaterialWeakPtr;
	TWeakObjectPtr<const UObject> SdfFontMaterialOverrideWeakPtr;
	TWeakObjectPtr<const UObject> OutlineMaterialWeakPtr;
	FName DebugFontMaterialName;
	FName DebugSdfFontMaterialOverrideName;
	FName DebugOutlineMaterialName;
#endif
};

/** Information for rendering one non-shaped character */
struct FCharacterEntry
{
	/** The character this entry is for */
	TCHAR Character = 0;
	/** The index of the glyph from the FreeType face that this entry is for */
	uint32 GlyphIndex = 0;
	/** The raw font data this character was rendered with */
	const FFontData* FontData = nullptr;
	/** The kerning cache that this entry uses */
	TSharedPtr<FFreeTypeKerningCache> KerningCache;
	/** Scale that was applied when rendering this character */
	float FontScale = 0.0f;
	/** Any additional scale that should be applied when rendering this glyph */
	float BitmapRenderScale = 0.0f;
	/** Start X location of the character in the texture */
	uint16 StartU = 0;
	/** Start Y location of the character in the texture */
	uint16 StartV = 0;
	/** X Size of the character in the texture */
	uint16 USize = 0;
	/** Y Size of the character in the texture */
	uint16 VSize = 0;
	/** The vertical distance from the baseline to the topmost border of the character */
	int16 VerticalOffset = 0;
	/** The horizontal distance from the origin to the left most border of the character */
	int16 HorizontalOffset = 0;
	/** The largest vertical distance below the baseline for any character in the font */
	int16 GlobalDescender = 0;
	/** The amount to advance in X before drawing the next character in a string */
	int16 XAdvance = 0;
	/** Index to a specific texture in the font cache. */
	uint8 TextureIndex = 0;
	/** The fallback level this character represents */
	EFontFallback FallbackLevel = EFontFallback::FF_Max;
	/** 1 if this entry has kerning, 0 otherwise. */
	bool HasKerning = false;
	/** 1 if this entry supports outline rendering, 0 otherwise. */
	bool SupportsOutline = false;
	/** 1 if this entry is valid, 0 otherwise. */
	bool Valid = false;
};

/**
 * Manages a potentially large list of non-shaped characters
 * Uses a directly indexed by TCHAR array until space runs out and then maps the rest to conserve memory
 * Every character indexed by TCHAR could potentially cost a lot of memory of a lot of empty entries are created
 * because characters being used are far apart
 */
class FCharacterList
{
public:
	SLATECORE_API FCharacterList( const FSlateFontKey& InFontKey, FSlateFontCache& InFontCache );

	/**
	 * Gets data about how to render and measure a character.
	 * Caching and atlasing it if needed.
	 * Subsequent calls may invalidate previous pointers.
	 *
	 * @param Character			The character to get
	 * @param MaxFontFallback	The maximum fallback level that can be used when resolving glyphs
	 * @return				Data about the character
	 */
	SLATECORE_API const FCharacterEntry& GetCharacter(TCHAR Character, const EFontFallback MaxFontFallback);

#if WITH_EDITORONLY_DATA
	/** Check to see if our cached data is potentially stale for our font */
	SLATECORE_API bool IsStale() const;
#endif	// WITH_EDITORONLY_DATA

	/**
	 * Gets a kerning value for a pair of characters
	 *
	 * @param FirstChar			The first character in the pair
	 * @param SecondChar		The second character in the pair
	 * @param MaxFontFallback	The maximum fallback level that can be used when resolving glyphs
	 * @return The kerning value
	 */
	SLATECORE_API int8 GetKerning(TCHAR FirstChar, TCHAR SecondChar, const EFontFallback MaxFontFallback);

	/**
	 * Gets a kerning value for a pair of character entries
	 *
	 * @param FirstCharacterEntry	The first character entry in the pair
	 * @param SecondCharacterEntry	The second character entry in the pair
	 * @return The kerning value
	 */
	SLATECORE_API int8 GetKerning( const FCharacterEntry& FirstCharacterEntry, const FCharacterEntry& SecondCharacterEntry );

	/**
	 * @return The global max height for any character in this font
	 */
	SLATECORE_API uint16 GetMaxHeight() const;

	/** 
	 * Returns the baseline for the font used by this character 
	 *
	 * @return The offset from the bottom of the max character height to the baseline. Be aware that the value will be negative.
	 */
	SLATECORE_API int16 GetBaseline() const;

private:
	/**
	 * Returns whether the specified character is valid for caching (i.e. whether it matches the FontFallback level)
	 *
	 * @param Character			The character to check
	 * @param MaxFontFallback	The maximum fallback level that can be used when resolving glyphs
	 */
	bool CanCacheCharacter(TCHAR Character, const EFontFallback MaxFontFallback) const;

	/**
	 * Caches a new character
	 * 
	 * @param Character	The character to cache
	 */
	const FCharacterEntry* CacheCharacter(TCHAR Character);

private:
	/** Entries for larger character sets to conserve memory */
	TMap<TCHAR, FCharacterEntry> MappedEntries;

	/** Font for this character list */
	FSlateFontKey FontKey;
	/** Reference to the font cache for accessing new unseen characters */
	FSlateFontCache& FontCache;
#if WITH_EDITORONLY_DATA
	/** The history revision of the cached composite font */
	int32 CompositeFontHistoryRevision;
#endif	// WITH_EDITORONLY_DATA
	/** The global max height for any character in this font */
	mutable uint16 MaxHeight;
	/** The offset from the bottom of the max character height to the baseline. */
	mutable int16 Baseline;
};

/**
 * Font caching implementation
 * Caches characters into textures as needed
 */
class FSlateFontCache : public ISlateAtlasProvider, public FSlateFlushableAtlasCache
{
	friend FCharacterList;

public:
	/**
	 * Constructor
	 *
	 * @param InTextureSize The size of the atlas texture
	 * @param InFontAlas	Platform specific font atlas resource
	 */
	SLATECORE_API FSlateFontCache( TSharedRef<ISlateFontAtlasFactory> InFontAtlasFactory, ESlateTextureAtlasThreadId InOwningThread);
	SLATECORE_API virtual ~FSlateFontCache();

	/** ISlateAtlasProvider */
	SLATECORE_API virtual int32 GetNumAtlasPages() const override;
	SLATECORE_API virtual FSlateShaderResource* GetAtlasPageResource(const int32 InIndex) const override;
	SLATECORE_API virtual bool IsAtlasPageResourceAlphaOnly(const int32 InIndex) const override;
#if WITH_ATLAS_DEBUGGING
	virtual FAtlasSlotInfo GetAtlasSlotInfoAtPosition(FIntPoint InPosition, int32 AtlasIndex) const override;
#endif
	/** 
	 * Performs text shaping on the given string using the given font info. Returns you the shaped text sequence to use for text rendering via FSlateDrawElement::MakeShapedText.
	 * When using the version which takes a start point and length, the text outside of the given range won't be shaped, but will provide context information to allow the shaping to function correctly.
	 * ShapeBidirectionalText is used when you have text that may contain a mixture of LTR and RTL text runs.
	 * 
	 * @param InText				The string to shape
	 * @param InTextStart			The start position of the text to shape
	 * @param InTextLen				The length of the text to shape
	 * @param InFontInfo			Information about the font that the string is drawn with
	 * @param InFontScale			The scale to apply to the font
	 * @param InBaseDirection		The overall reading direction of the text (see TextBiDi::ComputeBaseDirection). This will affect where some characters (such as brackets and quotes) are placed within the resultant shaped text
	 * @param InTextShapingMethod	The text shaping method to use
	 * @param OutLoadingFontFaces	An array (if provided) that keeps track of the unique font faces that are loading for a text during shaping.
	 */
	SLATECORE_API FShapedGlyphSequenceRef ShapeBidirectionalText( const FString& InText, const FSlateFontInfo &InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InBaseDirection, const ETextShapingMethod InTextShapingMethod , TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces = nullptr) const;
	SLATECORE_API FShapedGlyphSequenceRef ShapeBidirectionalText( const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo &InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InBaseDirection, const ETextShapingMethod InTextShapingMethod , TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces = nullptr) const;

	/** 
	 * Performs text shaping on the given range of the string using the given font info. Returns you the shaped text sequence to use for text rendering via FSlateDrawElement::MakeShapedText.
	 * When using the version which takes a start point and length, the text outside of the given range won't be shaped, but will provide context information to allow the shaping to function correctly.
	 * ShapeUnidirectionalText is used when you have text that all reads in the same direction (either LTR or RTL).
	 * 
	 * @param InText				The string containing the sub-string to shape
	 * @param InTextStart			The start position of the text to shape
	 * @param InTextLen				The length of the text to shape
	 * @param InFontInfo			Information about the font that the string is drawn with
	 * @param InFontScale			The scale to apply to the font
	 * @param InTextDirection		The reading direction of the text to shape (valid values are LeftToRight or RightToLeft)
	 * @param InTextShapingMethod	The text shaping method to use
	 * @param OutLoadingFontFaces	An array (if provided) that keeps track of the unique font faces that are loading for a text during shaping.
	 */
	SLATECORE_API FShapedGlyphSequenceRef ShapeUnidirectionalText( const FString& InText, const FSlateFontInfo &InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InTextDirection, const ETextShapingMethod InTextShapingMethod , TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces = nullptr) const;
	SLATECORE_API FShapedGlyphSequenceRef ShapeUnidirectionalText( const TCHAR* InText, const int32 InTextStart, const int32 InTextLen, const FSlateFontInfo &InFontInfo, const float InFontScale, const TextBiDi::ETextDirection InTextDirection, const ETextShapingMethod InTextShapingMethod , TArray<TWeakPtr<FFreeTypeFace>>* OutLoadingFontFaces = nullptr) const;

	/**
	 * Performs text shaping on the overflow glyph sequence for a given font. The overflow sequence is used to replace characters that are clipped
	 */
	SLATECORE_API FShapedGlyphSequenceRef ShapeOverflowEllipsisText(const FSlateFontInfo& InFontInfo, const float InFontScale);

	/** 
	 * Gets information for how to draw all non-shaped characters in the specified string. Caches characters as they are found
	 * 
	 * @param InFontInfo		Information about the font that the string is drawn with
	 * @param FontScale			The scale to apply to the font
	 * @param OutCharacterEntries	Populated array of character entries. Indices of characters in Text match indices in this array
	 */
	SLATECORE_API class FCharacterList& GetCharacterList( const FSlateFontInfo &InFontInfo, float FontScale, const FFontOutlineSettings& InOutlineSettings = FFontOutlineSettings::NoOutline);

	/**
	 * Get the atlas information for the given shaped glyph. This information will be cached if required 
	 */
	SLATECORE_API FShapedGlyphFontAtlasData GetShapedGlyphFontAtlasData(const FShapedGlyphEntry& InShapedGlyph, const FFontOutlineSettings& InOutlineSettings);
	
	/**
	 * Get the atlas information and the scaled metrics of a given shaped sdf glyph. This information will be cached if required.
	 */
	SLATECORE_API FSdfGlyphFontAtlasData GetSdfGlyphFontAtlasData(const FShapedGlyphEntry& InShapedGlyph, const FFontOutlineSettings& InOutlineSettings);

	/**
	 * Gets the overflow glyph sequence for a given font. The overflow sequence is used to replace characters that are clipped
	 */
	UE_DEPRECATED(5.1, "GetOverflowEllipsisText is known to create dangling pointer. Use FShapedTextCache::FindOrAddOverflowEllipsisText.")
	SLATECORE_API FShapedGlyphSequenceRef GetOverflowEllipsisText(const FSlateFontInfo& InFontInfo, const float InFontScale);

public:
	/**
	 * Flush the given object out of the cache
	 */
	SLATECORE_API void FlushObject( const UObject* const InObject );

	/**
	 * Flush the given composite font out of the cache
	 */
	SLATECORE_API void FlushCompositeFont(const FCompositeFont& InCompositeFont);

	/** 
	 * Flush the cache if needed
	 */
	SLATECORE_API bool ConditionalFlushCache();

	/**
	 * Updates the texture used for rendering
	 */
	SLATECORE_API void UpdateCache();

	/**
	 * Releases rendering resources
	 */
	SLATECORE_API void ReleaseResources();

	/**
	 * Event called after releasing the rendering resources in ReleaseResources
	 */
	FOnReleaseFontResources& OnReleaseResources() { return OnReleaseResourcesDelegate; }

	/**
	 * Get the texture resource for a font atlas at a given index
	 * 
	 * @param Index	The index of the texture 
	 * @return Handle to the texture resource
	 */
	ISlateFontTexture* GetFontTexture( uint32 Index ) { return &AllFontTextures[Index].Get(); }

	/**
	 * Returns the font to use from the default typeface
	 *
	 * @param InFontInfo	A descriptor of the font to get the default typeface for
	 * 
	 * @return The raw font data
	 */
	SLATECORE_API const FFontData& GetDefaultFontData( const FSlateFontInfo& InFontInfo ) const;

	/**
	 * Returns the font to use from the typeface associated with the given codepoint
	 *
	 * @param InFontInfo		A descriptor of the font to get the typeface for
	 * @param InCodepoint		The codepoint to get the typeface associated with
	 * @param OutScalingFactor	The scaling factor applied to characters rendered with the given font
	 * 
	 * @return The raw font data
	 */
	SLATECORE_API const FFontData& GetFontDataForCodepoint( const FSlateFontInfo& InFontInfo, const UTF32CHAR InCodepoint, float& OutScalingFactor ) const;

	/**
	 * Returns true if the font to use from the typeface associated with the given codepoint is already loaded
	 *
	 * @param InFontInfo		A descriptor of the font to get the typeface for
	 * @param InCodepoint		The codepoint to get the typeface associated with
	 *
	 * @return FontData is already loaded
	 */
	SLATECORE_API bool HasLoadedFontForCodepoint(const FSlateFontInfo& InFontInfo, const UTF32CHAR InCodepoint) const;

	/**
	 * Returns the height of the largest character in the font. 
	 *
	 * @param InFontInfo	A descriptor of the font to get character size for 
	 * @param FontScale		The scale to apply to the font
	 * 
	 * @return The largest character height
	 */
	SLATECORE_API uint16 GetMaxCharacterHeight( const FSlateFontInfo& InFontInfo, float FontScale ) const;

	/**
	 * Returns the baseline for the specified font.
	 *
	 * @param InFontInfo	A descriptor of the font to get character size for 
	 * @param FontScale		The scale to apply to the font
	 * 
	 * @return The offset from the bottom of the max character height to the baseline.
	 */
	SLATECORE_API int16 GetBaseline( const FSlateFontInfo& InFontInfo, float FontScale ) const;

	/**
	 * Get the underline metrics for the specified font.
	 *
	 * @param InFontInfo			A descriptor of the font to get character size for
	 * @param FontScale				The scale to apply to the font
	 * @param OutUnderlinePos		The offset from the baseline to the center of the underline bar
	 * @param OutUnderlineThickness	The thickness of the underline bar
	 */
	SLATECORE_API void GetUnderlineMetrics( const FSlateFontInfo& InFontInfo, const float FontScale, int16& OutUnderlinePos, int16& OutUnderlineThickness ) const;

	/**
	 * Get the strike metrics for the specified font.
	 *
	 * @param InFontInfo			A descriptor of the font to get character size for
	 * @param FontScale				The scale to apply to the font
	 * @param OutStrikeLinePos		The offset from the baseline to the center of the strike bar
	 * @param OutStrikeLineThickness The thickness of the strike bar
	 */
	SLATECORE_API void GetStrikeMetrics( const FSlateFontInfo& InFontInfo, const float FontScale, int16& OutStrikeLinePos, int16& OutStrikeLineThickness ) const;

	/**
	 * Calculates the kerning amount for a pair of characters
	 *
	 * @param InFontData	The font that used to draw the string with the first and second characters
	 * @param InSize		The size of the font to draw
	 * @param First			The first character in the pair
	 * @param Second		The second character in the pair
	 * @return The kerning amount, 0 if no kerning
	 */
	SLATECORE_API int8 GetKerning( const FFontData& InFontData, const float InSize, TCHAR First, TCHAR Second, float Scale ) const;

	/**
	 * @return Whether or not the font used has kerning information
	 */
	SLATECORE_API bool HasKerning( const FFontData& InFontData ) const;

	/**
	 * Whether or not the specified character, within the specified font, can be loaded with the specified maximum font fallback level
	 *
	 * @param InFontData		Information about the font to load
	 * @param InCodepoint		The codepoint being loaded
	 * @param MaxFallbackLevel	The maximum fallback level to try for the font
	 * @return					Whether or not the character can be loaded
	 */
	SLATECORE_API bool CanLoadCodepoint(const FFontData& InFontData, const UTF32CHAR InCodepoint, EFontFallback MaxFallbackLevel = EFontFallback::FF_NoFallback) const;

	/**
	 * Returns the font attributes for the specified font.
	 *
	 * @param InFontData	The font to get attributes for 
	 * 
	 * @return The font attributes for the specified font.
	 */
	SLATECORE_API const TSet<FName>& GetFontAttributes( const FFontData& InFontData ) const;

	/**
	 * Get the available sub-face data from the given font.
	 * Typically there will only be one face unless this is a TTC/OTC font.
	 * The index of the returned entry can be passed as InFaceIndex to the FFreeTypeFace constructor.
	 */
	SLATECORE_API TArray<FString> GetAvailableFontSubFaces(FFontFaceDataConstRef InMemory) const;
	SLATECORE_API TArray<FString> GetAvailableFontSubFaces(const FString& InFilename) const;

	/**
	 * Issues a request to clear all cached data from the cache
	 */
	SLATECORE_API void RequestFlushCache(const FString& FlushReason);

	/**
	 * Clears just the cached font data, but leaves the atlases alone
	 */
	SLATECORE_API void FlushData();

	/**
	 * Gets the allocated font face data for a font data asset
	 */
	SLATECORE_API SIZE_T GetFontDataAssetResidentMemory(const UObject* FontDataAsset) const;

#if WITH_EDITOR && WITH_FREETYPE
	SLATECORE_API bool EnsurePreprocessedFontGeometry(FFontFaceDataRef FontFaceData);
#endif

private:
	// Non-copyable
	SLATECORE_API FSlateFontCache(const FSlateFontCache&);
	SLATECORE_API FSlateFontCache& operator=(const FSlateFontCache&);

	/**
	 * Returns the index for AllFontTextures array based on index in the concatenation of
	 * GrayscaleFontAtlasIndices, ColorFontAtlasIndices, and MsdfFontAtlasIndices
	 */
	SLATECORE_API int32 GetAllFontTexturesIndex(const int32 InIndex) const;

	/**
	 * Clears all cached data from the cache
	 */
	SLATECORE_API bool FlushCache();

	/**
	 * Clears out any pending UFont objects that were requested to be flushed
	 */
	SLATECORE_API void FlushFontObjects();

	/** Called after the active culture has changed */
	SLATECORE_API void HandleCultureChanged();

	/**
	 * Add a new entries into a cache atlas
	 *
	 * @param InFontInfo	Information about the font being used for the characters
	 * @param Characters	The characters to cache
	 * @param FontScale		The font scale to use
	 * @return true if the characters could be cached. false if the cache is full
	 */
	SLATECORE_API bool AddNewEntry(const FShapedGlyphEntry& InShapedGlyph, const FFontOutlineSettings& InOutlineSettings, FShapedGlyphFontAtlasData& OutAtlasData);

	SLATECORE_API bool AddNewEntry(const FCharacterRenderData InRenderData, uint8& OutTextureIndex, uint16& OutGlyphX, uint16& OutGlyphY, uint16& OutGlyphWidth, uint16& OutGlyphHeight, uint8& OutPaddingOffset);

#if !UE_BUILD_SHIPPING
	/** Dump statistics about Font cache usage if needed */
	SLATECORE_API void ConditionalDumpFontCacheStats() const;
#endif

private:

	/** FreeType library instance (owned by this font cache) */
	TUniquePtr<FFreeTypeLibrary> FTLibrary;

	/** FreeType low-level cache directory (owned by this font cache) */
	TUniquePtr<FFreeTypeCacheDirectory> FTCacheDirectory;

	/** High-level composite font cache (owned by this font cache) */
	TUniquePtr<FCompositeFontCache> CompositeFontCache;

	/** FreeType font renderer (owned by this font cache) */
	TUniquePtr<FSlateFontRenderer> FontRenderer;

	/** HarfBuzz text shaper (owned by this font cache) */
	TUniquePtr<FSlateTextShaper> TextShaper;

	/** Sdf Rasterizer for Freetype fonts */
	TUniquePtr<FSlateSdfGenerator> SdfGenerator;

	/** Mapping Font keys to cached data */
	TMap<FSlateFontKey, TUniquePtr<FCharacterList>, FDefaultSetAllocator, FSlateFontKeyFuncs<TUniquePtr<FCharacterList>>> FontToCharacterListCache;

	/** Mapping shaped glyphs to their cached atlas data */
	TMap<FShapedGlyphEntryKey, TSharedRef<FShapedGlyphFontAtlasData>> ShapedGlyphToAtlasData;

	/** Mapping signed distance field glyphs to their cached atlas data */
	TMap<FSdfGlyphEntryKey, TSharedRef<FSdfGlyphFontAtlasData>> SdfGlyphToAtlasData;

	/** Mapping unfinished signed distance field tasks to their cached atlas data */
	TMap<FSdfGlyphTaskKey, TSharedRef<FSdfGlyphFontAtlasData>> SdfTaskToAtlasData;

	/** Array of grayscale font atlas indices for use with AllFontTextures (cast the element to FSlateFontAtlas) */
	TArray<uint8> GrayscaleFontAtlasIndices;

	/** Array of color font atlas indices for use with AllFontTextures (cast the element to FSlateFontAtlas) */
	TArray<uint8> ColorFontAtlasIndices;

	/** Array of multi-channel distance field font atlas indices for use with AllFontTextures (cast the element to FSlateFontAtlas) */
	TArray<uint8> MsdfFontAtlasIndices;

	/** Array of any non-atlased font texture indices for use with AllFontTextures */
	TArray<uint8> NonAtlasedTextureIndices;

	/** Array of all font textures - both atlased and non-atlased */
	TArray<TSharedRef<ISlateFontTexture>> AllFontTextures;

	/** Factory for creating new font atlases */
	TSharedRef<ISlateFontAtlasFactory> FontAtlasFactory;

	/** Whether or not we have a pending request to flush the cache when it is safe to do so */
	volatile bool bFlushRequested;

	/** Critical section preventing concurrent access to FontObjectsToFlush */
	mutable FCriticalSection FontObjectsToFlushCS;

	/** Array of UFont objects that the font cache has been requested to flush. Since GC can happen while the loading screen is running, the request may be deferred until the next call to ConditionalFlushCache */
	TArray<const UObject*> FontObjectsToFlush;

	/** Called after releasing the rendering resources in ReleaseResources */
	FOnReleaseFontResources OnReleaseResourcesDelegate;

	ESlateTextureAtlasThreadId OwningThread;

	/** Overflow text string to use to replace clipped characters */
	FText EllipsisText;

};

