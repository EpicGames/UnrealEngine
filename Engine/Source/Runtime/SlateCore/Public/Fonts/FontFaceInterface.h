// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Fonts/CompositeFont.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "FontFaceInterface.generated.h"

class UObject;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UFontFaceInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IFontFaceInterface
{
	GENERATED_IINTERFACE_BODY()

#if WITH_EDITORONLY_DATA
	/** Initialize this font face from legacy bulk data. */
	virtual void InitializeFromBulkData(const FString& InFilename, const EFontHinting InHinting, const void* InBulkDataPtr, const int32 InBulkDataSizeBytes) = 0;
#endif // WITH_EDITORONLY_DATA

	/** Get the filename of the font to use. This may not actually exist on disk in editor builds and we should load the face buffer instead. */
	virtual const FString& GetFontFilename() const = 0;

	/** Get the hinting algorithm to use with the font. */
	virtual EFontHinting GetHinting() const = 0;

	/** Get the enum controlling how this font should be loaded at runtime. */
	virtual EFontLoadingPolicy GetLoadingPolicy() const = 0;

	/** Get the method to use when laying out the font? */
	virtual EFontLayoutMethod GetLayoutMethod() const = 0;

	/** Returns true if the ascend is overridden. */
	virtual bool IsAscendOverridden() const = 0;

	/** Returns the overridden value of the ascend. This value will be used only if IsAscendOverridden returns true. */
	virtual int32 GetAscendOverriddenValue() const = 0;

	/** Returns true if the descend is overridden. */
	virtual bool IsDescendOverridden() const = 0;

	/** Returns the overridden value of the descend. This value will be used only if IsDescendOverridden returns true. */
	virtual int32 GetDescendOverriddenValue() const = 0;

	/** Returns the percentage of the font height that's used as the height the strike brush. */
	virtual int32 GetStrikeBrushHeightPercentage() const {return 60;};

	/** Get the data buffer containing the data for the current font face. */
	virtual FFontFaceDataConstRef GetFontFaceData() const = 0;

	/** Returns the font face's rasterization settings. */
	virtual FFontRasterizationSettings GetRasterizationSettings() const = 0;
};
