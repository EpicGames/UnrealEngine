// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Fonts/FontTypes.h"
#include "Types/SlateVector2.h"

struct FSlateFontMeasureCache;

typedef class FLRUStringCache FMeasureCache;
struct FSlateFontMeasureCache;

class FSlateFontMeasure final
{
public:

	static SLATECORE_API TSharedRef< FSlateFontMeasure > Create( const TSharedRef<class FSlateFontCache>& FontCache );

public:

	/** 
	 * Measures the width and height of a passed in string.  The height is the maximum height of the largest character in the font/size pair
	 * 
	 * @param Text			The string to measure
	 * @param InFontInfo	Information about the font that the string is drawn with
	 * @param FontScale	The scale to apply to the font
	 * @return The width and height of the string.
	 */
	SLATECORE_API UE::Slate::FDeprecateVector2DResult Measure( FStringView Text, const FSlateFontInfo &InFontInfo, float FontScale = 1.0f ) const;

	/** 
	 * Measures the width and height of a passed in text.  The height is the maximum height of the largest character in the font/size pair
	 * 
	 * @param Text			The text to measure
	 * @param InFontInfo	Information about the font that the text is drawn with
	 * @param FontScale	The scale to apply to the font
	 * @return The width and height of the text.
	 */
	SLATECORE_API UE::Slate::FDeprecateVector2DResult Measure( const FText& Text, const FSlateFontInfo &InFontInfo, float FontScale = 1.0f ) const;
	
	/** 
	 * Measures the width and height of a passed in string.  The height is the maximum height of the largest character in the font/size pair
	 * 
	 * @param Text			The string to measure
	 * @param StartIndex	The inclusive index to start measuring the string at
	 * @param EndIndex		The inclusive index to stop measuring the string at
	 * @param InFontInfo	Information about the font that the string is drawn with
	 * @param FontScale	The scale to apply to the font
	 * @return The width and height of the string.
	 */
	SLATECORE_API UE::Slate::FDeprecateVector2DResult Measure( FStringView Text, int32 StartIndex, int32 EndIndex, const FSlateFontInfo &InFontInfo, bool IncludeKerningWithPrecedingChar = true, float FontScale = 1.0f ) const;

	/** 
	 * Finds the last whole character index before the specified position in pixels along the string horizontally
	 * 
	 * @param String				The string to measure
	 * @param InFontInfo	Information about the font used to draw the string
	 * @param HorizontalOffset  Offset horizontally into the string, in pixels
	 *
	 * @return The index of the last whole character before the specified horizontal offset
	 */
	SLATECORE_API int32 FindLastWholeCharacterIndexBeforeOffset( FStringView Text, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, float FontScale = 1.0f ) const;

	/** 
	 * Finds the last whole character index before the specified position in pixels along the string horizontally
	 * 
	 * @param String				The string to measure
	 * @param InFontInfo	Information about the font used to draw the string
	 * @param HorizontalOffset  Offset horizontally into the string, in pixels
	 *
	 * @return The index of the last whole character before the specified horizontal offset
	 */
	SLATECORE_API int32 FindLastWholeCharacterIndexBeforeOffset( const FText& Text, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, float FontScale = 1.0f ) const;

	/** 
	 * Finds the last whole character index before the specified position in pixels along the text horizontally
	 * 
	 * @param Text			The text to measure
	 * @param StartIndex	The inclusive index to start measuring the string at
	 * @param EndIndex		The inclusive index to stop measuring the string at
	 * @param InFontInfo	Information about the font used to draw the string
	 * @param HorizontalOffset  Offset horizontally into the string, in pixels
	 *
	 * @return The index of the last whole character before the specified horizontal offset
	 */
	SLATECORE_API int32 FindLastWholeCharacterIndexBeforeOffset( FStringView Text, int32 StartIndex, int32 EndIndex, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, bool IncludeKerningWithPrecedingChar = true, float FontScale = 1.0f ) const;

	/** 
	 * Finds the first whole character index after the specified position in pixels along the string horizontally
	 * 
	 * @param String				The string to measure
	 * @param InFontInfo	Information about the font used to draw the string
	 * @param HorizontalOffset  Offset horizontally into the string, in pixels
	 *
	 * @return The index of the first whole character after the specified horizontal offset
	 */
	SLATECORE_API int32 FindFirstWholeCharacterIndexAfterOffset( FStringView Text, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, float FontScale = 1.0f ) const;

	/** 
	 * Finds the first whole character index after the specified position in pixels along the string horizontally
	 * 
	 * @param String				The string to measure
	 * @param InFontInfo	Information about the font used to draw the string
	 * @param HorizontalOffset  Offset horizontally into the string, in pixels
	 *
	 * @return The index of the first whole character after the specified horizontal offset
	 */
	SLATECORE_API int32 FindFirstWholeCharacterIndexAfterOffset( const FText& Text, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, float FontScale = 1.0f ) const;

	/** 
	 * Finds the first whole character index after the specified position in pixels along the string horizontally
	 * 
	 * @param String				The string to measure
	 * @param StartIndex	The inclusive index to start measuring the string at
	 * @param EndIndex		The inclusive index to stop measuring the string at
	 * @param InFontInfo	Information about the font used to draw the string
	 * @param HorizontalOffset  Offset horizontally into the string, in pixels
	 *
	 * @return The index of the first whole character after the specified horizontal offset
	 */
	SLATECORE_API int32 FindFirstWholeCharacterIndexAfterOffset( FStringView Text, int32 StartIndex, int32 EndIndex, const FSlateFontInfo& InFontInfo, const int32 HorizontalOffset, bool IncludeKerningWithPrecedingChar = true, float FontScale = 1.0f ) const;

	/** 
	 * Finds the character index at the specified position in pixels along the string horizontally
	 * 
	 * @param Text	The string to measure
	 * @param FontName	The name of the font the string is drawn with
	 * @param FontSize	The size of the font
	 * @param HorizontalOffset  Offset horizontally into the string, in pixels
	 * @param FontScale	The scale to apply to the font
	 *
	 * @return The index of the character closest to the specified horizontal offset
	 */
	SLATECORE_API int32 FindCharacterIndexAtOffset( FStringView Text, const FSlateFontInfo &InFontInfo, const int32 HorizontalOffset, float FontScale = 1.0f ) const;

	/** 
	 * Finds the character index at the specified position in pixels along the text horizontally
	 * 
	 * @param Text	The text to measure
	 * @param FontName	The name of the font the text is drawn with
	 * @param FontSize	The size of the font
	 * @param HorizontalOffset  Offset horizontally into the text, in pixels
	 * @param FontScale	The scale to apply to the font
	 *
	 * @return The index of the character closest to the specified horizontal offset
	 */
	SLATECORE_API int32 FindCharacterIndexAtOffset( const FText& Text, const FSlateFontInfo &InFontInfo, const int32 HorizontalOffset, float FontScale = 1.0f ) const;

	/** 
	 * Finds the character index at the specified position in pixels along the string horizontally
	 * 
	 * @param Text	The string to measure
	 * @param StartIndex	The inclusive index to start measuring the string at
	 * @param EndIndex		The inclusive index to stop measuring the string at
	 * @param FontName	The name of the font the string is drawn with
	 * @param FontSize	The size of the font
	 * @param HorizontalOffset  Offset horizontally into the string, in pixels
	 * @param FontScale	The scale to apply to the font
	 *
	 * @return The index of the character closest to the specified horizontal offset
	 */
	SLATECORE_API int32 FindCharacterIndexAtOffset( FStringView Text, int32 StartIndex, int32 EndIndex, const FSlateFontInfo &InFontInfo, const int32 HorizontalOffset, bool IncludeKerningWithPrecedingChar = true, float FontScale = 1.0f ) const;

	/**
	 * Returns the height of the largest character in the font. 
	 *
	 * @param InFontInfo	A descriptor of the font to get character size for 
	 * @param FontScale		The scale to apply to the font
	 * 
	 * @return The largest character height
	 */
	SLATECORE_API uint16 GetMaxCharacterHeight( const FSlateFontInfo& InFontInfo, float FontScale = 1.0f ) const;

	/**
	 * Returns the kerning value for the specified pair of characters.
	 *
	 * @param InFontInfo           A descriptor of the font to get kerning for
	 * @param FontScale            The scale to apply to the font
	 * @param PreviousCharacter    The character preceding the current character that you want to measure kerning for
	 * @param CurrentCharacter     The current character to you to measure kerning for when precede by the specified PreviousCharacter
	 *
	 * @return the kerning value used between the two specified characters
	 */
	SLATECORE_API int8 GetKerning(const FSlateFontInfo& InFontInfo, float FontScale, TCHAR PreviousCharacter, TCHAR CurrentCharacter) const;

	/**
	 * Returns the baseline for the specified font.
	 *
	 * @param InFontInfo	A descriptor of the font to get character size for 
	 * @param FontScale		The scale to apply to the font
	 * 
	 * @return The offset from the bottom of the max character height to the baseline.
	 */
	SLATECORE_API int16 GetBaseline( const FSlateFontInfo& InFontInfo, float FontScale = 1.0f ) const;

	SLATECORE_API void FlushCache();


private:

	FSlateFontMeasure( const TSharedRef<class FSlateFontCache>& InFontCache );

	enum ELastCharacterIndexFormat
	{
		// The last whole character before the horizontal offset
		LastWholeCharacterBeforeOffset,
		// The character directly at the offset
		CharacterAtOffset,
		// Not used
		Unused,
	};

	/** 
	 * Measures a string, optionally stopped after the specified horizontal offset in pixels is reached
	 * 
	 * @param Text	The string to measure
	 * @param StartIndex	The inclusive index to start measuring the string at
	 * @param EndIndex	The inclusive index to stop measuring the string at
	 * @param FontName	The name of the font the string is drawn with
	 * @param FontSize	The size of the font
	 * @param StopAfterHorizontalOffset  Offset horizontally into the string to stop measuring characters after, in pixels (or INDEX_NONE)
	 * @param CharIndexFormat  Behavior to use for StopAfterHorizontalOffset
	 * @param OutCharacterIndex  The index of the last character processed (used with StopAfterHorizontalOffset)
	 * @param FontScale	The scale to apply when measuring the text
	 *
	 * @return The width and height of the string.
	 */
	FVector2f MeasureStringInternal( FStringView Text, int32 StartIndex, int32 EndIndex, const FSlateFontInfo& InFontInfo, bool IncludeKerningWithPrecedingChar, float FontScale, int32 StopAfterHorizontalOffset, ELastCharacterIndexFormat CharIndexFormat, int32& OutLastCharacterIndex ) const;

	/**
	 * Check to see if there's an existing cached measurement, or failing that, add a new entry so that we can cache a new measurement
	 */
	FMeasureCache* FindOrAddMeasureCache( const FSlateFontInfo& InFontInfo, const float InFontScale ) const;

private:
	/** Mapping Font keys to cached data */
	mutable TMap<FSlateFontKey, TSharedPtr<FSlateFontMeasureCache>, FDefaultSetAllocator, FSlateFontKeyFuncs<TSharedPtr<FSlateFontMeasureCache>>> FontToMeasureCache;

	TSharedRef<class FSlateFontCache> FontCache;
};
