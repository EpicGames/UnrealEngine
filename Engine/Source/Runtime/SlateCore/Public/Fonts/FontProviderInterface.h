// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "FontRasterizationMode.h"
#include "FontSdfSettings.h"

#include "FontProviderInterface.generated.h"

class UObject;
struct FCompositeFont;

UINTERFACE(MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UFontProviderInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IFontProviderInterface
{
	GENERATED_IINTERFACE_BODY()

	virtual const FCompositeFont* GetCompositeFont() const = 0;

};
