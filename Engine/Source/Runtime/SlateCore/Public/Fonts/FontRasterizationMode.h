// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "FontRasterizationMode.generated.h"

/** Enumerates supported font rasterization modes. */
UENUM()
enum class EFontRasterizationMode : uint8
{
	/** Glyphs are rasterized directly into alpha mask bitmaps per size and skew. */
	Bitmap,

	/** Glyphs are rasterized into multi-channel signed distance fields, which are size and skew agnostic. */
	Msdf UMETA(DisplayName = "Multi-Channel Distance Field"),

	/** Glyphs are rasterized into single-channel signed distance fields, which are size and skew agnostic. More memory efficient but corners may appear rounded. */
	Sdf UMETA(DisplayName = "Signed Distance Field"),

	/** Glyphs are rasterized into approximate distance fields, which are size and skew agnostic. More memory and computationally efficient but lower quality. */
	SdfApproximation UMETA(DisplayName = "Approximate Signed Distance Field (fast)")
};

/** Returns true if the font rasterization Mode is one of the distance field-based modes, which are part of the "Slate SDF text" feature */
inline bool IsSdfFontRasterizationMode(EFontRasterizationMode Mode)
{
	switch (Mode)
	{
		case EFontRasterizationMode::Msdf:
		case EFontRasterizationMode::Sdf:
		case EFontRasterizationMode::SdfApproximation:
			return true;
		case EFontRasterizationMode::Bitmap:
		default:
			return false;
	}
}
