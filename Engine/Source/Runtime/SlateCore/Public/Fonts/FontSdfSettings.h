// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "FontSdfSettings.generated.h"

/**
 * Settings for signed distance field fonts.
 */
USTRUCT(BlueprintType)
struct FFontSdfSettings
{
	GENERATED_USTRUCT_BODY()

	/** The base pixels per em for generated distance fields */
	UPROPERTY(EditAnywhere, Category=SdfFont, meta = (ClampMin = 8, ClampMax = 256, DisplayName="Base SDF px/em"))
	int32 BasePpem = 32;

	/** Returns the clamped and aligned pixels/em */
	inline int32 GetClampedPpem() const
	{
		return FMath::Clamp(AlignArbitrary(BasePpem, 4), 8, 256);
	}
};
