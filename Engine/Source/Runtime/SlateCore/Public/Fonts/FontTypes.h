// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Textures/TextureAtlas.h"

class FSlateShaderResource;


DECLARE_MULTICAST_DELEGATE_OneParam(FOnReleaseFontResources, const class FSlateFontCache&);


struct FSlateFontKey
{
public:
	FSlateFontKey( const FSlateFontInfo& InInfo, const FFontOutlineSettings& InFontOutlineSettings, const float InScale )
		: FontInfo( InInfo )
		, OutlineSettings(InFontOutlineSettings)
		, Scale( InScale )
		, KeyHash( 0 )
	{
		KeyHash = HashCombine(KeyHash, GetLegacyTypeHash(FontInfo));
		KeyHash = HashCombine(KeyHash, GetTypeHash(OutlineSettings));
		KeyHash = HashCombine(KeyHash, GetTypeHash(Scale));
	}

	inline const FSlateFontInfo& GetFontInfo() const
	{
		return FontInfo;
	}

	inline float GetScale() const
	{
		return Scale;
	}

	inline const FFontOutlineSettings& GetFontOutlineSettings() const
	{
		return OutlineSettings;
	}

	inline bool IsIdenticalToForCaching(const FSlateFontKey& Other) const
	{
		return FontInfo.IsLegacyIdenticalTo(Other.FontInfo)
			&& OutlineSettings.IsIdenticalToForCaching(Other.OutlineSettings)
			&& Scale == Other.Scale;
	}

	friend inline uint32 GetTypeHash( const FSlateFontKey& Key )
	{
		return Key.KeyHash;
	}

private:
	FSlateFontInfo FontInfo;
	FFontOutlineSettings OutlineSettings;
	float Scale;
	uint32 KeyHash;
};


template<typename ValueType>
struct FSlateFontKeyFuncs : BaseKeyFuncs<TPair<FSlateFontKey, ValueType>, FSlateFontKey, false>
{
	typedef BaseKeyFuncs <
		TPair<FSlateFontKey, ValueType>,
		FSlateFontKey
	> Super;
	typedef typename Super::ElementInitType ElementInitType;
	typedef typename Super::KeyInitType     KeyInitType;

	inline static const FSlateFontKey& GetSetKey(ElementInitType Element)
	{
		return Element.Key;
	}

	inline static bool Matches(const FSlateFontKey& A, const FSlateFontKey& B)
	{
		return A.IsIdenticalToForCaching(B);
	}

	inline static uint32 GetKeyHash(const FSlateFontKey& Identifier)
	{
		return GetTypeHash(Identifier);
	}
};


/** Contains pixel data for a character rendered from freetype as well as measurement info */
struct FCharacterRenderData
{
	/** Raw pixels of the rendered character */
	TArray<uint8> RawPixels;
	/** Width of the character in pixels */
	int16 SizeX = 0;
	/** Height of the character in pixels */
	int16 SizeY = 0;
	/** The vertical distance from the baseline to the topmost border of the glyph bitmap */
	int16 VerticalOffset = 0;
	/** The horizontal distance from the origin to the leftmost border of the character */
	int16 HorizontalOffset = 0;
	/** Type of glyph rasterization */
	ESlateFontAtlasContentType ContentType = ESlateFontAtlasContentType::Alpha;
	/** True if the rendered character supports outlines, false otherwise */
	bool bSupportsOutline = false;
};

/** For a deferred atlas character insertion, this contains the subregion of the atlas previously reserved for a character
 *  and the pixels to copy into it. The subregion corresponds to the initial reserved atlas slot extents after padding */
struct FDeferredCharacterRenderData
{
	/** Destination subregion width */
	int16 USize = 0;
	/** Destination subregion height */
	int16 VSize = 0;
	/** Destination subregion x offset */
	int16 StartU = 0;
	/** Destination subregion y offset */
	int16 StartV = 0;
	/** Source pixels to copy in subregion defined below */
	TArray<uint8> RawPixels;
};

/**
 * Interface to all Slate font textures, both atlased and non-atlased
 */
class ISlateFontTexture
{
public:
	virtual ~ISlateFontTexture() = default;

	/**
	 * Returns the texture resource used by Slate
	 */
	virtual class FSlateShaderResource* GetSlateTexture() const = 0;

	/**
	 * Returns the texture resource used the Engine
	 */
	virtual class FTextureResource* GetEngineTexture() = 0;

	/**
	 * Returns the type of content in the texture
	 */
	virtual ESlateFontAtlasContentType GetContentType() const = 0;

	/**
	 * Releases rendering resources of this texture
	 */
	virtual void ReleaseRenderingResources() = 0;

	/**
 * Retrieves a copy of the texture atlas data in this texture.
 * This is a slow operation so this is recommended primarily for debugging and visualization tools.
 *
 * @param OutData	An array that a caller provides. This array will be populated with a copy of the data in the atlas. It is up to the caller to verify that the array is populated and not empty.
 */
	virtual void GetAtlasDataCopy(TArray<uint8>& OutData) const = 0;
};

/** 
 * Representation of a texture for fonts in which characters are packed tightly based on their bounding rectangle 
 */
class FSlateFontAtlas : public ISlateFontTexture, public FSlateTextureAtlas
{
public:
	SLATECORE_API FSlateFontAtlas(uint32 InWidth, uint32 InHeight, ESlateFontAtlasContentType InContentType, ESlateTextureAtlasPaddingStyle InPaddingStyle);
	SLATECORE_API virtual ~FSlateFontAtlas();

	//~ Begin ISlateFontTexture interface 
	SLATECORE_API virtual ESlateFontAtlasContentType GetContentType() const override final;
	virtual FSlateShaderResource* GetAtlasTexture() const override { return GetSlateTexture(); }
	virtual void ReleaseRenderingResources() { ReleaseResources(); }
	SLATECORE_API virtual void GetAtlasDataCopy(TArray<uint8>& OutData) const override;
	//~ End ISlateFontTexture interface

	/**
	 * Flushes all cached data.
	 */
	SLATECORE_API void Flush();

	/** 
	 * Adds a character to the texture.
	 *
	 * @param CharInfo	Information about the size of the character
	 */
	SLATECORE_API const struct FAtlasedTextureSlot* AddCharacter( const FCharacterRenderData& CharInfo );

	/**
	 * Reserve a slot for a character but dont't update the texture yet.
	 *
	 * @param InSizeX	Width of the character
	 * @param InSizeY	Height of the character
	 */
	bool BeginDeferredAddCharacter( const int16 InSizeX, const int16 InSizeY, FDeferredCharacterRenderData& OutCharInfo);
	
	/**
	 * Update a character in the texture for already reserved or added slot.
	 *
	 * @param CharInfo	Information about the location and size of the character
	 */
	void EndDeferredAddCharacter( const FDeferredCharacterRenderData& CharInfo );

protected:
	ESlateFontAtlasContentType ContentType;
};

class ISlateFontAtlasFactory
{
public:
	virtual ~ISlateFontAtlasFactory() = default;
	virtual FIntPoint GetAtlasSize(ESlateFontAtlasContentType InContentType) const = 0;
	virtual TSharedRef<FSlateFontAtlas> CreateFontAtlas(ESlateFontAtlasContentType InContentType) const = 0;
	virtual TSharedPtr<ISlateFontTexture> CreateNonAtlasedTexture(const uint32 InWidth, const uint32 InHeight, ESlateFontAtlasContentType InContentType, const TArray<uint8>& InRawData) const = 0;
};
