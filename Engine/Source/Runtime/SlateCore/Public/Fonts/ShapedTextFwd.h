// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FShapedGlyphSequence;

/** Immutable pointer/reference to a shaped text sequence. This is what gets returned by the font cache, and is used throughout the rest of the rendering pipeline to avoid copying data */
class FShapedGlyphSequence;
typedef TSharedPtr<const FShapedGlyphSequence> FShapedGlyphSequencePtr;
typedef TSharedRef<const FShapedGlyphSequence> FShapedGlyphSequenceRef;
