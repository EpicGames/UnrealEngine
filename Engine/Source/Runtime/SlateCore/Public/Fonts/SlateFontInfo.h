// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Fonts/CompositeFont.h"
#include "HAL/IConsoleManager.h"
#include "SlateFontInfo.generated.h"

namespace FontConstants
{
	/** The  DPI we render at (horizontal and vertical) */
	inline const uint32 RenderDPI = 96;
}

/**
 * Sets the maximum font fallback level, for when a character can't be found in the selected font set.
 * UI code that renders strings from a third party (e.g. player chat in a multiplayer game) may want to restrict font fallback to prevent potential performance problems.
 */
enum class EFontFallback : uint8
{
	/** No fallback font */
	FF_NoFallback,
	/** Fallback to last resort font set */
	FF_LastResortFallback,
	/** Tries all fallbacks */
	FF_Max
};

/**
 * Settings for applying an outline to a font
 */
USTRUCT(BlueprintType)
struct FFontOutlineSettings
{
	GENERATED_USTRUCT_BODY()

	/** Size of the outline in slate units (at 1.0 font scale this unit is a pixel)*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=OutlineSettings, meta=(ClampMin="0", ClampMax="1024"))
	int32 OutlineSize;

	/** When enabled, outlines have sharp mitered corners, otherwise they are rounded. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = OutlineSettings, meta = (DisplayName="Mitered Corners"))
	bool bMiteredCorners;

	/**
	 * When enabled the outline will be completely translucent where the filled area will be.  This allows for a separate fill alpha value
	 * The trade off when enabling this is slightly worse quality for completely opaque fills where the inner outline border meets the fill area
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = OutlineSettings)
	bool bSeparateFillAlpha;

	/**
	 * When enabled the outline will be applied to any drop shadow that uses this font
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = OutlineSettings)
	bool bApplyOutlineToDropShadows;

	/** Optional material to apply to the outline */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SlateStyleRules, meta=(AllowedClasses="/Script/Engine.MaterialInterface"))
	TObjectPtr<UObject> OutlineMaterial;

	/** The color of the outline for any character in this font */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=OutlineSettings)
	FLinearColor OutlineColor;

	FFontOutlineSettings()
		: OutlineSize(0)
		, bMiteredCorners(false)
		, bSeparateFillAlpha(false)
		, bApplyOutlineToDropShadows(false)
		, OutlineMaterial(nullptr)
		, OutlineColor(FLinearColor::Black)
	{}

	FFontOutlineSettings(int32 InOutlineSize, FLinearColor InColor = FLinearColor::Black)
		: OutlineSize(InOutlineSize)
		, bMiteredCorners(false)
		, bSeparateFillAlpha(false)
		, bApplyOutlineToDropShadows(false)
		, OutlineMaterial(nullptr)
		, OutlineColor(InColor)
	{}

	inline bool IsIdenticalToForCaching(const FFontOutlineSettings& Other) const
	{
		// Ignore OutlineMaterial && OutlineColor because they do not affect the cached glyph.
		return OutlineSize == Other.OutlineSize
			&&  bMiteredCorners == Other.bMiteredCorners
			&&  bSeparateFillAlpha == Other.bSeparateFillAlpha;
	}

	inline bool IsIdenticalTo(const FFontOutlineSettings& Other) const
	{
		return
			OutlineSize == Other.OutlineSize &&
			bMiteredCorners == Other.bMiteredCorners &&
			bSeparateFillAlpha == Other.bSeparateFillAlpha &&
			bApplyOutlineToDropShadows == Other.bApplyOutlineToDropShadows &&
			OutlineMaterial == Other.OutlineMaterial &&
			OutlineColor == Other.OutlineColor;
	}

	friend inline uint32 GetTypeHash(const FFontOutlineSettings& OutlineSettings)
	{
		uint32 Hash = 0;
		// Ignore OutlineMaterial && OutlineColor because they do not affect the cached glyph.
		Hash = HashCombine(Hash, GetTypeHash(OutlineSettings.OutlineSize));
		Hash = HashCombine(Hash, GetTypeHash(OutlineSettings.bMiteredCorners));
		Hash = HashCombine(Hash, GetTypeHash(OutlineSettings.bSeparateFillAlpha));
		return Hash;
	}

	bool IsVisible() const
	{
		return OutlineSize > 0 && OutlineColor.A > 0;
	}

#if WITH_EDITORONLY_DATA
	bool Serialize(FArchive& Ar);
	void PostSerialize(const FArchive& Ar);
#endif

	static SLATECORE_API FFontOutlineSettings NoOutline;
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FFontOutlineSettings>
	: public TStructOpsTypeTraitsBase2<FFontOutlineSettings>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};
#endif

/**
 * A representation of a font in Slate.
 */
USTRUCT(BlueprintType, meta = (HasNativeMake = "/Script/SlateBaseRenderer.SlateFontInfoBlueprintLibrary.MakeSlateFontInfo"))
struct FSlateFontInfo
{
	GENERATED_USTRUCT_BODY()

	/** The font object (valid when used from UMG or a Slate widget style asset) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SlateStyleRules, meta=(AllowedClasses="/Script/Engine.Font", DisplayName="Font Family"))
	TObjectPtr<const UObject> FontObject;

	/** The material to use when rendering */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SlateStyleRules, meta=(AllowedClasses="/Script/Engine.MaterialInterface"))
	TObjectPtr<UObject> FontMaterial;

	/** Settings for applying an outline to a font */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SlateStyleRules)
	FFontOutlineSettings OutlineSettings;

	/** The composite font data to use (valid when used with a Slate style set in C++) */
	TSharedPtr<const FCompositeFont> CompositeFont;

	/** The name of the font to use from the default typeface (None will use the first entry) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SlateStyleRules, meta=(DisplayName="Typeface", EditCondition="FontObject"))
	FName TypefaceFontName;

	/**
	 * The font size is a measure in point values. The conversion of points to Slate Units is done at 96 DPI.
	 * So if you're using a tool like Photoshop to prototype layouts and UI mock ups, you can change the UMG Font settings
	 * to ensure that UMG font size is displayed in its 72 DPI equivalent, even if Slate will still use 96 DPI internally.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SlateStyleRules, meta=(ClampMin=1, ClampMax=1000))
	float Size;

	/** The uniform spacing (or tracking) between all characters in the text. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SlateStyleRules, meta=(ClampMin=-1000, ClampMax=10000))
	int32 LetterSpacing = 0;

	/** A skew amount to apply to the text. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SlateStyleRules, meta=(ClampMin=-5, ClampMax=5))
	float SkewAmount = 0.0f;

	/** The font fallback level. Runtime only, don't set on shared FSlateFontInfo, as it may change the font elsewhere (make a copy). */
	EFontFallback FontFallback;

	/** Enable pseudo-monospaced font. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SlateStyleRules, meta=(DisplayName="Monospacing"))
	bool bForceMonospaced = false;

	/** When enabled, whole quads are filled by the material without automatically stenciling the text - this needs to be done within the material (using Font Signed Distance node). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SlateStyleRules)
	bool bMaterialIsStencil = false;

	/** The uniform width to apply to all characters when bForceMonospaced is enabled, proportional of the font Size. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=SlateStyleRules, meta=(ClampMin=0, EditCondition = "bForceMonospaced==true", EditConditionHides))
	float MonospacedWidth = 1.0f;

#if WITH_EDITORONLY_DATA
private:

	/** The name of the font */
	UPROPERTY()
	FName FontName_DEPRECATED;

	/** The hinting algorithm to use with the font */
	UPROPERTY()
	EFontHinting Hinting_DEPRECATED;
#endif

public:

	/** Default constructor. */
	SLATECORE_API FSlateFontInfo();

	/** Dtor. Must be inline*/
	~FSlateFontInfo() = default;

	/**
	 * Creates and initializes a new instance with the specified font, size, and emphasis.
	 *
	 * @param InCompositeFont The font instance to use.
	 * @param InSize The size of the font.
	 * @param InTypefaceFontName The name of the font to use from the default typeface (None will use the first entry)
	 */
	SLATECORE_API FSlateFontInfo( TSharedPtr<const FCompositeFont> InCompositeFont, const float InSize, const FName& InTypefaceFontName = NAME_None, const FFontOutlineSettings& InOutlineSettings = FFontOutlineSettings());

	/**
	 * Creates and initializes a new instance with the specified font, size, and emphasis.
	 *
	 * @param InFontObject The font instance to use.
	 * @param InSize The size of the font.
	 * @param InFamilyFontName The name of the font to use from the default typeface (None will use the first entry)
	 */
	SLATECORE_API FSlateFontInfo( const UObject* InFontObject, const float InSize, const FName& InTypefaceFontName = NAME_None, const FFontOutlineSettings& InOutlineSettings = FFontOutlineSettings());

	/**
	 * Creates and initializes a new instance with the specified font name and size.
	 *
	 * @param InFontName The name of the font.
	 * @param InSize The size of the font.
	 * @param InHinting The type of hinting to use for the font.
	 */
	UE_DEPRECATED(5.6, "Use constructor with FCompositeFont instead.")
	SLATECORE_API FSlateFontInfo( const FString& InFontName, float InSize, EFontHinting InHinting = EFontHinting::Default, const FFontOutlineSettings& InOutlineSettings = FFontOutlineSettings());

	/**
	 * Creates and initializes a new instance with the specified font name and size.
	 *
	 * @param InFontName The name of the font.
	 * @param InSize The size of the font.
	 * @param InHinting The type of hinting to use for the font.
	 */
	UE_DEPRECATED(5.6, "Use constructor with FCompositeFont instead.")
	SLATECORE_API FSlateFontInfo( const FName& InFontName, float InSize, EFontHinting InHinting = EFontHinting::Default );

	/**
	 * Creates and initializes a new instance with the specified font name and size.
	 *
	 * @param InFontName The name of the font.
	 * @param InSize The size of the font.
	 * @param InHinting The type of hinting to use for the font.
	 */
	UE_DEPRECATED(5.6, "Use constructor with FCompositeFont instead.")
	SLATECORE_API FSlateFontInfo( const ANSICHAR* InFontName, float InSize, EFontHinting InHinting = EFontHinting::Default );

	/**
	 * Creates and initializes a new instance with the specified font name and size.
	 *
	 * @param InFontName The name of the font.
	 * @param InSize The size of the font.
	 * @param InHinting The type of hinting to use for the font.
	 */
	UE_DEPRECATED(5.6, "Use constructor with FCompositeFont instead.")
	SLATECORE_API FSlateFontInfo( const WIDECHAR* InFontName, float InSize, EFontHinting InHinting = EFontHinting::Default );

public:
	/**
	* We need to verify equality without taking into account some more recently added
	* UPROPERTY.. this is required and used only by the legacy FSlateFontKey.
	*/
	inline bool IsLegacyIdenticalTo(const FSlateFontInfo& Other) const
	{
		return FontObject == Other.FontObject
			&& OutlineSettings.IsIdenticalToForCaching(Other.OutlineSettings)
			&& CompositeFont == Other.CompositeFont
			&& TypefaceFontName == Other.TypefaceFontName
			&& GetClampSize() == Other.GetClampSize();
	}

	inline  bool IsIdenticalTo(const FSlateFontInfo& Other) const
	{
		return FontObject == Other.FontObject
			&& FontMaterial == Other.FontMaterial
			&& OutlineSettings.IsIdenticalTo(Other.OutlineSettings)
			&& CompositeFont == Other.CompositeFont
			&& TypefaceFontName == Other.TypefaceFontName
			&& Size == Other.Size
			&& LetterSpacing == Other.LetterSpacing
			&& SkewAmount == Other.SkewAmount
			&& bForceMonospaced == Other.bForceMonospaced
			&& (bForceMonospaced ? MonospacedWidth == Other.MonospacedWidth : true);
	}

	inline bool operator==(const FSlateFontInfo& Other) const
	{
		return IsIdenticalTo(Other);
	}

	/**
	 * Check to see whether this font info has a valid composite font pointer set (either directly or via a UFont)
	 */
	SLATECORE_API bool HasValidFont() const;

	/**
	 * Get the composite font pointer associated with this font info (either directly or via a UFont)
	 * @note This function will return the fallback font if this font info itself does not contain a valid font. If you want to test whether this font info is empty, use HasValidFont
	 */
	SLATECORE_API const FCompositeFont* GetCompositeFont() const;

	/** Get the font size clamp for the font renderer (on 16bits) */
	SLATECORE_API float GetClampSize() const;

	/** Get the skew amount clamp for the text shaper */
	SLATECORE_API float GetClampSkew() const;

	/**
	* We need a Type Hash that does not take into account some more recently added 
	* UPROPERTY.. this is required and used only by the legacy FSlateFontKey.
	*/
	friend inline uint32 GetLegacyTypeHash(const FSlateFontInfo& FontInfo)
	{
		uint32 Hash = 0;
		Hash = HashCombine(Hash, GetTypeHash(FontInfo.FontObject));
		Hash = HashCombine(Hash, GetTypeHash(FontInfo.CompositeFont));
		Hash = HashCombine(Hash, GetTypeHash(FontInfo.TypefaceFontName));
		Hash = HashCombine(Hash, GetTypeHash(FontInfo.GetClampSize()));
		return Hash;
	}

	/**
	 * Calculates a type hash value for a font info.
	 *
	 * Type hashes are used in certain collection types, such as TMap.
	 *
	 * @param FontInfo The font info to calculate the hash for.
	 * @return The hash value.
	 */
	friend inline uint32 GetTypeHash( const FSlateFontInfo& FontInfo )
	{
		uint32 Hash = 0;
		Hash = HashCombine(Hash, GetTypeHash(FontInfo.FontObject));
		Hash = HashCombine(Hash, GetTypeHash(FontInfo.CompositeFont));
		Hash = HashCombine(Hash, GetTypeHash(FontInfo.FontMaterial));
		Hash = HashCombine(Hash, GetTypeHash(FontInfo.TypefaceFontName));
		Hash = HashCombine(Hash, GetTypeHash(FontInfo.GetClampSize()));
		Hash = HashCombine(Hash, GetTypeHash(FontInfo.LetterSpacing));
		Hash = HashCombine(Hash, GetTypeHash(FontInfo.GetClampSkew()));
		Hash = HashCombine(Hash, GetTypeHash(FontInfo.OutlineSettings));
		return Hash;
	}

#if WITH_EDITORONLY_DATA
	/**
	 * Used to upgrade legacy font into so that it uses composite fonts
	 */
	SLATECORE_API void PostSerialize(const FArchive& Ar);
#endif

	SLATECORE_API void AddReferencedObjects(FReferenceCollector& Collector);

private:

	/**
	 * Used to upgrade legacy font into so that it uses composite fonts
	 */
	void UpgradeLegacyFontInfo(FName LegacyFontName, EFontHinting LegacyHinting);
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FSlateFontInfo>
	: public TStructOpsTypeTraitsBase2<FSlateFontInfo>
{
	enum 
	{
		WithPostSerialize = true,
	};
};
#endif
