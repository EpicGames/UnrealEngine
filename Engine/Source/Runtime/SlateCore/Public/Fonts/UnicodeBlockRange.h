// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Range.h"

/** Enumeration of pre-defined Unicode block ranges that can be used to access entries from FUnicodeBlockRange */
enum class EUnicodeBlockRange : uint16
{
// Macro parameter names here have to be TEXT_KEY and TEXT_LITERAL and not LOCTEXT_KEY/LOCTEXT_LITERAL
// to not confuse the localization tooling.	
#define REGISTER_UNICODE_BLOCK_RANGE(LOWERBOUND, UPPERBOUND, SYMBOLNAME, TEXT_KEY, TEXT_LITERAL) SYMBOLNAME,
#include "UnicodeBlockRange.inl"
#undef REGISTER_UNICODE_BLOCK_RANGE
};

/** Pre-defined Unicode block ranges that can be used with the character ranges in sub-fonts */
struct FUnicodeBlockRange
{
	/** Index enum of this block */
	EUnicodeBlockRange Index;

	/** Display name key for this block. (Use GetDisplayName() method) */
	const TCHAR* DisplayNameKey;

	/** Display name literal for this block. (Use GetDisplayName() method) */
	const TCHAR* DisplayNameLiteral;

	/** Range lower bound of this block. (Use GetRange() method) */
	int32 RangeLower;

	/** Range upper bound of this block. (Use GetRange() method) */
	int32 RangeUpper;

	/** Returns an array containing all of the pre-defined block ranges */
	static SLATECORE_API TArrayView<const FUnicodeBlockRange> GetUnicodeBlockRanges();

	/** Returns the block corresponding to the given enum */
	static SLATECORE_API FUnicodeBlockRange GetUnicodeBlockRange(const EUnicodeBlockRange InBlockIndex);

	/** Resolve display name. */
	FText GetDisplayName() const
	{
		return FText::AsLocalizable_Advanced(TEXT("UnicodeBlock"), DisplayNameKey, DisplayNameLiteral);
	}

	/** Resolve range. */
	FInt32Range GetRange() const
	{
		return FInt32Range(FInt32Range::BoundsType::Inclusive(RangeLower), FInt32Range::BoundsType::Inclusive(RangeUpper));
	}
};
