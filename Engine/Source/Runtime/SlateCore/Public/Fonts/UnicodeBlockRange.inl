// Copyright Epic Games, Inc. All Rights Reserved.

// XMacro header
// #define REGISTER_UNICODE_BLOCK_RANGE(LOWERBOUND, UPPERBOUND, SYMBOLNAME, LOCTEXT_KEY, LOCTEXT_LITERAL) before including this header

REGISTER_UNICODE_BLOCK_RANGE(0x0000, 0x001f, ControlCharacter,							"ControlCharacter", "Control Character")
REGISTER_UNICODE_BLOCK_RANGE(0x0020, 0x007f, BasicLatin,								"BasicLatin", "Basic Latin")
REGISTER_UNICODE_BLOCK_RANGE(0x0080, 0x00ff, Latin1Supplement,							"Latin1Supplement", "Latin - 1 Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x0100, 0x017f, LatinExtendedA,							"LatinExtendedA", "Latin Extended - A")
REGISTER_UNICODE_BLOCK_RANGE(0x0180, 0x024f, LatinExtendedB,							"LatinExtendedB", "Latin Extended - B")
REGISTER_UNICODE_BLOCK_RANGE(0x0250, 0x02af, IPAExtensions,								"IPAExtensions", "IPA Extensions")
REGISTER_UNICODE_BLOCK_RANGE(0x02b0, 0x02ff, SpacingModifierLetters,					"SpacingModifierLetters", "Spacing Modifier Letters")
REGISTER_UNICODE_BLOCK_RANGE(0x0300, 0x036f, CombiningDiacriticalMarks,					"CombiningDiacriticalMarks", "Combining Diacritical Marks")
REGISTER_UNICODE_BLOCK_RANGE(0x0370, 0x03ff, GreekAndCoptic,							"GreekAndCoptic", "Greek and Coptic")
REGISTER_UNICODE_BLOCK_RANGE(0x0400, 0x04ff, Cyrillic,									"Cyrillic", "Cyrillic")
REGISTER_UNICODE_BLOCK_RANGE(0x0500, 0x052f, CyrillicSupplementary,						"CyrillicSupplementary", "Cyrillic Supplementary")
REGISTER_UNICODE_BLOCK_RANGE(0x0530, 0x058f, Armenian,									"Armenian", "Armenian")
REGISTER_UNICODE_BLOCK_RANGE(0x0590, 0x05ff, Hebrew,									"Hebrew", "Hebrew")
REGISTER_UNICODE_BLOCK_RANGE(0x0600, 0x06ff, Arabic,									"Arabic", "Arabic")
REGISTER_UNICODE_BLOCK_RANGE(0x0700, 0x074f, Syriac,									"Syriac", "Syriac")
REGISTER_UNICODE_BLOCK_RANGE(0x0750, 0x077f, ArabicSupplement,							"ArabicSupplement", "Arabic Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x0780, 0x07bf, Thaana,									"Thaana", "Thaana")
REGISTER_UNICODE_BLOCK_RANGE(0x07c0, 0x07ff, NKo,										"NKo", "NKo")
REGISTER_UNICODE_BLOCK_RANGE(0x0800, 0x083f, Samaritan,									"Samaritan", "Samaritan")
REGISTER_UNICODE_BLOCK_RANGE(0x0840, 0x085f, Mandaic,									"Mandaic", "Mandaic")
REGISTER_UNICODE_BLOCK_RANGE(0x08a0, 0x08ff, ArabicExtendedA,							"ArabicExtendedA", "Arabic Extended - A")
REGISTER_UNICODE_BLOCK_RANGE(0x0900, 0x097f, Devanagari,								"Devanagari", "Devanagari")
REGISTER_UNICODE_BLOCK_RANGE(0x0980, 0x09ff, Bengali,									"Bengali", "Bengali")
REGISTER_UNICODE_BLOCK_RANGE(0x0a00, 0x0a7f, Gurmukhi,									"Gurmukhi", "Gurmukhi")
REGISTER_UNICODE_BLOCK_RANGE(0x0a80, 0x0aff, Gujarati,									"Gujarati", "Gujarati")
REGISTER_UNICODE_BLOCK_RANGE(0x0b00, 0x0b7f, Oriya,										"Oriya", "Oriya")
REGISTER_UNICODE_BLOCK_RANGE(0x0b80, 0x0bff, Tamil,										"Tamil", "Tamil")
REGISTER_UNICODE_BLOCK_RANGE(0x0c00, 0x0c7f, Telugu,									"Telugu", "Telugu")
REGISTER_UNICODE_BLOCK_RANGE(0x0c80, 0x0cff, Kannada,									"Kannada", "Kannada")
REGISTER_UNICODE_BLOCK_RANGE(0x0d00, 0x0d7f, Malayalam,									"Malayalam", "Malayalam")
REGISTER_UNICODE_BLOCK_RANGE(0x0d80, 0x0dff, Sinhala,									"Sinhala", "Sinhala")
REGISTER_UNICODE_BLOCK_RANGE(0x0e00, 0x0e7f, Thai,										"Thai", "Thai")
REGISTER_UNICODE_BLOCK_RANGE(0x0e80, 0x0eff, Lao,										"Lao", "Lao")
REGISTER_UNICODE_BLOCK_RANGE(0x0f00, 0x0fff, Tibetan,									"Tibetan", "Tibetan")
REGISTER_UNICODE_BLOCK_RANGE(0x1000, 0x109f, Myanmar,									"Myanmar", "Myanmar")
REGISTER_UNICODE_BLOCK_RANGE(0x10a0, 0x10ff, Georgian,									"Georgian", "Georgian")
REGISTER_UNICODE_BLOCK_RANGE(0x1100, 0x11ff, HangulJamo,								"HangulJamo", "Hangul Jamo")
REGISTER_UNICODE_BLOCK_RANGE(0x1200, 0x137f, Ethiopic,									"Ethiopic", "Ethiopic")
REGISTER_UNICODE_BLOCK_RANGE(0x1380, 0x139f, EthiopicSupplement,						"EthiopicSupplement", "Ethiopic Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x13a0, 0x13ff, Cherokee,									"Cherokee", "Cherokee")
REGISTER_UNICODE_BLOCK_RANGE(0x1400, 0x167f, UnifiedCanadianAboriginalSyllabics,		"UnifiedCanadianAboriginalSyllabics", "Unified Canadian Aboriginal Syllabics")
REGISTER_UNICODE_BLOCK_RANGE(0x1680, 0x169f, Ogham,										"Ogham", "Ogham")
REGISTER_UNICODE_BLOCK_RANGE(0x16a0, 0x16ff, Runic,										"Runic", "Runic")
REGISTER_UNICODE_BLOCK_RANGE(0x1700, 0x171f, Tagalog,									"Tagalog", "Tagalog")
REGISTER_UNICODE_BLOCK_RANGE(0x1720, 0x173f, Hanunoo,									"Hanunoo", "Hanunoo")
REGISTER_UNICODE_BLOCK_RANGE(0x1740, 0x175f, Buhid,										"Buhid", "Buhid")
REGISTER_UNICODE_BLOCK_RANGE(0x1760, 0x177f, Tagbanwa,									"Tagbanwa", "Tagbanwa")
REGISTER_UNICODE_BLOCK_RANGE(0x1780, 0x17ff, Khmer,										"Khmer", "Khmer")
REGISTER_UNICODE_BLOCK_RANGE(0x1800, 0x18af, Mongolian,									"Mongolian", "Mongolian")
REGISTER_UNICODE_BLOCK_RANGE(0x18b0, 0x18ff, UnifiedCanadianAboriginalSyllabicsExtended,"UnifiedCanadianAboriginalSyllabicsExtended", "Unified Canadian Aboriginal Syllabics Extended")
REGISTER_UNICODE_BLOCK_RANGE(0x1900, 0x194f, Limbu,										"Limbu", "Limbu")
REGISTER_UNICODE_BLOCK_RANGE(0x1950, 0x197f, TaiLe,										"TaiLe", "Tai Le")
REGISTER_UNICODE_BLOCK_RANGE(0x1980, 0x19Df, NewTaiLue,									"NewTaiLue", "New Tai Lue")
REGISTER_UNICODE_BLOCK_RANGE(0x19e0, 0x19ff, KhmerSymbols,								"KhmerSymbols", "Khmer Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x1a00, 0x1a1f, Buginese,									"Buginese", "Buginese")
REGISTER_UNICODE_BLOCK_RANGE(0x1a20, 0x1aaf, TaiTham,									"TaiTham", "Tai Tham")
REGISTER_UNICODE_BLOCK_RANGE(0x1ab0, 0x1aff, CombiningDiacriticalMarksExtended,			"CombiningDiacriticalMarksExtended", "Combining Diacritical Marks Extended")
REGISTER_UNICODE_BLOCK_RANGE(0x1b00, 0x1b7f, Balinese,									"Balinese", "Balinese")
REGISTER_UNICODE_BLOCK_RANGE(0x1b80, 0x1bbf, Sundanese,									"Sundanese", "Sundanese")
REGISTER_UNICODE_BLOCK_RANGE(0x1bc0, 0x1bff, Batak,										"Batak", "Batak")
REGISTER_UNICODE_BLOCK_RANGE(0x1c00, 0x1c4f, Lepcha,									"Lepcha", "Lepcha")
REGISTER_UNICODE_BLOCK_RANGE(0x1c50, 0x1c7f, OlChiki,									"OlChiki", "Ol Chiki")
REGISTER_UNICODE_BLOCK_RANGE(0x1c80, 0x1c87, CyrillicExtendedC,							"CyrillicExtendedC", "Cyrillic Extended C")
REGISTER_UNICODE_BLOCK_RANGE(0x1cc0, 0x1ccf, SundaneseSupplement,						"SundaneseSupplement", "Sundanese Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x1cd0, 0x1cff, VedicExtensions,							"VedicExtensions", "Vedic Extensions")
REGISTER_UNICODE_BLOCK_RANGE(0x1d00, 0x1d7f, PhoneticExtensions,						"PhoneticExtensions", "Phonetic Extensions")
REGISTER_UNICODE_BLOCK_RANGE(0x1d80, 0x1dbf, PhoneticExtensionsSupplement,				"PhoneticExtensionsSupplement", "Phonetic Extensions Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x1dc0, 0x1dff, CombiningDiacriticalMarksSupplement,		"CombiningDiacriticalMarksSupplement", "Combining Diacritical Marks Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x1e00, 0x1eff, LatinExtendedAdditional,					"LatinExtendedAdditional", "Latin Extended Additional")
REGISTER_UNICODE_BLOCK_RANGE(0x1f00, 0x1fff, GreekExtended,								"GreekExtended", "Greek Extended")
REGISTER_UNICODE_BLOCK_RANGE(0x2000, 0x206f, GeneralPunctuation,						"GeneralPunctuation", "General Punctuation")
REGISTER_UNICODE_BLOCK_RANGE(0x2070, 0x209f, SuperscriptsAndSubscripts,					"SuperscriptsAndSubscripts", "Superscripts and Subscripts")
REGISTER_UNICODE_BLOCK_RANGE(0x20a0, 0x20cf, CurrencySymbols,							"CurrencySymbols", "Currency Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x20d0, 0x20ff, CombiningDiacriticalMarksForSymbols,		"CombiningDiacriticalMarksForSymbols", "Combining Diacritical Marks for Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x2100, 0x214f, LetterlikeSymbols,							"LetterlikeSymbols", "Letterlike Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x2150, 0x218f, NumberForms,								"NumberForms", "Number Forms")
REGISTER_UNICODE_BLOCK_RANGE(0x2190, 0x21ff, Arrows,									"Arrows", "Arrows")
REGISTER_UNICODE_BLOCK_RANGE(0x2200, 0x22ff, MathematicalOperators,						"MathematicalOperators", "Mathematical Operators")
REGISTER_UNICODE_BLOCK_RANGE(0x2300, 0x23ff, MiscellaneousTechnical,					"MiscellaneousTechnical", "Miscellaneous Technical")
REGISTER_UNICODE_BLOCK_RANGE(0x2400, 0x243f, ControlPictures,							"ControlPictures", "Control Pictures")
REGISTER_UNICODE_BLOCK_RANGE(0x2440, 0x245f, OpticalCharacterRecognition,				"OpticalCharacterRecognition", "Optical Character Recognition")
REGISTER_UNICODE_BLOCK_RANGE(0x2460, 0x24ff, EnclosedAlphanumerics,						"EnclosedAlphanumerics", "Enclosed Alphanumerics")
REGISTER_UNICODE_BLOCK_RANGE(0x2500, 0x257f, BoxDrawing,								"BoxDrawing", "Box Drawing")
REGISTER_UNICODE_BLOCK_RANGE(0x2580, 0x259f, BlockElements,								"BlockElements", "Block Elements")
REGISTER_UNICODE_BLOCK_RANGE(0x25a0, 0x25ff, GeometricShapes,							"GeometricShapes", "Geometric Shapes")
REGISTER_UNICODE_BLOCK_RANGE(0x2600, 0x26ff, MiscellaneousSymbols,						"MiscellaneousSymbols", "Miscellaneous Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x2700, 0x27bf, Dingbats,									"Dingbats", "Dingbats")
REGISTER_UNICODE_BLOCK_RANGE(0x27c0, 0x27ef, MiscellaneousMathematicalSymbolsA,			"MiscellaneousMathematicalSymbolsA", "Miscellaneous Mathematical Symbols - A")
REGISTER_UNICODE_BLOCK_RANGE(0x27f0, 0x27ff, SupplementalArrowsA,						"SupplementalArrowsA", "Supplemental Arrows - A")
REGISTER_UNICODE_BLOCK_RANGE(0x2800, 0x28ff, BraillePatterns,							"BraillePatterns", "Braille Patterns")
REGISTER_UNICODE_BLOCK_RANGE(0x2900, 0x297f, SupplementalArrowsB,						"SupplementalArrowsB", "Supplemental Arrows - B")
REGISTER_UNICODE_BLOCK_RANGE(0x2980, 0x29ff, MiscellaneousMathematicalSymbolsB,			"MiscellaneousMathematicalSymbolsB", "Miscellaneous Mathematical Symbols - B")
REGISTER_UNICODE_BLOCK_RANGE(0x2a00, 0x2aff, SupplementalMathematicalOperators,			"SupplementalMathematicalOperators", "Supplemental Mathematical Operators")
REGISTER_UNICODE_BLOCK_RANGE(0x2b00, 0x2bff, MiscellaneousSymbolsAndArrows,				"MiscellaneousSymbolsAndArrows", "Miscellaneous Symbols and Arrows")
REGISTER_UNICODE_BLOCK_RANGE(0x2c00, 0x2c5f, Glagolitic,								"Glagolitic", "Glagolitic")
REGISTER_UNICODE_BLOCK_RANGE(0x2c60, 0x2c7f, LatinExtendedC,							"LatinExtendedC", "Latin Extended - C")
REGISTER_UNICODE_BLOCK_RANGE(0x2c80, 0x2cff, Coptic,									"Coptic", "Coptic")
REGISTER_UNICODE_BLOCK_RANGE(0x2d00, 0x2d2f, GeorgianSupplement,						"GeorgianSupplement", "Georgian Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x2d30, 0x2d7f, Tifinagh,									"Tifinagh", "Tifinagh")
REGISTER_UNICODE_BLOCK_RANGE(0x2d80, 0x2ddf, EthiopicExtended,							"EthiopicExtended", "Ethiopic Extended")
REGISTER_UNICODE_BLOCK_RANGE(0x2de0, 0x2dff, CyrillicExtendedA,							"CyrillicExtendedA", "Cyrillic Extended - A")
REGISTER_UNICODE_BLOCK_RANGE(0x2e00, 0x2e7f, SupplementalPunctuation,					"SupplementalPunctuation", "Supplemental Punctuation")
REGISTER_UNICODE_BLOCK_RANGE(0x2e80, 0x2eff, CJKRadicalsSupplement,						"CJKRadicalsSupplement", "CJK Radicals Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x2f00, 0x2fdf, KangxiRadicals,							"KangxiRadicals", "Kangxi Radicals")
REGISTER_UNICODE_BLOCK_RANGE(0x2ff0, 0x2fff, IdeographicDescriptionCharacters,			"IdeographicDescriptionCharacters", "Ideographic Description Characters")
REGISTER_UNICODE_BLOCK_RANGE(0x3000, 0x303f, CJKSymbolsAndPunctuation,					"CJKSymbolsAndPunctuation", "CJK Symbols and Punctuation")
REGISTER_UNICODE_BLOCK_RANGE(0x3040, 0x309f, Hiragana,									"Hiragana", "Hiragana")
REGISTER_UNICODE_BLOCK_RANGE(0x30a0, 0x30ff, Katakana,									"Katakana", "Katakana")
REGISTER_UNICODE_BLOCK_RANGE(0x3100, 0x312f, Bopomofo,									"Bopomofo", "Bopomofo")
REGISTER_UNICODE_BLOCK_RANGE(0x3130, 0x318f, HangulCompatibilityJamo,					"HangulCompatibilityJamo", "Hangul Compatibility Jamo")
REGISTER_UNICODE_BLOCK_RANGE(0x3190, 0x319f, Kanbun,									"Kanbun", "Kanbun")
REGISTER_UNICODE_BLOCK_RANGE(0x31a0, 0x31bf, BopomofoExtended,							"BopomofoExtended", "Bopomofo Extended")
REGISTER_UNICODE_BLOCK_RANGE(0x31c0, 0x31ef, CJKStrokes,								"CJKStrokes", "CJK Strokes")
REGISTER_UNICODE_BLOCK_RANGE(0x31f0, 0x31ff, KatakanaPhoneticExtensions,				"KatakanaPhoneticExtensions", "Katakana Phonetic Extensions")
REGISTER_UNICODE_BLOCK_RANGE(0x3200, 0x32ff, EnclosedCJKLettersAndMonths,				"EnclosedCJKLettersAndMonths", "Enclosed CJK Letters and Months")
REGISTER_UNICODE_BLOCK_RANGE(0x3300, 0x33ff, CJKCompatibility,							"CJKCompatibility", "CJK Compatibility")
REGISTER_UNICODE_BLOCK_RANGE(0x3400, 0x4dbf, CJKUnifiedIdeographsExtensionA,			"CJKUnifiedIdeographsExtensionA", "CJK Unified Ideographs Extension A")
REGISTER_UNICODE_BLOCK_RANGE(0x4dc0, 0x4dff, YijingHexagramSymbols,						"YijingHexagramSymbols", "Yijing Hexagram Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x4e00, 0x9fff, CJKUnifiedIdeographs,						"CJKUnifiedIdeographs", "CJK Unified Ideographs")
REGISTER_UNICODE_BLOCK_RANGE(0xa000, 0xa48f, YiSyllables,								"YiSyllables", "Yi Syllables")
REGISTER_UNICODE_BLOCK_RANGE(0xa490, 0xa4cf, YiRadicals,								"YiRadicals", "Yi Radicals")
REGISTER_UNICODE_BLOCK_RANGE(0xa4d0, 0xa4ff, Lisu,										"Lisu", "Lisu")
REGISTER_UNICODE_BLOCK_RANGE(0xa500, 0xa63f, Vai,										"Vai", "Vai")
REGISTER_UNICODE_BLOCK_RANGE(0xa640, 0xa69f, CyrillicExtendedB,							"CyrillicExtendedB", "Cyrillic Extended - B")
REGISTER_UNICODE_BLOCK_RANGE(0xa6a0, 0xa6ff, Bamum,										"Bamum", "Bamum")
REGISTER_UNICODE_BLOCK_RANGE(0xa700, 0xa71f, ModifierToneLetters,						"ModifierToneLetters", "Modifier Tone Letters")
REGISTER_UNICODE_BLOCK_RANGE(0xa720, 0xa7ff, LatinExtendedD,							"LatinExtendedD", "Latin Extended - D")
REGISTER_UNICODE_BLOCK_RANGE(0xa800, 0xa82f, SylotiNagri,								"SylotiNagri", "Syloti Nagri")
REGISTER_UNICODE_BLOCK_RANGE(0xa830, 0xa83f, CommonIndicNumberForms,					"CommonIndicNumberForms", "Common Indic Number Forms")
REGISTER_UNICODE_BLOCK_RANGE(0xa840, 0xa87f, PhagsPa,									"PhagsPa", "Phags - pa")
REGISTER_UNICODE_BLOCK_RANGE(0xa880, 0xa8df, Saurashtra,								"Saurashtra", "Saurashtra")
REGISTER_UNICODE_BLOCK_RANGE(0xa8e0, 0xa8ff, DevanagariExtended,						"DevanagariExtended", "Devanagari Extended")
REGISTER_UNICODE_BLOCK_RANGE(0xa900, 0xa92f, KayahLi,									"KayahLi", "Kayah Li")
REGISTER_UNICODE_BLOCK_RANGE(0xa930, 0xa95f, Rejang,									"Rejang", "Rejang")
REGISTER_UNICODE_BLOCK_RANGE(0xa960, 0xa97f, HangulJamoExtendedA,						"HangulJamoExtendedA", "Hangul Jamo Extended - A")
REGISTER_UNICODE_BLOCK_RANGE(0xa980, 0xa9df, Javanese,									"Javanese", "Javanese")
REGISTER_UNICODE_BLOCK_RANGE(0xa9e0, 0xa9ff, MyanmarExtendedB,							"MyanmarExtendedB", "Myanmar Extended - B")
REGISTER_UNICODE_BLOCK_RANGE(0xaa00, 0xaa5f, Cham,										"Cham", "Cham")
REGISTER_UNICODE_BLOCK_RANGE(0xaa60, 0xaa7f, MyanmarExtendedA,							"MyanmarExtendedA", "Myanmar Extended - A")
REGISTER_UNICODE_BLOCK_RANGE(0xaa80, 0xaadf, TaiViet,									"TaiViet", "Tai Viet")
REGISTER_UNICODE_BLOCK_RANGE(0xaae0, 0xaaff, MeeteiMayekExtensions,						"MeeteiMayekExtensions", "Meetei Mayek Extensions")
REGISTER_UNICODE_BLOCK_RANGE(0xab00, 0xab2f, EthiopicExtendedA,							"EthiopicExtendedA", "Ethiopic Extended - A")
REGISTER_UNICODE_BLOCK_RANGE(0xab30, 0xab6f, LatinExtendedE,							"LatinExtendedE", "Latin Extended - E")
REGISTER_UNICODE_BLOCK_RANGE(0xab70, 0xabbf, CherokeeSupplement,						"CherokeeSupplement", "Cherokee Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0xabc0, 0xabff, MeeteiMayek,								"MeeteiMayek", "Meetei Mayek")
REGISTER_UNICODE_BLOCK_RANGE(0xac00, 0xd7af, HangulSyllables,							"HangulSyllables", "Hangul Syllables")
REGISTER_UNICODE_BLOCK_RANGE(0xd7b0, 0xd7ff, HangulJamoExtendedB,						"HangulJamoExtendedB", "Hangul Jamo Extended - B")
REGISTER_UNICODE_BLOCK_RANGE(0xd800, 0xdb7f, HighSurrogates,							"HighSurrogates", "High Surrogates")
REGISTER_UNICODE_BLOCK_RANGE(0xdb80, 0xdbff, HighPrivateUseSurrogates,					"HighPrivateUseSurrogates", "High Private Use Surrogates")
REGISTER_UNICODE_BLOCK_RANGE(0xdc00, 0xdfff, LowSurrogates,								"LowSurrogates", "Low Surrogates")
REGISTER_UNICODE_BLOCK_RANGE(0xe000, 0xf8ff, PrivateUseArea,							"PrivateUseArea", "Private Use Area")
REGISTER_UNICODE_BLOCK_RANGE(0xf900, 0xfaff, CJKCompatibilityIdeographs,				"CJKCompatibilityIdeographs", "CJK Compatibility Ideographs")
REGISTER_UNICODE_BLOCK_RANGE(0xfb00, 0xfb4f, AlphabeticPresentationForms,				"AlphabeticPresentationForms", "Alphabetic Presentation Forms")
REGISTER_UNICODE_BLOCK_RANGE(0xfb50, 0xfdff, ArabicPresentationFormsA,					"ArabicPresentationFormsA", "Arabic Presentation Forms - A")
REGISTER_UNICODE_BLOCK_RANGE(0xfe00, 0xfe0f, VariationSelectors,						"VariationSelectors", "Variation Selectors")
REGISTER_UNICODE_BLOCK_RANGE(0xfe10, 0xfe1f, VerticalForms,								"VerticalForms", "Vertical Forms")
REGISTER_UNICODE_BLOCK_RANGE(0xfe20, 0xfe2f, CombiningHalfMarks,						"CombiningHalfMarks", "Combining Half Marks")
REGISTER_UNICODE_BLOCK_RANGE(0xfe30, 0xfe4f, CJKCompatibilityForms,						"CJKCompatibilityForms", "CJK Compatibility Forms")
REGISTER_UNICODE_BLOCK_RANGE(0xfe50, 0xfe6f, SmallFormVariants,							"SmallFormVariants", "Small Form Variants")
REGISTER_UNICODE_BLOCK_RANGE(0xfe70, 0xfeff, ArabicPresentationFormsB,					"ArabicPresentationFormsB", "Arabic Presentation Forms - B")
REGISTER_UNICODE_BLOCK_RANGE(0xff00, 0xffef, HalfwidthAndFullwidthForms,				"HalfwidthAndFullwidthForms", "Halfwidth and Fullwidth Forms")
REGISTER_UNICODE_BLOCK_RANGE(0xfff0, 0xffff, Specials,									"Specials", "Specials")

// The following are outside the Basic Multilingual Plane, so may not be supported by UE on all platforms
REGISTER_UNICODE_BLOCK_RANGE(0x10000, 0x1007f, LinearBSyllabary,						"LinearBSyllabary", "Linear B Syllabary")
REGISTER_UNICODE_BLOCK_RANGE(0x10080, 0x100ff, LinearBIdeograms,						"LinearBIdeograms", "Linear B Ideograms")
REGISTER_UNICODE_BLOCK_RANGE(0x10100, 0x1013f, AegeanNumbers,							"AegeanNumbers", "Aegean Numbers")
REGISTER_UNICODE_BLOCK_RANGE(0x10140, 0x1018f, AncientGreekNumbers,						"AncientGreekNumbers", "Ancient Greek Numbers")
REGISTER_UNICODE_BLOCK_RANGE(0x10190, 0x101cf, AncientSymbols,							"AncientSymbols", "Ancient Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x101d0, 0x101ff, PhaistosDisc,							"PhaistosDisc", "Phaistos Disc")
REGISTER_UNICODE_BLOCK_RANGE(0x10280, 0x1029f, Lycian,									"Lycian", "Lycian")
REGISTER_UNICODE_BLOCK_RANGE(0x102a0, 0x102df, Carian,									"Carian", "Carian")
REGISTER_UNICODE_BLOCK_RANGE(0x102e0, 0x102ff, CopticEpactNumbers,						"CopticEpactNumbers", "Coptic Epact Numbers")
REGISTER_UNICODE_BLOCK_RANGE(0x10300, 0x1032f, OldItalic,								"OldItalic", "Old Italic")
REGISTER_UNICODE_BLOCK_RANGE(0x10330, 0x1034f, Gothic,									"Gothic", "Gothic")
REGISTER_UNICODE_BLOCK_RANGE(0x10350, 0x1037f, OldPermic,								"OldPermic", "Old Permic")
REGISTER_UNICODE_BLOCK_RANGE(0x10380, 0x1039f, Ugaritic,								"Ugaritic", "Ugaritic")
REGISTER_UNICODE_BLOCK_RANGE(0x103a0, 0x103df, OldPersian,								"Old Persian", "Old Persian")
REGISTER_UNICODE_BLOCK_RANGE(0x10400, 0x1044f, Deseret,									"Deseret", "Deseret")
REGISTER_UNICODE_BLOCK_RANGE(0x10450, 0x1047f, Shavian,									"Shavian", "Shavian")
REGISTER_UNICODE_BLOCK_RANGE(0x10480, 0x104af, Osmanya,									"Osmanya", "Osmanya")
REGISTER_UNICODE_BLOCK_RANGE(0x104b0, 0x104ff, Osage,									"Osage", "Osage")
REGISTER_UNICODE_BLOCK_RANGE(0x10500, 0x1052f, Elbasan,									"Elbasan", "Elbasan")
REGISTER_UNICODE_BLOCK_RANGE(0x10530, 0x1056f, CaucasianAlbanian,						"CaucasianAlbanian", "Caucasian Albanian")
REGISTER_UNICODE_BLOCK_RANGE(0x10600, 0x1077f, LinearA,									"LinearA", "Linear A")
REGISTER_UNICODE_BLOCK_RANGE(0x10800, 0x1083f, CypriotSyllabary,						"CypriotSyllabary", "Cypriot Syllabary")
REGISTER_UNICODE_BLOCK_RANGE(0x10840, 0x1085f, ImperialAramaic,							"ImperialAramaic", "Imperial Aramaic")
REGISTER_UNICODE_BLOCK_RANGE(0x10860, 0x1087f, Palmyrene,								"Palmyrene", "Palmyrene")
REGISTER_UNICODE_BLOCK_RANGE(0x10880, 0x108af, Nabataean,								"Nabataean", "Nabataean")
REGISTER_UNICODE_BLOCK_RANGE(0x108e0, 0x108ff, Hatran,									"Hatran", "Hatran")
REGISTER_UNICODE_BLOCK_RANGE(0x10900, 0x1091f, Phoenician,								"Phoenician", "Phoenician")
REGISTER_UNICODE_BLOCK_RANGE(0x10920, 0x1093f, Lydian,									"Lydian", "Lydian")
REGISTER_UNICODE_BLOCK_RANGE(0x10980, 0x1099f, MeroiticHieroglyphs,						"MeroiticHieroglyphs", "Meroitic Hieroglyphs")
REGISTER_UNICODE_BLOCK_RANGE(0x109a0, 0x109ff, MeroiticCursive,							"MeroiticCursive", "Meroitic Cursive")
REGISTER_UNICODE_BLOCK_RANGE(0x10a00, 0x10a5f, Kharoshthi,								"Kharoshthi", "Kharoshthi")
REGISTER_UNICODE_BLOCK_RANGE(0x10a60, 0x10a7f, OldSouthArabian,							"OldSouthArabian", "Old South Arabian")
REGISTER_UNICODE_BLOCK_RANGE(0x10a80, 0x10a9f, OldNorthArabian,							"OldNorthArabian", "Old North Arabian")
REGISTER_UNICODE_BLOCK_RANGE(0x10ac0, 0x10aff, Manichaean,								"Manichaean", "Manichaean")
REGISTER_UNICODE_BLOCK_RANGE(0x10b00, 0x10b3f, Avestan,									"Avestan", "Avestan")
REGISTER_UNICODE_BLOCK_RANGE(0x10b40, 0x10b5f, InscriptionalParthian,					"InscriptionalParthian", "Inscriptional Parthian")
REGISTER_UNICODE_BLOCK_RANGE(0x10b60, 0x10b7f, InscriptionalPahlavi,					"InscriptionalPahlavi", "Inscriptional Pahlavi")
REGISTER_UNICODE_BLOCK_RANGE(0x10b80, 0x10baf, PsalterPahlavi,							"PsalterPahlavi", "Psalter Pahlavi")
REGISTER_UNICODE_BLOCK_RANGE(0x10c00, 0x10c4f, OldTurkic,								"OldTurkic", "Old Turkic")
REGISTER_UNICODE_BLOCK_RANGE(0x10c80, 0x10cff, OldHungarian,							"OldHungarian", "Old Hungarian")
REGISTER_UNICODE_BLOCK_RANGE(0x10e60, 0x10e7f, RumiNumeralSymbols,						"RumiNumeralSymbols", "Rumi Numeral Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x11000, 0x1107f, Brahmi,									"Brahmi", "Brahmi")
REGISTER_UNICODE_BLOCK_RANGE(0x11080, 0x110cf, Kaithi,									"Kaithi", "Kaithi")
REGISTER_UNICODE_BLOCK_RANGE(0x110d0, 0x110ff, SoraSompeng,								"SoraSompeng", "Sora Sompeng")
REGISTER_UNICODE_BLOCK_RANGE(0x11100, 0x1114f, Chakma,									"Chakma", "Chakma")
REGISTER_UNICODE_BLOCK_RANGE(0x11150, 0x1117f, Mahajani,								"Mahajani", "Mahajani")
REGISTER_UNICODE_BLOCK_RANGE(0x11180, 0x111df, Sharada,									"Sharada", "Sharada")
REGISTER_UNICODE_BLOCK_RANGE(0x111e0, 0x111ff, SinhalaArchaicNumbers,					"SinhalaArchaicNumbers", "Sinhala Archaic Numbers")
REGISTER_UNICODE_BLOCK_RANGE(0x11200, 0x1124f, Khojki,									"Khojki", "Khojki")
REGISTER_UNICODE_BLOCK_RANGE(0x11280, 0x112af, Multani,									"Multani", "Multani")
REGISTER_UNICODE_BLOCK_RANGE(0x112b0, 0x112ff, Khudawadi,								"Khudawadi", "Khudawadi")
REGISTER_UNICODE_BLOCK_RANGE(0x11300, 0x1137f, Grantha,									"Grantha", "Grantha")
REGISTER_UNICODE_BLOCK_RANGE(0x11400, 0x1147f, Newa,									"Newa", "Newa")
REGISTER_UNICODE_BLOCK_RANGE(0x11480, 0x114df, Tirhuta,									"Tirhuta", "Tirhuta")
REGISTER_UNICODE_BLOCK_RANGE(0x11580, 0x115ff, Siddham,									"Siddham", "Siddham")
REGISTER_UNICODE_BLOCK_RANGE(0x11600, 0x1165f, Modi,									"Modi", "Modi")
REGISTER_UNICODE_BLOCK_RANGE(0x11660, 0x1167f, MongolianSupplement,						"MongolianSupplement", "Mongolian Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x11680, 0x116cf, Takri,									"Takri", "Takri")
REGISTER_UNICODE_BLOCK_RANGE(0x11700, 0x1173f, Ahom,									"Ahom", "Ahom")
REGISTER_UNICODE_BLOCK_RANGE(0x118a0, 0x118ff, WarangCiti,								"WarangCiti", "Warang Citi")
REGISTER_UNICODE_BLOCK_RANGE(0x11ac0, 0x11aff, PauCinHau,								"PauCinHau", "Pau Cin Hau")
REGISTER_UNICODE_BLOCK_RANGE(0x11c00, 0x11c6f, Bhaiksuki,								"Bhaiksuki", "Bhaiksuki")
REGISTER_UNICODE_BLOCK_RANGE(0x11c70, 0x11cbf, Marchen,									"Marchen", "Marchen")
REGISTER_UNICODE_BLOCK_RANGE(0x12000, 0x123ff, Cuneiform,								"Cuneiform", "Cuneiform")
REGISTER_UNICODE_BLOCK_RANGE(0x12400, 0x1247f, CuneiformNumbersAndPunctuation,			"CuneiformNumbersAndPunctuation", "Cuneiform Numbers and Punctuation")
REGISTER_UNICODE_BLOCK_RANGE(0x12480, 0x1254f, EarlyDynasticCuneiform,					"EarlyDynasticCuneiform", "Early Dynastic Cuneiform")
REGISTER_UNICODE_BLOCK_RANGE(0x13000, 0x1342f, EgyptianHieroglyphs,						"EgyptianHieroglyphs", "Egyptian Hieroglyphs")
REGISTER_UNICODE_BLOCK_RANGE(0x14400, 0x1467f, AnatolianHieroglyphs,					"AnatolianHieroglyphs", "Anatolian Hieroglyphs")
REGISTER_UNICODE_BLOCK_RANGE(0x16800, 0x16a3f, BamumSupplement,							"BamumSupplement", "Bamum Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x16a40, 0x16a6f, Mro,										"Mro", "Mro")
REGISTER_UNICODE_BLOCK_RANGE(0x16ad0, 0x16aff, BassaVah,								"BassaVah", "Bassa Vah")
REGISTER_UNICODE_BLOCK_RANGE(0x16b00, 0x16b8f, PahawhHmong,								"PahawhHmong", "Pahawh Hmong")
REGISTER_UNICODE_BLOCK_RANGE(0x16f00, 0x16f9f, Miao,									"Miao", "Miao")
REGISTER_UNICODE_BLOCK_RANGE(0x16fe0, 0x16fff, IdeographicSymbolsAndPunctuation,		"IdeographicSymbolsAndPunctuation", "Ideographic Symbols and Punctuation")
REGISTER_UNICODE_BLOCK_RANGE(0x17000, 0x187ff, Tangut,									"Tangut", "Tangut")
REGISTER_UNICODE_BLOCK_RANGE(0x18800, 0x18aff, TangutComponents,						"TangutComponents", "Tangut Components")
REGISTER_UNICODE_BLOCK_RANGE(0x1b000, 0x1b0ff, KanaSupplement,							"KanaSupplement", "Kana Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x1bc00, 0x1bc9f, Duployan,								"Duployan", "Duployan")
REGISTER_UNICODE_BLOCK_RANGE(0x1bca0, 0x1bcaf, ShorthandFormatControls,					"ShorthandFormatControls", "Shorthand Format Controls")
REGISTER_UNICODE_BLOCK_RANGE(0x1d000, 0x1d0ff, ByzantineMusicalSymbols,					"ByzantineMusicalSymbols", "Byzantine Musical Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x1d100, 0x1d1ff, MusicalSymbols,							"MusicalSymbols", "Musical Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x1d200, 0x1d24f, AncientGreekMusicalNotation,				"AncientGreekMusicalNotation", "Ancient Greek Musical Notation")
REGISTER_UNICODE_BLOCK_RANGE(0x1d300, 0x1d35f, TaiXuanJingSymbols,						"TaiXuanJingSymbols", "Tai Xuan Jing Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x1d360, 0x1d37f, CountingRodNumerals,						"CountingRodNumerals", "Counting Rod Numerals")
REGISTER_UNICODE_BLOCK_RANGE(0x1d400, 0x1d7ff, MathematicalAlphanumericSymbols,			"MathematicalAlphanumericSymbols", "Mathematical Alphanumeric Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x1d800, 0x1daaf, SuttonSignWriting,						"SuttonSignWriting", "Sutton SignWriting")
REGISTER_UNICODE_BLOCK_RANGE(0x1e000, 0x1e02f, GlagoliticSupplement,					"GlagoliticSupplement", "Glagolitic Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x1e800, 0x1e8df, MendeKikakui,							"MendeKikakui", "Mende Kikakui")
REGISTER_UNICODE_BLOCK_RANGE(0x1e900, 0x1e95f, Adlam,									"Adlam", "Adlam")
REGISTER_UNICODE_BLOCK_RANGE(0x1ee00, 0x1eeff, ArabicMathematicalAlphabeticSymbols,		"ArabicMathematicalAlphabeticSymbols", "Arabic Mathematical Alphabetic Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x1f000, 0x1f02f, MahjongTiles,							"MahjongTiles", "Mahjong Tiles")
REGISTER_UNICODE_BLOCK_RANGE(0x1f030, 0x1f09f, DominoTiles,								"DominoTiles", "Domino Tiles")
REGISTER_UNICODE_BLOCK_RANGE(0x1f0a0, 0x1f0ff, PlayingCards,							"PlayingCards", "Playing Cards")
REGISTER_UNICODE_BLOCK_RANGE(0x1f100, 0x1f1ff, EnclosedAlphanumericSupplement,			"EnclosedAlphanumericSupplement", "Enclosed Alphanumeric Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x1f200, 0x1f2ff, EnclosedIdeographicSupplement,			"EnclosedIdeographicSupplement", "Enclosed Ideographic Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0x1f300, 0x1f5ff, MiscellaneousSymbolsAndPictographs,		"MiscellaneousSymbolsAndPictographs", "Miscellaneous Symbols and Pictographs")
REGISTER_UNICODE_BLOCK_RANGE(0x1f600, 0x1f64f, EmoticonsEmoji,							"EmoticonsEmoji", "Emoticons (Emoji)")
REGISTER_UNICODE_BLOCK_RANGE(0x1f650, 0x1f67f, OrnamentalDingbats,						"OrnamentalDingbats", "Ornamental Dingbats")
REGISTER_UNICODE_BLOCK_RANGE(0x1f680, 0x1f6ff, TransportAndMapSymbols,					"TransportAndMapSymbols", "Transport and Map Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x1f700, 0x1f77f, AlchemicalSymbols,						"AlchemicalSymbols", "Alchemical Symbols")
REGISTER_UNICODE_BLOCK_RANGE(0x1f780, 0x1f7ff, GeometricShapesExtended,					"GeometricShapesExtended", "Geometric Shapes Extended")
REGISTER_UNICODE_BLOCK_RANGE(0x1f800, 0x1f8ff, SupplementalArrowsC,						"SupplementalArrowsC", "Supplemental Arrows - C")
REGISTER_UNICODE_BLOCK_RANGE(0x1f900, 0x1f9ff, SupplementalSymbolsAndPictographs,		"SupplementalSymbolsAndPictographs", "Supplemental Symbols and Pictographs")
REGISTER_UNICODE_BLOCK_RANGE(0x20000, 0x2a6d6, CJKUnifiedIdeographsExtensionB,			"CJKUnifiedIdeographsExtensionB", "CJK Unified Ideographs Extension B")
REGISTER_UNICODE_BLOCK_RANGE(0x2a700, 0x2b734, CJKUnifiedIdeographsExtensionC,			"CJKUnifiedIdeographsExtensionC", "CJK Unified Ideographs Extension C")
REGISTER_UNICODE_BLOCK_RANGE(0x2b740, 0x2b81d, CJKUnifiedIdeographsExtensionD,			"CJKUnifiedIdeographsExtensionD", "CJK Unified Ideographs Extension D")
REGISTER_UNICODE_BLOCK_RANGE(0x2b820, 0x2cea1, CJKUnifiedIdeographsExtensionE,			"CJKUnifiedIdeographsExtensionE", "CJK Unified Ideographs Extension E")
REGISTER_UNICODE_BLOCK_RANGE(0x2f800, 0x2fa1f, CJKCompatibilityIdeographsSupplement,	"CJKCompatibilityIdeographsSupplement", "CJK Compatibility Ideographs Supplement")
REGISTER_UNICODE_BLOCK_RANGE(0xe0000, 0xe007f, Tags,									"Tags", "Tags")
REGISTER_UNICODE_BLOCK_RANGE(0xe0100, 0xe01ef, VariationSelectorsSupplement,			"VariationSelectorsSupplement", "Variation Selectors Supplement")
