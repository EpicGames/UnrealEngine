// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/ICursor.h"
#include "Input/ReplyBase.h"

class SWidget;
class SWindow;

/**
 * A reply to the OnQueryCursor event.
 */
class FCursorReply : public TReplyBase<FCursorReply>
{
public:

	/**
	 * Makes a NULL response meaning no prefersce.
	 * i.e. If your widget returns this, its parent will get to decide what the cursor shoudl be.
	 * This is the default behavior for a widget.
	 */
	static FCursorReply Unhandled()
	{
		return FCursorReply();
	}
		
	/**
	 * Respond with a specific cursor.
	 * This cursor will be used and no other widgets will be asked.
	 */
	static FCursorReply Cursor( EMouseCursor::Type InCursor )
	{
		return FCursorReply( InCursor );
	}

	/** @return The window to render the Cursor Widget in. */
	TSharedPtr<SWindow> GetCursorWindow() const { return CursorWindow; }
	
	/** @return The custom Cursor Widget to render if set and the event was handled. */
	TSharedPtr<SWidget> GetCursorWidget() const { return CursorWidget; }

	/** @return The requested MouseCursor if no custom widget is set and the event was handled. */
	EMouseCursor::Type GetCursorType() const { return MouseCursor; }

	/** Set the Cursor Widget, used by slate application to set the cursor widget if the MapCursor returns a widget. */
	void SetCursorWidget(TSharedPtr<SWindow> InCursorWindow, TSharedPtr<SWidget> InCursorWidget) { CursorWindow = InCursorWindow; CursorWidget = InCursorWidget; }

private:

	/** Internal constructor - makes a NULL result. */
	FCursorReply()
		: TReplyBase<FCursorReply>(false)
		, MouseCursor( EMouseCursor::Default )
	{ }
		
	/** Internal constructor - makes a non-NULL result. */
	FCursorReply( EMouseCursor::Type InCursorType )
		: TReplyBase<FCursorReply>(true)
		, MouseCursor( InCursorType )
	{ }

	/** Window to render for cursor */
	TSharedPtr<SWindow> CursorWindow;

	/** Custom widget to render for cursor */
	TSharedPtr<SWidget> CursorWidget;
		
	/** The cursor type must be set is CursorWidget is invalid */
	EMouseCursor::Type MouseCursor;
};
