// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/CursorReply.h"
#include "Input/Events.h"

class SWidget;
class SWindow;

/**
 * A base class for a DragAndDrop operation which supports reflection.
 * Drag and Drop is inherently quite stateful.
 * Implementing a custom DragDropOperation allows for dedicated handling of
 * a drag drop operation which keeps a decorator window (optionally)
 * Implement derived types with DRAG_DROP_OPERATOR_TYPE (see below)
 */
class FDragDropOperation
	: public TSharedFromThis<FDragDropOperation>
{
public:

	/**
	 * Default constructor.
	 */
	SLATECORE_API FDragDropOperation();

	/**
	 * Destructor.
	 */
	SLATECORE_API virtual ~FDragDropOperation( );

	/** Check if this drag and drop operation can cast safely to the specified template type */
	template<class TType> bool IsOfType() const 
	{
		return IsOfTypeImpl(TType::GetTypeId());
	}

	/**
	* Checks if this drag and drop operation is affected by a given PointerEvent.
	*
	* @param PointerEvent        The mouse event we are checking against
	* @return true if the the pointer event affects this drag and drop operation (default true).
	*/
	virtual bool AffectedByPointerEvent(const FPointerEvent& PointerEvent) {return true;}

	/**
	 * Invoked when the drag and drop operation has ended.
	 * 
	 * @param bDropWasHandled   true when the drop was handled by some widget; false otherwise
	 * @param MouseEvent        The mouse event which caused the on drop to be called.
	 */
	SLATECORE_API virtual void OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent );

	/** 
	 * Called when the mouse was moved during a drag and drop operation
	 *
	 * @param DragDropEvent    The event that describes this drag drop operation.
	 */
	SLATECORE_API virtual void OnDragged( const class FDragDropEvent& DragDropEvent );
	
	/** Allows drag/drop operations to override the current cursor */
	SLATECORE_API virtual FCursorReply OnCursorQuery();

	/**
	 * Gets the widget that will serve as the decorator unless overridden. 
	 * If you do not override, you will have no decorator
	 */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const { return TSharedPtr<SWidget>(); }

	/** Gets the desired position of the decorator in absolute space. */
	virtual FVector2D GetDecoratorPosition() const { return FVector2D(0, 0); }
	
	/** Alters the visibility of the window */
	SLATECORE_API virtual void SetDecoratorVisibility(bool bVisible);

	/** Is this drag Drop operation going to interact with applications outside of Slate */
	virtual bool IsExternalOperation() const { return false; }

	/** @return true if the operation does not create a decorator window.  Used for game drag drop operations. */
	virtual bool IsWindowlessOperation() const { return bCreateNewWindow == false; }

	/** 
	 * Sets the cursor to override the drag drop operations cursor with so that a 
	 * control can give temporary feedback, for example - a slashed circle telling 
	 * the user 'this can't be dropped here'.
	 */
	SLATECORE_API void SetCursorOverride( TOptional<EMouseCursor::Type> CursorType );

	/**
	 * Checks whether this drag and drop operation can cast safely to the specified type.
	 */
	virtual bool IsOfTypeImpl(const FString& Type) const
	{
		return false;
	}
	 
	virtual FStringView GetTypeIdString() const
	{
		return FStringView();
	}
protected:

	/**
	 * Constructs the window and widget if applicable.
	 */
	SLATECORE_API virtual void Construct();

	/**
	 * Creates a window for the cursor decorator.
	 */
	SLATECORE_API void CreateCursorDecoratorWindow();

	/**
	 * Destroys the cursor decorator window.
	 */
	SLATECORE_API void DestroyCursorDecoratorWindow();

	virtual TSharedPtr<FDragDropOperation> ConvertTo(const FString& TypeId)
	{
		return IsOfTypeImpl(TypeId) ? AsShared() : TSharedPtr<FDragDropOperation>();
	}

protected:

	/**
	 * Create a new window for the decorator content.  If this is false,
	 * SlateApplication will draw the decorator directly using the 
	 */
	bool bCreateNewWindow;

	/** The window that owns the decorator widget */
	TSharedPtr<SWindow> CursorDecoratorWindow;

	/** Mouse cursor used by the drag drop operation */
	TOptional<EMouseCursor::Type> MouseCursor;

	/** Mouse cursor used to temporarily replace the drag drops cursor */
	TOptional<EMouseCursor::Type> MouseCursorOverride;
};


/** Like a mouse event but with content */
class FDragDropEvent : public FPointerEvent
{
public:
	 /**
	  * Construct a DragDropEvent.
	  *
	  * @param InMouseEvent  The mouse event that is causing this drag and drop event to occur.
	  * @param InContent     The content being dragged.
	  */
	 FDragDropEvent( const FPointerEvent& InMouseEvent, const TSharedPtr<FDragDropOperation> InContent )
		: FPointerEvent( InMouseEvent )
		, Content( InContent )
	{
	}

	/** @return the content being dragged */
	TSharedPtr<FDragDropOperation> GetOperation() const
	{
		return Content;
	}

	/** @return the content being dragged if it matched the 'OperationType'; invalid Ptr otherwise. */
	template<typename OperationType>
	TSharedPtr<OperationType> GetOperationAs() const;

private:
	/** the content being dragged */
	TSharedPtr<FDragDropOperation> Content;
};


/**
 * Invoked when a drag and drop is finished.
 * This allows the widget that started the drag/drop to respond to the end of the operation.
 *
 * @param bWasDropHandled   True when the drag and drop operation was handled by some widget;
 *                          False when no widget handled the drop.
 * @param DragDropEvent     The Drop event that terminated the whole DragDrop operation
 */
DECLARE_DELEGATE_TwoParams( FOnDragDropEnded,
	bool /* bWasDropHandled */,
	const FDragDropEvent& /* DragDropEvent */
)

/**
 * A delegate invoked on the initiator of the DragDrop operation.
 * This delegate is invoked periodically during the DragDrop, and
 * gives the initiator an opportunity to respond to the current state of
 * the process. e.g. Create and update a custom cursor.
 */
DECLARE_DELEGATE_OneParam( FOnDragDropUpdate,
	const FDragDropEvent& /* DragDropEvent */
)



/**
 * All drag drop operations that require type checking must include this macro.
 * Example Usage:
 *	class FMyDragDropOp : public FDragDropOperation
 *	{
 *	public:
 *		DRAG_DROP_OPERATOR_TYPE(FMyDragDropOp, FDragDropOperation)
 *		...
 *	};
 */

#define DRAG_DROP_OPERATOR_TYPE(TYPE, BASE) \
	static const FString& GetTypeId() { static FString Type = TEXT(#TYPE); return Type; } \
	virtual FStringView GetTypeIdString() const override { return GetTypeId(); } \
	virtual bool IsOfTypeImpl(const FString& Type) const override { return GetTypeId() == Type || BASE::IsOfTypeImpl(Type); }


/**
 * An external drag and drop operation that originates outside of slate.
 * E.g. an OLE drag and drop.
 */
class FExternalDragOperation : public FDragDropOperation
{
private:
	/** A private constructor to ensure that the appropriate "New" factory method is used below */
	FExternalDragOperation(){}

	virtual bool IsExternalOperation() const override { return true; }

public:
	DRAG_DROP_OPERATOR_TYPE(FExternalDragOperation, FDragDropOperation)

	/** Creates a new external text drag operation */
	static SLATECORE_API TSharedRef<FExternalDragOperation> NewText( FString InText );
	/** Creates a new external file drag operation */
	static SLATECORE_API TSharedRef<FExternalDragOperation> NewFiles( TArray<FString> InFileNames );
	/** Creates a new external combined drag operation */
	static SLATECORE_API TSharedRef<FExternalDragOperation> NewOperation( FString InText, TArray<FString> InFileNames );

	/** @return true if this is a text drag operation */
	bool HasText() const {return !!(DragType & DragText);}
	/** @return true if this is a file drag operation */
	bool HasFiles() const {return !!(DragType & DragFiles);}
	
	/** @return the text from this drag operation */
	const FString& GetText() const { ensure(HasText()); return DraggedText;}
	/** @return the filenames from this drag operation */
	const TArray<FString>& GetFiles() const { ensure(HasFiles()); return DraggedFileNames;}

private:
	FString DraggedText;
	TArray<FString> DraggedFileNames;

	enum EExternalDragType
	{
		DragText = 1<<0,
		DragFiles = 1<<1,
	};
	uint8 DragType;
};


class FGameDragDropOperation : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FGameDragDropOperation, FDragDropOperation)

	/**
	* Default constructor.
	*/
	SLATECORE_API FGameDragDropOperation();

	SLATECORE_API virtual FVector2D GetDecoratorPosition() const override;

protected:
	/** The absolute position of the decorator. */
	FVector2D DecoratorPosition;
};

#include "Input/DragAndDrop.inl" // IWYU pragma: export
