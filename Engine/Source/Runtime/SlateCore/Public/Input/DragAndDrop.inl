// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FDragDropEvent;

/** @return the content being dragged if it matched the 'OperationType'; invalid Ptr otherwise. */
template<typename OperationType>
TSharedPtr<OperationType> FDragDropEvent::GetOperationAs() const
{
	if (Content.IsValid() && Content->IsOfType<OperationType>())
	{
		return StaticCastSharedPtr<OperationType>(Content);
	}
	else
	{
		return TSharedPtr<OperationType>();
	}
}