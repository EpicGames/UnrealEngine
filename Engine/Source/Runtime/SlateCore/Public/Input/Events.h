// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "InputCoreTypes.h"
#include "Types/SlateEnums.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Layout/Geometry.h"
#include "Types/SlateVector2.h"
#include "Events.generated.h"

class FWidgetPath;
class SWidget;
class SWindow;

/**
* Context for focus change
*/
UENUM()
enum class EFocusCause : uint8
{
	/** Focus was changed because of a mouse action. */
	Mouse,

	/** Focus was changed in response to a navigation, such as the arrow keys, TAB key, controller DPad, ... */
	Navigation,

	/** Focus was changed because someone asked the application to change it. */
	SetDirectly,

	/** Focus was explicitly cleared via the escape key or other similar action. */
	Cleared,

	/** Focus was changed because another widget lost focus, and focus moved to a new widget. */
	OtherWidgetLostFocus,

	/** Focus was set in response to the owning window being activated. */
	WindowActivate,
};

/**
 * FFocusEvent is used when notifying widgets about keyboard focus changes
 * It is passed to event handlers dealing with keyboard focus
 */
USTRUCT(BlueprintType)
struct FFocusEvent
{
	GENERATED_USTRUCT_BODY()

public:

	/**
	 * UStruct Constructor.  Not meant for normal usage.
	 */
	FFocusEvent()
		: Cause(EFocusCause::SetDirectly)
		, UserIndex(0)
	{ }

	/**
	 * Constructor.  Events are immutable once constructed.
	 *
	 * @param  InCause  The cause of the focus event
	 */
	FFocusEvent(const EFocusCause InCause, uint32 InUserIndex)
		: Cause(InCause)
		, UserIndex(InUserIndex)
	{ }

	/**
	 * Queries the reason for the focus change
	 *
	 * @return  The cause of the focus change
	 */
	EFocusCause GetCause() const
	{
		return Cause;
	}

	/**
	 * Queries the user that is changing focus
	 *
	 * @return  The user that is changing focus
	 */
	uint32 GetUser() const
	{
		return UserIndex;
	}

private:

	/** The cause of the focus change */
	EFocusCause Cause;

	/** User that is changing focus*/
	uint32 UserIndex;
};


USTRUCT(BlueprintType)
struct FCaptureLostEvent
{
	GENERATED_USTRUCT_BODY()

public:
	/**
	* UStruct Constructor.  Not meant for normal usage.
	*/
	FCaptureLostEvent()
		: UserIndex(0)
		, PointerIndex(0)
	{ }

	FCaptureLostEvent(int32 InUserIndex, int32 InPointerIndex)
		: UserIndex(InUserIndex)
		, PointerIndex(InPointerIndex)
	{ }

	/** User that is losing capture */
	int32 UserIndex;
	
	/** Pointer (Finger) that lost capture. */
	int32 PointerIndex;
};


/**
 * Represents the current and last cursor position in a "virtual window" for events that are routed to widgets transformed in a 3D scene.
 */
struct FVirtualPointerPosition
{
	FVirtualPointerPosition()
		: CurrentCursorPosition(FVector2f::ZeroVector)
		, LastCursorPosition(FVector2f::ZeroVector)
	{}

	FVirtualPointerPosition(const UE::Slate::FDeprecateVector2DParameter& InCurrentCursorPosition, const UE::Slate::FDeprecateVector2DParameter& InLastCursorPosition)
		: CurrentCursorPosition(InCurrentCursorPosition)
		, LastCursorPosition(InLastCursorPosition)
	{}

	UE::Slate::FDeprecateVector2DResult CurrentCursorPosition;
	UE::Slate::FDeprecateVector2DResult LastCursorPosition;
};

/**
 * Base class for all mouse and keyevents.
 */
USTRUCT(BlueprintType)
struct FInputEvent
{
	GENERATED_USTRUCT_BODY()

public:

	/**
	 * UStruct Constructor.  Not meant for normal usage.
	 */
	FInputEvent()
		: ModifierKeys(FModifierKeysState())
		, bIsRepeat(false)
		, UserIndex(0)
		, InputDeviceId(INPUTDEVICEID_NONE)
		, EventPath(nullptr)
		, EventTimestamp(FPlatformTime::Cycles64())	/* Initalize the timestamp of event input event to the current CPU cycle */
	{ }

	/**
	 * Constructor.  Events are immutable once constructed.
	 *
	 * @param  InModifierKeys  Modifier key state for this event
	 * @param  bInIsRepeat  True if this key is an auto-repeated keystroke
	 */
	FInputEvent(const FModifierKeysState& InModifierKeys, const int32 InUserIndex, const bool bInIsRepeat)
		: ModifierKeys(InModifierKeys)
		, bIsRepeat(bInIsRepeat)
		, UserIndex(InUserIndex)
		, InputDeviceId(FInputDeviceId::CreateFromInternalId(InUserIndex))
		, EventPath(nullptr)
		, EventTimestamp(FPlatformTime::Cycles64())	/* Initalize the timestamp of event input event to the current CPU cycle */
	{ }

	FInputEvent(const FModifierKeysState& InModifierKeys, const FInputDeviceId InDeviceId, const bool bInIsRepeat)
		: ModifierKeys(InModifierKeys)
		, bIsRepeat(bInIsRepeat)
		, InputDeviceId(InDeviceId)
		, EventPath(nullptr)
		, EventTimestamp(FPlatformTime::Cycles64())	/* Initalize the timestamp of event input event to the current CPU cycle */
	{
		// Set the User Index to the PlatformUser ID by default for backwards compatibility
		UserIndex = GetPlatformUserId().GetInternalId();
	}

	/**
	 * Virtual destructor.
	 */
	SLATECORE_API virtual ~FInputEvent();

public:

	/**
	 * Returns whether or not this character is an auto-repeated keystroke
	 *
	 * @return  True if this character is a repeat
	 */
	bool IsRepeat() const
	{
		return bIsRepeat;
	}

	/**
	 * Returns true if either shift key was down when this event occurred
	 *
	 * @return  True if shift is pressed
	 */
	bool IsShiftDown() const
	{
		return ModifierKeys.IsShiftDown();
	}

	/**
	 * Returns true if left shift key was down when this event occurred
	 */
	bool IsLeftShiftDown() const
	{
		return ModifierKeys.IsLeftShiftDown();
	}

	/**
	 * Returns true if right shift key was down when this event occurred
	 */
	bool IsRightShiftDown() const
	{
		return ModifierKeys.IsRightShiftDown();
	}

	/**
	 * Returns true if either control key was down when this event occurred
	 */
	bool IsControlDown() const
	{
		return ModifierKeys.IsControlDown();
	}

	/**
	 * Returns true if left control key was down when this event occurred
	 */
	bool IsLeftControlDown() const
	{
		return ModifierKeys.IsLeftControlDown();
	}

	/**
	 * Returns true if right control key was down when this event occurred
	 */
	bool IsRightControlDown() const
	{
		return ModifierKeys.IsRightControlDown();
	}

	/**
	 * Returns true if either alt key was down when this event occurred
	 */
	bool IsAltDown() const
	{
		return ModifierKeys.IsAltDown();
	}

	/**
	 * Returns true if left alt key was down when this event occurred
	 */
	bool IsLeftAltDown() const
	{
		return ModifierKeys.IsLeftAltDown();
	}

	/**
	 * Returns true if right alt key was down when this event occurred
	 */
	bool IsRightAltDown() const
	{
		return ModifierKeys.IsRightAltDown();
	}

	/**
	 * Returns true if either command key was down when this event occurred
	 */
	bool IsCommandDown() const
	{
		return ModifierKeys.IsCommandDown();
	}

	/**
	 * Returns true if left command key was down when this event occurred
	 */
	bool IsLeftCommandDown() const
	{
		return ModifierKeys.IsLeftCommandDown();
	}

	/**
	 * Returns true if right command key was down when this event occurred
	 */
	bool IsRightCommandDown() const
	{
		return ModifierKeys.IsRightCommandDown();
	}

	/**
	 * Returns true if caps lock was on when this event occurred
	 */
	bool AreCapsLocked() const
	{
		return ModifierKeys.AreCapsLocked();
	}

	/**
	 * Returns the complete set of modifier keys
	 */
	const FModifierKeysState& GetModifierKeys() const
	{
		return ModifierKeys;
	}

	/**
	* Returns the index of the user that generated this event.
	*/
	uint32 GetUserIndex() const
	{
		return UserIndex;
	}

	/**
	 * Returns the input device that caused this event.
	 */
	FInputDeviceId GetInputDeviceId() const
	{
		return InputDeviceId;
	}

	/**
	 * Returns the associated platform user that caused this event
	 */
	FPlatformUserId GetPlatformUserId() const
	{
		return IPlatformInputDeviceMapper::Get().GetUserForInputDevice(InputDeviceId);
	}

	/**
	 * @return The CPU cycle timestamp of when this input event was created.  
	 */
	uint64 GetEventTimestamp() const
	{
		return EventTimestamp;
	}

	/**
	 * @return The amount of time in milliseconds which have passes since this input event was constructed
	 */
	double GetMillisecondsSinceEvent() const
	{
		return FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - EventTimestamp);
	}

	/** The event path provides additional context for handling */
	SLATECORE_API FGeometry FindGeometry(const TSharedRef<SWidget>& WidgetToFind) const;

	SLATECORE_API TSharedRef<SWindow> GetWindow() const;

	/** Set the widget path along which this event will be routed */
	void SetEventPath( const FWidgetPath& InEventPath )
	{
		EventPath = &InEventPath;
	}

	const FWidgetPath* GetEventPath() const
	{
		return EventPath;
	}

	SLATECORE_API virtual FText ToText() const;
	
	/** Is this event a pointer event (touch or cursor). */
	SLATECORE_API virtual bool IsPointerEvent() const;

	/** Is this event a key event. */
	SLATECORE_API virtual bool IsKeyEvent() const;

protected:

	// State of modifier keys when this event happened.
	FModifierKeysState ModifierKeys;

	// True if this key was auto-repeated.
	bool bIsRepeat;

	// The index of the user that caused the event.
	uint32 UserIndex;
	
	// The ID of the input device that caused this event.
	FInputDeviceId InputDeviceId;

	// Events are sent along paths. See (GetEventPath).
	const FWidgetPath* EventPath;

	// The timestamp of when this input event was recorded. This is the
	// cycle at which this event was recorded, populated by FPlatformTime::Cycles64()
	// at the time of construction.
	uint64 EventTimestamp;
};

template<>
struct TStructOpsTypeTraits<FInputEvent> : public TStructOpsTypeTraitsBase2<FInputEvent>
{
	enum
	{
		WithCopy = true,
	};
};

/**
 * FKeyEvent describes a key action (keyboard/controller key/button pressed or released.)
 * It is passed to event handlers dealing with key input.
 */
USTRUCT(BlueprintType)
struct FKeyEvent : public FInputEvent
{
	GENERATED_USTRUCT_BODY()

public:
	/**
	 * UStruct Constructor.  Not meant for normal usage.
	 */
	SLATECORE_API FKeyEvent();

	/**
	 * Constructor.  Events are immutable once constructed.
	 *
	 * @param  InKeyName  Character name
	 * @param  InModifierKeys  Modifier key state for this event
	 * @param  bInIsRepeat  True if this key is an auto-repeated keystroke
	 */
	SLATECORE_API FKeyEvent(const FKey InKey,
		const FModifierKeysState& InModifierKeys,
		const uint32 InUserIndex,
		const bool bInIsRepeat,
		const uint32 InCharacterCode,
		const uint32 InKeyCode
	);
	
	SLATECORE_API FKeyEvent(const FKey InKey,
		const FModifierKeysState& InModifierKeys,
		const FInputDeviceId InDeviceId,
		const bool bInIsRepeat,
		const uint32 InCharacterCode,
		const uint32 InKeyCode,
		const TOptional<int32> InOptionalSlateUserIndex = TOptional<int32>()
	);

	SLATECORE_API virtual ~FKeyEvent();

	/**
	 * Returns the name of the key for this event
	 *
	 * @return  Key name
	 */
	FKey GetKey() const
	{
		return Key;
	}

	/**
	 * Returns the character code for this event.
	 *
	 * @return  Character code or 0 if this event was not a character key press
	 */
	uint32 GetCharacter() const
	{
		return CharacterCode;
	}

	/**
	 * Returns the key code received from hardware before any conversion/mapping.
	 *
	 * @return  Key code received from hardware
	 */
	uint32 GetKeyCode() const
	{
		return KeyCode;
	}

	SLATECORE_API virtual FText ToText() const override;

	SLATECORE_API virtual bool IsKeyEvent() const override;

private:
	// Name of the key that was pressed.
	FKey Key;

	// The character code of the key that was pressed.  Only applicable to typed character keys, 0 otherwise.
	uint32 CharacterCode;

	// Original key code received from hardware before any conversion/mapping
	uint32 KeyCode;
};

template<>
struct TStructOpsTypeTraits<FKeyEvent> : public TStructOpsTypeTraitsBase2<FKeyEvent>
{
	enum
	{
		WithCopy = true,
	};
};


/**
 * FAnalogEvent describes a analog key value.
 * It is passed to event handlers dealing with analog keys.
 */
USTRUCT(BlueprintType)
struct FAnalogInputEvent
	: public FKeyEvent
{
	GENERATED_USTRUCT_BODY()

public:
	/**
	* UStruct Constructor.  Not meant for normal usage.
	*/
	FAnalogInputEvent()
		: FKeyEvent(FKey(), FModifierKeysState(), false, 0, 0, 0)
		, AnalogValue(0.0f)
	{
	}

	/**
	 * Constructor.  Events are immutable once constructed.
	 *
	 * @param  InKeyName  Character name
	 * @param  InModifierKeys  Modifier key state for this event
	 * @param  bInIsRepeat  True if this key is an auto-repeated keystroke
	 */
	FAnalogInputEvent(const FKey InKey,
		const FModifierKeysState& InModifierKeys,
		const uint32 InUserIndex,
		const bool bInIsRepeat,
		const uint32 InCharacterCode,
		const uint32 InKeyCode,
		const float InAnalogValue
		)
		: FKeyEvent(InKey, InModifierKeys, InUserIndex, bInIsRepeat, InCharacterCode, InKeyCode)
		, AnalogValue(InAnalogValue)
	{ }

	FAnalogInputEvent(const FKey InKey,
		const FModifierKeysState& InModifierKeys,
		const FInputDeviceId InDeviceId,
		const bool bInIsRepeat,
		const uint32 InCharacterCode,
		const uint32 InKeyCode,
		const float InAnalogValue,
		const TOptional<int32> InOptionalSlateUserIndex = TOptional<int32>()
	)
		: FKeyEvent(InKey, InModifierKeys, InDeviceId, bInIsRepeat, InCharacterCode, InKeyCode, InOptionalSlateUserIndex)
		, AnalogValue(InAnalogValue)
	{ }

	SLATECORE_API virtual ~FAnalogInputEvent();

	/**
	 * Returns the analog value between 0 and 1.
	 * 0 being not pressed at all, 1 being fully pressed.
	 * Non analog keys will only be 0 or 1.
	 *
	 * @return Analog value between 0 and 1.  1 being fully pressed, 0 being not pressed at all
	 */
	float GetAnalogValue() const { return AnalogValue; }

	SLATECORE_API virtual FText ToText() const override;

private:
	//  Analog value between 0 and 1 (0 being not pressed at all, 1 being fully pressed).
	float AnalogValue;
};

template<>
struct TStructOpsTypeTraits<FAnalogInputEvent> : public TStructOpsTypeTraitsBase2<FAnalogInputEvent>
{
	enum
	{
		WithCopy = true,
	};
};

/**
 * FCharacterEvent describes a keyboard action where the utf-16 code is given.  Used for OnKeyChar messages
 */
USTRUCT(BlueprintType)
struct FCharacterEvent
	: public FInputEvent
{
	GENERATED_USTRUCT_BODY()

public:
	/**
	 * UStruct Constructor.  Not meant for normal usage.
	 */
	FCharacterEvent()
		: FInputEvent(FModifierKeysState(), 0, false)
		, Character(0)
	{
	}

	FCharacterEvent(const TCHAR InCharacter, const FModifierKeysState& InModifierKeys, const uint32 InUserIndex, const bool bInIsRepeat)
		: FInputEvent(InModifierKeys, InUserIndex, bInIsRepeat)
		, Character(InCharacter)
	{ }

	FCharacterEvent(const TCHAR InCharacter, const FModifierKeysState& InModifierKeys, const FInputDeviceId InDeviceId, const bool bInIsRepeat)
		: FInputEvent(InModifierKeys, InDeviceId, bInIsRepeat)
		, Character(InCharacter)
	{ }

	SLATECORE_API virtual ~FCharacterEvent();

	/**
	 * Returns the character for this event
	 *
	 * @return  Character
	 */
	TCHAR GetCharacter() const
	{
		return Character;
	}

	SLATECORE_API virtual FText ToText() const override;	

private:

	// The character that was pressed.
	TCHAR Character;
};


template<>
struct TStructOpsTypeTraits<FCharacterEvent> : public TStructOpsTypeTraitsBase2<FCharacterEvent>
{
	enum
	{
		WithCopy = true,
	};
};

/**
 * Helper class to auto-populate a set with the set of "keys" a touch represents
 */
class FTouchKeySet
	: public TSet<FKey>
{
public:

	/**
	 * Creates and initializes a new instance with the specified key.
	 *
	 * @param Key The key to insert into the set.
	 */
	FTouchKeySet(FKey Key)
	{
		this->Add(Key);
	}

public:

	// The standard set consists of just the left mouse button key.
	SLATECORE_API static const FTouchKeySet StandardSet;

	// The empty set contains no valid keys.
	SLATECORE_API static const FTouchKeySet EmptySet;
};



/**
 * FPointerEvent describes a mouse or touch action (e.g. Press, Release, Move, etc).
 * It is passed to event handlers dealing with pointer-based input.
 */
USTRUCT(BlueprintType)
struct FPointerEvent
	: public FInputEvent
{
	GENERATED_USTRUCT_BODY()
public:

	/**
	 * UStruct Constructor.  Not meant for normal usage.
	 */
	FPointerEvent()
		: ScreenSpacePosition(FVector2f(0.f, 0.f))
		, LastScreenSpacePosition(FVector2f(0.f, 0.f))
		, CursorDelta(FVector2f(0.f, 0.f))
		, PressedButtons(FTouchKeySet::EmptySet)
		, EffectingButton()
		, PointerIndex(0)
		, TouchpadIndex(0)
		, Force(1.0f)
		, bIsTouchEvent(false)
		, GestureType(EGestureEvent::None)
		, GesturePhase(EGesturePhase::None)
		, WheelOrGestureDelta(0.0f, 0)
		, bIsDirectionInvertedFromDevice(false)
		, bIsTouchForceChanged(false)
		, bIsTouchFirstMove(false)
	{ }

	/** Events are immutable once constructed. */
	FPointerEvent(
		uint32 InPointerIndex,
		const UE::Slate::FDeprecateVector2DParameter& InScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InLastScreenSpacePosition,
		const TSet<FKey>& InPressedButtons,
		FKey InEffectingButton,
		float InWheelDelta,
		const FModifierKeysState& InModifierKeys
	)
		: FInputEvent(InModifierKeys, 0, false)
		, ScreenSpacePosition(InScreenSpacePosition)
		, LastScreenSpacePosition(InLastScreenSpacePosition)
		, CursorDelta(FVector2f(InScreenSpacePosition) - FVector2f(InLastScreenSpacePosition))
		, PressedButtons(InPressedButtons)
		, EffectingButton(InEffectingButton)
		, PointerIndex(InPointerIndex)
		, TouchpadIndex(0)
		, Force(1.0f)
		, bIsTouchEvent(false)
		, GestureType(EGestureEvent::None)
		, GesturePhase(EGesturePhase::None)
		, WheelOrGestureDelta(0.0f, InWheelDelta)
		, bIsDirectionInvertedFromDevice(false)
		, bIsTouchForceChanged(false)
		, bIsTouchFirstMove(false)
	{ }

	FPointerEvent(
		uint32 InUserIndex,
		uint32 InPointerIndex,
		const UE::Slate::FDeprecateVector2DParameter& InScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InLastScreenSpacePosition,
		const TSet<FKey>& InPressedButtons,
		FKey InEffectingButton,
		float InWheelDelta,
		const FModifierKeysState& InModifierKeys
	)
		: FInputEvent(InModifierKeys, InUserIndex, false)
		, ScreenSpacePosition(InScreenSpacePosition)
		, LastScreenSpacePosition(InLastScreenSpacePosition)
		, CursorDelta(FVector2f(InScreenSpacePosition) - FVector2f(InLastScreenSpacePosition))
		, PressedButtons(InPressedButtons)
		, EffectingButton(InEffectingButton)
		, PointerIndex(InPointerIndex)
		, TouchpadIndex(0)
		, Force(1.0f)
		, bIsTouchEvent(false)
		, GestureType(EGestureEvent::None)
		, GesturePhase(EGesturePhase::None)
		, WheelOrGestureDelta(0.0f, InWheelDelta)
		, bIsDirectionInvertedFromDevice(false)
		, bIsTouchForceChanged(false)
		, bIsTouchFirstMove(false)
	{ }

	FPointerEvent(
		FInputDeviceId InDeviceId,
		uint32 InPointerIndex,
		const UE::Slate::FDeprecateVector2DParameter& InScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InLastScreenSpacePosition,
		const TSet<FKey>& InPressedButtons,
		FKey InEffectingButton,
		float InWheelDelta,
		const FModifierKeysState& InModifierKeys,
		const TOptional<int32> InOptionalSlateUserIndex = TOptional<int32>()
	)
		: FInputEvent(InModifierKeys, InDeviceId, false)
		, ScreenSpacePosition(InScreenSpacePosition)
		, LastScreenSpacePosition(InLastScreenSpacePosition)
		, CursorDelta(FVector2f(InScreenSpacePosition) - FVector2f(InLastScreenSpacePosition))
		, PressedButtons(InPressedButtons)
		, EffectingButton(InEffectingButton)
		, PointerIndex(InPointerIndex)
		, TouchpadIndex(0)
		, Force(1.0f)
		, bIsTouchEvent(false)
		, GestureType(EGestureEvent::None)
		, GesturePhase(EGesturePhase::None)
		, WheelOrGestureDelta(0.0f, InWheelDelta)
		, bIsDirectionInvertedFromDevice(false)
		, bIsTouchForceChanged(false)
		, bIsTouchFirstMove(false)
	{
		if (InOptionalSlateUserIndex.IsSet())
		{
			UserIndex = InOptionalSlateUserIndex.GetValue();
		}
	}

	FPointerEvent(
		uint32 InUserIndex,
		uint32 InPointerIndex,
		const UE::Slate::FDeprecateVector2DParameter& InScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InLastScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InDelta,
		const TSet<FKey>& InPressedButtons,
		const FModifierKeysState& InModifierKeys
	)
		: FInputEvent(InModifierKeys, InUserIndex, false)
		, ScreenSpacePosition(InScreenSpacePosition)
		, LastScreenSpacePosition(InLastScreenSpacePosition)
		, CursorDelta(InDelta)
		, PressedButtons(InPressedButtons)
		, PointerIndex(InPointerIndex)
		, TouchpadIndex(0)
		, Force(1.0f)
		, bIsTouchEvent(false)
		, GestureType(EGestureEvent::None)
		, GesturePhase(EGesturePhase::None)
		, WheelOrGestureDelta(0.0f, 0.0f)
		, bIsDirectionInvertedFromDevice(false)
		, bIsTouchForceChanged(false)
		, bIsTouchFirstMove(false)
	{ }

	/** A constructor for raw mouse events */
	FPointerEvent(
		uint32 InPointerIndex,
		const UE::Slate::FDeprecateVector2DParameter& InScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InLastScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InDelta,
		const TSet<FKey>& InPressedButtons,
		const FModifierKeysState& InModifierKeys
	)
		: FInputEvent(InModifierKeys, 0, false)
		, ScreenSpacePosition(InScreenSpacePosition)
		, LastScreenSpacePosition(InLastScreenSpacePosition)
		, CursorDelta(InDelta)
		, PressedButtons(InPressedButtons)
		, PointerIndex(InPointerIndex)
		, TouchpadIndex(0)
		, Force(1.0f)
		, bIsTouchEvent(false)
		, GestureType(EGestureEvent::None)
		, GesturePhase(EGesturePhase::None)
		, WheelOrGestureDelta(0.0f, 0.0f)
		, bIsDirectionInvertedFromDevice(false)
		, bIsTouchForceChanged(false)
		, bIsTouchFirstMove(false)
	{ }

	UE_DEPRECATED(5.8, "TouchpadIndex is was never fully implemented and has been retired.")
	FPointerEvent(
		const uint32 InUserIndex,
		const uint32 InPointerIndex,
		const UE::Slate::FDeprecateVector2DParameter& InScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InLastScreenSpacePosition,
		const float InForce,
		const bool bPressLeftMouseButton,
		const bool bInIsForceChanged,
		const bool bInIsFirstMove,
		const FModifierKeysState& InModifierKeys,
		const uint32 InTouchpadIndex
	)
		: FInputEvent(InModifierKeys, InUserIndex, false)
		, ScreenSpacePosition(InScreenSpacePosition)
		, LastScreenSpacePosition(InLastScreenSpacePosition)
		, CursorDelta(FVector2f(InScreenSpacePosition) - FVector2f(InLastScreenSpacePosition))
		, PressedButtons(bPressLeftMouseButton ? FTouchKeySet::StandardSet : FTouchKeySet::EmptySet)
		, EffectingButton(EKeys::LeftMouseButton)
		, PointerIndex(InPointerIndex)
		, TouchpadIndex(InTouchpadIndex)
		, Force(InForce)
		, bIsTouchEvent(true)
		, GestureType(EGestureEvent::None)
		, GesturePhase(EGesturePhase::None)
		, WheelOrGestureDelta(0.0f, 0.0f)
		, bIsDirectionInvertedFromDevice(false)
		, bIsTouchForceChanged(bInIsForceChanged)
		, bIsTouchFirstMove(bInIsFirstMove)
	{ }

	FPointerEvent(
		const uint32 InUserIndex,
		const uint32 InPointerIndex,
		const UE::Slate::FDeprecateVector2DParameter& InScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InLastScreenSpacePosition,
		const float InForce,
		const bool bPressLeftMouseButton,
		const bool bInIsForceChanged = false,
		const bool bInIsFirstMove = false,
		const FModifierKeysState& InModifierKeys = FModifierKeysState()
	)
		: FInputEvent(InModifierKeys, InUserIndex, false)
		, ScreenSpacePosition(InScreenSpacePosition)
		, LastScreenSpacePosition(InLastScreenSpacePosition)
		, CursorDelta(FVector2f(InScreenSpacePosition) - FVector2f(InLastScreenSpacePosition))
		, PressedButtons(bPressLeftMouseButton ? FTouchKeySet::StandardSet : FTouchKeySet::EmptySet)
		, EffectingButton(EKeys::LeftMouseButton)
		, PointerIndex(InPointerIndex)
		, TouchpadIndex(0)
		, Force(InForce)
		, bIsTouchEvent(true)
		, GestureType(EGestureEvent::None)
		, WheelOrGestureDelta(0.0f, 0.0f)
		, bIsDirectionInvertedFromDevice(false)
		, bIsTouchForceChanged(bInIsForceChanged)
		, bIsTouchFirstMove(bInIsFirstMove)
	{
	}

	FPointerEvent(
		const FInputDeviceId InDeviceId,
		const uint32 InPointerIndex,
		const UE::Slate::FDeprecateVector2DParameter& InScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InLastScreenSpacePosition,
		const float InForce,
		const bool bPressLeftMouseButton,
		const bool bInIsForceChanged = false,
		const bool bInIsFirstMove = false,
		const FModifierKeysState& InModifierKeys = FModifierKeysState(),
		const TOptional<int32> InOptionalSlateUserIndex = TOptional<int32>()
	)
	: FInputEvent(InModifierKeys, InDeviceId, false)
		, ScreenSpacePosition(InScreenSpacePosition)
		, LastScreenSpacePosition(InLastScreenSpacePosition)
		, CursorDelta(FVector2f(InScreenSpacePosition) - FVector2f(InLastScreenSpacePosition))
		, PressedButtons(bPressLeftMouseButton ? FTouchKeySet::StandardSet : FTouchKeySet::EmptySet)
		, EffectingButton(EKeys::LeftMouseButton)
		, PointerIndex(InPointerIndex)
		, TouchpadIndex(0)
		, Force(InForce)
		, bIsTouchEvent(true)
		, GestureType(EGestureEvent::None)
		, WheelOrGestureDelta(0.0f, 0.0f)
		, bIsDirectionInvertedFromDevice(false)
		, bIsTouchForceChanged(bInIsForceChanged)
		, bIsTouchFirstMove(bInIsFirstMove)
	{
		if (InOptionalSlateUserIndex.IsSet())
		{
			UserIndex = InOptionalSlateUserIndex.GetValue();
		}
	}

	UE_DEPRECATED(5.8, "TouchpadIndex is was never fully implemented and has been retired.")
	FPointerEvent(
		const FInputDeviceId InDeviceId,
		const uint32 InPointerIndex,
		const UE::Slate::FDeprecateVector2DParameter& InScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InLastScreenSpacePosition,
		const float InForce,
		const bool bPressLeftMouseButton,
		const bool bInIsForceChanged,
		const bool bInIsFirstMove,
		const FModifierKeysState& InModifierKeys,
		const uint32 InTouchpadIndex,
		const TOptional<int32> InOptionalSlateUserIndex = TOptional<int32>()
	)
	: FInputEvent(InModifierKeys, InDeviceId, false)
		, ScreenSpacePosition(InScreenSpacePosition)
		, LastScreenSpacePosition(InLastScreenSpacePosition)
		, CursorDelta(FVector2f(InScreenSpacePosition) - FVector2f(InLastScreenSpacePosition))
		, PressedButtons(bPressLeftMouseButton ? FTouchKeySet::StandardSet : FTouchKeySet::EmptySet)
		, EffectingButton(EKeys::LeftMouseButton)
		, PointerIndex(InPointerIndex)
		, TouchpadIndex(InTouchpadIndex)
		, Force(InForce)
		, bIsTouchEvent(true)
		, GestureType(EGestureEvent::None)
		, GesturePhase(EGesturePhase::None)
		, WheelOrGestureDelta(0.0f, 0.0f)
		, bIsDirectionInvertedFromDevice(false)
		, bIsTouchForceChanged(bInIsForceChanged)
		, bIsTouchFirstMove(bInIsFirstMove)
	{
		if (InOptionalSlateUserIndex.IsSet())
		{
			UserIndex = InOptionalSlateUserIndex.GetValue();
		}
	}

	/** A constructor for gesture events */
	FPointerEvent(
		const UE::Slate::FDeprecateVector2DParameter& InScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InLastScreenSpacePosition,
		const TSet<FKey>& InPressedButtons,
		const FModifierKeysState& InModifierKeys,
		EGestureEvent InGestureType,
		EGesturePhase InGesturePhase,
		const UE::Slate::FDeprecateVector2DParameter& InGestureDelta,
		bool bInIsDirectionInvertedFromDevice
	)
		: FInputEvent(InModifierKeys, 0, false)
		, ScreenSpacePosition(InScreenSpacePosition)
		, LastScreenSpacePosition(InLastScreenSpacePosition)
		, CursorDelta(FVector2f(LastScreenSpacePosition) - FVector2f(ScreenSpacePosition))
		, PressedButtons(InPressedButtons)
		, PointerIndex(0)
		, TouchpadIndex(0)
		, Force(1.0f)
		, bIsTouchEvent(InGestureType != EGestureEvent::None)
		, GestureType(InGestureType)
		, GesturePhase(InGesturePhase)
		, WheelOrGestureDelta(InGestureDelta)
		, bIsDirectionInvertedFromDevice(bInIsDirectionInvertedFromDevice)
		, bIsTouchForceChanged(false)
		, bIsTouchFirstMove(false)
	{ }

	/** A constructor for gesture events with a specific input device */
	FPointerEvent(
		FInputDeviceId InDeviceId,
		const UE::Slate::FDeprecateVector2DParameter& InScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InLastScreenSpacePosition,
		const TSet<FKey>& InPressedButtons,
		const FModifierKeysState& InModifierKeys,
		EGestureEvent InGestureType,
		EGesturePhase InGesturePhase,
		const UE::Slate::FDeprecateVector2DParameter& InGestureDelta,
		bool bInIsDirectionInvertedFromDevice
	)
		: FInputEvent(InModifierKeys, InDeviceId, false)
		, ScreenSpacePosition(InScreenSpacePosition)
		, LastScreenSpacePosition(InLastScreenSpacePosition)
		, CursorDelta(FVector2f(LastScreenSpacePosition) - FVector2f(ScreenSpacePosition))
		, PressedButtons(InPressedButtons)
		, PointerIndex(0)
		, TouchpadIndex(0)
		, Force(1.0f)
		, bIsTouchEvent(InGestureType != EGestureEvent::None)
		, GestureType(InGestureType)
		, GesturePhase(InGesturePhase)
		, WheelOrGestureDelta(InGestureDelta)
		, bIsDirectionInvertedFromDevice(bInIsDirectionInvertedFromDevice)
		, bIsTouchForceChanged(false)
		, bIsTouchFirstMove(false)
	{ }

	/** A constructor to alter cursor positions */
	FPointerEvent(
		const FPointerEvent& Other,
		const UE::Slate::FDeprecateVector2DParameter& InScreenSpacePosition,
		const UE::Slate::FDeprecateVector2DParameter& InLastScreenSpacePosition)
	{
		*this = Other;
		ScreenSpacePosition = InScreenSpacePosition;
		LastScreenSpacePosition = InLastScreenSpacePosition;
	}

	SLATECORE_API virtual ~FPointerEvent();

public:

	/** Returns The position of the cursor in screen space */
	const UE::Slate::FDeprecateVector2DResult& GetScreenSpacePosition() const { return ScreenSpacePosition; }

	/** Returns the position of the cursor in screen space last time we handled an input event */
	const UE::Slate::FDeprecateVector2DResult& GetLastScreenSpacePosition() const { return LastScreenSpacePosition; }

	/** Returns the distance the mouse traveled since the last event was handled. */
	const UE::Slate::FDeprecateVector2DResult& GetCursorDelta() const { return CursorDelta; }

	/** Mouse buttons that are currently pressed */
	bool IsMouseButtonDown( FKey MouseButton ) const { return PressedButtons.Contains( MouseButton ); }

	/** Mouse button that caused this event to be raised (possibly FKey::Invalid) */
	FKey GetEffectingButton() const { return EffectingButton; }
	
	/** How much did the mouse wheel turn since the last mouse event */
	float GetWheelDelta() const { return UE_REAL_TO_FLOAT(WheelOrGestureDelta.Y); }

	/** Returns the unique identifier of the pointer (e.g., finger index) */
	uint32 GetPointerIndex() const { return PointerIndex; }

	UE_DEPRECATED(5.8, "TouchpadIndex is was never fully implemented and has been retired.")
	uint32 GetTouchpadIndex() const { return TouchpadIndex; }

	/** Returns the force of a touch (1.0f is mapped to an general touch force, < 1 is "light", > 1 is "heavy", and 10 is the max force possible) */
	float GetTouchForce() const { return Force; }

	/** Is this event a result from a touch (as opposed to a mouse) */
	bool IsTouchEvent() const { return bIsTouchEvent; }

	/** Is this event a special force-change touch event */
	bool IsTouchForceChangedEvent() const { return bIsTouchForceChanged; }

	/** Is this event a special first-move touch event */
	bool IsTouchFirstMoveEvent() const { return bIsTouchFirstMove; }

	/** Returns the type of touch gesture */
	EGestureEvent GetGestureType() const { return GestureType; }

	/** Returns the phase of the gesture (Started, Update, Ended) */
	EGesturePhase GetGesturePhase() const { return GesturePhase; }

	/** Returns the change in gesture value since the last gesture event of the same type. */
	const UE::Slate::FDeprecateVector2DResult& GetGestureDelta() const { return WheelOrGestureDelta; }

	/** Is the gesture delta inverted */
	bool IsDirectionInvertedFromDevice() const { return bIsDirectionInvertedFromDevice; }

	/** Returns the full set of pressed buttons */
	const TSet<FKey>& GetPressedButtons() const { return PressedButtons; }

	SLATECORE_API virtual FText ToText() const override;

	SLATECORE_API virtual bool IsPointerEvent() const override;

	template<typename PointerEventType>
	static PointerEventType MakeTranslatedEvent( const PointerEventType& InPointerEvent, const FVirtualPointerPosition& VirtualPosition )
	{
		PointerEventType NewEvent = InPointerEvent;
		NewEvent.ScreenSpacePosition = VirtualPosition.CurrentCursorPosition;
		NewEvent.LastScreenSpacePosition = VirtualPosition.LastCursorPosition;
		//NewEvent.CursorDelta = VirtualPosition.GetDelta();
		return NewEvent;
	}

private:

	UE::Slate::FDeprecateVector2DResult ScreenSpacePosition;
	UE::Slate::FDeprecateVector2DResult LastScreenSpacePosition;
	UE::Slate::FDeprecateVector2DResult CursorDelta;
	TSet<FKey> PressedButtons;
	FKey EffectingButton;
	uint32 PointerIndex;
	uint32 TouchpadIndex;
	float Force;
	bool bIsTouchEvent;
	EGestureEvent GestureType;
	EGesturePhase GesturePhase;
	UE::Slate::FDeprecateVector2DResult WheelOrGestureDelta;
	bool bIsDirectionInvertedFromDevice;
	bool bIsTouchForceChanged;
	bool bIsTouchFirstMove;
	// NOTE: If you add a new member, make sure you add it to the assignment operator.
};


template<>
struct TStructOpsTypeTraits<FPointerEvent> : public TStructOpsTypeTraitsBase2<FPointerEvent>
{
	enum
	{
		WithCopy = true,
	};
};


/**
 * FMotionEvent describes a touch pad action (press, move, lift)
 * It is passed to event handlers dealing with touch input.
 */
USTRUCT(BlueprintType)
struct FMotionEvent
	: public FInputEvent
{
	GENERATED_USTRUCT_BODY()

public:
	/**
	* UStruct Constructor.  Not meant for normal usage.
	*/
	FMotionEvent()
		: Tilt(FVector(0, 0, 0))
		, RotationRate(FVector(0, 0, 0))
		, Gravity(FVector(0, 0, 0))
		, Acceleration(FVector(0, 0, 0))
	{ }

	FMotionEvent(
		uint32 InUserIndex,
		const FVector& InTilt, 
		const FVector& InRotationRate, 
		const FVector& InGravity, 
		const FVector& InAcceleration
	)
		: FInputEvent(FModifierKeysState(), InUserIndex, false)
		, Tilt(InTilt)
		, RotationRate(InRotationRate)
		, Gravity(InGravity)
		, Acceleration(InAcceleration)
	{ }

	FMotionEvent(
		const FInputDeviceId InDeviceId,
		const FVector& InTilt, 
		const FVector& InRotationRate, 
		const FVector& InGravity, 
		const FVector& InAcceleration
	)
		: FInputEvent(FModifierKeysState(), InDeviceId, false)
		, Tilt(InTilt)
		, RotationRate(InRotationRate)
		, Gravity(InGravity)
		, Acceleration(InAcceleration)
	{ }

public:

	/** Returns the current tilt of the device/controller */
	const FVector& GetTilt() const { return Tilt; }

	/** Returns otation speed */
	const FVector& GetRotationRate() const { return RotationRate; }

	/** Returns the gravity vector (pointing down into the ground) */
	const FVector& GetGravity() const { return Gravity; }

	/** Returns the 3D acceleration of the device */
	const FVector& GetAcceleration() const { return Acceleration; }

private:

	// The current tilt of the device/controller.
	FVector Tilt;

	// The rotation speed.
	FVector RotationRate;

	// The gravity vector (pointing down into the ground).
	FVector Gravity;

	// The 3D acceleration of the device.
	FVector Acceleration;
};


template<>
struct TStructOpsTypeTraits<FMotionEvent> : public TStructOpsTypeTraitsBase2<FMotionEvent>
{
	enum
	{
		WithCopy = true,
	};
};

/**
* FNavigationEvent describes a navigation action (Left, Right, Up, Down)
* It is passed to event handlers dealing with navigation.
*/
USTRUCT(BlueprintType)
struct FNavigationEvent
	: public FInputEvent
{
	GENERATED_USTRUCT_BODY()

public:
	/**
	* UStruct Constructor.  Not meant for normal usage.
	*/
	FNavigationEvent()
		: NavigationType(EUINavigation::Invalid)
		, NavigationGenesis(ENavigationGenesis::User)
	{ }

	FNavigationEvent(const FModifierKeysState& InModifierKeys, const int32 InUserIndex, EUINavigation InNavigationType, ENavigationGenesis InNavigationGenesis)
		: FInputEvent(InModifierKeys, InUserIndex, false)
		, NavigationType(InNavigationType)
		, NavigationGenesis(InNavigationGenesis)
	{ }

	FNavigationEvent(const FModifierKeysState& InModifierKeys, const FInputDeviceId InDeviceId, EUINavigation InNavigationType, ENavigationGenesis InNavigationGenesis)
		: FInputEvent(InModifierKeys, InDeviceId, false)
		, NavigationType(InNavigationType)
		, NavigationGenesis(InNavigationGenesis)
	{ }

public:

	/** Returns the type of navigation request (Left, Right, Up, Down) */
	EUINavigation GetNavigationType() const { return NavigationType; }

	/** Returns the genesis of the navigation request (Keyboard, Controller, User) */
	ENavigationGenesis GetNavigationGenesis() const { return NavigationGenesis; }

private:

	// The navigation type
	EUINavigation NavigationType;

	// The navigation genesis
	ENavigationGenesis NavigationGenesis;
};


template<>
struct TStructOpsTypeTraits<FNavigationEvent> : public TStructOpsTypeTraitsBase2<FNavigationEvent>
{
	enum
	{
		WithCopy = true,
	};
};

/**
 * FWindowActivateEvent describes a window being activated or deactivated.
 * (i.e. brought to the foreground or moved to the background)
 * This event is only passed to top level windows; most widgets are incapable
 * of receiving this event.
 */
class FWindowActivateEvent
{
public:

	enum EActivationType
	{
		EA_Activate,
		EA_ActivateByMouse,
		EA_Deactivate
	};

	FWindowActivateEvent( EActivationType InActivationType, TSharedRef<SWindow> InAffectedWindow )
		: ActivationType(InActivationType)
		, AffectedWindow(InAffectedWindow)
	{ }

public:

	/** Describes what actually happened to the window (e.g. Activated, Deactivated, Activated by a mouse click) */
	EActivationType GetActivationType() const
	{
		return ActivationType;
	}

	/** Sets what actually happened to the window (e.g. Activated, Deactivated, Activated by a mouse click) */
	void SetActivationType(EActivationType InActivationType)
	{
		ActivationType = InActivationType;
	}

	/** The window that this activation/deactivation happened to */
	TSharedRef<SWindow> GetAffectedWindow() const
	{
		return AffectedWindow;
	}

private:

	EActivationType ActivationType;
	TSharedRef<SWindow> AffectedWindow;
};
