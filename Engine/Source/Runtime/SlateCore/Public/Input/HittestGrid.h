// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FastUpdate/SlateInvalidationWidgetSortOrder.h"
#include "Layout/SlateRect.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/Clipping.h"
#include "Input/Events.h"
#include "StructUtils/InstancedStruct.h"
#include "Widgets/SWidget.h"

class FArrangedChildren;
struct FNavigationMethod;

#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
namespace HittestGridDebuggingText
{
	extern SLATECORE_API const FText Valid;
	extern SLATECORE_API const FText PreviousWidgetIsBetter;
	extern SLATECORE_API const FText NotCompatibleWithUserIndex;
	extern SLATECORE_API const FText DoesNotIntersect;
	extern SLATECORE_API const FText NotADescendant;
	extern SLATECORE_API const FText Disabled;
	extern SLATECORE_API const FText ParentDisabled;
	extern SLATECORE_API const FText DoesNotSuportKeyboardFocus;
};
#endif // WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING

class ICustomHitTestPath
{
public:
	virtual ~ICustomHitTestPath()= default;

	virtual TArray<FWidgetAndPointer> GetBubblePathAndVirtualCursors( const FGeometry& InGeometry, FVector2D DesktopSpaceCoordinate, bool bIgnoreEnabledStatus ) const = 0;

	virtual void ArrangeCustomHitTestChildren( FArrangedChildren& ArrangedChildren ) const = 0;

	virtual TOptional<FVirtualPointerPosition> TranslateMouseCoordinateForCustomHitTestChild(const SWidget& ChildWidget, const FGeometry& MyGeometry, const FVector2D ScreenSpaceMouseCoordinate, const FVector2D LastScreenSpaceMouseCoordinate) const = 0;
};

class FHittestGrid : public FNoncopyable
{
	friend struct FNavigationMethod;
	friend struct FNavigationMethodOrthogonal;
	friend struct FNavigationRouting;

public:

	SLATECORE_API FHittestGrid();

	/**
	 * Given a Slate Units coordinate in virtual desktop space, perform a hittest
	 * and return the path along which the corresponding event would be bubbled, optionally filtering
	 * the hittest to widgets in a range of layers.
	 */
	SLATECORE_API TArray<FWidgetAndPointer> GetBubblePath(UE::Slate::FDeprecateVector2DParameter DesktopSpaceCoordinate, float CursorRadius, bool bIgnoreEnabledStatus, int32 UserIndex = INDEX_NONE, TInterval<int32> LayerRange = TInterval<int32>(std::numeric_limits<int32>::min(), std::numeric_limits<int32>::max()));

	/**
	 * Set the position and size of the hittest area in desktop coordinates
	 *
	 * @param HittestPositionInDesktop	The position of this hit testing area in desktop coordinates.
	 * @param HittestDimensions			The dimensions of this hit testing area.
	 *
	 * @return      Returns true if a clear of the hittest grid was required. 
	 */
	SLATECORE_API bool SetHittestArea(const UE::Slate::FDeprecateVector2DParameter& HittestPositionInDesktop, const UE::Slate::FDeprecateVector2DParameter& HittestDimensions, const UE::Slate::FDeprecateVector2DParameter& HitestOffsetInWindow = FVector2f::ZeroVector);

	/** Insert custom hit test data for a widget already in the grid */
	SLATECORE_API void InsertCustomHitTestPath(const SWidget* InWidget, const TSharedRef<ICustomHitTestPath>& CustomHitTestPath);

	/** Sets the current slate user index that should be associated with any added widgets */
	void SetUserIndex(int32 UserIndex) { CurrentUserIndex = UserIndex; }

	/** Set the culling rect to be used by the parent grid (in case we are appended to another grid). */
	void SetCullingRect(const FSlateRect& InCullingRect) { CullingRect = InCullingRect; }

	/** Set the owner SWidget to be used by the parent grid (in case we are appended to another grid). */
	void SetOwner(const SWidget* InOwner) { check(Owner == nullptr || Owner == InOwner); Owner = InOwner; }

	/** Gets current slate user index that should be associated with any added widgets */
	int32 GetUserIndex() const { return CurrentUserIndex; }

	/**
	 * Finds the next focusable widget by searching through the hit test grid
	 *
	 * @param StartingWidget  This is the widget we are starting at, and navigating from.
	 * @param Direction       The direction we should search in.
	 * @param NavigationReply The Navigation Reply to specify a boundary rule for the search.
	 * @param RuleWidget      The Widget that is applying the boundary rule, used to get the bounds of the Rule.
	 */
	SLATECORE_API TSharedPtr<SWidget> FindNextFocusableWidget(const FArrangedWidget& StartingWidget, const EUINavigation Direction, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget, int32 UserIndex);

	UE::Slate::FDeprecateVector2DResult GetGridSize() const { return GridSize; }
	UE::Slate::FDeprecateVector2DResult GetGridOrigin() const { return GridOrigin; }
	UE::Slate::FDeprecateVector2DResult GetGridWindowOrigin() const { return GridWindowOrigin; }

	/** Get the screen space rectangle that corresponds to the painted area of the widget **/
	SLATECORE_API FSlateRect GetWidgetRenderBoundingRect(const TSharedPtr<SWidget>& Widget) const;

	/** Returns true if the CellCoordinates are within the grid **/
	SLATECORE_API bool IsValidCellCoord(const FIntPoint& CellCoord) const;
	SLATECORE_API bool IsValidCellCoord(const int32 XCoord, const int32 YCoord) const;
	
	const FVector2f& GetCellSize() const { return CellSize; }

	/** Clear the grid */
	SLATECORE_API void Clear();

	/** Add SWidget from the HitTest Grid */
	SLATECORE_API void AddWidget(const SWidget* InWidget, int32 InBatchPriorityGroup, int32 InLayerId, FSlateInvalidationWidgetSortOrder InSecondarySort);

	/** Remove SWidget from the HitTest Grid */
	SLATECORE_API void RemoveWidget(const SWidget* InWidget);
	
	/** Update the widget SecondarySort without removing it and readding it again. */
	SLATECORE_API void UpdateWidget(const SWidget* InWidget, FSlateInvalidationWidgetSortOrder InSecondarySort);

	/** Check if SWidget is contained within the HitTest Grid */
	SLATECORE_API bool ContainsWidget(const SWidget* InWidget) const;

	/**
	 * Add an already existing grid that occupy the same space.
	 * The grid needs to have an owner, not be this grid and occupy the same space as this grid.
	 */
	SLATECORE_API void AddGrid(const TSharedRef<const FHittestGrid>& OtherGrid);

	/** Remove a grid that was appended. */
	SLATECORE_API void RemoveGrid(const TSharedRef<const FHittestGrid>& OtherGrid);

	/** Remove a grid that was appended. */
	SLATECORE_API void RemoveGrid(const SWidget* OtherGridOwner);

	SLATECORE_API void SetPaintToHitTestTransform(const FSlateRenderTransform& InTransform);

	struct FDebuggingFindNextFocusableWidgetArgs
	{
		struct FWidgetResult
		{
			const TSharedPtr<const SWidget> Widget;
			const FText Result;
			FWidgetResult(const TSharedPtr<const SWidget>& InWidget, FText InResult)
				: Widget(InWidget), Result(InResult) {}
		};
		const FArrangedWidget StartingWidget;
		const EUINavigation Direction;
		const FNavigationReply NavigationReply;
		const FArrangedWidget RuleWidget;
		const int32 UserIndex;
		const TSharedPtr<const SWidget> Result;
		TArray<FWidgetResult> IntermediateResults;
	};

#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
	DECLARE_MULTICAST_DELEGATE_TwoParams(FDebuggingFindNextFocusableWidget, const FHittestGrid* /*HittestGrid*/, const FDebuggingFindNextFocusableWidgetArgs& /*Info*/);
	static SLATECORE_API FDebuggingFindNextFocusableWidget OnFindNextFocusableWidgetExecuted;
#endif

#if WITH_SLATE_DEBUGGING
	SLATECORE_API void LogGrid() const;

	enum class EDisplayGridFlags
	{
		None = 0,
		HideDisabledWidgets = 1 << 0,					// Hide hit box for widgets that have IsEnabled false
		HideUnsupportedKeyboardFocusWidgets = 1 << 1,	// Hide hit box for widgets that have SupportsKeyboardFocus false
		UseFocusBrush = 1 << 2,
	};
	SLATECORE_API void DisplayGrid(int32 InLayer, const FGeometry& AllottedGeometry, FSlateWindowElementList& WindowElementList, EDisplayGridFlags DisplayFlags = EDisplayGridFlags::UseFocusBrush) const;

	SLATECORE_API void DisplayNavigation(int32 InLayer, const FGeometry& AllottedGeometry, FSlateWindowElementList& WindowElementList);

	struct FWidgetSortData
	{
		const TWeakPtr<SWidget> WeakWidget;
		int64 PrimarySort;
		FSlateInvalidationWidgetSortOrder SecondarySort;
	};
	SLATECORE_API TArray<FWidgetSortData> GetAllWidgetSortDatas() const;
#endif

private:
	/**
	 * Widget Data we maintain internally store along with the widget reference
	 */
	struct FWidgetData
	{
		FWidgetData(const TWeakPtr<SWidget>& InWidget, const FIntPoint& InUpperLeftCell, const FIntPoint& InLowerRightCell, int64 InPrimarySort, FSlateInvalidationWidgetSortOrder InSecondarySort, int32 InUserIndex)
			: WeakWidget(InWidget)
			, UpperLeftCell(InUpperLeftCell)
			, LowerRightCell(InLowerRightCell)
			, PrimarySort(InPrimarySort)
			, SecondarySort(InSecondarySort)
			, UserIndex(InUserIndex)
		{}
		TWeakPtr<SWidget> WeakWidget;
		TWeakPtr<ICustomHitTestPath> CustomPath;
		FIntPoint UpperLeftCell;
		FIntPoint LowerRightCell;
		int64 PrimarySort;
		FSlateInvalidationWidgetSortOrder SecondarySort;
		int32 UserIndex;

		TSharedPtr<SWidget> GetWidget() const { return WeakWidget.Pin(); }
	};

	struct FWidgetIndex
	{
		FWidgetIndex()
			: Grid(nullptr)
			, WidgetIndex(INDEX_NONE)
		{}
		FWidgetIndex(const FHittestGrid* InHittestGrid, int32 InIndex)
			: Grid(InHittestGrid)
			, WidgetIndex(InIndex)
		{}
		bool IsValid() const { return Grid != nullptr && Grid->WidgetArray.IsValidIndex(WidgetIndex); }
		const FWidgetData& GetWidgetData() const;
		const FSlateRect& GetCullingRect() const { return Grid->CullingRect; }
		const FHittestGrid* GetGrid() const { return Grid; }

	private:
		const FHittestGrid* Grid;
		int32 WidgetIndex;
	};

	struct FIndexAndDistance : FWidgetIndex
	{
		FIndexAndDistance()
			: FWidgetIndex()
			, DistanceSqToWidget(0)
		{}
		FIndexAndDistance(FWidgetIndex WidgetIndex, float InDistanceSq)
			: FWidgetIndex(WidgetIndex)
			, DistanceSqToWidget(InDistanceSq)
		{}
		float GetDistanceSqToWidget() const { return DistanceSqToWidget; }

	private:
		float DistanceSqToWidget;
	};

	struct FGridTestingParams;

	/**
	 * All the available space is partitioned into Cells.
	 * Each Cell contains a list of widgets that overlap the cell.
	 * The list is ordered from back to front.
	 */
	struct FCell
	{
	public:
		FCell() = default;

		void AddIndex(int32 WidgetIndex);
		void RemoveIndex(int32 WidgetIndex);
		void Reset();

		const TArray<int32>& GetWidgetIndexes() const { return WidgetIndexes; }
		
	private:
		TArray<int32> WidgetIndexes;
	};

	struct FAppendedGridData
	{
		FAppendedGridData(const SWidget* InCachedOwner, const TWeakPtr<const FHittestGrid>& InGrid)
			 : CachedOwner(InCachedOwner), Grid(InGrid)
		{ }
		const SWidget* CachedOwner; // Cached owner of the grid
		TWeakPtr<const FHittestGrid> Grid;
	};

	//~ Helper functions
	SLATECORE_API void ClearInternal(int32 TotalCells);

	/** Return the Index and distance to a hit given the testing params */
	SLATECORE_API FIndexAndDistance GetHitIndexFromCellIndex(const FGridTestingParams& Params) const;

	/** @returns true if the child is a paint descendant of the provided Parent. */
	SLATECORE_API bool IsDescendantOf(const SWidget* Parent, const FWidgetData& ChildData) const;

	/** Utility function for searching for the next focusable widget. */
	template<typename TCompareFunc, typename TSourceSideFunc, typename TDestSideFunc>
	TSharedPtr<SWidget> FindFocusableWidget(const FSlateRect WidgetRect, const FSlateRect SweptRect, int32 AxisIndex, int32 Increment, const EUINavigation Direction, const FNavigationReply& NavigationReply, TCompareFunc CompareFunc, TSourceSideFunc SourceSideFunc, TDestSideFunc DestSideFunc, int32 UserIndex, TArray<FDebuggingFindNextFocusableWidgetArgs::FWidgetResult>* IntermediatedResultPtr, TSet<TSharedPtr<SWidget>>* DisabledDestinations) const;

	/** Constrains a float position into the grid coordinate. */
	SLATECORE_API FIntPoint GetCellCoordinate(UE::Slate::FDeprecateVector2DParameter Position) const;

	/** Access a cell at coordinates X, Y. Coordinates are row and column indexes. */
	FORCEINLINE FCell& CellAt(const int32 X, const int32 Y)
	{
		checkfSlow((Y*NumCells.X + X) < Cells.Num(), TEXT("HitTestGrid CellAt() failed: X= %d Y= %d NumCells.X= %d NumCells.Y= %d Cells.Num()= %d"), X, Y, NumCells.X, NumCells.Y, Cells.Num());
		return Cells[Y*NumCells.X + X];
	}

	/** Access a cell at coordinates X, Y. Coordinates are row and column indexes. */
	FORCEINLINE const FCell& CellAt( const int32 X, const int32 Y ) const
	{
		checkfSlow((Y*NumCells.X + X) < Cells.Num(), TEXT("HitTestGrid CellAt() failed: X= %d Y= %d NumCells.X= %d NumCells.Y= %d Cells.Num()= %d"), X, Y, NumCells.X, NumCells.Y, Cells.Num());
		return Cells[Y*NumCells.X + X];
	}

	/** Is the other grid compatible with this grid. */
	SLATECORE_API bool CanBeAppended(const FHittestGrid* OtherGrid) const;

	/** Are both grid of the same size. */
	SLATECORE_API bool SameSize(const FHittestGrid* OtherGrid) const;

	using FCollapsedHittestGridArray = TArray<const FHittestGrid*, TInlineAllocator<16>>;
	/** Get all the hittest grid appended to this grid. */
	SLATECORE_API void GetCollapsedHittestGrid(FCollapsedHittestGridArray& OutResult) const;

	using FCollapsedWidgetsArray = TArray<FWidgetIndex, TInlineAllocator<100>>;
	/** Return the list of all the widget in that cell. */
	SLATECORE_API void GetCollapsedWidgets(FCollapsedWidgetsArray& Out, const int32 X, const int32 Y) const;

	/** Remove appended hittest grid that are not valid anymore. */
	SLATECORE_API void RemoveStaleAppendedHittestGrid();

	static constexpr bool IsCompatibleUserIndex(int32 RequestedUserIndex, int32 TestUserIndex)
	{
		// INDEX_NONE means the user index is compatible with all
		return RequestedUserIndex == INDEX_NONE || TestUserIndex == INDEX_NONE|| RequestedUserIndex == TestUserIndex;
	}

	TInstancedStruct<FNavigationMethod> GetNavigationMethod(const FArrangedWidget& StartingWidget);
	TSharedPtr<SWidget> FindNextFocusableWidgetDefault(const FArrangedWidget& StartingWidget, const EUINavigation Direction, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget, int32 UserIndex);
	TSharedPtr<SWidget> FindNextFocusableWidgetExperimental(const FArrangedWidget& StartingWidget, const EUINavigation Direction, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget, int32 UserIndex);
	
	bool IsWidgetDataInLayerRange(const FWidgetData& WidgetData, const TInterval<int32>& LayerRange);

private:
	/** Map of all the widgets currently in the hit test grid to their stable index. */
	TMap<const SWidget*, int32> WidgetMap;

	/** Stable indexed sparse array of all the widget data we track. */
	TSparseArray<FWidgetData> WidgetArray;

	/** The cells that make up the space partition. */
	TArray<FCell> Cells;

	/** The collapsed grid cached untiled it's dirtied. */
	TArray<FAppendedGridData> AppendedGridArray;

	/** A grid needs a owner to be appended. */
	const SWidget* Owner;

	/** Culling Rect used when the widget was painted. */
	FSlateRect CullingRect;

	/** The size of the grid in cells. */
	FIntPoint NumCells;

	/** Where the 0,0 of the upper-left-most cell corresponds to in desktop space. */
	FVector2f GridOrigin;

	/** Where the 0,0 of the upper-left-most cell corresponds to in window space. */
	FVector2f GridWindowOrigin;

	/** The Size of the current grid. */
	FVector2f GridSize;

	/** The current slate user index that should be associated with any added widgets */
	int32 CurrentUserIndex;

	/** Optional delta transform applied during hit-testing.
		Used by RetainerWidget to reconcile hittest space with the child’s paint space
		when the retainer is non-grid aligned (i.e rotation) */
	TOptional<FSlateRenderTransform> PaintToHitTestTransform;

	/** The size in pixels of a cell */
	static const FVector2f CellSize;

#if WITH_SLATE_DEBUGGING
	/** Cache the state of the most recent FindNextFocusableWidget request*/
	TInstancedStruct<FNavigationMethod> LastNavigationMethod;	
#endif
};

#if WITH_SLATE_DEBUGGING
ENUM_CLASS_FLAGS(FHittestGrid::EDisplayGridFlags);
#endif
