// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/ISlateMetaData.h"
#include "Input/NavigationReply.h"

/**
* FNavigationTransition is a description of a navigation change between two widgets. 
* When a user attempts to navigate from a widget, 2 Transitions are reported. 
* The Outgoing Transition is executed on the OldFocusedWidget.
* The Incoming Transition is executed on the NewFOcusedWidget.
*/

enum class ENavigationTransitionDirection
{
	Incoming, 
	Outgoing
};

struct FNavigationTransition
{
	FNavigationTransition(EUINavigation Type, TWeakPtr<SWidget> OldFocusedWidget, TWeakPtr<SWidget> NewFocusedWidget, ENavigationTransitionDirection Direction)
		: Type(Type)
		, OldFocusedWidget(OldFocusedWidget)
		, NewFocusedWidget(NewFocusedWidget)
		, Direction(Direction)
	{}

	EUINavigation Type;
	TWeakPtr<SWidget> OldFocusedWidget;
	TWeakPtr<SWidget> NewFocusedWidget;
	ENavigationTransitionDirection Direction;
};

DECLARE_DELEGATE_OneParam(FOnNavigationTransitionDelegate, const FNavigationTransition&);

/**
 * FNavigationTransitionMetadatais an optional data field on SWidget instances that provides users with additional information about navigation transitions.
 */
struct FNavigationTransitionMetadata : ISlateMetaData
{
	SLATE_METADATA_TYPE(FNavigationTransitionMetadata, ISlateMetaData);

	FOnNavigationTransitionDelegate OnNavigationTransition;
};