// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"
#include "Input/HittestGrid.h"

#include "NavigationMethod.generated.h"

USTRUCT()
struct FNavigationMethod
{
	GENERATED_BODY()

public:
	using FDebugWidgetResult = FHittestGrid::FDebuggingFindNextFocusableWidgetArgs::FWidgetResult;

	FNavigationMethod() = default;
	virtual ~FNavigationMethod() = default;
	SLATECORE_API virtual void Initialize(const FHittestGrid* InHittestGrid, TArray<FDebugWidgetResult>* InIntermediateResultsPtr);
	SLATECORE_API virtual TSharedPtr<SWidget> FindNextFocusableWidget(const FArrangedWidget& StartingWidget, const EUINavigation Direction, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget, int32 InUserIndex);
	
protected:
	SLATECORE_API FIntPoint GetCellCoordinate(FVector2f Position) const;
	SLATECORE_API bool IsValidCellCoordinate(int32 X, int32 Y) const;
	SLATECORE_API bool IsParentsEnabled(const SWidget* Widget);

	using FWidgetFunc = TFunction<bool(const TSharedPtr<SWidget>)>;
	SLATECORE_API void ForEachFocusableWidgetsInCell(int32 X, int32 Y, const FNavigationReply& NavigationReply, int32 UserIndex, FWidgetFunc WidgetFunc);
	
	const FHittestGrid* HittestGrid;
	TArray<FDebugWidgetResult>* IntermediateResultsPtr;
	TSet<TSharedPtr<SWidget>> DisabledDestinations;

private:
	static FString DefaultNavigationMethod;
	static TMap<FString, TSharedPtr<FNavigationMethod>> RegisteredNavigationMethods;

#if WITH_SLATE_DEBUGGING
public:
	SLATECORE_API virtual void DrawDebug(int32 InLayer, const FGeometry& AllottedGeometry, FSlateWindowElementList& WindowElementList);
#endif // WITH_SLATE_DEBUGGING

protected:
#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
	SLATECORE_API void AddDebugIntermediateResult(const TSharedPtr<const SWidget>& InWidget, const FText Result);
#endif
};
