// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "NavigationReply.generated.h"

class SWidget;
enum class EUINavigation : uint8;

UENUM(BlueprintType)
enum class EUINavigationRule : uint8
{
	/** Allow the movement to continue in that direction, seeking the next navigable widget automatically. */
	Escape,
	/** Move to a specific widget. */
	Explicit,
	/**
	 * Wrap movement inside this container, causing the movement to cycle around from the opposite side, 
	 * if the navigation attempt would have escaped.
	 */
	Wrap,
	/** Stops movement in this direction */
	Stop,
	/** Custom navigation handled by user code. */
	Custom,
	/** Custom navigation handled by user code if the boundary is hit. */
	CustomBoundary,
	/** Invalid Rule */
	Invalid
};

DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SWidget>, FNavigationDelegate, EUINavigation);

/**
 * A FNavigationReply is something that a Slate navigation event returns to the system to notify it about the boundary rules for navigation.
 * For example, a widget may handle an OnNavigate event by asking the system to wrap if it's boundary is hit.
 * To do this, return FNavigationReply::Wrap().
 */
class FNavigationReply
{
public:

	/** The widget that ultimately specified the boundary rule for the navigation */
	const TSharedPtr<SWidget> GetHandler() const { return EventHandler; }

	/** Get the navigation boundary rule. */
	EUINavigationRule GetBoundaryRule() const { return BoundaryRule; }

	/** If the event replied with a constant explicit boundary rule this returns the desired focus recipient. Otherwise returns an invalid pointer. */
	const TSharedPtr<SWidget>& GetFocusRecipient() const { return FocusRecipient; }

	/** If the event replied with a delegate explicit boundary rule this returns the delegate to get the focus recipient. Delegate will be unbound if a constant widget was provided. */
	const FNavigationDelegate& GetFocusDelegate() const { return FocusDelegate; }

public:

	/**
	 * An event should return a FNavigationReply::Explicit() to let the system know to navigate to an explicit widget at the bounds of this widget.
	 */
	static FNavigationReply Explicit(TSharedPtr<SWidget> InFocusRecipient)
	{
		FNavigationReply NewReply;
		NewReply.BoundaryRule = EUINavigationRule::Explicit;
		NewReply.FocusRecipient = InFocusRecipient;
		return NewReply;
	}

	/**
	* An event should return a FNavigationReply::Custom() to let the system know to call a custom delegate to get the widget to navigate to.
	*/
	static FNavigationReply Custom(const FNavigationDelegate& InFocusDelegate)
	{
		FNavigationReply NewReply;
		NewReply.BoundaryRule = EUINavigationRule::Custom;
		NewReply.FocusDelegate = InFocusDelegate;
		return NewReply;
	}

	/**
	 * An event should return a FNavigationReply::CustomBoundary() to let the system know to call a custom delegate to get the widget to navigate to is the widgets boundary is hit.
	 */
	static FNavigationReply CustomBoundary(const FNavigationDelegate& InFocusDelegate)
	{
		FNavigationReply NewReply;
		NewReply.BoundaryRule = EUINavigationRule::CustomBoundary;
		NewReply.FocusDelegate = InFocusDelegate;
		return NewReply;
	}

	/**
	 * An event should return a FNavigationReply::Wrap() to let the system know to wrap at the bounds of this widget.
	 */
	static FNavigationReply Wrap()
	{
		FNavigationReply NewReply;
		NewReply.BoundaryRule = EUINavigationRule::Wrap;
		return NewReply;
	}

	/**
	 * An event should return a FNavigationReply::Stop() to let the system know to stop at the bounds of this widget.
	 */
	static FNavigationReply Stop()
	{
		FNavigationReply NewReply;
		NewReply.BoundaryRule = EUINavigationRule::Stop;
		return NewReply;
	}

	/**
	 * An event should return a FNavigationReply::Escape() to let the system know that a navigation can escape the bounds of this widget.
	 */
	static FNavigationReply Escape()
	{
		FNavigationReply NewReply;
		NewReply.BoundaryRule = EUINavigationRule::Escape;
		return NewReply;
	}

private:

	/** Set the widget that handled the event; undefined if never handled. This method is to be used by SlateApplication only! */
	FNavigationReply& SetHandler(const TSharedRef<SWidget>& InHandler)
	{
		this->EventHandler = InHandler;
		return *this;
	}

	/**
	 * Hidden default constructor.
	 */
	FNavigationReply()
		: EventHandler(nullptr)
		, FocusRecipient(nullptr)
		, FocusDelegate()
		, BoundaryRule(EUINavigationRule::Escape)
	{ }

	/**
	 * Verbose Constructor. Used by SWidget when constructing from MetaData.
	 */
	FNavigationReply(EUINavigationRule InBoundaryRule, TSharedPtr<SWidget> InFocusRecipient, const FNavigationDelegate& InFocusDelegate)
		: EventHandler(nullptr)
		, FocusRecipient(InFocusRecipient)
		, FocusDelegate(InFocusDelegate)
		, BoundaryRule(InBoundaryRule)
	{ }

private:

	TSharedPtr<SWidget> EventHandler;
	TSharedPtr<SWidget> FocusRecipient;
	FNavigationDelegate FocusDelegate;
	EUINavigationRule BoundaryRule;

	friend class FSlateApplication;
	friend class FSlateNavigationEventSimulator;
	friend class SWidget;
};
