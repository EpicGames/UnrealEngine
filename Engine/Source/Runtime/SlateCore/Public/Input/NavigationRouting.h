// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreMinimal.h"

#include "NavigationRouting.generated.h"

class SWidget;
class FHittestGrid;
class FNavigationReply;
class FSlateRect;

UENUM()
enum class EWidgetNavigationRoutingPolicy : uint8
{
	AcceptFocus UMETA(DisplayName="Accept Focus", Tooltip = "On navigation, this widget, or widgets within this boundary, will receive focus. Focus will not be routed"),
	RouteToTopMostChild  UMETA(DisplayName="Route To Top Most Child", Tooltip="On navigation, this widget, or widgets within this boundary, will route focus to topmost visibile widget in this container"),
	RouteToBottomMostChild  UMETA(DisplayName="Route To Bottom Most Child", Tooltip="On navigation, this widget, or widgets within this boundary, will route focus to bottommost visibile widget in this container"),
	RouteToLeftMostChild UMETA(DisplayName="Route To Left Most Child", Tooltip="On navigation, this widget, or widgets within this boundary, will route focus to leftmost visibile widget in this container"),
	RouteToRightMostChild UMETA(DisplayName="Route To Right Most Child", Tooltip="On navigation, this widget, or widgets within this boundary, will route focus to rightmost visibile widget in this container"),
	RouteToTopLeftChild UMETA(DisplayName="Route To Top Left Child", Tooltip="On navigation, this widget, or widgets within this boundary, will route focus to widget in the top left container"),
	RouteToTopRightChild UMETA(DisplayName="Route To Top Right Child", Tooltip="On navigation, this widget, or widgets within this boundary, will route focus to widget in the top right container"),
	RouteToBottomLeftChild UMETA(DisplayName="Route To Bottom Left Child", Tooltip="On navigation, this widget, or widgets within this boundary, will route focus to widget in the bottom left container"),
	RouteToBottomRightChild UMETA(DisplayName="Route To Bottom Right Child", Tooltip="On navigation, this widget, or widgets within this boundary, will route focus to widget in the bottom right container"),
	MAX UMETA(Hidden),
	Default = AcceptFocus UMETA(Hidden)
};

struct FNavigationRoutingParams
{
	FNavigationRoutingParams(TSharedPtr<SWidget> StartingWidget, TSharedPtr<SWidget> FocusedWidget, const FHittestGrid* HittestGrid, const FNavigationReply& NavigationReply, int32 UserIndex)
		: StartingWidget(StartingWidget)
		, FocusedWidget(FocusedWidget)
		, HittestGrid(HittestGrid)
		, NavigationReply(NavigationReply)
		, UserIndex(UserIndex)
	{}
	TSharedPtr<SWidget> StartingWidget;
	TSharedPtr<SWidget> FocusedWidget;
	const FHittestGrid* HittestGrid;
	const FNavigationReply& NavigationReply;
	int32 UserIndex;
};

struct FNavigationRouting
{
	static TSharedPtr<SWidget> RouteNavigationFocus(const FNavigationRoutingParams& Params);

private:
	using FGetEdgeFunc = TFunction<float(const FSlateRect&)>;
	static TSharedPtr<SWidget> RouteNavigationFocusToEdge(const FNavigationRoutingParams& Params, int Direction, int AxisIndex, const FSlateRect& BoundaryRect, const FVector2f& StartingPoint, FGetEdgeFunc GetEdgeFunc);
};