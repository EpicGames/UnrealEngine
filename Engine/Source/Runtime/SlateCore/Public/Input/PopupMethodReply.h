// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/ReplyBase.h"

enum class EPopupMethod : uint8
{
	/** Creating a new window allows us to place popups outside of the window in which the menu anchor resides. */
	CreateNewWindow,
	/** Place the popup into the current window. Applications that intend to run in fullscreen cannot create new windows, so they must use this method.. */
	UseCurrentWindow
};

enum class EShouldThrottle : uint8
{
	No,
	Yes
};

/** Reply informs Slate how it should express the popup: by creating a new window or by reusing the existing window. */
class FPopupMethodReply : public TReplyBase<FPopupMethodReply>
{
public:
	/** Create a reply that signals not having an opinion about the popup method */
	static FPopupMethodReply Unhandled()
	{
		return FPopupMethodReply(false, EPopupMethod::CreateNewWindow);
	}

	/** Create a reply that specifies how a popup should be handled. */
	static FPopupMethodReply UseMethod(EPopupMethod WithMethod)
	{
		return FPopupMethodReply(true, WithMethod);
	}

	/** Specify whether we should throttle the engine ticking s.t. the UI is most responsive when this popup is up. */
	FPopupMethodReply& SetShouldThrottle(EShouldThrottle InShouldThrottle)
	{
		ShouldThrottle = InShouldThrottle;
		return Me();
	}

	/** Should we throttle the engine? */
	EShouldThrottle GetShouldThrottle() const
	{
		return ShouldThrottle;
	}

	/** Which method to use for the popup: new window or reuse current window */
	EPopupMethod GetPopupMethod() const
	{
		return PopupMethod;
	}

	/** Alias for IsEventHandled for situations where this is used as optional */
	bool IsSet() const
	{
		return IsEventHandled();
	}


public:
	FPopupMethodReply()
		:TReplyBase<FPopupMethodReply>(false)
		, PopupMethod(EPopupMethod::CreateNewWindow)
		, ShouldThrottle(EShouldThrottle::Yes)
	{}

private:
	FPopupMethodReply(bool bIsHandled, EPopupMethod InMethod)
		: TReplyBase<FPopupMethodReply>(bIsHandled)
		, PopupMethod(InMethod)
		, ShouldThrottle(EShouldThrottle::Yes)
	{
	}

	EPopupMethod PopupMethod;
	EShouldThrottle ShouldThrottle;
};
