// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Types/SlateEnums.h"
#include "Input/ReplyBase.h"
#include "Input/Events.h"
#include "Input/DragAndDrop.h"

class SWidget;

// @todo Slate: this is highly sketchy, as FDragDropOperation is declared in Slate,
// but cannot currently be brought into SlateCore due to dependencies to SWindow.
class FDragDropOperation;

/**
 * A Reply is something that a Slate event returns to the system to notify it about certain aspect of how an event was handled.
 * For example, a widget may handle an OnMouseDown event by asking the system to give mouse capture to a specific Widget.
 * To do this, return FReply::CaptureMouse( NewMouseCapture ).
 */
class FReply : public TReplyBase<FReply>
{
public:
		
	/** An event should return a FReply::Handled().CaptureMouse( SomeWidget ) as a means of asking the system to forward all mouse events to SomeWidget */
	FReply& CaptureMouse( TSharedRef<SWidget> InMouseCaptor )
	{
		this->MouseCaptor = InMouseCaptor;
		return Me();
	}

	/**
	 * Enables the use of high precision (raw input) mouse movement, for more accurate mouse movement without mouse ballistics
	 * NOTE: This implies mouse capture and hidden mouse movement.  Releasing mouse capture deactivates this mode.
	 */
	FReply& UseHighPrecisionMouseMovement( TSharedRef<SWidget> InMouseCaptor )
	{
		this->MouseCaptor = InMouseCaptor;
		this->bUseHighPrecisionMouse = true;
		return Me();
	}

	/**
	 * An event should return FReply::Handled().SetMousePos to ask Slate to move the mouse cursor to a different location
	 */
	SLATECORE_API FReply& SetMousePos( const FIntPoint& NewMousePos );

	/** An event should return FReply::Handled().SetUserFocus( SomeWidget ) as a means of asking the system to set users focus to the provided widget*/
	SLATECORE_API FReply& SetUserFocus(TSharedRef<SWidget> GiveMeFocus, EFocusCause ReasonFocusIsChanging = EFocusCause::SetDirectly, bool bInAllUsers = false);

	/** An event should return a FReply::Handled().ClearUserFocus() to ask the system to clear user focus*/
	FReply& ClearUserFocus(bool bInAllUsers = false)
	{
		return ClearUserFocus(EFocusCause::SetDirectly, bInAllUsers);
	}

	/** An event should return a FReply::Handled().ClearUserFocus() to ask the system to clear user focus*/
	SLATECORE_API FReply& ClearUserFocus(EFocusCause ReasonFocusIsChanging, bool bInAllUsers = false);

	/** 
	 *	An event should rarely invoke FReply::CancelFocusRequest. This will change the reply
	 *  so that it no longer contains a request for the system to set or clear focus. 
	 *  This is most useful when a reply has been cached for use in async operations.
	 */
	SLATECORE_API FReply& CancelFocusRequest();

	/** An event should return FReply::Handled().SetNavigation( NavigationType ) as a means of asking the system to attempt a navigation*/
	FReply& SetNavigation(EUINavigation InNavigationType, const ENavigationGenesis InNavigationGenesis, const ENavigationSource InNavigationSource = ENavigationSource::FocusedWidget)
	{
		this->NavigationType = InNavigationType;
		this->NavigationGenesis = InNavigationGenesis;
		this->NavigationSource = InNavigationSource;
		this->NavigationDestination = nullptr;
		return Me();
	}

	/** An event should return FReply::Handled().SetNavigation( NavigationDestination ) as a means of asking the system to attempt a navigation to the specified destination*/
	FReply& SetNavigation(TSharedRef<SWidget> InNavigationDestination, const ENavigationGenesis InNavigationGenesis, const ENavigationSource InNavigationSource = ENavigationSource::FocusedWidget)
	{
		this->NavigationType = EUINavigation::Invalid;
		this->NavigationGenesis = InNavigationGenesis;
		this->NavigationSource = InNavigationSource;
		this->NavigationDestination = InNavigationDestination;
		return Me();
	}

	/**
	 * An event should return FReply::Handled().LockMouseToWidget( SomeWidget ) as a means of asking the system to 
	 * Lock the mouse so it cannot move out of the bounds of the widget.
	 */
	FReply& LockMouseToWidget( TSharedRef<SWidget> InWidget )
	{
		this->MouseLockWidget = InWidget;
		this->bShouldReleaseMouseLock = false;
		return Me();
	}

	/** 
	 * An event should return a FReply::Handled().ReleaseMouseLock() to ask the system to release mouse lock on a widget
	 */
	FReply& ReleaseMouseLock()
	{
		this->bShouldReleaseMouseLock = true;
		MouseLockWidget.Reset();
		return Me();
	}

	/** 
	 * An event should return a FReply::Handled().ReleaseMouse() to ask the system to release mouse capture
	 * NOTE: Deactivates high precision mouse movement if activated.
	 */
	FReply& ReleaseMouseCapture()
	{
		this->MouseCaptor.Reset();
		this->bReleaseMouseCapture = true;
		this->bUseHighPrecisionMouse = false;
		return Me();
	}

	/**
	 * Ask Slate to detect if a user started dragging in this widget.
	 * If a drag is detected, Slate will send an OnDragDetected event.
	 *
	 * @param DetectDragInMe  Detect dragging in this widget
	 * @param MouseButton     This button should be pressed to detect the drag
	 */
	FReply& DetectDrag( const TSharedRef<SWidget>& DetectDragInMe, FKey MouseButton )
	{
		this->DetectDragForWidget = DetectDragInMe;
		this->DetectDragForMouseButton = MouseButton;
		return Me();
	}

	/**
	 * An event should return FReply::Handled().BeginDragDrop( Content ) to initiate a drag and drop operation.
	 *
	 * @param InDragDropContent  The content that is being dragged. This could be a widget, or some arbitrary data
	 *
	 * @return reference back to the FReply so that this call can be chained.
	 */
	FReply& BeginDragDrop(TSharedRef<FDragDropOperation> InDragDropContent)
	{
		this->DragDropContent = InDragDropContent;
		return Me();
	}

	/** An event should return FReply::Handled().EndDragDrop() to request that the current drag/drop operation be terminated. */
	FReply& EndDragDrop()
	{
		this->bEndDragDrop = true;
		return Me();
	}

	/** Ensures throttling for Slate UI responsiveness is not done on mouse down */
	FReply& PreventThrottling()
	{
		this->bPreventThrottling = true;
		return Me();
	}

public:

	/** True if this reply indicated that we should release mouse capture as a result of the event being handled */
	bool ShouldReleaseMouse() const { return bReleaseMouseCapture; }

	/** true if this reply indicated that we should set focus as a result of the event being handled */
	bool ShouldSetUserFocus() const { return bSetUserFocus; }

	/** true if this reply indicated that we should release focus as a result of the event being handled */
	bool ShouldReleaseUserFocus() const { return bReleaseUserFocus; }

	/** If the event replied with a request to change the user focus whether it should do it for all users or just the current UserIndex */
	bool AffectsAllUsers() const { return bAllUsers; }

	/** True if this reply indicated that we should use high precision mouse movement */
	bool ShouldUseHighPrecisionMouse() const { return bUseHighPrecisionMouse; }

	/** True if the reply indicated that we should release mouse lock */
	bool ShouldReleaseMouseLock() const { return bShouldReleaseMouseLock; }

	/** Whether or not we should throttle on mouse down */
	bool ShouldThrottle() const { return !bPreventThrottling; }

	/** Returns the widget that the mouse should be locked to (if any) */
	TSharedPtr<SWidget> GetMouseLockWidget() const { return MouseLockWidget.Pin(); }

	/** When not nullptr, user focus has been requested to be set on the FocusRecipient. */
	TSharedPtr<SWidget> GetUserFocusRecepient() const { return FocusRecipient.Pin(); }

	/** Get the reason that a focus change is being requested. */
	EFocusCause GetFocusCause() const { return FocusChangeReason; }

	/** If the event replied with a request to capture the mouse, this returns the desired mouse captor. Otherwise returns an invalid pointer. */
	TSharedPtr<SWidget> GetMouseCaptor() const { return MouseCaptor.Pin(); }

	/** Get the navigation type (Invalid if no navigation is requested). */
	EUINavigation GetNavigationType() const { return NavigationType; }

	/** Get the genesis of the navigation. */
	ENavigationGenesis GetNavigationGenesis() const { return NavigationGenesis; }

	/** Get the source of the navigation. */
	ENavigationSource GetNavigationSource() const { return NavigationSource; }

	/** Get the widget that is the navigation destination. */
	TSharedPtr<SWidget> GetNavigationDestination() const { return NavigationDestination.Pin(); }

	/** @return the Content that we should use for the Drag and Drop operation; Invalid SharedPtr if a drag and drop operation is not requested*/
	const TSharedPtr<FDragDropOperation>& GetDragDropContent() const { return DragDropContent; }

	/** @return true if the user has asked us to terminate the ongoing drag/drop operation */
	bool ShouldEndDragDrop() const { return bEndDragDrop; }

	/** @return a widget for which to detect a drag; Invalid SharedPtr if no drag detection requested */
	TSharedPtr<SWidget> GetDetectDragRequest() const { return DetectDragForWidget.Pin(); }

	/** @return the mouse button for which we are detecting a drag */
	FKey GetDetectDragRequestButton() const { return DetectDragForMouseButton; }

	/** @return The coordinates of the requested mouse position */
	const TOptional<FIntPoint>& GetRequestedMousePos() const { return RequestedMousePos; }

	/** Converts the reply into an string representation. */
	SLATECORE_API FString ToString();
		
public:

	/**
	 * An event should return a FReply::Handled() to let the system know that an event was handled.
	 */
	[[nodiscard]] static FReply Handled( )
	{
		return FReply(true);
	}

	/**
	 * An event should return a FReply::Unhandled() to let the system know that an event was unhandled.
	 */
	[[nodiscard]] static FReply Unhandled( )
	{
		return FReply(false);
	}

public:
	/** Default copy constructor. */
	SLATECORE_API FReply(const FReply&);

	/** Default copy-assignment operator. */
	SLATECORE_API FReply& operator=(const FReply&);

private:

	/**
	 * Hidden default constructor.
	 */
	SLATECORE_API FReply( bool bIsHandled );

private:
	friend class FSlateApplication;

private:

	TOptional<FIntPoint> RequestedMousePos;
	TWeakPtr<SWidget> EventHandler;
	TWeakPtr<SWidget> MouseCaptor;
	TWeakPtr<SWidget> FocusRecipient;
	TWeakPtr<SWidget> MouseLockWidget;
	TWeakPtr<SWidget> DetectDragForWidget;
	TWeakPtr<SWidget> NavigationDestination;
	FKey DetectDragForMouseButton;
	TSharedPtr<FDragDropOperation> DragDropContent;
	EFocusCause FocusChangeReason;
	EUINavigation NavigationType;
	ENavigationGenesis NavigationGenesis;
	ENavigationSource NavigationSource;
	uint32 bReleaseMouseCapture : 1;
	uint32 bSetUserFocus : 1;
	uint32 bReleaseUserFocus : 1;
	uint32 bAllUsers : 1;
	uint32 bShouldReleaseMouseLock : 1;
	uint32 bUseHighPrecisionMouse : 1;
	uint32 bPreventThrottling : 1;
	uint32 bEndDragDrop : 1;
};
