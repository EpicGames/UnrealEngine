// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class SWidget;

/**
 * Base class for all the ways that a Slate Widget can reply to events.
 * Typical replies include requests to set focus, capture or release the pointer,
 * signify a cursor preference, etc.
 */
class FReplyBase
{
public:

	/** @return true if this this reply is a result of the event being handled; false otherwise. */
	bool IsEventHandled() const { return bIsHandled; }
	
	/** The widget that ultimately handled the event */
	const TSharedPtr<SWidget> GetHandler() const { return EventHandler; }
		
protected:

	/** A reply can be handled or unhandled. Any widget handling events decides whether it has handled the event. */
	FReplyBase( bool InIsHandled )
	: bIsHandled(InIsHandled)
	{
	}

	/** Has a widget handled an event. */ 
	bool bIsHandled;
	
	/** Widget that handled the event that generated this reply. */
	TSharedPtr<SWidget> EventHandler;
};

template<typename ReplyType>
class TReplyBase : public FReplyBase
{
public:
	TReplyBase( bool bIsHandled )
	: FReplyBase( bIsHandled )
	{
	}

protected:
	friend class FEventRouter;

	/** Set the widget that handled the event; undefined if never handled. This method is to be used by SlateApplication only! */
	ReplyType& SetHandler( const TSharedRef<SWidget>& InHandler )
	{
		this->EventHandler = InHandler;
		return Me();
	}

	/** @return a reference to this reply */
	ReplyType& Me()
	{
		return static_cast<ReplyType&>(*this);
	}
};

/** A reply type for events that return a void reply. e.g. OnMouseLeave() */
class FNoReply : public TReplyBase<FNoReply>
{
public:
	static FNoReply Unhandled(){ return FNoReply(); }
	FNoReply() : TReplyBase<FNoReply>(false){}
};
