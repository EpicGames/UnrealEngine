// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Layout/ArrangedWidget.h"

/**
 * The results of an ArrangeChildren are always returned as an FArrangedChildren.
 * FArrangedChildren supports a filter that is useful for excluding widgets with unwanted
 * visibilities.
 */
class FArrangedChildren
{
	private:
	
	EVisibility VisibilityFilter;
	
	public:

	typedef TArray<FArrangedWidget, TInlineAllocator<4>> FArrangedWidgetArray;

	/**
	 * Construct a new container for arranged children that only accepts children that match the VisibilityFilter.
	 * e.g.
	 *  FArrangedChildren ArrangedChildren( VIS_All ); // Children will be included regardless of visibility
	 *  FArrangedChildren ArrangedChildren( EVisibility::Visible ); // Only visible children will be included
	 *  FArrangedChildren ArrangedChildren( EVisibility::Collapsed | EVisibility::Hidden ); // Only hidden and collapsed children will be included.
	 */
	FArrangedChildren( EVisibility InVisibilityFilter, bool bInAllow3DWidgets = false )
	: VisibilityFilter( InVisibilityFilter )
	, bAllow3DWidgets( bInAllow3DWidgets )
	{
	}

	// @todo hittest2.0 : we should get rid of this eventually.
	static FArrangedChildren Hittest2_FromArray(const TArrayView<FWidgetAndPointer> InWidgets)
	{
		FArrangedChildren Temp( EVisibility::All );
		Temp.Array.Reserve(InWidgets.Num());
		for (const FWidgetAndPointer& WidgetAndPointer : InWidgets)
		{
			Temp.Array.Add(WidgetAndPointer);
		}		
		return Temp;
	}

	/** Reverse the order of the arranged children */
	void Reverse()
	{
		int32 LastElementIndex = Array.Num() - 1;
		for (int32 WidgetIndex = 0; WidgetIndex < Array.Num()/2; ++WidgetIndex )
		{
			Array.Swap( WidgetIndex, LastElementIndex - WidgetIndex );
		}
	}

	/**
	 * Add an arranged widget (i.e. widget and its resulting geometry) to the list of Arranged children.
	 *
	 * @param VisibilityOverride   The arrange function may override the visibility of the widget for the purposes
	 *                             of layout or performance (i.e. prevent redundant call to Widget->GetVisibility())
	 * @param InWidgetGeometry     The arranged widget (i.e. widget and its geometry)
	 */
	inline void AddWidget(EVisibility VisibilityOverride, const FArrangedWidget& InWidgetGeometry)
	{
		if ( Accepts(VisibilityOverride) )
		{
			Array.Add(InWidgetGeometry);
		}
	}

	inline void InsertWidget(EVisibility VisibilityOverride, const FArrangedWidget& InWidgetGeometry, int32 Index)
	{
		if (Accepts(VisibilityOverride))
		{
			Array.Insert(InWidgetGeometry, Index);
		}
	}

	/** Reserves sufficient memory for the given number of arranged widgets. */
	inline void Reserve(FArrangedWidgetArray::SizeType Number)
	{
		Array.Reserve(Number);
	}
	
	/**
	 * Add an arranged widget (i.e. widget and its resulting geometry) to the list of Arranged children
	 * based on the the visibility filter and the arranged widget's visibility
	 */
	SLATECORE_API void AddWidget(const FArrangedWidget& InWidgetGeometry);

	SLATECORE_API void InsertWidget(const FArrangedWidget& InWidgetGeometry, int32 Index);

	inline void SetFilter(EVisibility InVisibility)
	{
		VisibilityFilter = InVisibility;
	}

	inline EVisibility GetFilter() const
	{
		return VisibilityFilter;
	}

	inline bool Accepts(EVisibility InVisibility) const
	{
		return EVisibility::DoesVisibilityPassFilter(InVisibility, VisibilityFilter);
	}

	inline bool Allows3DWidgets() const
	{
		return bAllow3DWidgets;
	}

	// We duplicate parts of TArray's interface here!
	// Inheriting from "public TArray<FArrangedWidget>"
	// saves us this boilerplate, but causes instantiation of
	// many template combinations that we do not want.

private:
	/** Internal representation of the array widgets */
	FArrangedWidgetArray Array;

	bool bAllow3DWidgets;
public:

	inline FArrangedWidgetArray& GetInternalArray()
	{
		return Array;
	}

	inline const FArrangedWidgetArray& GetInternalArray() const
	{
		return Array;
	}

	inline int32 Num() const
	{
		return Array.Num();
	}

	inline const FArrangedWidget& operator[]( int32 Index ) const
	{
		return Array[Index];
	}

	inline FArrangedWidget& operator[](int32 Index)
	{
		return Array[Index];
	}

	inline const FArrangedWidget& Last() const
	{
		return Array.Last();
	}

	inline FArrangedWidget& Last()
	{
		return Array.Last();
	}

	inline int32 FindItemIndex(const FArrangedWidget& ItemToFind) const
	{
		return Array.Find(ItemToFind);
	}

	template<typename PredicateType>
	inline int32 IndexOfByPredicate(const PredicateType& Pred) const
	{
		return Array.IndexOfByPredicate( Pred );
	}

	template<typename PredicateType>
	inline int32 FindLastByPredicate(const PredicateType& Pred) const
	{
		return Array.FindLastByPredicate(Pred);
	}

	inline void Remove(int32 Index, int32 Count=1)
	{
		Array.RemoveAt(Index, Count);
	}

	inline void Append(const FArrangedChildren& Source)
	{
		Array.Append( Source.Array );
	}

	inline void Empty()
	{
		Array.Reset();
	}

	inline bool IsValidIndex(int32 Index)
	{
		return Array.IsValidIndex(Index);
	}
};
