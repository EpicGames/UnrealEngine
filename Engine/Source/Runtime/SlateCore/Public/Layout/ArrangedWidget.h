// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Geometry.h"
#include "Input/Events.h"

#define UE_API SLATECORE_API

class SWidget;

/**
 * A pair: Widget and its Geometry. Widgets populate an list of WidgetGeometries
 * when they arrange their children. See SWidget::ArrangeChildren.
 */
class FArrangedWidget
{
public:

	FArrangedWidget( TSharedRef<SWidget> InWidget, const FGeometry& InGeometry )
		: Geometry(InGeometry)
		, Widget(InWidget)
	{ }

	SLATECORE_API static const FArrangedWidget& GetNullWidget();

public:

	/** The widget that is being arranged. */
	SWidget* GetWidgetPtr() const
	{
		return &Widget.Get();
	}

	/**
	 * Gets the string representation of the Widget and corresponding Geometry.
	 *
	 * @return String representation.
	 */
	FString ToString( ) const;

public:

	/**
	 * Compares this widget arrangement with another for equality.
	 *
	 * @param Other The other arrangement to compare with.
	 * @return true if the two arrangements are equal, false otherwise.
	 */
	bool operator==( const FArrangedWidget& Other ) const 
	{
		return Widget == Other.Widget;
	}

public:

	/** The widget's geometry. */
	FGeometry Geometry;

	/** The widget that is being arranged. */
	TSharedRef<SWidget> Widget;
};

struct FWidgetAndPointer : public FArrangedWidget
{
public:
	UE_API FWidgetAndPointer();
	UE_API FWidgetAndPointer(const FArrangedWidget& InWidget);
	UE_API FWidgetAndPointer(const FArrangedWidget& InWidget, TOptional<FVirtualPointerPosition> InPosition);

	TOptional<FVirtualPointerPosition> GetPointerPosition() const
	{
		return OptionalPointerPosition;
	}

	void SetPointerPosition(TOptional<FVirtualPointerPosition> InPosition)
	{
		OptionalPointerPosition = InPosition;
	}

private:
	TOptional<FVirtualPointerPosition> OptionalPointerPosition;
};

#undef UE_API
