// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/WidgetSlotWithAttributeSupport.h"
#include "Layout/FlowDirection.h"
#include "Layout/Margin.h"
#include "Misc/Optional.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#include <type_traits>


/** Mixin to add the alignment functionality to a base slot. */
template <typename MixedIntoType>
class TAlignmentWidgetSlotMixin
{
public:
	TAlignmentWidgetSlotMixin()
		: HAlignment(HAlign_Fill)
		, VAlignment(VAlign_Fill)
	{}

	TAlignmentWidgetSlotMixin(const EHorizontalAlignment InHAlign, const EVerticalAlignment InVAlign)
		: HAlignment(InHAlign)
		, VAlignment(InVAlign)
	{}
	
public:
	struct FSlotArgumentsMixin
	{
	private:
		friend TAlignmentWidgetSlotMixin;

	public:
		typename MixedIntoType::FSlotArguments& HAlign(EHorizontalAlignment InHAlignment)
		{
			_HAlignment = InHAlignment;
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

		typename MixedIntoType::FSlotArguments& VAlign(EVerticalAlignment InVAlignment)
		{
			_VAlignment = InVAlignment;
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

	private:
		TOptional<EHorizontalAlignment> _HAlignment;
		TOptional<EVerticalAlignment> _VAlignment;
	};

protected:
	void ConstructMixin(const FChildren& SlotOwner, FSlotArgumentsMixin&& InArgs)
	{
		HAlignment = InArgs._HAlignment.Get(HAlignment);
		VAlignment = InArgs._VAlignment.Get(VAlignment);
	}

public:
	void SetHorizontalAlignment(EHorizontalAlignment Alignment)
	{
		if (HAlignment != Alignment)
		{
			HAlignment = Alignment;
			static_cast<MixedIntoType*>(this)->Invalidate(EInvalidateWidgetReason::Layout);
		}
	}

	EHorizontalAlignment GetHorizontalAlignment() const
	{
		return HAlignment;
	}

	void SetVerticalAlignment(EVerticalAlignment Alignment)
	{
		if (VAlignment != Alignment)
		{
			VAlignment = Alignment;
			static_cast<MixedIntoType*>(this)->Invalidate(EInvalidateWidgetReason::Layout);
		}
	}

	EVerticalAlignment GetVerticalAlignment() const
	{
		return VAlignment;
	}

private:
	/** Horizontal positioning of child within the allocated slot */
	EHorizontalAlignment HAlignment;
	/** Vertical positioning of child within the allocated slot */
	EVerticalAlignment VAlignment;
};


/** Mixin to add the alignment functionality to a base slot that is also a single children. */
template <typename MixedIntoType>
class TAlignmentSingleWidgetSlotMixin
{
public:
	template<typename WidgetType, typename V = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value>::type>
	TAlignmentSingleWidgetSlotMixin(WidgetType& InParent)
		: HAlignment(HAlign_Fill)
		, VAlignment(VAlign_Fill)
	{}

	template<typename WidgetType, typename V = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value>::type>
	TAlignmentSingleWidgetSlotMixin(WidgetType& InParent, const EHorizontalAlignment InHAlign, const EVerticalAlignment InVAlign)
		: HAlignment(InHAlign)
		, VAlignment(InVAlign)
	{}

public:
	struct FSlotArgumentsMixin
	{
	private:
		friend TAlignmentSingleWidgetSlotMixin;

	public:
		typename MixedIntoType::FSlotArguments& HAlign(EHorizontalAlignment InHAlignment)
		{
			_HAlignment = InHAlignment;
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

		typename MixedIntoType::FSlotArguments& VAlign(EVerticalAlignment InVAlignment)
		{
			_VAlignment = InVAlignment;
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

	private:
		TOptional<EHorizontalAlignment> _HAlignment;
		TOptional<EVerticalAlignment> _VAlignment;
	};

protected:
	void ConstructMixin(FSlotArgumentsMixin&& InArgs)
	{
		HAlignment = InArgs._HAlignment.Get(HAlignment);
		VAlignment = InArgs._VAlignment.Get(VAlignment);
	}

public:
	// HAlign will be deprecated soon. Use SetVerticalAlignment or construct a new slot with FSlotArguments
	MixedIntoType& HAlign(EHorizontalAlignment InHAlignment)
	{
		HAlignment = InHAlignment;
		return *(static_cast<MixedIntoType*>(this));
	}

	// VAlign will be deprecated soon. Use SetVerticalAlignment or construct a new slot with FSlotArguments
	MixedIntoType& VAlign(EVerticalAlignment InVAlignment)
	{
		VAlignment = InVAlignment;
		return *(static_cast<MixedIntoType*>(this));
	}

public:
	void SetHorizontalAlignment(EHorizontalAlignment Alignment)
	{
		if (HAlignment != Alignment)
		{
			HAlignment = Alignment;
			static_cast<MixedIntoType*>(this)->Invalidate(EInvalidateWidgetReason::Layout);
		}
	}

	EHorizontalAlignment GetHorizontalAlignment() const
	{
		return HAlignment;
	}

	void SetVerticalAlignment(EVerticalAlignment Alignment)
	{
		if (VAlignment != Alignment)
		{
			VAlignment = Alignment;
			static_cast<MixedIntoType*>(this)->Invalidate(EInvalidateWidgetReason::Layout);
		}
	}

	EVerticalAlignment GetVerticalAlignment() const
	{
		return VAlignment;
	}

private:
	/** Horizontal positioning of child within the allocated slot */
	EHorizontalAlignment HAlignment;
	/** Vertical positioning of child within the allocated slot */
	EVerticalAlignment VAlignment;
};


/** Mixin to add the padding functionality to a base slot. */
template <typename MixedIntoType>
class TPaddingWidgetSlotMixin
{
public:
	TPaddingWidgetSlotMixin()
		: SlotPaddingAttribute(*static_cast<MixedIntoType*>(this))
	{}
	TPaddingWidgetSlotMixin(const FMargin& Margin)
		: SlotPaddingAttribute(*static_cast<MixedIntoType*>(this), Margin)
	{}

public:
	struct FSlotArgumentsMixin
	{
	private:
		friend TPaddingWidgetSlotMixin;
		TAttribute<FMargin> _Padding;

	public:
		typename MixedIntoType::FSlotArguments& Padding(TAttribute<FMargin> InPadding)
		{
			_Padding = MoveTemp(InPadding);
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

		typename MixedIntoType::FSlotArguments& Padding(float Uniform)
		{
			_Padding = FMargin(Uniform);
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

		typename MixedIntoType::FSlotArguments& Padding(float Horizontal, float Vertical)
		{
			_Padding = FMargin(Horizontal, Vertical);
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

		typename MixedIntoType::FSlotArguments& Padding(float Left, float Top, float Right, float Bottom)
		{
			_Padding = FMargin(Left, Top, Right, Bottom);
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}
	};

protected:
	void ConstructMixin(const FChildren& SlotOwner, FSlotArgumentsMixin&& InArgs)
	{
		if (InArgs._Padding.IsSet())
		{
			SlotPaddingAttribute.Assign(*static_cast<MixedIntoType*>(this), MoveTemp(InArgs._Padding));
		}
	}

	static void RegisterAttributesMixin(FSlateWidgetSlotAttributeInitializer& AttributeInitializer)
	{
		SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(MixedIntoType, AttributeInitializer, "Slot.Padding", SlotPaddingAttribute, EInvalidateWidgetReason::Layout);
	}

public:
	void SetPadding(TAttribute<FMargin> InPadding)
	{
		SlotPaddingAttribute.Assign(static_cast<MixedIntoType&>(*this), MoveTemp(InPadding));
	}

	FMargin GetPadding() const
	{
		return SlotPaddingAttribute.Get();
	}

private:
	using SlotPaddingCompareType = TSlateAttributeComparePredicate<>;
	using SlotPaddingType = ::SlateAttributePrivate::TSlateContainedAttribute<FMargin, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, SlotPaddingCompareType>;
	SlotPaddingType SlotPaddingAttribute;
};


/** Mixin to add the padding functionality to a base slot that is also a single children. */
template <typename MixedIntoType, EInvalidateWidgetReason InPaddingInvalidationReason = EInvalidateWidgetReason::Layout>
class TPaddingSingleWidgetSlotMixin
{
public:
	template<typename WidgetType, typename V = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value>::type>
	TPaddingSingleWidgetSlotMixin(WidgetType& InParent)
		: SlotPaddingAttribute(InParent)
	{}

	template<typename WidgetType, typename V = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value>::type>
	TPaddingSingleWidgetSlotMixin(WidgetType& InParent, const FMargin & Margin)
		: SlotPaddingAttribute(InParent, Margin)
	{}

public:
	struct FSlotArgumentsMixin
	{
	private:
		friend TPaddingSingleWidgetSlotMixin;
		TAttribute<FMargin> _Padding;

	public:
		typename MixedIntoType::FSlotArguments& Padding(TAttribute<FMargin> InPadding)
		{
			_Padding = MoveTemp(InPadding);
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

		typename MixedIntoType::FSlotArguments& Padding(float Uniform)
		{
			_Padding = FMargin(Uniform);
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

		typename MixedIntoType::FSlotArguments& Padding(float Horizontal, float Vertical)
		{
			_Padding = FMargin(Horizontal, Vertical);
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

		typename MixedIntoType::FSlotArguments& Padding(float Left, float Top, float Right, float Bottom)
		{
			_Padding = FMargin(Left, Top, Right, Bottom);
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}
	};

protected:
	void ConstructMixin(FSlotArgumentsMixin&& InArgs)
	{
		if (InArgs._Padding.IsSet())
		{
			SWidget* OwnerWidget = static_cast<MixedIntoType*>(this)->GetOwnerWidget();
			check(OwnerWidget);
			SlotPaddingAttribute.Assign(*OwnerWidget, MoveTemp(InArgs._Padding));
		}
	}

public:
	// Padding will be deprecated soon. Use SetPadding or construct a new slot with FSlotArguments
	MixedIntoType& Padding(TAttribute<FMargin> InPadding)
	{
		SetPadding(MoveTemp(InPadding));
		return *(static_cast<MixedIntoType*>(this));
	}

	// Padding will be deprecated soon. Use SetPadding or construct a new slot with FSlotArguments
	MixedIntoType& Padding(float Uniform)
	{
		SetPadding(FMargin(Uniform));
		return *(static_cast<MixedIntoType*>(this));
	}

	// Padding will be deprecated soon. Use SetPadding or construct a new slot with FSlotArguments
	MixedIntoType& Padding(float Horizontal, float Vertical)
	{
		SetPadding(FMargin(Horizontal, Vertical));
		return *(static_cast<MixedIntoType*>(this));
	}

	// Padding will be deprecated soon. Use SetPadding or construct a new slot with FSlotArguments
	MixedIntoType& Padding(float Left, float Top, float Right, float Bottom)
	{
		SetPadding(FMargin(Left, Top, Right, Bottom));
		return *(static_cast<MixedIntoType*>(this));
	}

public:
	void SetPadding(TAttribute<FMargin> InPadding)
	{
		SWidget* OwnerWidget = static_cast<MixedIntoType*>(this)->GetOwnerWidget();
		check(OwnerWidget);
		SlotPaddingAttribute.Assign(*OwnerWidget, MoveTemp(InPadding));
	}

	FMargin GetPadding() const
	{
		return SlotPaddingAttribute.Get();
	}

public:
	using SlotPaddingInvalidationType = typename std::conditional<InPaddingInvalidationReason == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InPaddingInvalidationReason>>::type;
	using SlotPaddingAttributeType = SlateAttributePrivate::TSlateMemberAttribute<FMargin, SlotPaddingInvalidationType, TSlateAttributeComparePredicate<>>;
	using SlotPaddingAttributeRefType = SlateAttributePrivate::TSlateMemberAttributeRef<SlotPaddingAttributeType>;

	template<typename WidgetType, typename V = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value>::type>
	SlotPaddingAttributeRefType GetSlotPaddingAttribute() const
	{
		WidgetType* Widget = static_cast<WidgetType*>(static_cast<MixedIntoType*>(this)->GetOwnerWidget());
		check(Widget);
		return SlotPaddingAttributeRefType(Widget->template SharedThis<WidgetType>(Widget), SlotPaddingAttribute);
	}

protected:
	SlotPaddingAttributeType SlotPaddingAttribute;
};


/** Mixin to add resizing functionality to a base slot. */
template <typename MixedIntoType>
class TResizingWidgetSlotMixin
{
public:
	TResizingWidgetSlotMixin()
		: SizeRule(FSizeParam::SizeRule_Stretch)
		, SizeValue(static_cast<MixedIntoType&>(*this), 1.f)
		, ShrinkSizeValue(static_cast<MixedIntoType&>(*this), 1.f)
		, MinSize(static_cast<MixedIntoType&>(*this), 0.0f)
		, MaxSize(static_cast<MixedIntoType&>(*this), 0.0f)
	{}

	struct FSlotArgumentsMixin
	{
	private:
		friend TResizingWidgetSlotMixin;

	public:
		typename MixedIntoType::FSlotArguments& SizeParam(TOptional<FSizeParam> InSizeParam)
		{
			_SizeParam = InSizeParam;
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

		typename MixedIntoType::FSlotArguments& MinSize(TAttribute<float> InMinSize)
		{
			_MinSize = MoveTemp(InMinSize);
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

		typename MixedIntoType::FSlotArguments& MaxSize(TAttribute<float> InMaxSize)
		{
			_MaxSize = MoveTemp(InMaxSize);
			return static_cast<typename MixedIntoType::FSlotArguments&>(*this);
		}

	protected:
		TOptional<FSizeParam> _SizeParam;
		TAttribute<float> _MinSize;
		TAttribute<float> _MaxSize;
	};

protected:
	void ConstructMixin(const FChildren& SlotOwner, FSlotArgumentsMixin&& InArgs)
	{
		if (InArgs._MinSize.IsSet())
		{
			SetMinSize(MoveTemp(InArgs._MinSize));
		}

		if (InArgs._MaxSize.IsSet())
		{
			SetMaxSize(MoveTemp(InArgs._MaxSize));
		}

		if (InArgs._SizeParam.IsSet())
		{
			SetSizeParam(MoveTemp(InArgs._SizeParam.GetValue()));
		}
	}

	static void RegisterAttributesMixin(FSlateWidgetSlotAttributeInitializer& AttributeInitializer)
	{
		SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(MixedIntoType, AttributeInitializer, "Slot.MinSize", MinSize, EInvalidateWidgetReason::Layout);
		SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(MixedIntoType, AttributeInitializer, "Slot.MaxSize", MaxSize, EInvalidateWidgetReason::Layout);
		SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(MixedIntoType, AttributeInitializer, "Slot.SizeValue", SizeValue, EInvalidateWidgetReason::Layout)
			.UpdatePrerequisite("Slot.MinSize")
			.UpdatePrerequisite("Slot.MaxSize");
		SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(MixedIntoType, AttributeInitializer, "Slot.ShrinkSizeValue", ShrinkSizeValue, EInvalidateWidgetReason::Layout)
			.UpdatePrerequisite("Slot.MinSize")
			.UpdatePrerequisite("Slot.MaxSize");
	}

public:
	/** Get the space rule this slot should occupy along panel's direction. */
	FSizeParam::ESizeRule GetSizeRule() const
	{
		return SizeRule;
	}

	/**
	 * Get the size parameter for the space rule.
	 * Used for size rule SizeRule_Stretch and SizeRule_StretchContent.
	 */
	float GetSizeValue() const
	{
		return SizeValue.Get();
	}

	/**
	 * Get the size parameter for the space rule, used when the slot size is shrinking below desired size.
	 * Used for size rule SizeRule_StretchContent.
	 */
	float GetShrinkSizeValue() const
	{
		return ShrinkSizeValue.Get();
	}

	/** Set the size Param of the slot, It could be a FStretch, FStretchContent, or a FAuto. */
	void SetSizeParam(FSizeParam InSizeParam)
	{
		SizeRule = InSizeParam.SizeRule;

		// ShrinkSizeValue is only used for StretchContent.
		// If ShrinkValue is not set, make it equal to the Value.
		if (SizeRule == FSizeParam::SizeRule_StretchContent)
		{
			if (InSizeParam.ShrinkValue.IsSet())
			{
				ShrinkSizeValue.Assign(static_cast<MixedIntoType&>(*this), MoveTemp(InSizeParam.ShrinkValue));
			}
			else
			{
				ShrinkSizeValue.Assign(static_cast<MixedIntoType&>(*this), InSizeParam.Value); // Make copy, let SizeValue use move.
			}
		}
		else
		{
			ShrinkSizeValue.Set(static_cast<MixedIntoType&>(*this), 1.f); // Reset
		}

		SizeValue.Assign(static_cast<MixedIntoType&>(*this), MoveTemp(InSizeParam.Value));
	}

	/**
	 * The widget's DesiredSize will be used as the space required.
	 */
	void SetSizeToAuto()
	{
		SetSizeParam(FAuto());
	}

	/**
	 * The available space will be distributed proportionately to each slots stretch coefficient.
	 * A slot with coefficient of 2 will get assigned twice as much available space as slot with coefficient 1.
	 * @param InStretchCoefficient Stretch coefficient for this slot.
	 */
	void SetSizeToStretch(TAttribute<float> InStretchCoefficient)
	{
		SetSizeParam(FStretch(MoveTemp(InStretchCoefficient)));
	}

	/**
	 * The widget's content size is adjusted proportionally to fit the available space.
	 * The slots size starts at DesiredSize, and a slot with coefficient of 2 will get adjusted twice as much as slot with coefficient 1 to fit the available space.
	 * @param InStretchCoefficient Stretch coefficient for this slot.
	 * @param InShrinkStretchCoefficient If specified, this stretch coefficient is used when the slots is shrinking below desired size. Otherwise InStretchCoefficient is used for both shrink and grow.
	 */
	void SetSizeToStretchContent(TAttribute<float> InStretchCoefficient, TAttribute<float> InShrinkStretchCoefficient = TAttribute<float>())
	{
		SetSizeParam(FStretchContent(MoveTemp(InStretchCoefficient), MoveTemp(InShrinkStretchCoefficient)));
	}

	/** Get the min size the slot can be.*/
	float GetMinSize() const
	{
		return MinSize.Get();
	}

	/** Set the min size in SlateUnit this slot can be. */
	void SetMinSize(TAttribute<float> InMinSize)
	{
		MinSize.Assign(static_cast<MixedIntoType&>(*this), MoveTemp(InMinSize));
	}

	/** Get the max size the slot can be.*/
	float GetMaxSize() const
	{
		return MaxSize.Get();
	}

	/** Set the max size in SlateUnit this slot can be. */
	void SetMaxSize(TAttribute<float> InMaxSize)
	{
		MaxSize.Assign(static_cast<MixedIntoType&>(*this), MoveTemp(InMaxSize));
	}

protected:
	/** The sizing rule to use, see ESizeRule for more info how the different rules work. */
	FSizeParam::ESizeRule SizeRule;

	template <typename InObjectType>
	using TSlateSlotAttribute = ::SlateAttributePrivate::TSlateContainedAttribute<InObjectType, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeComparePredicate<>>;

	/** The actual value this size parameter stores. */
	TSlateSlotAttribute<float> SizeValue;

	/** The actual value this size parameter stores, used for shrinking (negative if not defined, use SizeValue). */
	TSlateSlotAttribute<float> ShrinkSizeValue;

	/** The min size that this slot can be */
	TSlateSlotAttribute<float> MinSize;

	/** The max size that this slot can be (0 if no max) */
	TSlateSlotAttribute<float> MaxSize;
};


/** A templated basic slot that can be used by layout. */
template <typename SlotType>
class TBasicLayoutWidgetSlot : public TWidgetSlotWithAttributeSupport<SlotType>
	, public TPaddingWidgetSlotMixin<SlotType>
	, public TAlignmentWidgetSlotMixin<SlotType>
{
public:
	TBasicLayoutWidgetSlot()
		: TWidgetSlotWithAttributeSupport<SlotType>()
		, TPaddingWidgetSlotMixin<SlotType>()
		, TAlignmentWidgetSlotMixin<SlotType>()
	{}

	TBasicLayoutWidgetSlot(FChildren& InOwner)
		: TWidgetSlotWithAttributeSupport<SlotType>(InOwner)
		, TPaddingWidgetSlotMixin<SlotType>()
		, TAlignmentWidgetSlotMixin<SlotType>()
	{}

	TBasicLayoutWidgetSlot(const EHorizontalAlignment InHAlign, const EVerticalAlignment InVAlign)
		: TWidgetSlotWithAttributeSupport<SlotType>()
		, TPaddingWidgetSlotMixin<SlotType>()
		, TAlignmentWidgetSlotMixin<SlotType>(InHAlign, InVAlign)
	{}

	TBasicLayoutWidgetSlot(FChildren& InOwner, const EHorizontalAlignment InHAlign, const EVerticalAlignment InVAlign)
		: TWidgetSlotWithAttributeSupport<SlotType>(InOwner)
		, TPaddingWidgetSlotMixin<SlotType>()
		, TAlignmentWidgetSlotMixin<SlotType>(InHAlign, InVAlign)
	{}

public:
	SLATE_SLOT_BEGIN_ARGS_TwoMixins(TBasicLayoutWidgetSlot, TSlotBase<SlotType>, TPaddingWidgetSlotMixin<SlotType>, TAlignmentWidgetSlotMixin<SlotType>)
	SLATE_SLOT_END_ARGS()

	void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
	{
		TWidgetSlotWithAttributeSupport<SlotType>::Construct(SlotOwner, MoveTemp(InArgs));
		TPaddingWidgetSlotMixin<SlotType>::ConstructMixin(SlotOwner, MoveTemp(InArgs));
		TAlignmentWidgetSlotMixin<SlotType>::ConstructMixin(SlotOwner, MoveTemp(InArgs));
	}

	static void RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer)
	{
		TWidgetSlotWithAttributeSupport<SlotType>::RegisterAttributes(AttributeInitializer);
		TPaddingWidgetSlotMixin<SlotType>::RegisterAttributesMixin(AttributeInitializer);
	}
};


/** The basic slot that can be used by layout. */
class FBasicLayoutWidgetSlot : public TBasicLayoutWidgetSlot<FBasicLayoutWidgetSlot>
{
public:
	using TBasicLayoutWidgetSlot<FBasicLayoutWidgetSlot>::TBasicLayoutWidgetSlot;
};
