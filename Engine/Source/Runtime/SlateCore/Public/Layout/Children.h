// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SNullWidget.h"
#include "SlotBase.h"
#include "Layout/ChildrenBase.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Widgets/SWidget.h"


/**
 * Occasionally you may need to keep multiple discrete sets of children with differing slot requirements.
 * This data structure can be used to link multiple FChildren under a single accessor so you can always return
 * all children from GetChildren, but internally manage them in their own child lists.
 */
class FCombinedChildren final : public FChildren
{
public:
	using FChildren::FChildren;

	void AddChildren(FChildren& InLinkedChildren)
	{
		LinkedChildren.Add(&InLinkedChildren);
	}

	/** @return the number of children */
	virtual int32 Num() const override
	{
		int32 TotalNum = 0;
		for (const FChildren* Children : LinkedChildren)
		{
			TotalNum += Children->Num();
		}

		return TotalNum;
	}

	virtual TSharedRef<SWidget> GetChildAt(int32 Index) override
	{
		int32 TotalNum = 0;
		for (FChildren* Children : LinkedChildren)
		{
			const int32 NewTotal = TotalNum + Children->Num();
			if (NewTotal > Index)
			{
				return Children->GetChildAt(Index - TotalNum);
			}
			TotalNum = NewTotal;
		}
		// This result should never occur users should always access a valid index for child slots.
		check(false);
		return SNullWidget::NullWidget;
	}

	virtual TSharedRef<const SWidget> GetChildAt(int32 Index) const override
	{
		int32 TotalNum = 0;
		for (const FChildren* Children : LinkedChildren)
		{
			const int32 NewTotal = TotalNum + Children->Num();
			if (NewTotal > Index)
			{
				return Children->GetChildAt(Index - TotalNum);
			}
			TotalNum = NewTotal;
		}
		// This result should never occur users should always access a valid index for child slots.
		check(false);
		return SNullWidget::NullWidget;
	}

protected:
	virtual int32 NumSlot() const override
	{
		int32 TotalNum = 0;
		for (const FChildren* Children : LinkedChildren)
		{
			TotalNum += Children->NumSlot();
		}

		return TotalNum;
	}

	virtual const FSlotBase& GetSlotAt(int32 ChildIndex) const override
	{
		int32 TotalNum = 0;
		for (const FChildren* Children : LinkedChildren)
		{
			const int32 NewTotal = TotalNum + Children->NumSlot();
			if (NewTotal > ChildIndex)
			{
				return Children->GetSlotAt(ChildIndex - TotalNum);
			}
			TotalNum = NewTotal;
		}

		// This result should never occur users should always access a valid index for child slots.
		check(false);
		static FSlotBase NullSlot;
		return NullSlot;
	}

	virtual FWidgetRef GetChildRefAt(int32 Index) override
	{
		int32 TotalNum = 0;
		for (FChildren* Children : LinkedChildren)
		{
			const int32 NewTotal = TotalNum + Children->Num();
			if (NewTotal > Index)
			{
				return Children->GetChildRefAt(Index - TotalNum);
			}
			TotalNum = NewTotal;
		}
		// This result should never occur users should always access a valid index for child slots.
		check(false);
		return FWidgetRef(ReferenceConstruct, SNullWidget::NullWidget.Get());
	}

	virtual FConstWidgetRef GetChildRefAt(int32 Index) const override
	{
		int32 TotalNum = 0;
		for (const FChildren* Children : LinkedChildren)
		{
			const int32 NewTotal = TotalNum + Children->Num();
			if (NewTotal > Index)
			{
				return Children->GetChildRefAt(Index - TotalNum);
			}
			TotalNum = NewTotal;
		}
		// This result should never occur users should always access a valid index for child slots.
		check(false);
		return FConstWidgetRef(ReferenceConstruct, SNullWidget::NullWidget.Get());
	}

protected:
	TArray<FChildren*> LinkedChildren;
};


/**
 * Widgets with no Children can return an instance of FNoChildren.
 * For convenience a shared instance FNoChildren::NoChildrenInstance can be used.
 */
class FNoChildren final : public FChildren
{
public:
	static SLATECORE_API FNoChildren NoChildrenInstance;

public:
	FNoChildren(SWidget* InOwner)
		: FChildren(InOwner)
	{
	}

	FNoChildren(SWidget* InOwner, FName InName)
		: FChildren(InOwner, InName)
	{
	}

	virtual int32 Num() const override { return 0; }
	
	virtual TSharedRef<SWidget> GetChildAt( int32 ) override
	{
		// Nobody should be getting a child when there aren't any children.
		// We expect this to crash!
		check( false );
		return SNullWidget::NullWidget;
	}
	
	virtual TSharedRef<const SWidget> GetChildAt( int32 ) const override
	{
		// Nobody should be getting a child when there aren't any children.
		// We expect this to crash!
		check( false );
		return SNullWidget::NullWidget;
	}

private:
	friend class SWidget;
	virtual const FSlotBase& GetSlotAt(int32 ChildIndex) const override
	{
		check(false);
		static FSlotBase NullSlot;
		return NullSlot;
	}

	virtual FWidgetRef GetChildRefAt(int32 Index) override
	{
		check(false);
		return FWidgetRef(ReferenceConstruct, SNullWidget::NullWidget.Get());
	}
	
	virtual FConstWidgetRef GetChildRefAt(int32 Index) const override
	{
		check(false);
		return FConstWidgetRef(ReferenceConstruct, SNullWidget::NullWidget.Get());
	}
};


/**
 * For widgets that do not own their content, but are responsible for presenting someone else's content.
 * e.g. Tooltips are just presented by the owner window; not actually owned by it. They can go away at any time
 *      and then they'll just stop being shown.
 */
template <typename ChildType>
class TWeakChild final : public FChildren
{
public:
	using FChildren::FChildren;

	virtual int32 Num() const override
	{
		return WidgetPtr.IsValid() ? 1 : 0 ;
	}

	virtual TSharedRef<SWidget> GetChildAt( int32 ChildIndex ) override
	{
		check(ChildIndex == 0);
		return GetWidget();
	}

	virtual TSharedRef<const SWidget> GetChildAt( int32 ChildIndex ) const override
	{
		check(ChildIndex == 0);
		return GetWidget();
	}

private:
	virtual int32 NumSlot() const override
	{
		return 0;
	}

	virtual const FSlotBase& GetSlotAt(int32 ChildIndex) const override
	{
		static FSlotBase NullSlot;
		check(ChildIndex == 0);
		return NullSlot;
	}

	virtual FWidgetRef GetChildRefAt(int32 ChildIndex) override
	{
		check(ChildIndex == 0);
		TSharedPtr<SWidget> Widget = WidgetPtr.Pin();
		return (Widget.IsValid()) ? FWidgetRef(CopyConstruct, Widget.ToSharedRef()) : FWidgetRef(ReferenceConstruct, SNullWidget::NullWidget.Get());
	}
	virtual FConstWidgetRef GetChildRefAt(int32 ChildIndex) const override
	{
		check(ChildIndex == 0);
		TSharedPtr<SWidget> Widget = WidgetPtr.Pin();
		return (Widget.IsValid()) ? FConstWidgetRef(CopyConstruct, Widget.ToSharedRef()) : FConstWidgetRef(ReferenceConstruct, SNullWidget::NullWidget.Get());
	}

public:
	void AttachWidget(const TSharedPtr<SWidget>& InWidget)
	{
		if (WidgetPtr != InWidget)
		{
			WidgetPtr = InWidget;
			GetOwner().Invalidate(EInvalidateWidgetReason::ChildOrder);

			if (InWidget.IsValid() && InWidget != SNullWidget::NullWidget)
			{
				InWidget->AssignParentWidget(GetOwner().AsShared());
			}
		}
	}

	void DetachWidget()
	{
		if (TSharedPtr<SWidget> Widget = WidgetPtr.Pin())
		{
			if (Widget != SNullWidget::NullWidget)
			{
				Widget->ConditionallyDetatchParentWidget(&GetOwner());
			}

			WidgetPtr.Reset();
		}
	}

	TSharedRef<SWidget> GetWidget() const
	{
		ensure(Num() > 0);
		TSharedPtr<SWidget> Widget = WidgetPtr.Pin();
		return (Widget.IsValid()) ? Widget.ToSharedRef() : SNullWidget::NullWidget;
	}

private:
	TWeakPtr<ChildType> WidgetPtr;
};


/** A FChildren that has only one child and can take a templated slot. */
template<typename SlotType>
class TSingleWidgetChildrenWithSlot : public FChildren, protected TSlotBase<SlotType>
{
public:
	TSingleWidgetChildrenWithSlot(SWidget* InOwner)
		: FChildren(InOwner)
		, TSlotBase<SlotType>(static_cast<const FChildren&>(*this))
	{
	}

	TSingleWidgetChildrenWithSlot(SWidget* InOwner, FName InName)
		: FChildren(InOwner, InName)
		, TSlotBase<SlotType>(static_cast<const FChildren&>(*this))
	{
	}

	TSingleWidgetChildrenWithSlot(std::nullptr_t) = delete;

public:
	virtual int32 Num() const override { return 1; }

	virtual TSharedRef<SWidget> GetChildAt(int32 ChildIndex) override
	{
		check(ChildIndex == 0);
		return this->GetWidget();
	}

	virtual TSharedRef<const SWidget> GetChildAt(int32 ChildIndex) const override
	{
		check(ChildIndex == 0);
		return this->GetWidget();
	}

public:
	struct FSlotArguments : protected TSlotBase<SlotType>::FSlotArguments
	{
		FSlotArguments()
			: TSlotBase<SlotType>::FSlotArguments(TSlotBase<SlotType>::ConstructSlotIsFChildren)
		{
		}

		friend TSingleWidgetChildrenWithSlot;

		typename SlotType::FSlotArguments& operator[](const TSharedRef<SWidget>& InChildWidget)
		{
			TSlotBase<SlotType>::FSlotArguments::AttachWidget(InChildWidget);
			return static_cast<typename SlotType::FSlotArguments&>(*this);
		}
	};

	void Construct(FSlotArguments&& InArgs)
	{
		TSlotBase<SlotType>::Construct(*this, MoveTemp(InArgs));
	}

public:
	TSlotBase<SlotType>& AsSlot() { return *this; }
	const TSlotBase<SlotType>& AsSlot() const { return *this; }

	using TSlotBase<SlotType>::GetOwnerWidget;
	using TSlotBase<SlotType>::AttachWidget;
	using TSlotBase<SlotType>::DetachWidget;
	using TSlotBase<SlotType>::GetWidget;
	using TSlotBase<SlotType>::Invalidate;
	SlotType& operator[](const TSharedRef<SWidget>& InChildWidget)
	{
		this->AttachWidget(InChildWidget);
		return static_cast<SlotType&>(*this);
	}

	SlotType& Expose(SlotType*& OutVarToInit)
	{
		OutVarToInit = static_cast<SlotType*>(this);
		return static_cast<SlotType&>(*this);
	}

private:
	virtual const FSlotBase& GetSlotAt(int32 ChildIndex) const override
	{
		check(ChildIndex == 0);
		return *this;
	}

	virtual FWidgetRef GetChildRefAt(int32 ChildIndex) override
	{
		check(ChildIndex == 0);
		return FWidgetRef(ReferenceConstruct, this->GetWidget().Get());
	}
	virtual FConstWidgetRef GetChildRefAt(int32 ChildIndex) const override
	{
		check(ChildIndex == 0);
		return FConstWidgetRef(ReferenceConstruct, this->GetWidget().Get());
	}
};


/** A FChildren that has only one child. */
class FSingleWidgetChildrenWithSlot final : public TSingleWidgetChildrenWithSlot<FSingleWidgetChildrenWithSlot>
{
public:
	using TSingleWidgetChildrenWithSlot<FSingleWidgetChildrenWithSlot>::TSingleWidgetChildrenWithSlot;
};


/** A FChildren that has only one child and support alignment and padding. */
template<EInvalidateWidgetReason InPaddingInvalidationReason = EInvalidateWidgetReason::Layout>
class TSingleWidgetChildrenWithBasicLayoutSlot : public TSingleWidgetChildrenWithSlot<TSingleWidgetChildrenWithBasicLayoutSlot<InPaddingInvalidationReason>>
	, public TPaddingSingleWidgetSlotMixin<TSingleWidgetChildrenWithBasicLayoutSlot<InPaddingInvalidationReason>, InPaddingInvalidationReason>
	, public TAlignmentSingleWidgetSlotMixin<TSingleWidgetChildrenWithBasicLayoutSlot<InPaddingInvalidationReason>>
{
private:
	using ParentType = TSingleWidgetChildrenWithSlot<TSingleWidgetChildrenWithBasicLayoutSlot<InPaddingInvalidationReason>>;
	using PaddingMixinType = TPaddingSingleWidgetSlotMixin<TSingleWidgetChildrenWithBasicLayoutSlot<InPaddingInvalidationReason>, InPaddingInvalidationReason>;
	using AlignmentMixinType = TAlignmentSingleWidgetSlotMixin<TSingleWidgetChildrenWithBasicLayoutSlot<InPaddingInvalidationReason>>;

public:
	template<typename WidgetType, typename V = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value>::type>
	TSingleWidgetChildrenWithBasicLayoutSlot(WidgetType* InOwner)
		: ParentType(InOwner)
		, PaddingMixinType(*InOwner)
		, AlignmentMixinType(*InOwner, HAlign_Fill, VAlign_Fill)
	{
	}

	template<typename WidgetType, typename V = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value>::type>
	TSingleWidgetChildrenWithBasicLayoutSlot(WidgetType* InOwner, const EHorizontalAlignment InHAlign, const EVerticalAlignment InVAlign)
		: ParentType(InOwner)
		, PaddingMixinType(*InOwner)
		, AlignmentMixinType(*InOwner, InHAlign, InVAlign)
	{
	}

	TSingleWidgetChildrenWithBasicLayoutSlot(std::nullptr_t) = delete;
	TSingleWidgetChildrenWithBasicLayoutSlot(std::nullptr_t, const EHorizontalAlignment InHAlign, const EVerticalAlignment InVAlign) = delete;

public:
	SLATE_SLOT_BEGIN_ARGS_TwoMixins(TSingleWidgetChildrenWithBasicLayoutSlot, ParentType, PaddingMixinType, AlignmentMixinType)
	SLATE_SLOT_END_ARGS()

	void Construct(FSlotArguments&& InArgs)
	{
		ParentType::Construct(MoveTemp(InArgs));
		PaddingMixinType::ConstructMixin(MoveTemp(InArgs));
		AlignmentMixinType::ConstructMixin( MoveTemp(InArgs));
	}
};


class FSingleWidgetChildrenWithBasicLayoutSlot : public TSingleWidgetChildrenWithBasicLayoutSlot<>
{
	using TSingleWidgetChildrenWithBasicLayoutSlot<>::TSingleWidgetChildrenWithBasicLayoutSlot;
};


/**
 * A generic FChildren that stores children along with layout-related information.
 * The type containing Widget* and layout info is specified by ChildType.
 * ChildType must have a public member SWidget* Widget;
 */
template<typename SlotType>
class TPanelChildren final : public FChildren
{
private:
	TArray<TUniquePtr<SlotType>> Children;
	static constexpr bool bSupportSlotWithSlateAttribute = std::is_base_of<TWidgetSlotWithAttributeSupport<SlotType>, SlotType>::value;

protected:
	virtual const FSlotBase& GetSlotAt(int32 ChildIndex) const override
	{
		return *Children[ChildIndex];
	}

	virtual FWidgetRef GetChildRefAt(int32 ChildIndex) override
	{
		return FWidgetRef(ReferenceConstruct, Children[ChildIndex]->GetWidget().Get());
	}

	virtual FConstWidgetRef GetChildRefAt(int32 ChildIndex) const override
	{
		return FConstWidgetRef(ReferenceConstruct, Children[ChildIndex]->GetWidget().Get());
	}

public:
	using FChildren::FChildren;
	
	virtual int32 Num() const override
	{
		return Children.Num();
	}

	virtual TSharedRef<SWidget> GetChildAt( int32 Index ) override
	{
		return Children[Index]->GetWidget();
	}

	virtual TSharedRef<const SWidget> GetChildAt( int32 Index ) const override
	{
		return Children[Index]->GetWidget();
	}

	virtual bool SupportSlotWithSlateAttribute() const override
	{
		return bSupportSlotWithSlateAttribute;
	}

public:
	int32 AddSlot(typename SlotType::FSlotArguments&& SlotArgument)
	{
		TUniquePtr<SlotType> NewSlot = SlotArgument.StealSlot();
		check(NewSlot.Get());
		int32 Result = Children.Add(MoveTemp(NewSlot));
		Children[Result]->Construct(*this, MoveTemp(SlotArgument));
		return Result;
	}

	void AddSlots(TArray<typename SlotType::FSlotArguments> SlotArguments)
	{
		Children.Reserve(Children.Num() + SlotArguments.Num());
		for (typename SlotType::FSlotArguments& Arg : SlotArguments)
		{
			AddSlot(MoveTemp(Arg));
		}
	}

	void RemoveAt( int32 Index )
	{
		// NOTE:
		// We don't do any invalidating here, that's handled by the FSlotBase, which eventually calls ConditionallyDetatchParentWidget

		// Steal the instance from the array, then free the element.
		// This alleviates issues where (misbehaving) destructors on the children may call back into this class and query children while they are being destroyed.
		TUniquePtr<SlotType> SlotToRemove = MoveTemp(Children[Index]);
		Children.RemoveAt(Index);
		SlotToRemove.Reset();
	}

	/** Removes the corresponding widget from the set of children if it exists.  Returns the index it found the child at, INDEX_NONE otherwise. */
	int32 Remove(const TSharedRef<SWidget>& SlotWidget)
	{
		for (int32 SlotIdx = 0; SlotIdx < Num(); ++SlotIdx)
		{
			if (SlotWidget == Children[SlotIdx]->GetWidget())
			{
				Children.RemoveAt(SlotIdx);
				return SlotIdx;
			}
		}

		return INDEX_NONE;
	}

	void Empty(int32 Slack = 0)
	{
		// NOTE:
		// We don't do any invalidating here, that's handled by the FSlotBase, which eventually calls ConditionallyDetatchParentWidget

		// We empty children by first transferring them onto a stack-owned array, then freeing the elements.
		// This alleviates issues where (misbehaving) destructors on the children may call back into this class and query children while they are being destroyed.
		// By storing the children on the stack first, we defer the destruction of children until after we have emptied our owned container.
		TArray<TUniquePtr<SlotType>> ChildrenCopy = MoveTemp(Children);

		// Explicitly calling Empty is not really necessary (it is already empty/moved-from now), but we call it for safety
		Children.Empty();

		// ChildrenCopy will now be emptied and moved back (to preserve any allocated memory)
		ChildrenCopy.Empty(Slack);
		Children = MoveTemp(ChildrenCopy);
	}

	void InsertSlot(typename SlotType::FSlotArguments&& SlotArgument, int32 Index)
	{
		TUniquePtr<SlotType> NewSlot = SlotArgument.StealSlot();
		check(NewSlot.Get());
		Children.Insert(MoveTemp(NewSlot), Index);
		Children[Index]->Construct(*this, MoveTemp(SlotArgument));
	}

	void Move(int32 IndexToMove, int32 IndexToDestination)
	{
		{
			TUniquePtr<SlotType> SlotToMove = MoveTemp(Children[IndexToMove]);
			Children.RemoveAt(IndexToMove);
			Children.Insert(MoveTemp(SlotToMove), IndexToDestination);
			if constexpr (bSupportSlotWithSlateAttribute)
			{
				check(Children.Num() > 0);
				Children[0]->RequestSortAttribute();
			}
		}

		GetOwner().Invalidate(EInvalidateWidgetReason::ChildOrder);
	}

	void Reserve( int32 NumToReserve )
	{
		Children.Reserve(NumToReserve);
	}

	bool IsValidIndex( int32 Index ) const
	{
		return Children.IsValidIndex( Index );
	}

	const SlotType& operator[](int32 Index) const
	{
		return *Children[Index];
	}
	SlotType& operator[](int32 Index)
	{
		return *Children[Index];
	}

	template <class PREDICATE_CLASS>
	void Sort( const PREDICATE_CLASS& Predicate )
	{
		if (Children.Num() > 0)
		{
			Children.Sort([&Predicate](const TUniquePtr<SlotType>& One, const TUniquePtr<SlotType>& Two)
			{
				return Predicate(*One, *Two);
			});

			if constexpr (bSupportSlotWithSlateAttribute)
			{
				Children[0]->RequestSortAttribute();
			}

			GetOwner().Invalidate(EInvalidateWidgetReason::ChildOrder);
		}
	}

	template <class PREDICATE_CLASS>
	void StableSort(const PREDICATE_CLASS& Predicate)
	{
		if (Children.Num() > 0)
		{
			Children.StableSort([&Predicate](const TUniquePtr<SlotType>& One, const TUniquePtr<SlotType>& Two)
			{
				return Predicate(*One, *Two);
			});

			if constexpr (bSupportSlotWithSlateAttribute)
			{
				Children[0]->RequestSortAttribute();
			}

			GetOwner().Invalidate(EInvalidateWidgetReason::ChildOrder);
		}
	}

	void Swap( int32 IndexA, int32 IndexB )
	{
		Children.Swap(IndexA, IndexB);

		if constexpr (bSupportSlotWithSlateAttribute)
		{
			check(Children.Num() > 0);
			Children[0]->RequestSortAttribute();
		}

		GetOwner().Invalidate(EInvalidateWidgetReason::ChildOrder);
	}

public:
	/** At the end of the scope a slot will be constructed and added to the FChildren. */
	struct FScopedWidgetSlotArguments final : public SlotType::FSlotArguments
	{
	public:
		FScopedWidgetSlotArguments(TUniquePtr<SlotType> InSlot, TPanelChildren<SlotType>& InChildren, int32 InIndex)
			: SlotType::FSlotArguments(MoveTemp(InSlot))
			, Children(InChildren)
			, Index(InIndex)
		{
		}
		FScopedWidgetSlotArguments(TUniquePtr<SlotType> InSlot, TPanelChildren<SlotType>& InChildren, int32 InIndex, TFunction<void(const SlotType*, int32)> OnAdded)
			: SlotType::FSlotArguments(MoveTemp(InSlot))
			, Children(InChildren)
			, Index(InIndex)
			, Added(OnAdded)
		{
		}
	
		FScopedWidgetSlotArguments(const FScopedWidgetSlotArguments&) = delete;
		FScopedWidgetSlotArguments& operator=(const FScopedWidgetSlotArguments&) = delete;
		FScopedWidgetSlotArguments(FScopedWidgetSlotArguments&&) = default;
		FScopedWidgetSlotArguments& operator=(FScopedWidgetSlotArguments&&) = default;
	
		~FScopedWidgetSlotArguments()
		{
			if (const SlotType* SlotPtr = this->GetSlot())	// Is nullptr when the FScopedWidgetSlotArguments was moved-constructed.
			{
				if (Index == INDEX_NONE)
				{
					Index = Children.AddSlot(MoveTemp(*this));
				}
				else
				{
					Children.InsertSlot(MoveTemp(*this), Index);
				}
				if (Added)
				{
					Added(SlotPtr, Index);
				}
			}
		}

	private:
		TPanelChildren<SlotType>& Children;
		int32 Index;
		TFunction<void(const SlotType*, int32)> Added;
	};
};


template<typename SlotType>
class TPanelChildrenConstIterator
{
public:
	TPanelChildrenConstIterator(const TPanelChildren<SlotType>& InContainer, EFlowDirection InLayoutFlow)
		: Container(InContainer)
		, LayoutFlow(InLayoutFlow)
	{
		Reset();
	}

	TPanelChildrenConstIterator(const TPanelChildren<SlotType>& InContainer, EOrientation InOrientation, EFlowDirection InLayoutFlow)
		: Container(InContainer)
		, LayoutFlow(InOrientation == Orient_Vertical ? EFlowDirection::LeftToRight : InLayoutFlow)
	{
		Reset();
	}

	/** Advances iterator to the next element in the container. */
	TPanelChildrenConstIterator<SlotType>& operator++()
	{
		switch (LayoutFlow)
		{
		default:
		case EFlowDirection::LeftToRight:
			++Index;
			break;
		case EFlowDirection::RightToLeft:
			--Index;
			break;
		}

		return *this;
	}

	/** Moves iterator to the previous element in the container. */
	TPanelChildrenConstIterator<SlotType>& operator--()
	{
		switch (LayoutFlow)
		{
		default:
		case EFlowDirection::LeftToRight:
			--Index;
			break;
		case EFlowDirection::RightToLeft:
			++Index;
			break;
		}

		return *this;
	}

	const SlotType& operator* () const
	{
		return Container[Index];
	}

	const SlotType* operator->() const
	{
		return &Container[Index];
	}

	/** conversion to "bool" returning true if the iterator has not reached the last element. */
	inline explicit operator bool() const
	{
		return Container.IsValidIndex(Index);
	}

	/** Returns an index to the current element. */
	int32 GetIndex() const
	{
		return Index;
	}

	/** Resets the iterator to the first element. */
	void Reset()
	{
		switch (LayoutFlow)
		{
		default:
		case EFlowDirection::LeftToRight:
			Index = 0;
			break;
		case EFlowDirection::RightToLeft:
			Index = Container.Num() - 1;
			break;
		}
	}

	/** Sets iterator to the last element. */
	void SetToEnd()
	{
		switch (LayoutFlow)
		{
		default:
		case EFlowDirection::LeftToRight:
			Index = Container.Num() - 1;
			break;
		case EFlowDirection::RightToLeft:
			Index = 0;
			break;
		}
	}

private:

	const TPanelChildren<SlotType>& Container;
	int32 Index;
	EFlowDirection LayoutFlow;
};



/**
 * Some advanced widgets contain no layout information, and do not require slots.
 * Those widgets may wish to store a specialized type of child widget.
 * In those cases, using TSlotlessChildren is convenient.
 *
 * TSlotlessChildren should not be used for general-purpose widgets.
 *
 * If bManagedByWidget is false, the children will not be attached or detached from the parent and no invalidation will be triggerred
 */
template<typename ChildType, bool bManagedByWidget = true>
class TSlotlessChildren final : public FChildren
{
private:
	TArray<TSharedRef<ChildType>> Children;

	virtual int32 NumSlot() const override
	{
		return 0;
	}

	virtual const FSlotBase& GetSlotAt(int32 ChildIndex) const override
	{
		// @todo slate : slotless children should be removed altogether; for now they return a fake slot.
		static FSlotBase NullSlot;
		check(false);
		return NullSlot;
	}

	virtual FWidgetRef GetChildRefAt(int32 ChildIndex) override
	{
		return FWidgetRef(ReferenceConstruct, Children[ChildIndex].Get());
	}
	
	virtual FConstWidgetRef GetChildRefAt(int32 ChildIndex) const override
	{
		return FConstWidgetRef(ReferenceConstruct, Children[ChildIndex].Get());
	}

public:
	TSlotlessChildren(SWidget* InOwner, bool InbChangesInvalidatePrepass = true) requires (bManagedByWidget)
		: FChildren(InOwner)
		, bChangesInvalidatePrepass(InbChangesInvalidatePrepass)
	{
	}

	TSlotlessChildren(SWidget* InOwner) requires (!bManagedByWidget)
		: FChildren(InOwner)
		, bChangesInvalidatePrepass(false)
	{
	}

	TSlotlessChildren(std::nullptr_t, bool InbChangesInvalidatePrepass = true) = delete;

	virtual int32 Num() const override
	{
		return Children.Num();
	}

	virtual TSharedRef<SWidget> GetChildAt( int32 Index ) override
	{
		return Children[Index];
	}

	virtual TSharedRef<const SWidget> GetChildAt( int32 Index ) const override
	{
		return Children[Index];
	}

	int32 Add( const TSharedRef<ChildType>& Child )
	{
		if constexpr (bManagedByWidget)
		{
			if (bChangesInvalidatePrepass)
			{
				GetOwner().Invalidate(EInvalidateWidgetReason::ChildOrder);
			}
		}

		int32 Index = Children.Add(Child);

		if constexpr (bManagedByWidget)
		{
			if (Child != SNullWidget::NullWidget)
			{
				Child->AssignParentWidget(GetOwner().AsShared());
			}
		}

		return Index;
	}

	void Reset(int32 NewSize = 0)
	{
		if constexpr (bManagedByWidget)
		{
			for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
			{
				TSharedRef<SWidget> Child = GetChildAt(ChildIndex);
				if (Child != SNullWidget::NullWidget)
				{
					Child->ConditionallyDetatchParentWidget(&GetOwner());
				}
			}
		}

		// We reset children by first transferring them onto a stack-owned array, then freeing the elements.
		// This alleviates issues where (misbehaving) destructors on the children may call back into this class and query children while they are being destroyed.
		// By storing the children on the stack first, we defer the destruction of children until after we have reset our owned container.
		TArray<TSharedRef<ChildType>> ChildrenCopy = MoveTemp(Children);

		// Explicitly calling Reset is not really necessary (it is already empty/moved-from now), but we call it for safety
		Children.Reset();

		// ChildrenCopy will now be reset and moved back (to preserve any allocated memory)
		ChildrenCopy.Reset(NewSize);
		Children = MoveTemp(ChildrenCopy);
	}

	void Empty(int32 Slack = 0)
	{
		if constexpr (bManagedByWidget)
		{
			for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
			{
				TSharedRef<SWidget> Child = GetChildAt(ChildIndex);
				if (Child != SNullWidget::NullWidget)
				{
					Child->ConditionallyDetatchParentWidget(&GetOwner());
				}
			}
		}

		// We empty children by first transferring them onto a stack-owned array, then freeing the elements.
		// This alleviates issues where (misbehaving) destructors on the children may call back into this class and query children while they are being destroyed.
		// By storing the children on the stack first, we defer the destruction of children until after we have emptied our owned container.
		TArray<TSharedRef<ChildType>> ChildrenCopy = MoveTemp(Children);

		// Explicitly calling Empty is not really necessary (it is already empty/moved-from now), but we call it for safety
		Children.Empty();

		// ChildrenCopy will now be emptied and moved back (to preserve any allocated memory)
		ChildrenCopy.Empty(Slack);
		Children = MoveTemp(ChildrenCopy);
	}

	void Insert(const TSharedRef<ChildType>& Child, int32 Index)
	{
		if constexpr (bManagedByWidget)
		{
			if (bChangesInvalidatePrepass)
			{
				GetOwner().Invalidate(EInvalidateWidgetReason::ChildOrder);
			}
		}

		Children.Insert(Child, Index);

		if constexpr (bManagedByWidget)
		{
			if (Child != SNullWidget::NullWidget)
			{
				Child->AssignParentWidget(GetOwner().AsShared());
			}
		}
	}

	int32 Remove( const TSharedRef<ChildType>& Child )
	{
		if constexpr (bManagedByWidget)
		{
			if (Child != SNullWidget::NullWidget)
			{
				Child->ConditionallyDetatchParentWidget(&GetOwner());
			}
		}

		return Children.Remove( Child );
	}

	void RemoveAt( int32 Index )
	{
		if constexpr (bManagedByWidget)
		{
			TSharedRef<SWidget> Child = GetChildAt(Index);
			if (Child != SNullWidget::NullWidget)
			{
				Child->ConditionallyDetatchParentWidget(&GetOwner());
			}
		}

		// Note: Child above ensures the instance we're removing from the array won't run its destructor until after it's fully removed from the array
		Children.RemoveAt( Index );
	}

	int32 Find( const TSharedRef<ChildType>& Item ) const
	{
		return Children.Find( Item );
	}

	TArray< TSharedRef< ChildType > > AsArrayCopy() const
	{
		return TArray<TSharedRef<ChildType>>(Children);
	}

	const TSharedRef<ChildType>& operator[](int32 Index) const { return Children[Index]; }
	TSharedRef<ChildType>& operator[](int32 Index) { return Children[Index]; }

	template <class PREDICATE_CLASS>
	void Sort( const PREDICATE_CLASS& Predicate )
	{
		Children.Sort( Predicate );
		if constexpr (bManagedByWidget)
		{
			if (bChangesInvalidatePrepass)
			{
				GetOwner().Invalidate(EInvalidateWidgetReason::ChildOrder);
			}
		}
	}

	void Swap( int32 IndexA, int32 IndexB )
	{
		Children.Swap(IndexA, IndexB);
		if constexpr (bManagedByWidget)
		{
			if (bChangesInvalidatePrepass)
			{
				GetOwner().Invalidate(EInvalidateWidgetReason::ChildOrder);
			}
		}
	}

private:
	bool bChangesInvalidatePrepass;
};

template <typename ChildType>
using TUnmanagedSlotlessChildren = TSlotlessChildren<ChildType, false>;


/** Required to implement GetChildren() in a way that can dynamically return the currently active child. */
template<typename SlotType, typename Attr>
class TOneDynamicChildBase final : public FChildren
{
public:
	TOneDynamicChildBase(SWidget* InOwner, TPanelChildren<SlotType>* InAllChildren, const Attr* InWidgetIndex)
		: FChildren(InOwner)
		, AllChildren(InAllChildren)
		, WidgetIndex(InWidgetIndex)
	{
		check(InAllChildren);
		check(WidgetIndex);
	}

	TOneDynamicChildBase(SWidget* InOwner, TPanelChildren<SlotType>* InAllChildren, std::nullptr_t) = delete;
	TOneDynamicChildBase(SWidget* InOwner, std::nullptr_t, const Attr* InWidgetIndex) = delete;
	TOneDynamicChildBase(SWidget* InOwner, std::nullptr_t, std::nullptr_t) = delete;
	TOneDynamicChildBase(std::nullptr_t, TPanelChildren<SlotType>* InAllChildren, std::nullptr_t) = delete;
	TOneDynamicChildBase(std::nullptr_t, std::nullptr_t, const Attr* InWidgetIndex) = delete;
	TOneDynamicChildBase(std::nullptr_t, std::nullptr_t, std::nullptr_t) = delete;

	virtual int32 Num() const override { return AllChildren->Num() > 0 ? 1 : 0; }

	virtual TSharedRef<SWidget> GetChildAt(int32 Index) override
	{
		check(Index == 0);
		return AllChildren->GetChildAt(WidgetIndex->Get());
	}

	virtual TSharedRef<const SWidget> GetChildAt(int32 Index) const override
	{
		check(Index == 0);
		return AllChildren->GetChildAt(WidgetIndex->Get());
	}

private:
	virtual const FSlotBase& GetSlotAt(int32 ChildIndex) const override
	{
		return (*AllChildren)[ChildIndex];
	}

	virtual FWidgetRef GetChildRefAt(int32 ChildIndex) override
	{
		check(ChildIndex == 0);
		return static_cast<FChildren*>(AllChildren)->GetChildRefAt(WidgetIndex->Get());
	}
	
	virtual FConstWidgetRef GetChildRefAt(int32 ChildIndex) const override
	{
		check(ChildIndex == 0);
		return static_cast<const FChildren*>(AllChildren)->GetChildRefAt(WidgetIndex->Get());
	}

	TPanelChildren<SlotType>* AllChildren;
	const Attr* WidgetIndex;
};

template <typename SlotType>
using TOneDynamicChild = TOneDynamicChildBase<SlotType, TAttribute<int32>>;
