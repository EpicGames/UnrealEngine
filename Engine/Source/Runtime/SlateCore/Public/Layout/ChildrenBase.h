// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlotBase.h"
#include "SlateGlobals.h"
#include "Types/ReflectionMetadata.h"

#ifndef UE_WITH_SLATE_CHILDREN_DEBUGGING
#define UE_WITH_SLATE_CHILDREN_DEBUGGING !(UE_BUILD_SHIPPING)
#endif


class SWidget;
class FSlotBase;


/**
 * FChildren is an interface that must be implemented by all child containers.
 * It allows iteration over a list of any Widget's children regardless of how
 * the underlying Widget happens to store its children.
 * 
 * FChildren is intended to be returned by the GetChildren() method.
 */
class FChildren
{
public:
	FChildren(SWidget* InOwner)
		: Owner(InOwner)
		, Name("Children")
	{
		check(InOwner);
	}
	FChildren(SWidget* InOwner, FName InName)
		: Owner(InOwner)
		, Name(InName)
	{
		check(InOwner);
		check(!Name.IsNone());
	}

	FChildren(std::nullptr_t) = delete;

	//~ Prevents allocation of FChildren. It creates confusion between FSlot and FChildren.
	void* operator new (size_t) = delete;
	void* operator new[](size_t) = delete;

	/** @return the number of children */
	virtual int32 Num() const = 0;
	/** @return pointer to the Widget at the specified Index. */
	virtual TSharedRef<SWidget> GetChildAt( int32 Index ) = 0;
	/** @return const pointer to the Widget at the specified Index. */
	virtual TSharedRef<const SWidget> GetChildAt( int32 Index ) const = 0;

	/** @return the SWidget that own the FChildren */
	SWidget& GetOwner() const
	{
		Debug_TestDestroyTag();
		return *Owner;
	}

	/** Applies the predicate to all the widgets contained by the FChildren. */
	template<typename Predicate>
	inline void ForEachWidget(Predicate&& Pred)
	{
#if UE_WITH_SLATE_CHILDREN_DEBUGGING 
		Debug_TestDestroyTag();
		TGuardValue<bool> IteratingGuard(Debug_bIsIteratingChildren, true);
#endif

		int32 WidgetCount = Num();
		for (int32 Index = 0; Index < WidgetCount; ++Index)
		{
			FWidgetRef WidgetRef = GetChildRefAt(Index);
			Pred(WidgetRef.GetWidget());
		}
	}

	/** Applies the predicate to all the widgets contained by the FChildren. */
	template<typename Predicate>
	inline void ForEachWidget(Predicate&& Pred) const
	{
#if UE_WITH_SLATE_CHILDREN_DEBUGGING 
		Debug_TestDestroyTag();
		TGuardValue<bool> IteratingGuard(Debug_bIsIteratingChildren, true);
#endif

		int32 WidgetCount = Num();
		for (int32 Index = 0; Index < WidgetCount; ++Index)
		{
			FConstWidgetRef WidgetRef = GetChildRefAt(Index);
			Pred(WidgetRef.GetWidget());
		}
	}

	/** @return the number of slot the children has. */
	virtual int32 NumSlot() const
	{
		return Num();
	}

	/** @return the const reference to the slot at the specified Index */
	virtual const FSlotBase& GetSlotAt(int32 ChildIndex) const = 0;

	/** @returns if the Children can support slot with SlateSlotAttribute. */
	virtual bool SupportSlotWithSlateAttribute() const
	{
		return false;
	}

	/** Option to give a name to Children to SlotAttribute purposes or for debugging. */
	FName GetName() const
	{
		Debug_TestDestroyTag();
		return Name;
	}

protected:
	friend class FCombinedChildren;
	template<typename SlotType, typename Attr>
	friend class TOneDynamicChildBase;

	enum ECopyConstruct { CopyConstruct };
	enum ERefConstruct { ReferenceConstruct };

	struct FWidgetRef
	{
	private:
		TOptional<TSharedRef<SWidget>> WidgetCopy;
		SWidget& WidgetReference;

	public:
		FWidgetRef(ECopyConstruct, TSharedRef<SWidget> InWidgetCopy)
			: WidgetCopy(MoveTemp(InWidgetCopy))
			, WidgetReference(WidgetCopy.GetValue().Get())
		{}
		FWidgetRef(ERefConstruct, SWidget& InWidgetRef)
			: WidgetCopy()
			, WidgetReference(InWidgetRef)
		{}
		FWidgetRef(const FWidgetRef& Other)
			: WidgetCopy(Other.WidgetCopy)
			, WidgetReference(WidgetCopy.IsSet() ? WidgetCopy.GetValue().Get() : Other.WidgetReference)
		{}
		FWidgetRef(FWidgetRef&& Other)
			: WidgetCopy(MoveTemp(Other.WidgetCopy))
			, WidgetReference(WidgetCopy.IsSet() ? WidgetCopy.GetValue().Get() : Other.WidgetReference)
		{}
		FWidgetRef& operator=(const FWidgetRef&) = delete;
		FWidgetRef& operator=(FWidgetRef&&) = delete;
		SWidget& GetWidget() const
		{
			return WidgetReference;
		}
	};

	struct FConstWidgetRef
	{
	private:
		TOptional<TSharedRef<const SWidget>> WidgetCopy;
		const SWidget& WidgetReference;

	public:
		FConstWidgetRef(ECopyConstruct, TSharedRef<const SWidget> InWidgetCopy)
			: WidgetCopy(MoveTemp(InWidgetCopy))
			, WidgetReference(WidgetCopy.GetValue().Get())
		{}
		FConstWidgetRef(ERefConstruct, const SWidget& InWidgetRef)
			: WidgetCopy()
			, WidgetReference(InWidgetRef)
		{}
		FConstWidgetRef(const FConstWidgetRef& Other)
			: WidgetCopy(Other.WidgetCopy)
			, WidgetReference(WidgetCopy.IsSet() ? WidgetCopy.GetValue().Get() : Other.WidgetReference)
		{}
		FConstWidgetRef(FConstWidgetRef&& Other)
			: WidgetCopy(MoveTemp(Other.WidgetCopy))
			, WidgetReference(WidgetCopy.IsSet() ? WidgetCopy.GetValue().Get() : Other.WidgetReference)
		{}
		FConstWidgetRef& operator=(const FConstWidgetRef&) = delete;
		FConstWidgetRef& operator=(FConstWidgetRef&&) = delete;
		const SWidget& GetWidget() const
		{
			return WidgetReference;
		}
	};

	/** @return ref to the Widget at the specified Index. */
	virtual FWidgetRef GetChildRefAt(int32 Index) = 0;
	/** @return ref to the Widget at the specified Index. */
	virtual FConstWidgetRef GetChildRefAt(int32 Index) const = 0;

protected:
#if UE_WITH_SLATE_CHILDREN_DEBUGGING 
	virtual ~FChildren()
	{
		Debug_TestDestroyTag();
		Debug_DestroyedTag = 0xA3;
		UE_CLOGF(Debug_bIsIteratingChildren, LogSlate, Fatal,
			"Destroying widget while iterating children! Owner: %ls [%ls]",
			*FReflectionMetaData::GetWidgetDebugInfo(Owner),
			*Name.ToString());
	}

	void Debug_TestDestroyTag() const
	{
		/** There is no guarantee that it will work. The memory could have been reused and the new data can luckily matches what we are looking for. */
		UE_CLOGF(Debug_DestroyedTag != 0xDC, LogSlate, Fatal, "The FChildren is destroyed. You probably have 1 widget owned by 2 different FChildren.");
	}

#else

	virtual ~FChildren() = default;

	void Debug_TestDestroyTag() const
	{
	}

#endif // !WITH_SLATE_DEBUGGING

private:
	SWidget* Owner;
	FName Name;

#if UE_WITH_SLATE_CHILDREN_DEBUGGING
	mutable bool Debug_bIsIteratingChildren = false;
	mutable uint8 Debug_DestroyedTag = 0xDC;
#endif // WITH_SLATE_DEBUGGING
};
