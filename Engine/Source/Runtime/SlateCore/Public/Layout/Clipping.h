// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Layout/SlateRect.h"
#include "Rendering/RenderingCommon.h"
#include "Clipping.generated.h"

/**
 * This enum controls clipping of widgets in Slate.  By default all SWidgets do not need to clip their children.
 * Most of the time, you don't need to clip, the only times it becomes important is when something might become hidden
 * due to panning.  You should use this wisely, as Slate can not batch across clipping areas, so if widget A and widget B
 * are set to EWidgetClipping::Yes, no drawing that happens inside their widget trees will ever be batch together, adding
 * additional GPU overhead.
 */
UENUM(BlueprintType)
enum class EWidgetClipping : uint8
{
	/**
	 * This widget does not clip children, it and all children inherit the clipping area of the last widget that clipped.
	 */
	Inherit,
	/**
	 * This widget clips content the bounds of this widget.  It intersects those bounds with any previous clipping area.
	 */
	ClipToBounds,
	/**
	 * This widget clips to its bounds.  It does NOT intersect with any existing clipping geometry, it pushes a new clipping 
	 * state.  Effectively allowing it to render outside the bounds of hierarchy that does clip.
	 * 
	 * NOTE: This will NOT allow you ignore the clipping zone that is set to [Yes - Always].
	 */
	ClipToBoundsWithoutIntersecting UMETA(DisplayName = "Clip To Bounds - Without Intersecting (Advanced)"),
	/**
	 * This widget clips to its bounds.  It intersects those bounds with any previous clipping area.
	 *
	 * NOTE: This clipping area can NOT be ignored, it will always clip children.  Useful for hard barriers
	 * in the UI where you never want animations or other effects to break this region.
	 */
	ClipToBoundsAlways UMETA(DisplayName = "Clip To Bounds - Always (Advanced)"),
	/**
	 * This widget clips to its bounds when it's Desired Size is larger than the allocated geometry 
	 * the widget is given.  If that occurs, it work like [Yes].
	 * 
	 * NOTE: This mode was primarily added for Text, which is often placed into containers that eventually 
	 * are resized to not be able to support the length of the text.  So rather than needing to tag every 
	 * container that could contain text with [Yes], which would result in almost no batching, this mode 
	 * was added to dynamically adjust the clipping if needed.  The reason not every panel is set to OnDemand, 
	 * is because not every panel returns a Desired Size that matches what it plans to render at.
	 */
	OnDemand UMETA(DisplayName = "On Demand (Advanced)")
};


/**
 * The Clipping Zone represents some arbitrary plane segment that can be used to clip the geometry in Slate.
 */
class FSlateClippingZone
{
public:

	FVector2f TopLeft;
	FVector2f TopRight;
	FVector2f BottomLeft;
	FVector2f BottomRight;

	SLATECORE_API explicit FSlateClippingZone(const FShortRect& AxisAlignedRect);
	SLATECORE_API explicit FSlateClippingZone(const FSlateRect& AxisAlignedRect);
	SLATECORE_API explicit FSlateClippingZone(const FGeometry& BoundingGeometry);
	SLATECORE_API explicit FSlateClippingZone(const FPaintGeometry& PaintingGeometry);
	SLATECORE_API FSlateClippingZone(const UE::Slate::FDeprecateVector2DParameter& InTopLeft, const UE::Slate::FDeprecateVector2DParameter& InTopRight, const UE::Slate::FDeprecateVector2DParameter& InBottomLeft, const UE::Slate::FDeprecateVector2DParameter& InBottomRight);
	FSlateClippingZone() {}

	/**  */
	inline bool GetShouldIntersectParent() const
	{
		return bIntersect;
	}

	/**  */
	inline void SetShouldIntersectParent(bool bValue)
	{
		bIntersect = bValue;
	}

	/**  */
	inline bool GetAlwaysClip() const
	{
		return bAlwaysClip;
	}

	/**  */
	inline void SetAlwaysClip(bool bValue)
	{
		bAlwaysClip = bValue;
	}

	/** Is the clipping rect axis aligned?  If it is, we can use a much cheaper clipping solution. */
	inline bool IsAxisAligned() const
	{
		return bIsAxisAligned;
	}

	/**
	 * Indicates if this clipping state has a zero size, aka is empty.  We only possibly report true for
	 * scissor clipping zones.
	 */
	inline bool HasZeroArea() const
	{
		if (bIsAxisAligned)
		{
			FVector2f Difference = TopLeft - BottomRight;
			return FMath::IsNearlyZero(Difference.X) || FMath::IsNearlyZero(Difference.Y);
		}

		return false;
	}

	/** Is a point inside the clipping zone? */
	SLATECORE_API bool IsPointInside(const UE::Slate::FDeprecateVector2DParameter& Point) const;

	/**
	 * Intersects two clipping zones and returns the new clipping zone that would need to be used.
	 * This can only be called between two axis aligned clipping zones.
	 */
	SLATECORE_API FSlateClippingZone Intersect(const FSlateClippingZone& Other) const;

	/**
	 * Gets the bounding box of the points making up this clipping zone.
	 */
	SLATECORE_API FSlateRect GetBoundingBox() const;

	bool operator==(const FSlateClippingZone& Other) const
	{
		return bIsAxisAligned == Other.bIsAxisAligned &&
			bIntersect == Other.bIntersect &&
			bAlwaysClip == Other.bAlwaysClip &&
			TopLeft == Other.TopLeft &&
			TopRight == Other.TopRight &&
			BottomLeft == Other.BottomLeft &&
			BottomRight == Other.BottomRight;
	}

	uint32 ComputeHash() const
	{
		return HashCombine(
			HashCombine(GetTypeHash(TopLeft), GetTypeHash(TopRight)),
			HashCombine(GetTypeHash(BottomLeft), GetTypeHash(BottomRight))
		);
	}

	FSlateClippingZone ConvertRelativeToAbsolute(const UE::Slate::FDeprecateVector2DParameter& WindowOffset) const
	{
		FSlateClippingZone Absolute(TopLeft + WindowOffset, TopRight + WindowOffset, BottomLeft + WindowOffset, BottomRight + WindowOffset);
		Absolute.bIsAxisAligned = bIsAxisAligned;
		Absolute.bIntersect = bIntersect;
		Absolute.bAlwaysClip = bAlwaysClip;

		return Absolute;
	}
private:
	SLATECORE_API void InitializeFromArbitraryPoints(const UE::Slate::FDeprecateVector2DParameter& InTopLeft, const UE::Slate::FDeprecateVector2DParameter& InTopRight, const UE::Slate::FDeprecateVector2DParameter& InBottomLeft, const UE::Slate::FDeprecateVector2DParameter& InBottomRight);

private:
	/** Is the clipping zone axis aligned?  Axis aligned clipping zones are much cheaper. */
	uint8 bIsAxisAligned : 1;
	/** Should this clipping zone intersect the current one? */
	uint8 bIntersect : 1;
	/** Should this clipping zone always clip, even if another zone wants to ignore intersection? */
	uint8 bAlwaysClip : 1;

	friend inline uint32 GetTypeHash(const FSlateClippingZone& Zone)
	{
		return Zone.ComputeHash();
	}
};

template<> struct TIsPODType<FSlateClippingZone> { enum { Value = true }; };

/**
 * Indicates the method of clipping that should be used on the GPU.
 */
enum class EClippingMethod : uint8
{
	Scissor,
	Stencil
};

/**
 * Indicates the method of clipping that should be used on the GPU.
 */
enum class EClippingFlags : uint8
{
	None		= 0,
	/** If the clipping state is always clip, we cache it at a higher level. */
	AlwaysClip	= 1 << 0
};

ENUM_CLASS_FLAGS(EClippingFlags)

/**
 * Captures everything about a single draw calls clipping state.
 */
class FSlateClippingState
{
public:
	SLATECORE_API FSlateClippingState(EClippingFlags InFlags = EClippingFlags::None);
	
	/** Is a point inside the clipping state? */
	SLATECORE_API bool IsPointInside(const UE::Slate::FDeprecateVector2DParameter& Point) const;

#if WITH_SLATE_DEBUGGING
	/** Set the state index that this clipping state originated from.  We just do this for debugging purposes. */
	inline void SetDebuggingStateIndex(int32 InStateIndex) const
	{
		Debugging_StateIndex = InStateIndex;
		Debugging_StateIndexFromFrame = GFrameNumber;
	}
#endif

	inline bool GetAlwaysClip() const { return EnumHasAllFlags(Flags, EClippingFlags::AlwaysClip); }

	inline bool GetShouldIntersectParent() const
	{
		if (GetClippingMethod() == EClippingMethod::Scissor)
		{
			return ScissorRect->GetShouldIntersectParent();
		}
		else
		{
			for (const FSlateClippingZone& Stencil : StencilQuads)
			{
				if (!Stencil.GetShouldIntersectParent())
				{
					return false;
				}
			}
		}

		return true;
	}

	/**
	 * Gets the type of clipping that is required by this clipping state.  The simpler clipping is
	 * scissor clipping, but that's only possible if the clipping rect is axis aligned.
	 */
	inline EClippingMethod GetClippingMethod() const
	{
		return ScissorRect.IsSet() ? EClippingMethod::Scissor : EClippingMethod::Stencil;
	}

	/**
	 * Indicates if this clipping state has a zero size, aka is empty.  We only possibly report true for
	 * scissor clipping zones.
	 */
	inline bool HasZeroArea() const
	{
		if (ScissorRect.IsSet())
		{
			return ScissorRect->HasZeroArea();
		}

		// Assume that stenciled clipping state has some area, don't bother computing it.
		return false;
	}

	bool operator==(const FSlateClippingState& Other) const
	{
		return Flags == Other.Flags &&
			ScissorRect == Other.ScissorRect &&
			StencilQuads == Other.StencilQuads;
	}

public:
	/** If this is a more expensive stencil clipping zone, this will be filled. */
	TArray<FSlateClippingZone> StencilQuads;

	/** If this is an axis aligned clipping state, this will be filled. */
	TOptional<FSlateClippingZone> ScissorRect;

private:

	/** The specialized flags needed for this clipping state. */
	EClippingFlags Flags;

#if WITH_SLATE_DEBUGGING
	/** For a given frame, this is a unique index into a state array of clipping zones that have been registered for a window being drawn. */
	mutable int32 Debugging_StateIndex;
	mutable int32 Debugging_StateIndexFromFrame;
#endif
};

struct FClipStateHandle
{
public:
	FClipStateHandle()
		: CachedClipState(nullptr)
		, PrecachedClipIndex(INDEX_NONE)
	{}

	int32 GetPrecachedClipIndex() const { return PrecachedClipIndex; }
	const FSlateClippingState* GetCachedClipState() const { return CachedClipState; }

	bool operator==(const FClipStateHandle& Other) const
	{
		return CachedClipState == Other.CachedClipState && PrecachedClipIndex == Other.PrecachedClipIndex;
	}

	void SetPreCachedClipIndex(int32 InClipIndex)
	{
		PrecachedClipIndex = InClipIndex;
	}

	void SetCachedClipState(const FSlateClippingState* CachedState)
	{
		CachedClipState = CachedState;
		PrecachedClipIndex = INDEX_NONE;
	}
private:
	const FSlateClippingState* CachedClipState;
	int32 PrecachedClipIndex;

};


class FSlateCachedClipState
{
public:
	FSlateCachedClipState(const FSlateClippingState& InState)
		: ClippingState(MakeShared<FSlateClippingState, ESPMode::ThreadSafe>(InState))
	{}

	TSharedRef<FSlateClippingState, ESPMode::ThreadSafe> ClippingState;
};


/**
 * The clipping manager maintain the running clip state.  This is used for both maintain and for hit testing.
 */
class FSlateClippingManager
{
public:
	SLATECORE_API FSlateClippingManager();

	SLATECORE_API int32 PushClip(const FSlateClippingZone& InClippingZone);
	SLATECORE_API int32 PushClippingState(const FSlateClippingState& InClipState);
	SLATECORE_API int32 GetClippingIndex() const;
	SLATECORE_API TOptional<FSlateClippingState> GetActiveClippingState() const;
	const TArray<int32>& GetClippingStack() const { return ClippingStack; }
	SLATECORE_API const TArray<FSlateClippingState>& GetClippingStates() const;
	SLATECORE_API void PopClip();
	SLATECORE_API void PopToStackIndex(int32 Index);
	int32 GetClippingIndexAtStackIndex(int32 StackIndex) const { return ClippingStack.IsValidIndex(StackIndex) ? ClippingStack[StackIndex] : INDEX_NONE; }
	int32 GetStackDepth() const { return ClippingStack.Num(); }
	SLATECORE_API const FSlateClippingState* GetPreviousClippingState(bool bWillIntersectWithParent) const;

	SLATECORE_API void ResetClippingState();

private:

	SLATECORE_API FSlateClippingState CreateClippingState(const FSlateClippingZone& InClipRect) const;

private:
	/** Maintains the current clipping stack, with the indexes in the array of clipping states.  Pushed and popped throughout the drawing process. */
	TArray< int32 > ClippingStack;

	/** The authoritative list of clipping states used when rendering.  Any time a clipping state is needed, it's added here. */
	TArray< FSlateClippingState > ClippingStates;
};
