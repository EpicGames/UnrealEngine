// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "FlowDirection.generated.h"

/**
 * Widgets may need to flow left or right depending upon the current culture/localization that's active.
 * This enum is used to request a specific layout flow.
 */
enum class EFlowDirection : uint8
{
	/** Desires content flows using a LTR layout */
	LeftToRight,
	/** Desires content flows using a RTL layout */
	RightToLeft,
};


/**
 * 
 */
UENUM()
enum class EFlowDirectionPreference : uint8
{
	/** Inherits the flow direction set by the parent widget. */
	Inherit,
	/** Begins laying out widgets using the current cultures layout direction preference, flipping the directionality of flows. */
	Culture,
	/** Forces a Left to Right layout flow. */
	LeftToRight,
	/** Forces a Right to Left layout flow. */
	RightToLeft
};


extern SLATECORE_API EFlowDirection GSlateFlowDirection;
extern SLATECORE_API int32 GSlateFlowDirectionShouldFollowCultureByDefault;

/**
 * Class containing utilities for getting layout localization information.
 */
class FLayoutLocalization
{
public:
	/** Gets the current expected flow direction based on localization. */
	static SLATECORE_API EFlowDirection GetLocalizedLayoutDirection();
};
