// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Layout/PaintGeometry.h"
#include "Layout/SlateRect.h"
#include "Layout/SlateRotatedRect.h"
#include "Layout/SlateRotatedRect.h"
#include "Math/TransformCalculus.h"
#include "Math/TransformCalculus2D.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Rendering/SlateRenderTransform.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"
#include "Types/SlateVector2.h"

#include "Geometry.generated.h"

class FArrangedWidget;
class FLayoutGeometry;
class SWidget;
template <typename T> struct TIsPODType;

/**
 * Represents the position, size, and absolute position of a Widget in Slate.
 * The absolute location of a geometry is usually screen space or
 * window space depending on where the geometry originated.
 * Geometries are usually paired with a SWidget pointer in order
 * to provide information about a specific widget (see FArrangedWidget).
 * A Geometry's parent is generally thought to be the Geometry of the
 * the corresponding parent widget.
 */
USTRUCT(BlueprintType)
struct FGeometry
{
	GENERATED_USTRUCT_BODY()

public:

	/**
	 * Default constructor. Creates a geometry with identity transforms.
	 */
	SLATECORE_API FGeometry();

	/**
	 * Copy constructor.
	 */
	FGeometry(const FGeometry& RHS) = default;

	/**
	 * !!! HACK!!! We're keeping members of FGeometry const to prevent mutability without making them private, for backward compatibility.
	 * But this means the assignment operator no longer works. We implement one ourselves now and force a memcpy.
	 */
	SLATECORE_API FGeometry& operator=(const FGeometry& RHS);

	/**
	 * !!! DEPRECATED FUNCTION !!! Use MakeChild taking a layout transform instead!
	 * Construct a new geometry given the following parameters:
	 * 
	 * @param OffsetFromParent         Local position of this geometry within its parent geometry.
	 * @param ParentAbsolutePosition   The absolute position of the parent geometry containing this geometry.
	 * @param InSize                   The size of this geometry.
	 * @param InScale                  The scale of this geometry with respect to Normal Slate Coordinates.
	 */
	FGeometry( const UE::Slate::FDeprecateVector2DParameter& OffsetFromParent, const UE::Slate::FDeprecateVector2DParameter& ParentAbsolutePosition, const UE::Slate::FDeprecateVector2DParameter& InLocalSize, float InScale )
		: Size(InLocalSize)
		, Scale(1.0f)
		, AbsolutePosition(0.0f, 0.0f)
		, bHasRenderTransform(false)
	{
		// Since OffsetFromParent is given as a LocalSpaceOffset, we MUST convert this offset into the space of the parent to construct a valid layout transform.
		// The extra TransformPoint below does this by converting the local offset to an offset in parent space.
		FVector2f LayoutOffset = TransformPoint(InScale, UE::Slate::CastToVector2f(OffsetFromParent));

		FSlateLayoutTransform ParentAccumulatedLayoutTransform(InScale, UE::Slate::CastToVector2f(ParentAbsolutePosition));
		FSlateLayoutTransform LocalLayoutTransform(LayoutOffset);
		FSlateLayoutTransform AccumulatedLayoutTransform = Concatenate(LocalLayoutTransform, ParentAccumulatedLayoutTransform);
		AccumulatedRenderTransform = TransformCast<FSlateRenderTransform>(AccumulatedLayoutTransform);
		// HACK to allow us to make FGeometry public members immutable to catch misuse.
		const_cast<FVector2f&>( AbsolutePosition ) = FVector2f(AccumulatedLayoutTransform.GetTranslation());
		const_cast<float&>( Scale ) = AccumulatedLayoutTransform.GetScale();
		const_cast<FVector2f&>( Position ) = FVector2f(LocalLayoutTransform.GetTranslation());
	}

private:
	/**
	 * Construct a new geometry with a given size in LocalSpace that is attached to a parent geometry with the given layout and render transform. 
	 * 
	 * @param InLocalSize						The size of the geometry in Local Space.
	 * @param InLocalLayoutTransform			A layout transform from local space to the parent geoemtry's local space.
	 * @param InLocalRenderTransform			A render-only transform in local space that will be prepended to the LocalLayoutTransform when rendering.
	 * @param InLocalRenderTransformPivot		Pivot in normalizes local space for the local render transform.
	 * @param ParentAccumulatedLayoutTransform	The accumulated layout transform of the parent widget. AccumulatedLayoutTransform = Concat(LocalLayoutTransform, ParentAccumulatedLayoutTransform).
	 * @param ParentAccumulatedRenderTransform	The accumulated render transform of the parent widget. AccumulatedRenderTransform = Concat(LocalRenderTransform, LocalLayoutTransform, ParentAccumulatedRenderTransform).
	 */
	FGeometry( 
		const UE::Slate::FDeprecateVector2DParameter& InLocalSize, 
		const FSlateLayoutTransform& InLocalLayoutTransform, 
		const FSlateRenderTransform& InLocalRenderTransform, 
		const UE::Slate::FDeprecateVector2DParameter& InLocalRenderTransformPivot, 
		const FSlateLayoutTransform& ParentAccumulatedLayoutTransform, 
		const FSlateRenderTransform& ParentAccumulatedRenderTransform)
		: Size(InLocalSize)
		, Scale(1.0f)
		, AbsolutePosition(0.0f, 0.0f)
		, AccumulatedRenderTransform(
			Concatenate(
				// convert the pivot to local space and make it the origin
				Inverse(TransformPoint(FScale2D(UE::Slate::CastToVector2f(InLocalSize)), UE::Slate::CastToVector2f(InLocalRenderTransformPivot))),
				// apply the render transform in local space centered around the pivot
				InLocalRenderTransform,
				// translate the pivot point back.
				TransformPoint(FScale2D(UE::Slate::CastToVector2f(InLocalSize)), UE::Slate::CastToVector2f(InLocalRenderTransformPivot)),
				// apply the layout transform next.
				InLocalLayoutTransform,
				// finally apply the parent accumulated transform, which takes us to the root.
				ParentAccumulatedRenderTransform
			)
		), bHasRenderTransform(true)
	{
		FSlateLayoutTransform AccumulatedLayoutTransform = Concatenate(InLocalLayoutTransform, ParentAccumulatedLayoutTransform);
		// HACK to allow us to make FGeometry public members immutable to catch misuse.
		const_cast<FVector2f&>( AbsolutePosition ) = FVector2f(AccumulatedLayoutTransform.GetTranslation());
		const_cast<float&>( Scale ) = AccumulatedLayoutTransform.GetScale();
		const_cast<FVector2f&>( Position ) = FVector2f(InLocalLayoutTransform.GetTranslation());
	}

	/**
	 * Construct a new geometry with a given size in LocalSpace (and identity render transform) that is attached to a parent geometry with the given layout and render transform. 
	 * 
	 * @param InLocalSize						The size of the geometry in Local Space.
	 * @param InLocalLayoutTransform			A layout transform from local space to the parent geoemtry's local space.
	 * @param ParentAccumulatedLayoutTransform	The accumulated layout transform of the parent widget. AccumulatedLayoutTransform = Concat(LocalLayoutTransform, ParentAccumulatedLayoutTransform).
	 * @param ParentAccumulatedRenderTransform	The accumulated render transform of the parent widget. AccumulatedRenderTransform = Concat(LocalRenderTransform, LocalLayoutTransform, ParentAccumulatedRenderTransform).
	 */
	FGeometry(
		const UE::Slate::FDeprecateVector2DParameter& InLocalSize,
		const FSlateLayoutTransform& InLocalLayoutTransform,
		const FSlateLayoutTransform& ParentAccumulatedLayoutTransform,
		const FSlateRenderTransform& ParentAccumulatedRenderTransform,
		bool bParentHasRenderTransform)
		: Size(UE::Slate::CastToVector2f(InLocalSize))
		, Scale(1.0f)
		, AbsolutePosition(0.0f, 0.0f)
		, AccumulatedRenderTransform(Concatenate(InLocalLayoutTransform, ParentAccumulatedRenderTransform))
		, bHasRenderTransform(bParentHasRenderTransform)
	{
		FSlateLayoutTransform AccumulatedLayoutTransform = Concatenate(InLocalLayoutTransform, ParentAccumulatedLayoutTransform);
		// HACK to allow us to make FGeometry public members immutable to catch misuse.
		const_cast<FVector2f&>( AbsolutePosition ) = FVector2f(AccumulatedLayoutTransform.GetTranslation());
		const_cast<float&>( Scale ) = AccumulatedLayoutTransform.GetScale();
		const_cast<FVector2f&>( Position ) = FVector2f(InLocalLayoutTransform.GetTranslation());
	}

public:

	/**
	 * Test geometries for strict equality (i.e. exact float equality).
	 *
	 * @param Other A geometry to compare to.
	 * @return true if this is equal to other, false otherwise.
	 */
	bool operator==( const FGeometry& Other ) const
	{
		return
			this->Size == Other.Size && 
			this->AbsolutePosition == Other.AbsolutePosition &&
			this->Position == Other.Position &&
			this->Scale == Other.Scale;
	}
	
	/**
	 * Test geometries for strict inequality (i.e. exact float equality negated).
	 *
	 * @param Other A geometry to compare to.
	 * @return false if this is equal to other, true otherwise.
	 */
	bool operator!=( const FGeometry& Other ) const
	{
		return !( this->operator==(Other) );
	}

public:
	/**
	 * Makes a new geometry that is essentially the root of a hierarchy (has no parent transforms to inherit).
	 * For a root Widget, the LayoutTransform is often the window DPI scale + window offset.
	 * 
	 * @param LocalSize			Size of the geometry in Local Space.
	 * @param LayoutTransform	Layout transform of the geometry.
	 * @return					The new root geometry
	 */
	inline static FGeometry MakeRoot(const UE::Slate::FDeprecateVector2DParameter& InLocalSize, const FSlateLayoutTransform& LayoutTransform)
	{
		return FGeometry(InLocalSize, LayoutTransform, FSlateLayoutTransform(), FSlateRenderTransform(), false);
	}
	/**
	 * Makes a new geometry that is essentially the root of a hierarchy (has no parent transforms to inherit).
	 * For a root Widget, the LayoutTransform is often the window DPI scale + window offset.
	 * 
	 * @param LocalSize			Size of the geometry in Local Space.
	 * @param LayoutTransform	Layout transform of the geometry.
	 * @return					The new root geometry
	 */
	inline static FGeometry MakeRoot(const UE::Slate::FDeprecateVector2DParameter& InLocalSize, const FSlateLayoutTransform& LayoutTransform, const FSlateRenderTransform& RenderTransform)
	{
		return FGeometry(InLocalSize, LayoutTransform, FSlateLayoutTransform(), FSlateRenderTransform(), !RenderTransform.IsIdentity());
	}
	/**
	 * Create a child geometry relative to this one with a given local space size, layout transform, and render transform.
	 * For example, a widget with a 5x5 margin will create a geometry for it's child contents having a LayoutTransform of Translate(5,5) and a LocalSize 10 units smaller 
	 * than it's own.
	 *
	 * @param LocalSize			The size of the child geometry in local space.
	 * @param LayoutTransform	Layout transform of the new child relative to this Geometry. Goes from the child's layout space to the this widget's layout space.
	 * @param RenderTransform	Render-only transform of the new child that is applied before the layout transform for rendering purposes only.
	 * @param RenderTransformPivot	Pivot in normalized local space of the Render transform.
	 *
	 * @return					The new child geometry.
	 */
	inline FGeometry MakeChild(const UE::Slate::FDeprecateVector2DParameter& InLocalSize, const FSlateLayoutTransform& LayoutTransform, const FSlateRenderTransform& RenderTransform, const UE::Slate::FDeprecateVector2DParameter& RenderTransformPivot) const
	{
		return FGeometry(InLocalSize, LayoutTransform, RenderTransform, RenderTransformPivot, GetAccumulatedLayoutTransform(), GetAccumulatedRenderTransform());
	}

	/**
	 * Create a child geometry relative to this one with a given local space size, layout transform, and identity render transform.
	 * For example, a widget with a 5x5 margin will create a geometry for it's child contents having a LayoutTransform of Translate(5,5) and a LocalSize 10 units smaller 
	 * than it's own.
	 *
	 * @param LocalSize			The size of the child geometry in local space.
	 * @param LayoutTransform	Layout transform of the new child relative to this Geometry. Goes from the child's layout space to the this widget's layout space.
	 *
	 * @return					The new child geometry.
	 */
	FORCEINLINE_DEBUGGABLE FGeometry MakeChild(const UE::Slate::FDeprecateVector2DParameter& InLocalSize, const FSlateLayoutTransform& LayoutTransform) const
	{
		return FGeometry(InLocalSize, LayoutTransform, GetAccumulatedLayoutTransform(), GetAccumulatedRenderTransform(), bHasRenderTransform);
	}
	FORCEINLINE_DEBUGGABLE FGeometry MakeChild(const FSlateRenderTransform& RenderTransform, const UE::Slate::FDeprecateVector2DParameter& RenderTransformPivot = FVector2f(0.5f, 0.5f)) const
	{
		return FGeometry(GetLocalSize(), FSlateLayoutTransform(), RenderTransform, RenderTransformPivot, GetAccumulatedLayoutTransform(), GetAccumulatedRenderTransform());
	}


	/**
	 * Create a child geometry+widget relative to this one using the given LayoutGeometry.
	 *
	 * @param ChildWidget		The child widget this geometry is being created for.
	 * @param LayoutGeometry	Layout geometry of the child.
	 *
	 * @return					The new child geometry.
	 */
	FArrangedWidget MakeChild(const TSharedRef<SWidget>& ChildWidget, const FLayoutGeometry& LayoutGeometry) const;

	/**
	 * Create a child geometry+widget relative to this one with a given local space size and layout transform.
	 * The Geometry inherits the child widget's render transform.
	 * For example, a widget with a 5x5 margin will create a geometry for it's child contents having a LayoutTransform of Translate(5,5) and a LocalSize 10 units smaller 
	 * than it's own.
	 *
	 * @param ChildWidget		The child widget this geometry is being created for.
	 * @param LocalSize			The size of the child geometry in local space.
	 * @param LayoutTransform	Layout transform of the new child relative to this Geometry. 
	 *
	 * @return					The new child geometry+widget.
	 */
	FArrangedWidget MakeChild(const TSharedRef<SWidget>& ChildWidget, const UE::Slate::FDeprecateVector2DParameter& InLocalSize, const FSlateLayoutTransform& LayoutTransform) const;

#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS

	/**
	 * !!! DEPRECATED FUNCTION !!! Use MakeChild taking a layout transform instead!
	 * Create a child geometry relative to this one with a given local space size and layout transform given by a scale+offset. The Render Transform is identity.
	 *
	 * @param ChildOffset	Offset of the child relative to the parent. Scale+Offset effectively define the layout transform.
	 * @param LocalSize		The size of the child geometry in local space.
	 * @param ChildScale	Scale of the child relative to the parent. Scale+Offset effectively define the layout transform.
	 *
	 * @return				The new child geometry.
	 */
	UE_DEPRECATED(5.2, "Use FGeometry MakeChild(const FVector2f& InLocalSize, const FSlateLayoutTransform& InLayoutTransform) instead.")
	inline FGeometry MakeChild(const FVector2D& ChildOffset, const FVector2D& InLocalSize, float ChildScale = 1.0f) const
	{
		// Since ChildOffset is given as a LocalSpaceOffset, we MUST convert this offset into the space of the parent to construct a valid layout transform.
		// The extra TransformPoint below does this by converting the local offset to an offset in parent space.
		return FGeometry(UE::Slate::CastToVector2f(InLocalSize), FSlateLayoutTransform(ChildScale, TransformPoint(ChildScale, UE::Slate::CastToVector2f(ChildOffset))), GetAccumulatedLayoutTransform(), GetAccumulatedRenderTransform(), bHasRenderTransform);
	}

#endif

	/**
	 * !!! DEPRECATED FUNCTION !!! Use MakeChild taking a layout transform instead!
	 * Create a child geometry+widget relative to this one with a given local space size and layout transform given by a scale+offset.
	 * The Geometry inherits the child widget's render transform.
	 *
	 * @param ChildWidget	The child widget this geometry is being created for.
	 * @param ChildOffset	Offset of the child relative to the parent. Scale+Offset effectively define the layout transform.
	 * @param LocalSize		The size of the child geometry in local space.
	 * @param ChildScale	Scale of the child relative to the parent. Scale+Offset effectively define the layout transform.
	 *
	 * @return				The new child geometry+widget.
	 */
	FArrangedWidget MakeChild(const TSharedRef<SWidget>& ChildWidget, const UE::Slate::FDeprecateVector2DParameter& ChildOffset, const UE::Slate::FDeprecateVector2DParameter& InLocalSize, float ChildScale = 1.0f) const;

	/**
	 * Create a paint geometry that represents this geometry.
	 * 
	 * @return	The new paint geometry.
	 */
	FORCEINLINE_DEBUGGABLE FPaintGeometry ToPaintGeometry() const
	{
		return FPaintGeometry(GetAccumulatedLayoutTransform(), GetAccumulatedRenderTransform(), FVector2f(Size), bHasRenderTransform);
	}

	/**
	 * Create a paint geometry relative to this one with a given local space size and layout transform.
	 * The paint geometry inherits the widget's render transform.
	 *
	 * @param LocalSize			The size of the child geometry in local space.
	 * @param LayoutTransform	Layout transform of the paint geometry relative to this Geometry. 
	 *
	 * @return					The new paint geometry derived from this one.
	 */
	FORCEINLINE_DEBUGGABLE FPaintGeometry ToPaintGeometry(const UE::Slate::FDeprecateVector2DParameter& InLocalSize, const FSlateLayoutTransform& InLayoutTransform) const
	{
		FSlateLayoutTransform NewAccumulatedLayoutTransform = Concatenate(InLayoutTransform, GetAccumulatedLayoutTransform());
		return FPaintGeometry(NewAccumulatedLayoutTransform, Concatenate(InLayoutTransform, GetAccumulatedRenderTransform()), UE::Slate::CastToVector2f(InLocalSize), bHasRenderTransform);
	}

	/**
	 * Create a paint geometry relative to this one with a given local space size and layout transform.
	 * The paint geometry inherits the widget's render transform.
	 *
	 * @param LocalSize			The size of the child geometry in local space.
	 * @param LayoutTransform	Layout transform of the paint geometry relative to this Geometry. 
	 *
	 * @return					The new paint geometry derived from this one.
	 */
	FORCEINLINE_DEBUGGABLE FPaintGeometry ToPaintGeometry(const UE::Slate::FDeprecateVector2DParameter& InLocalSize, const FSlateLayoutTransform& InLayoutTransform, const FSlateRenderTransform& RenderTransform, const UE::Slate::FDeprecateVector2DParameter& RenderTransformPivot = FVector2f(0.5f, 0.5f)) const
	{
		return MakeChild(InLocalSize, InLayoutTransform, RenderTransform, RenderTransformPivot).ToPaintGeometry();
	}

	/**
	 * Create a paint geometry with the same size as this geometry with a given layout transform.
	 * The paint geometry inherits the widget's render transform.
	 *
	 * @param LayoutTransform	Layout transform of the paint geometry relative to this Geometry. 
	 * 
	 * @return					The new paint geometry derived from this one.
	 */
	FORCEINLINE_DEBUGGABLE FPaintGeometry ToPaintGeometry(const FSlateLayoutTransform& LayoutTransform) const
	{
		return ToPaintGeometry(FVector2f(Size), LayoutTransform);
	}

#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS

	/**
	 * !!! DEPRECATED FUNCTION !!! Use ToPaintGeometry taking a layout transform instead!
	 * Create a paint geometry relative to this one with a given local space size and layout transform given as a scale+offset.
	 * The paint geometry inherits the widget's render transform.
	 *
	 * @param LocalOffset	Offset of the paint geometry relative to the parent. Scale+Offset effectively define the layout transform.
	 * @param LocalSize		The size of the paint geometry in local space.
	 * @param LocalScale	Scale of the paint geometry relative to the parent. Scale+Offset effectively define the layout transform.
	 * 
	 * @return				The new paint geometry derived from this one.
	 */
	UE_DEPRECATED(5.2, "Use FPaintGeometry ToPaintGeometry(const FVector2f& InLocalSize, const FSlateLayoutTransform& InLayoutTransform) instead.")
	FORCEINLINE_DEBUGGABLE FPaintGeometry ToPaintGeometry(const UE::Slate::FDeprecateVector2DParameter& InLocalOffset, const UE::Slate::FDeprecateVector2DParameter& InLocalSize, float InLocalScale = 1.0f) const
	{
		// Since ChildOffset is given as a LocalSpaceOffset, we MUST convert this offset into the space of the parent to construct a valid layout transform.
		// The extra TransformPoint below does this by converting the local offset to an offset in parent space.
		return ToPaintGeometry(InLocalSize, FSlateLayoutTransform(InLocalScale, TransformPoint(InLocalScale, UE::Slate::CastToVector2f(InLocalOffset))));
	}

#endif

	/**
	 * !!! DEPRECATED FUNCTION !!! Use ToPaintGeometry taking a layout transform instead!
	 * Create a paint geometry relative to this one with the same size size and a given local space offset.
	 * The paint geometry inherits the widget's render transform.
	 *
	 * @param LocalOffset	Offset of the paint geometry relative to the parent. Effectively defines the layout transform.
	 * 
	 * @return				The new paint geometry derived from this one.
	 */
	inline FPaintGeometry ToOffsetPaintGeometry(const UE::Slate::FDeprecateVector2DParameter& LocalOffset) const
	{
		return ToPaintGeometry(FSlateLayoutTransform(UE::Slate::CastToVector2f(LocalOffset)));
	}

	/**
	 * Create a paint geometry relative to this one that whose local space is "inflated" by the specified amount in each direction.
	 * The paint geometry inherits the widget's render transform.
	 *
	 * @param InflateAmount	Amount by which to "inflate" the geometry in each direction around the center point. Effectively defines a layout transform offset and an inflation of the LocalSize.
	 * 
	 * @return				The new paint geometry derived from this one.
	 */
	inline FPaintGeometry ToInflatedPaintGeometry(const UE::Slate::FDeprecateVector2DParameter& InflateAmount) const
	{
		FVector2f InflateAmount2f = UE::Slate::CastToVector2f(InflateAmount);
		// This essentially adds (or subtracts) a border around the widget. We scale the size then offset by the border amount.
		// Note this is not scaling child widgets, so the scale is not changing.
		FVector2f NewSize = FVector2f(Size) + InflateAmount2f* 2;
		return ToPaintGeometry(NewSize, FSlateLayoutTransform(-InflateAmount2f));
	}

	/** 
	 * Absolute coordinates could be either desktop or window space depending on what space the root of the widget hierarchy is in.
	 * 
	 * @return true if the provided location in absolute coordinates is within the bounds of this geometry. 
	 */
	inline bool IsUnderLocation(const UE::Slate::FDeprecateVector2DParameter& AbsoluteCoordinate) const
	{
		// this render transform invert is a little expensive. We might consider caching it?
		FSlateRotatedRect Rect = TransformRect(GetAccumulatedRenderTransform(), FSlateRotatedRect(FSlateRect(FVector2f(0.0f, 0.0f), FVector2f(Size))));
		return Rect.IsUnderLocation(AbsoluteCoordinate);
	}

	/** 
	 * Absolute coordinates could be either desktop or window space depending on what space the root of the widget hierarchy is in.
	 * 
	 * @return Transforms AbsoluteCoordinate into the local space of this Geometry. 
	 */
	FORCEINLINE_DEBUGGABLE UE::Slate::FDeprecateVector2DResult AbsoluteToLocal(UE::Slate::FDeprecateVector2DParameter AbsoluteCoordinate) const
	{
		// this render transform invert is a little expensive. We might consider caching it.
		return UE::Slate::FDeprecateVector2DResult(TransformPoint(Inverse(GetAccumulatedRenderTransform()), UE::Slate::CastToVector2f(AbsoluteCoordinate)));
	}

	/**
	 * Translates local coordinates into absolute coordinates
	 * 
	 * Absolute coordinates could be either desktop or window space depending on what space the root of the widget hierarchy is in.
	 * 
	 * @return  Absolute coordinates
	 */
	inline UE::Slate::FDeprecateVector2DResult LocalToAbsolute(UE::Slate::FDeprecateVector2DParameter LocalCoordinate) const
	{
		return UE::Slate::FDeprecateVector2DResult(TransformPoint(GetAccumulatedRenderTransform(), UE::Slate::CastToVector2f(LocalCoordinate)));
	}

	/**
	 * Translates the local coordinates into local coordinates that after being transformed into absolute space will be rounded
	 * to a whole number or approximately a whole number.  This is important for cases where you want to show a popup or a tooltip
	 * and not have the window start on a half pixel, which can cause the contents to jitter in relation to each other as the tooltip 
	 * or popup moves around.
	 */
	inline UE::Slate::FDeprecateVector2DResult LocalToRoundedLocal(UE::Slate::FDeprecateVector2DParameter LocalCoordinate) const
	{
		const FVector2f AbsoluteCoordinate = LocalToAbsolute(UE::Slate::CastToVector2f(LocalCoordinate));
		const FVector2f AbsoluteCoordinateRounded = FVector2f(FMath::RoundToFloat(AbsoluteCoordinate.X), FMath::RoundToFloat(AbsoluteCoordinate.Y));

		return AbsoluteToLocal(AbsoluteCoordinateRounded);
	}
	
	FORCEINLINE_DEBUGGABLE FSlateRect GetLayoutBoundingRect() const
	{
		return GetLayoutBoundingRect(FSlateRect(FVector2f(0.0f, 0.0f), FVector2f(Size)));
	}

	FORCEINLINE_DEBUGGABLE FSlateRect GetLayoutBoundingRect(const FMargin& LocalSpaceExtendBy) const
	{
		return GetLayoutBoundingRect(FSlateRect(FVector2f::ZeroVector, FVector2f(Size)).ExtendBy(LocalSpaceExtendBy));
	}

	FORCEINLINE_DEBUGGABLE FSlateRect GetLayoutBoundingRect(const FSlateRect& LocalSpaceRect) const
	{
		return TransformRect(GetAccumulatedLayoutTransform(), FSlateRotatedRect(LocalSpaceRect)).ToBoundingRect();
	}

	FORCEINLINE_DEBUGGABLE FSlateRect GetRenderBoundingRect() const
	{
		return GetRenderBoundingRect(FSlateRect(FVector2f(0.0f, 0.0f), FVector2f(Size)));
	}

	FORCEINLINE_DEBUGGABLE FSlateRect GetRenderBoundingRect(const FMargin& LocalSpaceExtendBy) const
	{
		return GetRenderBoundingRect(FSlateRect(FVector2f::ZeroVector, FVector2f(Size)).ExtendBy(LocalSpaceExtendBy));
	}

	FORCEINLINE_DEBUGGABLE FSlateRect GetRenderBoundingRect(const FSlateRect& LocalSpaceRect) const
	{
		return TransformRect(GetAccumulatedRenderTransform(), FSlateRotatedRect(LocalSpaceRect)).ToBoundingRect();
	}
	
	/** @return A String representation of this Geometry */
	SLATECORE_API FString ToString() const;

	/** 
	 * !!! DEPRECATED !!! This legacy function does not account for render transforms.
	 * 
	 * Absolute coordinates could be either desktop or window space depending on what space the root of the widget hierarchy is in.
	 *
	 * @return the size of the geometry in absolute space */
	inline UE::Slate::FDeprecateVector2DResult GetDrawSize() const
	{
		return UE::Slate::FDeprecateVector2DResult(TransformVector(GetAccumulatedLayoutTransform(), FVector2f(Size)));
	}

	/** @return the size of the geometry in local space. */
	inline UE::Slate::FDeprecateVector2DResult GetLocalSize() const { return Size; }

	/** @return the accumulated render transform. Shouldn't be needed in general. */
	inline const FSlateRenderTransform& GetAccumulatedRenderTransform() const { return AccumulatedRenderTransform; }

	/** @return the accumulated layout transform. Shouldn't be needed in general. */
	inline FSlateLayoutTransform GetAccumulatedLayoutTransform() const
	{
		return FSlateLayoutTransform(Scale, AbsolutePosition);
	}

	/**
	 * Special case method to append a layout transform to a geometry.
	 * This is used in cases where the FGeometry was arranged in window space
	 * and we need to add the root desktop translation.
	 * If you find yourself wanting to use this function, ask someone if there's a better way.
	 * 
	 * @param LayoutTransform	An additional layout transform to append to this geometry.
	 */
	FORCEINLINE_DEBUGGABLE void AppendTransform(const FSlateLayoutTransform& LayoutTransform)
	{
		FSlateLayoutTransform AccumulatedLayoutTransform = ::Concatenate(GetAccumulatedLayoutTransform(), LayoutTransform);
		AccumulatedRenderTransform = ::Concatenate(AccumulatedRenderTransform, LayoutTransform);
		const_cast<FVector2f&>( AbsolutePosition ) = FVector2f(AccumulatedLayoutTransform.GetTranslation());
		const_cast<float&>( Scale ) = AccumulatedLayoutTransform.GetScale();
	}

	/**
	 * Get the absolute position in render space.
	 */
	inline UE::Slate::FDeprecateVector2DResult GetAbsolutePosition() const
	{
		return UE::Slate::FDeprecateVector2DResult(AccumulatedRenderTransform.TransformPoint(FVector2f::ZeroVector));
	}

	/**
	 * Get the absolute size of the geometry in render space.
	 */
	inline UE::Slate::FDeprecateVector2DResult GetAbsoluteSize() const
	{
		return UE::Slate::FDeprecateVector2DResult(AccumulatedRenderTransform.TransformVector(FVector2f(GetLocalSize())));
	}

	/**
	 * Get the absolute position on the surface of the geometry using normalized coordinates.
	 *   (0,0) - upper left
	 *   (1,1) - bottom right
	 *
	 * Example: Say you wanted to know the center of the widget in absolute space, GetAbsolutePositionAtCoordinates(FVector2f(0.5f, 0.5f));
	 */
	inline UE::Slate::FDeprecateVector2DResult GetAbsolutePositionAtCoordinates(const UE::Slate::FDeprecateVector2DParameter& NormalCoordinates) const
	{
		return UE::Slate::FDeprecateVector2DResult(AccumulatedRenderTransform.TransformPoint(FVector2f(NormalCoordinates) * FVector2f(GetLocalSize())));
	}

	/**
	 * Get the local position on the surface of the geometry using normalized coordinates.
	 *   (0,0) - upper left
	 *   (1,1) - bottom right
	 *
	 * Example: Say you wanted to know the center of the widget in local space, GetLocalPositionAtCoordinates(FVector2f(0.5f, 0.5f));
	 */
	inline UE::Slate::FDeprecateVector2DResult GetLocalPositionAtCoordinates(const UE::Slate::FDeprecateVector2DParameter& NormalCoordinates) const
	{
		return UE::Slate::FDeprecateVector2DResult(Position + (FVector2f(NormalCoordinates) * GetLocalSize()));
	}

	bool HasRenderTransform() const { return bHasRenderTransform; }

public:
	/** 
	 * 
	 * !!! DEPRECATED !!! Use GetLocalSize() accessor instead of directly accessing public members.
	 * 
	 *	   This member has been made const to prevent mutation.
	 *	   There is no way to easily detect mutation of public members, thus no way to update the render transforms when they are modified.
	 * 
	 * 
	 * Size of the geometry in local space.
	 */
	const FDeprecateSlateVector2D /*Local*/Size;

	/** 
	 * !!! DEPRECATED !!! These legacy public members should ideally not be referenced, as they do not account for the render transform.
	 *     FGeometry manipulation should be done in local space as much as possible so logic can be done in aligned local space, but 
	 *     still support arbitrary render transforms.
	 * 
	 *	   This member has been made const to prevent mutation, which would also break render transforms, which are computed during construction.
	 *	   There is no way to easily detect mutation of public members, thus no way to update the render transforms when they are modified.
	 * 
	 * Scale in absolute space. Equivalent to the scale of the accumulated layout transform. 
	 * 
	 * Absolute coordinates could be either desktop or window space depending on what space the root of the widget hierarchy is in.
	 */
	const float /*Absolute*/Scale;

	/** 
	 * !!! DEPRECATED !!! These legacy public members should ideally not be referenced, as they do not account for the render transform.
	 *     FGeometry manipulation should be done in local space as much as possible so logic can be done in aligned local space, but 
	 *     still support arbitrary render transforms.
	 * 
	 *	   This member has been made const to prevent mutation, which would also break render transforms, which are computed during construction.
	 *	   There is no way to easily detect mutation of public members, thus no way to update the render transforms when they are modified.
	 * 
	 * Position in absolute space. Equivalent to the translation of the accumulated layout transform. 
	 * 
	 * Absolute coordinates could be either desktop or window space depending on what space the root of the widget hierarchy is in.
	 */
	const FVector2f AbsolutePosition = FVector2f::ZeroVector;	

	/** 
	 * !!! DEPRECATED !!! 
	 * 
	 * Position of the geometry with respect to its parent in local space. Equivalent to the translation portion of the Local->Parent layout transform.
	 * If you know your children have no additional scale applied to them, you can use this as the Local->Parent layout transform. If your children
	 * DO have additional scale applied, there is no way to determine the actual Local->Parent layout transform, since the scale is accumulated.
	 */
	const FVector2f /*Local*/Position = FVector2f::ZeroVector;

private:

	/** Concatenated Render Transform. Actual transform used for rendering.
	 * Formed as: Concat(LocalRenderTransform, LocalLayoutTransform, Parent->AccumulatedRenderTransform) 
	 * 
	 * For rendering, absolute coordinates will always be in window space (relative to the root window).
	 */
	FSlateRenderTransform AccumulatedRenderTransform;

	/**  */
	const uint8 bHasRenderTransform : 1;
};


template <> struct TIsPODType<FGeometry> { enum { Value = true }; };
