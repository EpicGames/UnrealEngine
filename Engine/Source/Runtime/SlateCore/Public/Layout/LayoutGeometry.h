// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/SlateRect.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Types/SlateVector2.h"

class FLayoutGeometry
{
public:
	FLayoutGeometry()
		: LocalSize(1.0f, 1.0f)
	{
	}

	explicit FLayoutGeometry(const FSlateLayoutTransform& InLocalToParent, const UE::Slate::FDeprecateVector2DParameter& SizeInLocalSpace)
		: LocalToParent(InLocalToParent)
		, LocalSize(FVector2f(SizeInLocalSpace))
	{
	}

	const FSlateLayoutTransform& GetLocalToParentTransform() const 
	{ 
		return LocalToParent; 
	}

	UE::Slate::FDeprecateVector2DResult GetSizeInLocalSpace() const
	{
		return UE::Slate::FDeprecateVector2DResult(LocalSize);
	}

	UE::Slate::FDeprecateVector2DResult GetSizeInParentSpace() const
	{
		return UE::Slate::FDeprecateVector2DResult(TransformVector(LocalToParent, LocalSize));
	}

	UE::Slate::FDeprecateVector2DResult GetOffsetInParentSpace() const
	{
		return UE::Slate::FDeprecateVector2DResult(LocalToParent.GetTranslation());
	}

	FSlateRect GetRectInLocalSpace() const
	{
		return FSlateRect(FVector2f(0.0f, 0.0f), LocalSize);
	}

	FSlateRect GetRectInParentSpace() const
	{
		return TransformRect(LocalToParent, GetRectInLocalSpace());
	}

private:
	FSlateLayoutTransform LocalToParent;
	FVector2f LocalSize;
};

