// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ArrangedChildren.h"
#include "FlowDirection.h"
#include "Layout/Children.h"
#include "Margin.h"
#include "SlateRect.h"
#include "Templates/UnrealTypeTraits.h"
#include "Types/SlateStructs.h"
#include "Visibility.h"
#include "Widgets/SBoxPanel.h"

struct AlignmentArrangeResult
{
	AlignmentArrangeResult( float InOffset, float InSize )
	: Offset(InOffset)
	, Size(InSize)
	{
	}
	
	float Offset;
	float Size;
};

/**
 *	Represents a single slot, used to perform arrangement operations.
 *  Allows layout with an arbitrary child order that can map back to a sequential array.
 */
struct FSlotProxy
{
	FSlotProxy() = default;

	/** Constructs a SlotProxy for the given slot index and FSlot. */
	template <typename SlotType>
	explicit FSlotProxy(const int32 InSlotIndex, const SlotType& InSlot)
		: FSlotProxy(InSlot.GetWidget())
	{
		SlotIndex = InSlotIndex;
		Padding = InSlot.GetPadding();
		MinSize = InSlot.GetMinSize();
		MaxSize = InSlot.GetMaxSize();
		SizeParam.SizeRule = InSlot.GetSizeRule();
		SizeParam.Value = InSlot.GetSizeValue();
		SizeParam.ShrinkValue = InSlot.GetShrinkSizeValue();
		HorizontalAlignment = InSlot.GetHorizontalAlignment();
		VerticalAlignment = InSlot.GetVerticalAlignment();
	}

	/** Performs the same functionality as FGeometry::MakeChild, without the need for the original Widget reference. */
	SLATECORE_API FGeometry MakeGeometry(const FGeometry& InParentGeometry, const FVector2f& InChildOffset, const FVector2f& InLocalSize) const;

	/**
	 * Applies the given slot's values to this proxy.
	 * Will return true if any values have changed from those stored.
	 * If CompareArranged is true, the "changed" flag is only true if the new DesiredSize doesn't match the previous ArrangedSize, rather than previous DesiredSize.
	 */
	template <typename SlotType, bool CompareArranged = false>
	bool UpdateFromSlot(const int32 InSlotIndex, const SlotType& InSlot)
	{
		bool bAnyValueChanged = UpdateFromSlot(InSlot.GetWidget());

		const FVector2f WidgetArrangedSize = InSlot.GetWidget()->GetTickSpaceGeometry().GetLocalSize();
		const FVector2f WidgetDesiredSize = InSlot.GetWidget()->GetDesiredSize();

		if constexpr (CompareArranged)
		{
			bAnyValueChanged = bAnyValueChanged || (!ArrangedSize.Equals(WidgetArrangedSize));
		}
		else
		{
			bAnyValueChanged = bAnyValueChanged || (!DesiredSize.Equals(WidgetDesiredSize));
		}

		ArrangedSize = WidgetArrangedSize;
		DesiredSize = WidgetDesiredSize;

		bAnyValueChanged = bAnyValueChanged || (SlotIndex != InSlotIndex);
		SlotIndex = InSlotIndex;

		bAnyValueChanged = bAnyValueChanged || (Padding != InSlot.GetPadding());
		Padding = InSlot.GetPadding();

		const float SlotMinSize = InSlot.GetMinSize();
		bAnyValueChanged = bAnyValueChanged || !FMath::IsNearlyEqual(MinSize, SlotMinSize);
		MinSize = SlotMinSize;

		const float SlotMaxSize = InSlot.GetMaxSize();
		bAnyValueChanged = bAnyValueChanged || !FMath::IsNearlyEqual(MaxSize, SlotMaxSize);
		MaxSize = SlotMaxSize;

		bAnyValueChanged = bAnyValueChanged || (SizeParam.SizeRule != InSlot.GetSizeRule());
		SizeParam.SizeRule = InSlot.GetSizeRule();

		const float SlotSizeValue = InSlot.GetSizeValue();
		bAnyValueChanged = bAnyValueChanged || !FMath::IsNearlyEqual(SizeParam.Value.Get(), SlotSizeValue);
		SizeParam.Value = SlotSizeValue;

		const float SlotShrinkSizeValue = InSlot.GetShrinkSizeValue();
		bAnyValueChanged = bAnyValueChanged || !FMath::IsNearlyEqual(SizeParam.ShrinkValue.Get(), SlotShrinkSizeValue);
		SizeParam.ShrinkValue = SlotShrinkSizeValue;

		bAnyValueChanged = bAnyValueChanged || (HorizontalAlignment != InSlot.GetHorizontalAlignment());
		HorizontalAlignment = InSlot.GetHorizontalAlignment();

		bAnyValueChanged = bAnyValueChanged || (VerticalAlignment != InSlot.GetVerticalAlignment());
		VerticalAlignment = InSlot.GetVerticalAlignment();

		return bAnyValueChanged;
	}

	/** Equality tested against the slot index. This makes it simple to find a slot by its index, even if the slot proxies are differently ordered. */
	bool operator==(const int32 InSlotIndex) const
	{
		return SlotIndex == InSlotIndex;
	}

	/** Default comparison operator for sorting by slot index. */
	bool operator<(const FSlotProxy& InOtherSlot) const
	{
		return SlotIndex < InOtherSlot.SlotIndex;
	}

private:
	SLATECORE_API explicit FSlotProxy(const TSharedRef<SWidget>& InWidget);

	/** Applies the given slot widget's values to this proxy. Will return true if any values have changed from those stored. */
	SLATECORE_API bool UpdateFromSlot(const TSharedRef<SWidget>& InWidget);

public:
	/** The original index of the slot represented by this proxy. */
	int32 SlotIndex = INDEX_NONE;

	/** Padding margin. */
	FMargin Padding = FMargin(0);

	/** Contained widget's desired size. */
	FVector2f DesiredSize = FVector2f::ZeroVector;

	/** Contained widget's arranged size (as it was last calculated). */
	FVector2f ArrangedSize = FVector2f::ZeroVector;

	/** Sizing option, ie. Auto, Fill. */
	FSizeParam SizeParam = FAuto();

	/** Min Size, if any. */
	float MinSize = 0.0f;

	/** Max Size, if any. */
	float MaxSize = 0.0f;

	/** Current visibility state. */
	EVisibility Visibility = EVisibility::Visible;

	/** Horizontal slot alignment. */
	EHorizontalAlignment HorizontalAlignment = HAlign_Fill;

	/** Vertical slot alignment. */
	EVerticalAlignment VerticalAlignment = VAlign_Fill;

	/** Last calculated RenderTransform. */
	TOptional<FSlateRenderTransform> RenderTransform;

	/** Corresponds with the RenderTransform. */
	FVector2f RenderTransformPivot = FVector2f::ZeroVector;

	/** Here for convenience and backwards compatibility. Not always valid. */
	TSharedPtr<SWidget> Widget = nullptr;
};

/** A custom iterator for FSlotProxy implementations, with behavior similar to TPanelChildrenConstIterator. */
template <typename SlotProxyType
	UE_REQUIRES(std::is_base_of_v<FSlotProxy, std::decay_t<SlotProxyType>>)>
class TSlotProxyIterator
{
public:
	TSlotProxyIterator(const TArrayView<SlotProxyType>& InContainer, EFlowDirection InLayoutFlow)
		: Container(InContainer)
		, LayoutFlow(InLayoutFlow)
	{
		Reset();
	}

	TSlotProxyIterator(const TArrayView<SlotProxyType>& InContainer, EOrientation InOrientation, EFlowDirection InLayoutFlow)
		: Container(InContainer)
		, LayoutFlow(InOrientation == Orient_Vertical ? EFlowDirection::LeftToRight : InLayoutFlow)
	{
		Reset();
	}

	/** Advances iterator to the next element in the container. */
	TSlotProxyIterator& operator++()
	{
		switch (LayoutFlow)
		{
		default:
		case EFlowDirection::LeftToRight:
			++Index;
			break;
		case EFlowDirection::RightToLeft:
			--Index;
			break;
		}

		return *this;
	}

	/** Moves iterator to the previous element in the container. */
	TSlotProxyIterator& operator--()
	{
		switch (LayoutFlow)
		{
		default:
		case EFlowDirection::LeftToRight:
			--Index;
			break;
		case EFlowDirection::RightToLeft:
			++Index;
			break;
		}

		return *this;
	}

	const SlotProxyType& operator* () const
	{
		return Container[Index];
	}

	SlotProxyType& operator* ()
	{
		return Container[Index];
	}

	const SlotProxyType* operator->() const
	{
		return &Container[Index];
	}

	SlotProxyType* operator->()
	{
		return &Container[Index];
	}

	/** conversion to "bool" returning true if the iterator has not reached the last element. */
	inline explicit operator bool() const
	{
		return Container.IsValidIndex(Index);
	}

	/** Returns an index to the current element. */
	int32 GetIndex() const
	{
		return Index;
	}

	/** Resets the iterator to the first element. */
	void Reset()
	{
		switch (LayoutFlow)
		{
		default:
		case EFlowDirection::LeftToRight:
			Index = 0;
			break;
		case EFlowDirection::RightToLeft:
			Index = Container.Num() - 1;
			break;
		}
	}

	/** Sets iterator to the last element. */
	void SetToEnd()
	{
		switch (LayoutFlow)
		{
		default:
		case EFlowDirection::LeftToRight:
			Index = Container.Num() - 1;
			break;
		case EFlowDirection::RightToLeft:
			Index = 0;
			break;
		}
	}

private:

	const TArrayView<SlotProxyType>& Container;
	int32 Index;
	EFlowDirection LayoutFlow;
};

namespace UE::Slate
{
	/**
	 * A wrapper for accessing slot parameters for a given SlotType. Not all SlotTypes support all parameters.
	 * Optional default values are used when the wrapped SlotType cannot provide the requested property (ie. FSlot doesn't provide it's index).
	 */
	template <typename SlotType, typename = void>
	struct TSlotAccessor
	{
		/** Get the index of the slot within it's parent container. */
		int32 GetIndex(const SlotType& InSlot, const int32 InDefaultValue = INDEX_NONE) const;

		/** Get the contained widget. This should always be valid, and return SNullWidget by default. */
		TSharedRef<SWidget> GetWidget(const SlotType& InSlot) const;

		/** Get the current visibility state of the slot. */
		EVisibility GetVisibility(const SlotType& InSlot, const EVisibility InDefaultValue = EVisibility::Visible) const;

		/** Get the slot padding. */
		FMargin GetPadding(const SlotType& InSlot, const FMargin& InDefaultValue = FMargin()) const;

		/** Get the slot's desired size as it was last calculated. */
		FVector2f GetDesiredSize(const SlotType& InSlot, const FVector2f& InDefaultValue = FVector2f::ZeroVector) const;

		/** Get the slot's arranged size as it was last calculated. */
		FVector2f GetArrangedSize(const SlotType& InSlot, const FVector2f& InDefaultValue = FVector2f::ZeroVector) const;

		/** Get the slot's applied size rule, ie. Auto, Fill. */
		FSizeParam::ESizeRule GetSizeRule(const SlotType& InSlot, const FSizeParam::ESizeRule InDefaultValue = FSizeParam::SizeRule_Auto) const;

		/** Get the slot's size value, applicable if the size rule is Fill or FillContent. */
		float GetSizeValue(const SlotType& InSlot, const float& InDefaultValue = 0.0f) const;

		/** Get the slot's shrink size value, applicable if the size rule is FillContent. */
		float GetShrinkSizeValue(const SlotType& InSlot, const float& InDefaultValue = 0.0f) const;

		/** Get the slot's minimum size, if set. A value of 0.0f indicates this is not set. */
		float GetMinSize(const SlotType& InSlot, const float& InDefaultValue = 0.0f) const;

		/** Get the slot's maximum size, if set. A value of 0.0f indicates this is not set. */
        float GetMaxSize(const SlotType& InSlot, const float& InDefaultValue = 0.0f) const;

		/** Makes an arranged widget for the given Slot. This usually wraps FGeometry::MakeChild(). */
		FArrangedWidget MakeArrangedWidget(const SlotType& InSlot, const FGeometry& InAllottedGeometry, const FVector2f& InLocalOffset, const FVector2f& InLocalSize) const;
	};

	template <typename SlotType>
	struct TSlotAccessor<
		SlotType,
		std::enable_if_t<
			std::is_base_of_v<TBasicLayoutWidgetSlot<SlotType>, std::decay_t<SlotType>>
			&& std::is_base_of_v<TResizingWidgetSlotMixin<SlotType>, std::decay_t<SlotType>>>>
	{
		/** Returns the provided DefaultValue. FSlot itself doesn't store it's index. */
		int32 GetIndex(const SlotType& InSlot, const int32 InDefaultValue = INDEX_NONE) const
		{
			return InDefaultValue;
		}

		/** Get the contained widget. This should always be valid, and return SNullWidget by default. */
		TSharedRef<SWidget> GetWidget(const SlotType& InSlot) const
		{
			return InSlot.GetWidget();
		}

		/** Get the current visibility state of the slot. */
		EVisibility GetVisibility(const SlotType& InSlot, const EVisibility InDefaultValue = EVisibility::Visible) const
		{
			return InSlot.GetWidget()->GetVisibility();
		}

		/** Get the slot padding. */
		FMargin GetPadding(const SlotType& InSlot, const FMargin& InDefaultValue = FMargin()) const
		{
			return InSlot.GetPadding();
		}

		/** Get the slot's desired size as it was last calculated. */
		FVector2f GetDesiredSize(const SlotType& InSlot, const FVector2f& InDefaultValue = FVector2f::ZeroVector) const
		{
			return InSlot.GetWidget()->GetDesiredSize();
		}

		/** Get the slot's arranged size as it was last calculated. */
		FVector2f GetArrangedSize(const SlotType& InSlot, const FVector2f& InDefaultValue = FVector2f::ZeroVector) const
		{
			return InSlot.GetWidget()->GetTickSpaceGeometry().Size;
		}

		/** Get the slot's applied size rule, ie. Auto, Fill. */
		FSizeParam::ESizeRule GetSizeRule(const SlotType& InSlot, const FSizeParam::ESizeRule InDefaultValue = FSizeParam::SizeRule_Auto) const
		{
			return InSlot.GetSizeRule();
		}

		/** Get the slot's size value, applicable if the size rule is Fill or FillContent. */
		float GetSizeValue(const SlotType& InSlot, const float& InDefaultValue = 0.0f) const
		{
			return InSlot.GetSizeValue();
		}

		/** Get the slot's shrink size value, applicable if the size rule is FillContent. */
		float GetShrinkSizeValue(const SlotType& InSlot, const float& InDefaultValue = 0.0f) const
		{
			return InSlot.GetShrinkSizeValue();
		}

		/** Get the slot's minimum size, if set. A value of 0.0f indicates this is not set. */
		float GetMinSize(const SlotType& InSlot, const float& InDefaultValue = 0.0f) const
		{
			return InSlot.GetMinSize();
		}

		/** Get the slot's maximum size, if set. A value of 0.0f indicates this is not set. */
		float GetMaxSize(const SlotType& InSlot, const float& InDefaultValue = 0.0f) const
		{
			return InSlot.GetMaxSize();
		}

		/** Makes an arranged widget for the given Slot. This usually wraps FGeometry::MakeChild(). */
		FArrangedWidget MakeArrangedWidget(const SlotType& InSlot, const FGeometry& InAllottedGeometry, const FVector2f& InLocalOffset, const FVector2f& InLocalSize) const
		{
			return InAllottedGeometry.MakeChild(
				// The child widget being arranged
				GetWidget(InSlot),
				// Child's local position (i.e. position within parent)
				InLocalOffset,
				// Child's size
				InLocalSize
			);
		}
	};

	/** Specialization for FSlotProxy (and derived). */
	template <typename SlotType>
	struct TSlotAccessor<
		SlotType,
		std::enable_if_t<std::is_base_of_v<FSlotProxy, std::decay_t<SlotType>>>>
	{
		/** Get the index of the slot within it's parent container. */
		int32 GetIndex(const FSlotProxy& InSlot, const int32 InDefaultValue = INDEX_NONE) const
		{
			return InSlot.SlotIndex;
		}

		/** Get the contained widget. This should always be valid, and return SNullWidget by default. */
		TSharedRef<SWidget> GetWidget(const FSlotProxy& InSlot) const
		{
			return InSlot.Widget.IsValid() ? InSlot.Widget.ToSharedRef() : SNullWidget::NullWidget;
		}

		/** Get the current visibility state of the slot. */
		EVisibility GetVisibility(const FSlotProxy& InSlot, const EVisibility InDefaultValue = EVisibility::Visible) const
		{
			return InSlot.Visibility;
		}

		/** Get the slot padding. */
		FMargin GetPadding(const FSlotProxy& InSlot, const FMargin& InDefaultValue = FMargin()) const
		{
			return InSlot.Padding;
		}

		/** Get the slot's desired size as it was last calculated. */
		FVector2f GetDesiredSize(const FSlotProxy& InSlot, const FVector2f& InDefaultValue = FVector2f::ZeroVector) const
		{
			return InSlot.DesiredSize;
		}

		/** Get the slot's arranged size as it was last calculated. */
		FVector2f GetArrangedSize(const FSlotProxy& InSlot, const FVector2f& InDefaultValue = FVector2f::ZeroVector) const
		{
			return InSlot.ArrangedSize;
		}

		/** Get the slot's applied size rule, ie. Auto, Fill. */
		FSizeParam::ESizeRule GetSizeRule(const FSlotProxy& InSlot, const FSizeParam::ESizeRule InDefaultValue = FSizeParam::SizeRule_Auto) const
		{
			return InSlot.SizeParam.SizeRule;
		}

		/** Get the slot's size value, applicable if the size rule is Fill or FillContent. */
		float GetSizeValue(const FSlotProxy& InSlot, const float& InDefaultValue = 0.0f) const
		{
			return InSlot.SizeParam.Value.Get();
		}

		/** Get the slot's shrink size value, applicable if the size rule is FillContent. */
		float GetShrinkSizeValue(const FSlotProxy& InSlot, const float& InDefaultValue = 0.0f) const
		{
			return InSlot.SizeParam.ShrinkValue.Get();
		}

		/** Get the slot's minimum size, if set. A value of 0.0f indicates this is not set. */
		FVector2f::FReal GetMinSize(const FSlotProxy& InSlot, const FVector2f::FReal& InDefaultValue = 0.0f) const
		{
			return InSlot.MinSize;
		}

		/** Get the slot's maximum size, if set. A value of 0.0f indicates this is not set. */
		FVector2f::FReal GetMaxSize(const FSlotProxy& InSlot, const FVector2f::FReal& InDefaultValue = 0.0f) const
		{
			return InSlot.MaxSize;
		}

		/** Makes an arranged widget for the given Slot. This usually wraps FGeometry::MakeChild(). */
		FArrangedWidget MakeArrangedWidget(const FSlotProxy& InSlot, const FGeometry& InAllottedGeometry, const FVector2f& InLocalOffset, const FVector2f& InLocalSize) const
		{
			return InAllottedGeometry.MakeChild(
				GetWidget(InSlot),
				InLocalOffset,
				InLocalSize);
		}
	};

	template <typename SlotType, typename = void>
	struct TSlotIterator
	{
	};

	template <typename SlotType>
	struct TSlotIterator<
		SlotType,
		std::enable_if_t<std::is_base_of_v<FSlotBase, std::decay_t<SlotType>>>>
	{
		using Type = TPanelChildrenConstIterator<SlotType>;
	};

	template <typename SlotType>
	struct TSlotIterator<
		SlotType,
		std::enable_if_t<std::is_base_of_v<FSlotProxy, std::decay_t<SlotType>>>>
	{
		using Type = TSlotProxyIterator<SlotType>;
	};
};

namespace ArrangeUtils
{
	/** Gets the alignment of an axis-agnostic int32 so that we can do alignment on an axis without caring about its orientation */
	template<EOrientation Orientation>
	struct GetChildAlignment
	{
		template<typename SlotType>
		static int32 AsInt(EFlowDirection InFlowDirection, const SlotType& InSlot );
	};

	template<>
	struct GetChildAlignment<Orient_Horizontal>
	{
		template<typename SlotType>
		static int32 AsInt(EFlowDirection InFlowDirection, const SlotType& InSlot )
		{
			EHorizontalAlignment HorizontalAlignment;
			if constexpr (std::is_base_of_v<FSlotProxy, std::decay_t<SlotType>>)
			{
				HorizontalAlignment = InSlot.HorizontalAlignment;
			}
			else
			{
				HorizontalAlignment = InSlot.GetHorizontalAlignment();
			}

			switch (InFlowDirection)
			{
			default:
			case EFlowDirection::LeftToRight:
				return static_cast<int32>(HorizontalAlignment);
			case EFlowDirection::RightToLeft:
				switch (HorizontalAlignment)
				{
				case HAlign_Left:
					return static_cast<int32>(HAlign_Right);
				case HAlign_Right:
					return static_cast<int32>(HAlign_Left);
				default:
					return static_cast<int32>(HorizontalAlignment);
				}
			}
		}
	};

	template<>
	struct GetChildAlignment<Orient_Vertical>
	{
		template<typename SlotType>
		static int32 AsInt(EFlowDirection InFlowDirection, const SlotType& InSlot )
		{
			EVerticalAlignment VerticalAlignment;
			if constexpr (std::is_base_of_v<FSlotProxy, std::decay_t<SlotType>>)
			{
				VerticalAlignment = InSlot.VerticalAlignment;
			}
			else
			{
				VerticalAlignment = InSlot.GetVerticalAlignment();
			}
			
			// InFlowDirection has no effect in vertical orientations.
			return static_cast<int32>(VerticalAlignment);
		}
	};

	/**
	 * Same as AlignChild but force the alignment to be fill.
	 * @return  Offset and Size of widget
	 */
	template<EOrientation Orientation>
	static AlignmentArrangeResult AlignFill(float AllottedSize, const FMargin& SlotPadding, const float ContentScale = 1.0f)
	{
		const FMargin& Margin = SlotPadding;
		const float TotalMargin = Margin.GetTotalSpaceAlong<Orientation>();
		const float MarginPre = (Orientation == Orient_Horizontal) ? Margin.Left : Margin.Top;
		return AlignmentArrangeResult(MarginPre, FMath::Max((AllottedSize - TotalMargin) * ContentScale, 0.f));
	}

	/**
	 * Same as AlignChild but force the alignment to be center.
	 * @return  Offset and Size of widget
	 */
	template<EOrientation Orientation>
	static AlignmentArrangeResult AlignCenter(float AllottedSize, float ChildDesiredSize, const FMargin& SlotPadding, const float ContentScale = 1.0f, bool bClampToParent = true)
	{
		const FMargin& Margin = SlotPadding;
		const float TotalMargin = Margin.GetTotalSpaceAlong<Orientation>();
		const float MarginPre = (Orientation == Orient_Horizontal) ? Margin.Left : Margin.Top;
		const float MarginPost = (Orientation == Orient_Horizontal) ? Margin.Right : Margin.Bottom;
		const float ChildSize = FMath::Max((bClampToParent ? FMath::Min(ChildDesiredSize, AllottedSize - TotalMargin) : ChildDesiredSize), 0.f);
		return AlignmentArrangeResult((AllottedSize - ChildSize) / 2.0f + MarginPre - MarginPost, ChildSize);
	}
}

static FMargin LayoutPaddingWithFlow(EFlowDirection InLayoutFlow, const FMargin& InPadding);

/**
 * Helper method to BoxPanel::ArrangeChildren.
 * 
 * @param AllottedSize         The size available to arrange the widget along the given orientation
 * @param ChildToArrange       The widget and associated layout information
 * @param SlotPadding          The padding to when aligning the child
 * @param ContentScale         The scale to apply to the child before aligning it.
 * @param bClampToParent       If true the child's size is clamped to the allotted size before alignment occurs, if false, the child's desired size is used, even if larger than the allotted size.
 * 
 * @return  Offset and Size of widget
 */
template<EOrientation Orientation, typename SlotType>
static AlignmentArrangeResult AlignChild(EFlowDirection InLayoutFlow, float AllottedSize, float ChildDesiredSize, const SlotType& ChildToArrange, const FMargin& SlotPadding, const float& ContentScale = 1.0f, bool bClampToParent = true)
{
	const FMargin& Margin = SlotPadding;
	const float TotalMargin = Margin.GetTotalSpaceAlong<Orientation>();
	const float MarginPre = ( Orientation == Orient_Horizontal ) ? Margin.Left : Margin.Top;
	const float MarginPost = ( Orientation == Orient_Horizontal ) ? Margin.Right : Margin.Bottom;

	const int32 Alignment = ArrangeUtils::GetChildAlignment<Orientation>::AsInt(InLayoutFlow, ChildToArrange);

	switch (Alignment)
	{
	case HAlign_Fill:
		return AlignmentArrangeResult(MarginPre, FMath::Max((AllottedSize - TotalMargin) * ContentScale, 0.f));
	}
	
	const float ChildSize = FMath::Max((bClampToParent ? FMath::Min(ChildDesiredSize, AllottedSize - TotalMargin) : ChildDesiredSize), 0.f);

	switch( Alignment )
	{
	case HAlign_Left: // same as Align_Top
		return AlignmentArrangeResult(MarginPre, ChildSize);
	case HAlign_Center:
		return AlignmentArrangeResult(( AllottedSize - ChildSize ) / 2.0f + MarginPre - MarginPost, ChildSize);
	case HAlign_Right: // same as Align_Bottom		
		return AlignmentArrangeResult(AllottedSize - ChildSize - MarginPost, ChildSize);
	}

	// Same as Fill
	return AlignmentArrangeResult(MarginPre, FMath::Max(( AllottedSize - TotalMargin ) * ContentScale, 0.f));
}

template<EOrientation Orientation, typename SlotType>
static AlignmentArrangeResult AlignChild(float AllottedSize, float ChildDesiredSize, const SlotType& ChildToArrange, const FMargin& SlotPadding, const float& ContentScale = 1.0f, bool bClampToParent = true)
{
	return AlignChild<Orientation, SlotType>(EFlowDirection::LeftToRight, AllottedSize, ChildDesiredSize, ChildToArrange, SlotPadding, ContentScale, bClampToParent);
}

template<EOrientation Orientation, typename SlotType>
static AlignmentArrangeResult AlignChild(EFlowDirection InLayoutFlow, float AllottedSize, const SlotType& ChildToArrange, const FMargin& SlotPadding, const float& ContentScale = 1.0f, bool bClampToParent = true)
{
	const FMargin& Margin = SlotPadding;
	const float TotalMargin = Margin.GetTotalSpaceAlong<Orientation>();
	const float MarginPre = ( Orientation == Orient_Horizontal ) ? Margin.Left : Margin.Top;
	const float MarginPost = ( Orientation == Orient_Horizontal ) ? Margin.Right : Margin.Bottom;

	const int32 Alignment = ArrangeUtils::GetChildAlignment<Orientation>::AsInt(InLayoutFlow, ChildToArrange);

	switch (Alignment)
	{
	case HAlign_Fill:
		return AlignmentArrangeResult(MarginPre, FMath::Max((AllottedSize - TotalMargin) * ContentScale, 0.f));
	}

	float ChildDesiredSize = 0.0f;
	if constexpr (std::is_base_of_v<FSlotProxy, std::decay_t<SlotType>>)
	{
		ChildDesiredSize = ( Orientation == Orient_Horizontal )
		? ( ChildToArrange.DesiredSize.X * ContentScale )
		: ( ChildToArrange.DesiredSize.Y * ContentScale );
	}
	else
	{
		ChildDesiredSize = ( Orientation == Orient_Horizontal )
		? ( ChildToArrange.GetWidget()->GetDesiredSize().X * ContentScale )
		: ( ChildToArrange.GetWidget()->GetDesiredSize().Y * ContentScale );
	}

	const float ChildSize = FMath::Max((bClampToParent ? FMath::Min(ChildDesiredSize, AllottedSize - TotalMargin) : ChildDesiredSize), 0.f);

	switch ( Alignment )
	{
	case HAlign_Left: // same as Align_Top
		return AlignmentArrangeResult(MarginPre, ChildSize);
	case HAlign_Center:
		return AlignmentArrangeResult(( AllottedSize - ChildSize ) / 2.0f + MarginPre - MarginPost, ChildSize);
	case HAlign_Right: // same as Align_Bottom		
		return AlignmentArrangeResult(AllottedSize - ChildSize - MarginPost, ChildSize);
	}

	// Same as Fill
	return AlignmentArrangeResult(MarginPre, FMath::Max((AllottedSize - TotalMargin) * ContentScale, 0.f));
}

template<EOrientation Orientation, typename SlotType>
static AlignmentArrangeResult AlignChild(float AllottedSize, const SlotType& ChildToArrange, const FMargin& SlotPadding, const float& ContentScale = 1.0f, bool bClampToParent = true)
{
	return AlignChild<Orientation, SlotType>(EFlowDirection::LeftToRight, AllottedSize, ChildToArrange, SlotPadding, ContentScale, bClampToParent);
}


/**
 * Arrange a ChildSlot within the AllottedGeometry and populate ArrangedChildren with the arranged result.
 * The code makes certain assumptions about the type of ChildSlot.
 */
template<typename SlotType>
static void ArrangeSingleChild(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, const SlotType& ChildSlot, const TAttribute<FVector2D>& ContentScale)
{
	ArrangeSingleChild<SlotType>(EFlowDirection::LeftToRight, AllottedGeometry, ArrangedChildren, ChildSlot, ContentScale);
}

template<typename SlotType>
static void ArrangeSingleChild(EFlowDirection InFlowDirection, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, const SlotType& ChildSlot, const TAttribute<FVector2D>& ContentScale)
{
	const EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
	if ( ArrangedChildren.Accepts(ChildVisibility) )
	{
		const FVector2D ThisContentScale = ContentScale.Get();
		const FMargin SlotPadding(LayoutPaddingWithFlow(InFlowDirection, ChildSlot.GetPadding()));
		const AlignmentArrangeResult XResult = AlignChild<Orient_Horizontal>(InFlowDirection, AllottedGeometry.GetLocalSize().X, ChildSlot, SlotPadding, ThisContentScale.X);
		const AlignmentArrangeResult YResult = AlignChild<Orient_Vertical>(AllottedGeometry.GetLocalSize().Y, ChildSlot, SlotPadding, ThisContentScale.Y);

		ArrangedChildren.AddWidget( ChildVisibility, AllottedGeometry.MakeChild(
				ChildSlot.GetWidget(),
				FVector2D(XResult.Offset, YResult.Offset),
				FVector2D(XResult.Size, YResult.Size)
		) );
	}
}

template<typename SlotType>
static void ArrangeSingleChild(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, const SlotType& ChildSlot, const FVector2D& ContentScale)
{
	ArrangeSingleChild<SlotType>(EFlowDirection::LeftToRight, AllottedGeometry, ArrangedChildren, ChildSlot, ContentScale);
}

template<typename SlotType>
static void ArrangeSingleChild(EFlowDirection InFlowDirection, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, const SlotType& ChildSlot, const FVector2D& ContentScale)
{
	const EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
	if (ArrangedChildren.Accepts(ChildVisibility))
	{
		const FVector2D ThisContentScale = ContentScale;
		const FMargin SlotPadding(LayoutPaddingWithFlow(InFlowDirection, ChildSlot.GetPadding()));
		const AlignmentArrangeResult XResult = AlignChild<Orient_Horizontal>(InFlowDirection, AllottedGeometry.GetLocalSize().X, ChildSlot, SlotPadding, ThisContentScale.X);
		const AlignmentArrangeResult YResult = AlignChild<Orient_Vertical>(AllottedGeometry.GetLocalSize().Y, ChildSlot, SlotPadding, ThisContentScale.Y);

		ArrangedChildren.AddWidget(ChildVisibility, AllottedGeometry.MakeChild(
			ChildSlot.GetWidget(),
			FVector2f(XResult.Size, YResult.Size),
			FSlateLayoutTransform(FVector2f(XResult.Offset, YResult.Offset))
		));
	}
}

/** Note that this accepts Child Proxies rather than the children themselves, allowing layout operations independent of the child widget array itself.
 * InputChildrenType supports both TPanelChildren<FSlot>, TArrayView/TConstArrayView<FSlotProxy>
 * If the provided type is TArrayView (non-const), the SlotProxy ArrangedSize is written to.
 */
template <
	EOrientation Orientation,
	typename InputChildrenType,
	typename InputSlotType
	UE_REQUIRES(
		(std::is_base_of_v<TPanelChildren<InputSlotType>, std::decay_t<InputChildrenType>>
		|| std::is_same_v<TConstArrayView<InputSlotType>, std::decay_t<InputChildrenType>>
		|| std::is_same_v<TArrayView<InputSlotType>, std::decay_t<InputChildrenType>>))>
static void ArrangeChildrenInStack(EFlowDirection InLayoutFlow, const InputChildrenType& InChildren, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, float InOffset, bool bInAllowShrink, FVector2D& OutArrangedSize)
{
	if (InChildren.Num() == 0)
	{
		return;
	}

	// Allotted space will be given to fixed-size children first.
	// Remaining space will be proportionately divided between stretch children (SizeRule_Stretch and SizeRule_StretchContent)
	// based on their stretch coefficient.

	// Helper function to clamp to max size, if the constraint is set.
	auto ClampSize = [](const float Size, const float MinSize, const float MaxSize)
	{
		return FMath::Clamp(
			Size,
			MinSize > 0.0f ? MinSize : 0.0f,
			MaxSize > 0.0f ? MaxSize : std::numeric_limits<float>::max());
	};

	float GrowStretchCoefficientTotal = 0.0f;
	float ShrinkStretchCoefficientTotal = 0.0f;
	float FixedSizeTotal = 0.0f;
	float StretchSizeTotal = 0.0f;

	struct FStretchItem
	{
		// Size of the item
		float Size = 0.0f;
		// Initial size of the item
		float BasisSize = 0.0f;
		// Min size constraint of the item.
		float MinSize = 0.0f;
		// Max size constraint of the item.
		float MaxSize = 0.0f;
		// Stretch coefficient when the items are growing.
		float GrowStretchValue = 0.0f;
		// Stretch coefficient when the items are shrinking.
		float ShrinkStretchValue = 0.0f;
		// True if the constraints of the item has been satisfied.
		bool bFrozen = false;
		// Sizing rule for the item.
		FSizeParam::ESizeRule SizeRule = FSizeParam::ESizeRule::SizeRule_Auto;
	};
	TArray<FStretchItem, TInlineAllocator<16>> StretchItems;
	StretchItems.Init({}, InChildren.Num());

	bool bAnyChildVisible = false;
	bool bAnyStretchContentItems = false;
	bool bAnyStretchItems = false;

	UE::Slate::TSlotAccessor<std::decay_t<InputSlotType>> SlotAccessor;

	// Compute the sum of stretch coefficients (SizeRule_Stretch & SizeRule_StretchContent) and space required by fixed-size widgets (SizeRule_Auto),
	// as well as the total desired size.
	for (int32 ChildIndex = 0; ChildIndex < InChildren.Num(); ++ChildIndex)
	{
		const InputSlotType& CurChild = InChildren[ChildIndex];

		if (SlotAccessor.GetVisibility(CurChild) != EVisibility::Collapsed)
		{
			bAnyChildVisible = true;

			// All widgets contribute their margin to the fixed space requirement
			FixedSizeTotal += SlotAccessor.GetPadding(CurChild).template GetTotalSpaceAlong<Orientation>();

			FVector2f ChildDesiredSize = SlotAccessor.GetDesiredSize(CurChild);

			// Auto-sized children contribute their desired size to the fixed space requirement
			float ChildSize = (Orientation == Orient_Vertical)
				? ChildDesiredSize.Y
				: ChildDesiredSize.X;

			const float MinSize = SlotAccessor.GetMinSize(CurChild);
			const float MaxSize = SlotAccessor.GetMaxSize(CurChild);

			FStretchItem& Item = StretchItems[ChildIndex];
			Item.MinSize = MinSize;
			Item.MaxSize = MaxSize;
			Item.SizeRule = SlotAccessor.GetSizeRule(CurChild);

			// Clamp to the max size if it was specified
			ChildSize = ClampSize(ChildSize, MinSize, MaxSize);

			if (Item.SizeRule == FSizeParam::SizeRule_Stretch)
			{
				// Using same shrink and grow since otherwise the transition would be discontinuous as (reference) basis size is 0.
				Item.GrowStretchValue = SlotAccessor.GetSizeValue(CurChild);
				Item.ShrinkStretchValue = Item.GrowStretchValue;
				Item.Size = 0.0f;
				Item.BasisSize = 0.0f;

				// For stretch children we save sum up the stretch coefficients
				GrowStretchCoefficientTotal += Item.GrowStretchValue;
				ShrinkStretchCoefficientTotal += Item.ShrinkStretchValue;
				StretchSizeTotal += ChildSize;

				bAnyStretchItems = true;
			}
			else if (Item.SizeRule == FSizeParam::SizeRule_StretchContent)
			{
				// Allow separate values from grow and shrink, as the adjustment is relative to the child size.
				Item.GrowStretchValue = FMath::Max(0.f, SlotAccessor.GetSizeValue(CurChild));
				Item.ShrinkStretchValue = FMath::Max(0.f, SlotAccessor.GetShrinkSizeValue(CurChild));
				Item.Size = ChildSize;
				Item.BasisSize = ChildSize;

				// For sized stretch we sum to coefficients, but also treat the size as fixed.
				GrowStretchCoefficientTotal += Item.GrowStretchValue;
				ShrinkStretchCoefficientTotal += Item.ShrinkStretchValue;
				StretchSizeTotal += ChildSize;

				bAnyStretchContentItems = true;
			}
			else
			{
				FixedSizeTotal += ChildSize;

				Item.GrowStretchValue = 0.0f;
				Item.ShrinkStretchValue = 0.0f;
				Item.Size = ChildSize;
				Item.BasisSize = ChildSize;
			}
		}
	}

	if (!bAnyChildVisible)
	{
		return;
	}

	// When shrink is not allowed, we'll ensure to use all the space desired by the stretchable widgets.
	const float MinAvailableSpace = bInAllowShrink ? 0.0f : StretchSizeTotal;

	const float AllottedSize = Orientation == Orient_Vertical
		? AllottedGeometry.GetLocalSize().Y
		: AllottedGeometry.GetLocalSize().X;

	// The space available for SizeRule_Stretch and SizeRule_StretchContent widgets is any space that wasn't taken up by fixed-sized widgets.
	float AvailableSpace = FMath::Max(MinAvailableSpace, AllottedSize - FixedSizeTotal);

	// Apply SizeRule_Stretch.
	if (bAnyStretchItems && GrowStretchCoefficientTotal > 0.0f)
	{
		// Distribute available space amongst the SizeRule_Stretch items proportional to the their stretch coefficient.
		float UsedSpace = 0.0f;
		for (FStretchItem& Item : StretchItems)
		{
			if (Item.SizeRule == FSizeParam::SizeRule_Stretch)
			{
				// Stretch widgets get a fraction of the space remaining after all the fixed-space requirements are met.
				// Supporting only one stretch value since otherwise the transition would be discontinuous as (reference) basis size is 0.
				const float Size = AvailableSpace * Item.GrowStretchValue / GrowStretchCoefficientTotal;

				Item.Size = ClampSize(Size, Item.MinSize, Item.MaxSize);
				
				UsedSpace += Item.Size;
			}
		}
		AvailableSpace -= UsedSpace;
	}

	// Apply SizeRule_StretchContent.
	const bool bIsGrowing = AvailableSpace > StretchSizeTotal;

	const bool bCanStretch = bIsGrowing
		? (GrowStretchCoefficientTotal > 0.0f)
		: (ShrinkStretchCoefficientTotal > 0.0f);

	if (bAnyStretchContentItems && bCanStretch)
	{
		// Each StretchContent item starts at desired size and shrinks or grows based on available size.
		// First, consume each items desired size from the available space.
		// The remainder is corrected by growing ot shrinking the items.
		int32 NumStretchContentItems = 0;
		for (FStretchItem& Item : StretchItems)
		{
			if (Item.SizeRule == FSizeParam::SizeRule_StretchContent)
			{
				AvailableSpace -= Item.Size;
				NumStretchContentItems++;

				// If the item cannot shrink or grow, mark it already frozen.
				if (bIsGrowing)
				{
					Item.bFrozen |= FMath::IsNearlyZero(Item.GrowStretchValue);
				}
				else
				{
					Item.bFrozen |= FMath::IsNearlyZero(Item.ShrinkStretchValue);
				}
			}
		}

		// Run number of passes to satisfy the StretchContent constraints.
		// On each pass distribute the available space to non-frozen items.
		// An item gets frozen if it's (min/max) constraints are violated.
		// This makes sure that we distribute all of the available space, event if small items collapse or if items clamp to max size.
		// Each iteration should solve at least one constraint.
		// In practice most layouts solve in 2 passes, we're capping to 5 iterations to keep things in fixed budget.
		const int32 MaxPasses = FMath::Min(NumStretchContentItems, 5);
		for (int32 Pass = 0; Pass < MaxPasses; Pass++)
		{
			// If no available space, stop.
			if (FMath::IsNearlyZero(AvailableSpace))
			{
				break;
			}

			// On each pass calculate the total coefficients for valid items.
			GrowStretchCoefficientTotal = 0.0f;
			ShrinkStretchCoefficientTotal = 0.0f;

			for (const FStretchItem& Item : StretchItems)
			{
				if (Item.SizeRule == FSizeParam::SizeRule_StretchContent
					&& !Item.bFrozen)
				{
					// Items are grown proportional to their stretch value.
					GrowStretchCoefficientTotal += Item.GrowStretchValue;
					// Items are shrank proportional to their stretch value and size. This is to emulate the flexbox behavior.
					ShrinkStretchCoefficientTotal += Item.ShrinkStretchValue * Item.BasisSize;
				}
			}

			const float StretchCoefficientTotal = bIsGrowing
				? GrowStretchCoefficientTotal
				: ShrinkStretchCoefficientTotal;

			// If none of the items can stretch, stop.
			if (StretchCoefficientTotal < UE_KINDA_SMALL_NUMBER)
			{
				break;
			}

			float ConsumedSpace = 0.0f;

			for (FStretchItem& Item : StretchItems)
			{
				if (Item.SizeRule == FSizeParam::SizeRule_StretchContent
					&& !Item.bFrozen)
				{
					const float SizeAdjust = bIsGrowing
						? (AvailableSpace * (Item.GrowStretchValue / GrowStretchCoefficientTotal))
						: (AvailableSpace * (Item.ShrinkStretchValue * Item.BasisSize / ShrinkStretchCoefficientTotal));

					// If the item cannot be adjusted anymore, mark it frozen.
					if (FMath::IsNearlyZero(SizeAdjust))
					{
						Item.bFrozen = true;
						continue;
					}

					const float MinSize = Item.MinSize;
					const float MaxSize = Item.MaxSize;
					const bool bHasMaxConstraint = MaxSize > 0.0f; 

					if ((Item.Size + SizeAdjust) <= MinSize)
					{
						// Adjustment goes past min constraint, apply what we can and freeze since the item cannot change anymore.
						ConsumedSpace += MinSize - Item.Size;
						Item.Size = MinSize;
						Item.bFrozen = true;
					}
					else if (bHasMaxConstraint
						&& (Item.Size + SizeAdjust) >= MaxSize)
					{
						// Adjustment goes past max constraint, apply what we can and freeze since the item cannot change anymore.
						ConsumedSpace += MaxSize - Item.Size;
						Item.Size = MaxSize;
						Item.bFrozen = true;
					}
					else
					{
						// Within constraints, adjust.
						ConsumedSpace += SizeAdjust;
						Item.Size += SizeAdjust;
					}
				}
			}

			AvailableSpace -= ConsumedSpace;
		}
	}

	// Now that we have the satisfied size requirements we can
	// arrange widgets top-to-bottom or left-to-right (depending on the orientation).
	float PositionSoFar = 0.0f;

	ArrangedChildren.Reserve(ArrangedChildren.Num() + InChildren.Num());

	using SlotIteratorType = typename UE::Slate::TSlotIterator<InputSlotType>::Type;

	// Track the bounds of the arranged widgets.
	FVector2D ArrangedWidgetsMin = FVector2D(FLT_MAX, FLT_MAX);
	FVector2D ArrangedWidgetsMax = FVector2D(FLT_MIN, FLT_MIN);

	for (SlotIteratorType It(InChildren, Orientation, InLayoutFlow); It; ++It)
	{
		const InputSlotType& CurChild = *It;

		const EVisibility ChildVisibility = SlotAccessor.GetVisibility(CurChild);

		// Figure out the area allocated to the child in the direction of BoxPanel
		// The area allocated to the slot is ChildSize + the associated margin.
		const float ChildSize = StretchItems[It.GetIndex()].Size;

		const FMargin SlotPadding(LayoutPaddingWithFlow(InLayoutFlow, SlotAccessor.GetPadding(CurChild)));

		FVector2f SlotSize = (Orientation == Orient_Vertical)
			? FVector2f(AllottedGeometry.GetLocalSize().X, ChildSize + SlotPadding.template GetTotalSpaceAlong<Orient_Vertical>())
			: FVector2f(ChildSize + SlotPadding.template GetTotalSpaceAlong<Orient_Horizontal>(), AllottedGeometry.GetLocalSize().Y);

		// Figure out the size and local position of the child within the slot
		const AlignmentArrangeResult XAlignmentResult = AlignChild<Orient_Horizontal>(InLayoutFlow, SlotSize.X, CurChild, SlotPadding);
		const AlignmentArrangeResult YAlignmentResult = AlignChild<Orient_Vertical>(SlotSize.Y, CurChild, SlotPadding);

		const FVector2f LocalPosition = (Orientation == Orient_Vertical)
			? FVector2f(XAlignmentResult.Offset, PositionSoFar + YAlignmentResult.Offset + InOffset)
			: FVector2f(PositionSoFar + XAlignmentResult.Offset + InOffset, YAlignmentResult.Offset);

		const FVector2f LocalSize = FVector2f(XAlignmentResult.Size, YAlignmentResult.Size);

		ArrangedWidgetsMin = FVector2D::Min(ArrangedWidgetsMin, static_cast<FVector2D>(LocalPosition));
		ArrangedWidgetsMax = FVector2D::Max(ArrangedWidgetsMax, static_cast<FVector2D>(LocalPosition + LocalSize));

		ArrangedChildren.AddWidget(
			ChildVisibility,
			SlotAccessor.MakeArrangedWidget(
				CurChild,
				AllottedGeometry,
				LocalPosition,
				LocalSize));

		if constexpr (std::is_same_v<TArrayView<InputSlotType>, std::decay_t<InputChildrenType>>
			&& !std::is_const_v<typename TArrayView<InputSlotType>::ElementType>)
		{
			InputSlotType& MutableChild = (*It);

			// If the FSlotProxy is writable, write it's ArrangedSize.
			MutableChild.ArrangedSize = LocalSize;
		}

		if (ChildVisibility != EVisibility::Collapsed)
		{
			// Offset the next child by the size of the current child and any post-child (bottom/right) margin
			PositionSoFar += (Orientation == Orient_Vertical) ? SlotSize.Y : SlotSize.X;
		}
	}

	OutArrangedSize = ArrangedWidgetsMax - ArrangedWidgetsMin;
}

template <
	EOrientation Orientation,
	typename SlotType
	UE_REQUIRES(std::is_base_of_v<FSlotProxy, std::decay_t<SlotType>>)>
static void ArrangeChildrenInStack(EFlowDirection InLayoutFlow, const TConstArrayView<SlotType>& Children, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, float InOffset, bool bInAllowShrink, FVector2D& OutArrangedSize)
{
	ArrangeChildrenInStack<Orientation, TConstArrayView<SlotType>, const SlotType>(InLayoutFlow, Children, AllottedGeometry, ArrangedChildren, InOffset, bInAllowShrink, OutArrangedSize);
}

template <
	EOrientation Orientation,
	typename SlotType
	UE_REQUIRES(std::is_base_of_v<FSlotProxy, std::decay_t<SlotType>>)>
static void ArrangeChildrenInStack(EFlowDirection InLayoutFlow, const TArrayView<SlotType>& Children, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, float InOffset, bool bInAllowShrink, FVector2D& OutArrangedSize)
{
	ArrangeChildrenInStack<Orientation, TArrayView<SlotType>, SlotType>(InLayoutFlow, Children, AllottedGeometry, ArrangedChildren, InOffset, bInAllowShrink, OutArrangedSize);
}

template <
	EOrientation Orientation,
	typename SlotType
	UE_REQUIRES(std::is_base_of_v<FSlotBase, std::decay_t<SlotType>>)>
static void ArrangeChildrenInStack(EFlowDirection InLayoutFlow, const TPanelChildren<SlotType>& Children, const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, float InOffset, bool bInAllowShrink)
{
	FVector2D Unused;
	ArrangeChildrenInStack<Orientation, TPanelChildren<SlotType>, SlotType>(InLayoutFlow, Children, AllottedGeometry, ArrangedChildren, InOffset, bInAllowShrink, Unused);
}

static FMargin LayoutPaddingWithFlow(EFlowDirection InLayoutFlow, const FMargin& InPadding)
{
	FMargin ReturnPadding(InPadding);
	if (InLayoutFlow == EFlowDirection::RightToLeft)
	{
		float Temp = ReturnPadding.Left;
		ReturnPadding.Left = ReturnPadding.Right;
		ReturnPadding.Right = Temp;
	}
	return ReturnPadding;
}

/**
* Given information about a popup and the space available for displaying that popup, compute best placement for it.
*
* @param InAnchor          Area relative to which popup is being created (e.g. the button part of a combo box)
* @param PopupRect         Proposed placement of popup; position may require adjustment.
* @param Orientation       Are we trying to show the popup above/below or left/right relative to the anchor?
* @param RectToFit         The space available for showing this popup; we want to fit entirely within it without clipping.
* @param bAllowFlip        Determines whether the popup can be flipped to the other side of the anchor when there
*                          is no space on the screen for it to fit at its preferred position.
*
* @return A best position within the RectToFit such that none of the popup clips outside of the RectToFit.
*/
SLATECORE_API UE::Slate::FDeprecateVector2DResult ComputePopupFitInRect(const FSlateRect& InAnchor, const FSlateRect& PopupRect, const EOrientation& Orientation, const FSlateRect& RectToFit, bool bAllowFlip = true);
