// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Types/SlateEnums.h"
#include "Types/SlateVector2.h"
#include "Margin.generated.h"


/**
 * Describes the space around a Widget.
 */
USTRUCT(BlueprintType)
struct FMargin
{
	GENERATED_USTRUCT_BODY()

	/** Holds the margin to the left. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	float Left;

	/** Holds the margin to the top. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	float Top;
	
	/** Holds the margin to the right. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	float Right;
	
	/** Holds the margin to the bottom. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	float Bottom;

public:

	/**
	 * Default constructor.
	 *
	 * The default margin size is zero on all four sides..
	 */
	FMargin( )
		: Left(0.0f)
		, Top(0.0f)
		, Right(0.0f)
		, Bottom(0.0f)
	{ }

	/** Construct a Margin with uniform space on all sides */
	FMargin(float UniformMargin)
		: Left(UniformMargin)
		, Top(UniformMargin)
		, Right(UniformMargin)
		, Bottom(UniformMargin)
	{ }
	
	/** Construct a Margin where Horizontal describes Left and Right spacing while Vertical describes Top and Bottom spacing */
	FMargin(float Horizontal, float Vertical)
		: Left(Horizontal)
		, Top(Vertical)
		, Right(Horizontal)
		, Bottom(Vertical)
	{ }

	/** Construct a Margin where Horizontal describes Left and Right spacing while Vertical describes Top and Bottom spacing */
	FMargin(const FVector2f InVector)
		: Left(InVector.X)
		, Top(InVector.Y)
		, Right(InVector.X)
		, Bottom(InVector.Y)
	{ }

	/** Construct a Margin where Horizontal describes Left and Right spacing while Vertical describes Top and Bottom spacing */
	FMargin(const FVector2d InVector)
		: Left(UE_REAL_TO_FLOAT(InVector.X))
		, Top(UE_REAL_TO_FLOAT(InVector.Y))
		, Right(UE_REAL_TO_FLOAT(InVector.X))
		, Bottom(UE_REAL_TO_FLOAT(InVector.Y))
	{ }
	
	/** Construct a Margin where the spacing on each side is individually specified. */
	FMargin(double InLeft, double InTop, double InRight, double InBottom)
		: Left((float)InLeft)
		, Top((float)InTop)
		, Right((float)InRight)
		, Bottom((float)InBottom)
	{ }

	/** Construct a Margin where the margins are coming from a FVector4 */
	FMargin(const FVector4f InVector)
		: Left(InVector.X)
		, Top(InVector.Y)
		, Right(InVector.Z)
		, Bottom(InVector.W)
	{ }

	/** Construct a Margin where the margins are coming from a FVector4 */
	FMargin(const FVector4d InVector)
		: Left(UE_REAL_TO_FLOAT(InVector.X))
		, Top(UE_REAL_TO_FLOAT(InVector.Y))
		, Right(UE_REAL_TO_FLOAT(InVector.Z))
		, Bottom(UE_REAL_TO_FLOAT(InVector.W))
	{ }
	
public:

	/**
	 * Multiply the margin by a scalar.
	 *
	 * @param Scale How much to scale the margin.
	 * @return An FMargin where each value is scaled by Scale.
	 */
	FMargin operator*( float Scale ) const
	{
		return FMargin(Left * Scale, Top * Scale, Right * Scale, Bottom * Scale);
	}

	/**
	 * Multiply the margin by another margin functioning as the scale.
	 *
	 * @param InScale How much to scale the margin.
	 * @return An FMargin where each value is scaled by Scale.
	 */
	FMargin operator*(const FMargin& InScale) const
	{
		return FMargin(Left * InScale.Left, Top * InScale.Top, Right * InScale.Right, Bottom * InScale.Bottom);
	}

	/**
	 * Adds another margin to this margin.
	 *
	 * @param Other The margin to add.
	 * @return A margin that represents this margin plus the other margin.
	 */
	FMargin operator+( const FMargin& InDelta ) const
	{
		return FMargin(Left + InDelta.Left, Top + InDelta.Top, Right + InDelta.Right, Bottom + InDelta.Bottom);
	}

	/**
	 * Subtracts another margin from this margin.
	 *
	 * @param Other The margin to subtract.
	 * @return A margin that represents this margin minues the other margin.
	 */
	FMargin operator-( const FMargin& Other ) const
	{
		return FMargin(Left - Other.Left, Top - Other.Top, Right - Other.Right, Bottom - Other.Bottom);
	}

	/**
	 * Compares this margin with another for equality.
	 *
	 * @param Other The other margin.
	 * @return true if the two margins are equal, false otherwise.
	 */
	bool operator==( const FMargin& Other ) const 
	{
		return (Left == Other.Left) && (Right == Other.Right) && (Top == Other.Top) && (Bottom == Other.Bottom);
	}

	/**
	 * Compares this margin with another for inequality.
	 *
	 * @param Other The other margin.
	 * @return true if the two margins are not equal, false otherwise.
	 */
	bool operator!=( const FMargin& Other ) const 
	{
		return Left != Other.Left || Right != Other.Right || Top != Other.Top || Bottom != Other.Bottom;
	}

public:

	/**
	 * Gets the offset for the top left.
	 *
	 * @return The top left offset.
	 */
	UE::Slate::FDeprecateVector2DResult GetTopLeft() const
	{
		return UE::Slate::FDeprecateVector2DResult(Left, Top);
	}
	FVector2f GetTopLeft2f() const
	{
		return FVector2f(Left, Top);
	}

	/**
	 * Gets the margin's total size.
	 *
	 * @return Cumulative margin size.
	 */
	UE::Slate::FDeprecateVector2DResult GetDesiredSize() const
	{
		return UE::Slate::FDeprecateVector2DResult(GetDesiredSize2f());
	}
	FVector2f GetDesiredSize2f() const
	{
		return FVector2f(Left + Right, Top + Bottom);
	}

	/**
	 * Gets the total horizontal or vertical margin.
	 *
	 * @return Cumulative horizontal margin.
	 */
	template<EOrientation Orientation>
	float GetTotalSpaceAlong( ) const
	{
		return 0.0f;
	}
};

template <> struct TIsPODType<FMargin> { enum { Value = true }; };

template<>
inline float FMargin::GetTotalSpaceAlong<Orient_Horizontal>( ) const { return Left + Right; }

template<>
inline float FMargin::GetTotalSpaceAlong<Orient_Vertical>( ) const { return Top + Bottom; }
