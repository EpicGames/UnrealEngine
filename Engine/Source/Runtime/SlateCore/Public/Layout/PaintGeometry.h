// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Rendering/SlateRenderTransform.h"
#include "Types/SlateVector2.h"

/**
 * A Paint geometry contains the window-space (draw-space) info to draw an element on the screen.
 * 
 * It contains the size of the element in the element's local space along with the 
 * transform needed to position the element in window space.
 * 
 * DrawPosition/DrawSize/DrawScale are maintained for legacy reasons, but are deprecated:
 * 
 *		The DrawPosition and DrawSize are already positioned and
 *		scaled for immediate consumption by the draw code.
 *
 *		The DrawScale is only applied to the internal aspects of the draw primitives. e.g. Line thickness, 3x3 grid margins, etc.
 */
struct FPaintGeometry
{
	/** 
	 * !!! DEPRECATED!!! Drawing should only happen in local space to ensure render transforms work.
	 *	WARNING: This legacy member represents is LAYOUT space, and does not account for render transforms.
	 *	
	 * Render-space position at which we will draw.
	 *
	 * 
	 */
	FVector2f DrawPosition;

	/**
	 * !!! DEPRECATED!!! Drawing should only happen in local space to ensure render transforms work.
	 *	WARNING: This legacy member represents is LAYOUT space, and does not account for render transforms.
	 * 
	 * Only affects the draw aspects of individual draw elements. e.g. line thickness, 3x3 grid margins 
	 */
	float DrawScale;

	/** Get the Size of the geometry in local space. Must call CommitTransformsIfUsingLegacyConstructor() first if legacy ctor is used. */
	UE::Slate::FDeprecateVector2DResult GetLocalSize() const { return UE::Slate::FDeprecateVector2DResult(LocalSize); }

	/** Access the final render transform. Must call CommitTransformsIfUsingLegacyConstructor() first if legacy ctor is used. */
	const FSlateRenderTransform& GetAccumulatedRenderTransform() const { return AccumulatedRenderTransform; }

	/**
	 * Support mutable geometries constructed in window space, and possibly mutated later, as all legacy members are public.
	 * In these cases we defer creating of the RenderTransform and LocalSize until rendering time to ensure that all member changes have finished.
	 * WARNING: Legacy usage does NOT support render transforms!
	 */
	void CommitTransformsIfUsingLegacyConstructor() const 
	{ 
		if (!bUsingLegacyConstructor) return;
		
		AccumulatedRenderTransform = FSlateRenderTransform(DrawScale, DrawPosition);
		FSlateLayoutTransform AccumulatedLayoutTransform = FSlateLayoutTransform(DrawScale, DrawPosition);
		LocalSize = TransformVector(Inverse(AccumulatedLayoutTransform), DrawSize);
	}

	bool HasRenderTransform() const { return bHasRenderTransform; }

private:
	// Mutable to support legacy constructors. Doesn't account for render transforms.
	mutable FVector2f DrawSize;

	// Mutable to support legacy constructors.
	mutable FVector2f LocalSize;

	// final render transform for drawing. Transforms from local space to window space for the draw element.
	// Mutable to support legacy constructors.
	mutable FSlateRenderTransform AccumulatedRenderTransform;

	// Support legacy constructors.
	uint8 bUsingLegacyConstructor : 1;

	uint8 bHasRenderTransform : 1;

public:
	/** Default ctor. */
	FPaintGeometry() 
		: DrawPosition(0.0f, 0.0f)
		, DrawScale(1.0f)
		, DrawSize(0.0f, 0.0f)
		, LocalSize(0.0f, 0.0f)
		, AccumulatedRenderTransform()
		, bUsingLegacyConstructor(true)
		, bHasRenderTransform(false)
	{}

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InLocalSize					The size in local space
	 * @param InAccumulatedLayoutTransform	The accumulated layout transform (from an FGeometry)
	 * @param InAccumulatedRenderTransform	The accumulated render transform (from an FGeometry)
	 */
	FPaintGeometry( const FSlateLayoutTransform& InAccumulatedLayoutTransform, const FSlateRenderTransform& InAccumulatedRenderTransform, const UE::Slate::FDeprecateVector2DParameter& InLocalSize, bool bInHasRenderTransform)
		: DrawPosition(InAccumulatedLayoutTransform.GetTranslation())
		, DrawScale(InAccumulatedLayoutTransform.GetScale())
		, DrawSize(0.0f, 0.0f)
		, LocalSize(InLocalSize)
		, AccumulatedRenderTransform(InAccumulatedRenderTransform)
		, bUsingLegacyConstructor(false)
		, bHasRenderTransform(bInHasRenderTransform)
	{ 
	}

	// !!! DEPRECATED!!! This is legacy and should be removed!
	FPaintGeometry( UE::Slate::FDeprecateVector2DParameter InDrawPosition, UE::Slate::FDeprecateVector2DParameter InDrawSize, float InDrawScale )
		: DrawPosition(InDrawPosition)
		, DrawScale(InDrawScale)
		, DrawSize(InDrawSize)
		, LocalSize(0.0f, 0.0f)
		, AccumulatedRenderTransform()
		, bUsingLegacyConstructor(true)
		, bHasRenderTransform(false)
	{ 
	}

	/**
	 * Special case method to append a layout transform to a paint geometry.
	 * This is used in cases where the FGeometry was arranged in desktop space
	 * and we need to undo the root desktop translation to get into window space.
	 * If you find yourself wanting to use this function, ask someone if there's a better way.
	 * 
	 * @param LayoutTransform	An additional layout transform to append to this paint geometry.
	 */
	void AppendTransform(const FSlateLayoutTransform& LayoutTransform)
	{
		AccumulatedRenderTransform = ::Concatenate(AccumulatedRenderTransform, LayoutTransform);
		DrawPosition = TransformPoint(LayoutTransform, DrawPosition);
		DrawScale = ::Concatenate(LayoutTransform.GetScale(), DrawScale);
	}

	/**
	 * Special case method to replace the render transform on a paint geometry.
	 * 
	 * @param RenderTransform	An additional layout transform to append to this paint geometry.
	 */
	void SetRenderTransform(const FSlateRenderTransform& RenderTransform)
	{
		AccumulatedRenderTransform = RenderTransform;
	}
};


template<> struct TIsPODType<FPaintGeometry> { enum { Value = true }; };
