// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"
#include "Types/SlateVector2.h"

/** 
 * A rectangle defined by upper-left and lower-right corners.
 * 
 * Assumes a "screen-like" coordinate system where the origin is in the top-left, with the Y-axis going down.
 * Functions like "contains" etc will not work with other conventions.
 * 
 *      +---------> X
 *      |
 *      |    (Left,Top)  
 *      |            o----o 
 *      |            |    |
 *      |            o----o 
 *      |                (Right, Bottom)
 *      \/
 *      Y
 */
class FSlateRect
{
public:

	float Left;
	float Top;
	float Right;
	float Bottom;

	explicit FSlateRect( float InLeft = -1, float InTop = -1, float InRight = -1, float InBottom = -1 )
		: Left(InLeft)
		, Top(InTop)
		, Right(InRight)
		, Bottom(InBottom)
	{ }

	FSlateRect( const UE::Slate::FDeprecateVector2DParameter& InStartPos, const UE::Slate::FDeprecateVector2DParameter& InEndPos )
		: Left(InStartPos.X)
		, Top(InStartPos.Y)
		, Right(InEndPos.X)
		, Bottom(InEndPos.Y)
	{ }

	/**
	 * Creates a rect from a top left point and extent. Provided as a factory function to not conflict
	 * with the TopLeft + BottomRight ctor.
	 */
	static FSlateRect FromPointAndExtent(const UE::Slate::FDeprecateVector2DParameter& TopLeft, const UE::Slate::FDeprecateVector2DParameter& Size)
	{
		return FSlateRect(TopLeft, TopLeft + Size);
	}
	
public:

	/**
	 * Determines if the rectangle has positive dimensions.
	 */
	inline bool IsValid() const
	{
		return !(Left == -1 && Right == -1 && Bottom == -1 && Top == -1) && Right >= Left && Bottom >= Top;
	}

	/**
	 * @return true, if the rectangle has an effective size of 0.
	 */
	inline bool IsEmpty() const
	{
		return GetArea() <= UE_SMALL_NUMBER;
	}

	/**
	 * Returns the size of the rectangle in each dimension.
	 *
	 * @return The size as a vector.
	 */
	inline UE::Slate::FDeprecateVector2DResult GetSize() const
	{
		return UE::Slate::FDeprecateVector2DResult(GetSize2f());
	}
	inline FVector2f GetSize2f() const
	{
		return FVector2f(Right - Left, Bottom - Top);
	}

	/**
	 * @return the area of the rectangle
	 */
	inline float GetArea() const
	{
		return (Right - Left) * (Bottom - Top);
	}

	/**
	 * Returns the center of the rectangle
	 * 
	 * @return The center point.
	 */
	inline UE::Slate::FDeprecateVector2DResult GetCenter() const
	{
		return UE::Slate::FDeprecateVector2DResult(GetCenter2f());
	}
	inline FVector2f GetCenter2f() const
	{
		return FVector2f(Left, Top) + GetSize2f() * 0.5f;
	}

	/**
	 * Returns the top-left position of the rectangle
	 * 
	 * @return The top-left position.
	 */
	inline UE::Slate::FDeprecateVector2DResult GetTopLeft() const
	{
		return UE::Slate::FDeprecateVector2DResult(GetTopLeft2f());
	}
	inline FVector2f GetTopLeft2f() const
	{
		return FVector2f(Left, Top);
	}

	/**
	 * Returns the top-right position of the rectangle
	 *
	 * @return The top-right position.
	 */
	inline UE::Slate::FDeprecateVector2DResult GetTopRight() const
	{
		return UE::Slate::FDeprecateVector2DResult(GetTopRight2f());
	}
	inline FVector2f GetTopRight2f() const
	{
		return FVector2f(Right, Top);
	}

	/**
	 * Returns the bottom-right position of the rectangle
	 * 
	 * @return The bottom-right position.
	 */
	inline UE::Slate::FDeprecateVector2DResult GetBottomRight() const
	{
		return UE::Slate::FDeprecateVector2DResult(GetBottomRight2f());
	}
	inline FVector2f GetBottomRight2f() const
	{
		return FVector2f(Right, Bottom);
	}

	/**
	 * Returns the bottom-left position of the rectangle
	 * 
	 * @return The bottom-left position.
	 */
	inline UE::Slate::FDeprecateVector2DResult GetBottomLeft() const
	{
		return UE::Slate::FDeprecateVector2DResult(GetBottomLeft2f());
	}
	inline FVector2f GetBottomLeft2f() const
	{
		return FVector2f(Left, Bottom);
	}

	/**
	 * Return a rectangle that is contracted on each side by the amount specified in each margin.
	 *
	 * @param InsetAmount The amount to contract the geometry.
	 *
	 * @return An inset rectangle.
	 */
	inline FSlateRect InsetBy(const struct FMargin& InsetAmount) const
	{
		return FSlateRect(Left + InsetAmount.Left, Top + InsetAmount.Top, Right - InsetAmount.Right, Bottom - InsetAmount.Bottom);
	}

	/**
	 * Return a rectangle that is extended on each side by the amount specified in each margin.
	 *
	 * @param ExtendAmount The amount to extend the geometry.
	 *
	 * @return An extended rectangle.
	 */
	inline FSlateRect ExtendBy(const FMargin& ExtendAmount) const
	{
		return FSlateRect(Left - ExtendAmount.Left, Top - ExtendAmount.Top, Right + ExtendAmount.Right, Bottom + ExtendAmount.Bottom);
	}

	/**
	 * Return a rectangle that is offset by the amount specified .
	 *
	 * @param OffsetAmount The amount to contract the geometry.
	 *
	 * @return An offset rectangle.
	 */
	inline FSlateRect OffsetBy( const UE::Slate::FDeprecateVector2DParameter& OffsetAmount ) const
	{
		return FSlateRect(GetTopLeft2f() + OffsetAmount, GetBottomRight2f() + OffsetAmount);
	}

	/**
	 * Return a rectangle that is scaled by the amount specified.
	 *
	 * @param ScaleBy The amount to scale the geometry.
	 *
	 * @return An scaled rectangle.
	 */
	inline FSlateRect ScaleBy(float ScaleBy) const
	{
		const FVector2f Delta = GetSize() * 0.5f * ScaleBy;
		return ExtendBy(FMargin(Delta));
	}

	/**
	 * Returns the rect that encompasses both rectangles
	 * 
	 * @param	Other	The other rectangle
	 *
	 * @return	Rectangle that is big enough to fit both rectangles
	 */
	inline FSlateRect Expand( const FSlateRect& Other ) const
	{
		return FSlateRect( FMath::Min( Left, Other.Left ), FMath::Min( Top, Other.Top ), FMath::Max( Right, Other.Right ), FMath::Max( Bottom, Other.Bottom ) );
	}

	/**
	 * Rounds the Left, Top, Right and Bottom fields and returns a new FSlateRect with rounded components.
	 */
	inline FSlateRect Round() const
	{
		return FSlateRect(
			FMath::RoundToFloat(Left),
			FMath::RoundToFloat(Top),
			FMath::RoundToFloat(Right),
			FMath::RoundToFloat(Bottom));
	}

	/**
	 * Returns the rectangle that is the intersection of this rectangle and Other.
	 * 
	 * @param	Other	The other rectangle
	 *
	 * @return	Rectangle over intersection.
	 */
	inline FSlateRect IntersectionWith(const FSlateRect& Other) const
	{
		bool bOverlapping;
		return IntersectionWith(Other, bOverlapping);
	}

	/**
	 * Returns the rectangle that is the intersection of this rectangle and Other, as well as if they were overlapping at all.
	 * 
	 * @param	Other	The other rectangle
	 * @param	OutOverlapping	[Out] Was there any overlap with the other rectangle.
	 *
	 * @return	Rectangle over intersection.
	 */
	FSlateRect IntersectionWith(const FSlateRect& Other, bool& OutOverlapping) const
	{
		FSlateRect Intersected( FMath::Max( this->Left, Other.Left ), FMath::Max(this->Top, Other.Top), FMath::Min( this->Right, Other.Right ), FMath::Min( this->Bottom, Other.Bottom ) );
		if ( (Intersected.Bottom < Intersected.Top) || (Intersected.Right < Intersected.Left) )
		{
			OutOverlapping = false;
			// The intersection has 0 area and should not be rendered at all.
			return FSlateRect(0,0,0,0);
		}
		else
		{
			OutOverlapping = true;
			return Intersected;
		}
	}

	/**
	 * Returns whether or not a point is inside the rectangle
	 * 
	 * @param Point	The point to check
	 * @return True if the point is inside the rectangle
	 */
	inline bool ContainsPoint( const UE::Slate::FDeprecateVector2DParameter& Point ) const
	{
		return Point.X >= Left && Point.X <= Right && Point.Y >= Top && Point.Y <= Bottom;
	}

	/**
	 * Returns whether or not the given rectangle is fully contained within this rectangle.
	 *
	 * @param Other	The rectangle to check
	 * @return True if Other is entirely inside this rectangle
	 */
	inline bool ContainsRect( const FSlateRect& Other ) const
	{
		return Other.Left >= Left && Other.Top >= Top && Other.Right <= Right && Other.Bottom <= Bottom;
	}

	bool operator==( const FSlateRect& Other ) const
	{
		return
			Left == Other.Left &&
			Top == Other.Top &&
			Right == Other.Right &&
			Bottom == Other.Bottom;
	}

	bool operator!=( const FSlateRect& Other ) const
	{
		return Left != Other.Left || Top != Other.Top || Right != Other.Right || Bottom != Other.Bottom;
	}

	friend FSlateRect operator+( const FSlateRect& A, const FSlateRect& B )
	{
		return FSlateRect( A.Left + B.Left, A.Top + B.Top, A.Right + B.Right, A.Bottom + B.Bottom );
	}

	friend FSlateRect operator-( const FSlateRect& A, const FSlateRect& B )
	{
		return FSlateRect( A.Left - B.Left, A.Top - B.Top, A.Right - B.Right, A.Bottom - B.Bottom );
	}

	friend FSlateRect operator*( float Scalar, const FSlateRect& Rect )
	{
		return FSlateRect( Rect.Left * Scalar, Rect.Top * Scalar, Rect.Right * Scalar, Rect.Bottom * Scalar );
	}

	/** Do rectangles A and B intersect? */
	static bool DoRectanglesIntersect( const FSlateRect& A, const FSlateRect& B )
	{
		//  Segments A and B do not intersect when:
		//
		//       (left)   A     (right)
		//         o-------------o
		//  o---o        OR         o---o
		//    B                       B
		//
		//
		// We assume the A and B are well-formed rectangles.
		// i.e. (Top,Left) is above and to the left of (Bottom,Right)
		const bool bDoNotOverlap =
			B.Right < A.Left || A.Right < B.Left ||
			B.Bottom < A.Top || A.Bottom < B.Top;

		return ! bDoNotOverlap;
	}

	/** Is rectangle B contained within rectangle A? */
	inline static bool IsRectangleContained( const FSlateRect& A, const FSlateRect& B )
	{
		return (A.Left <= B.Left) && (A.Right >= B.Right) && (A.Top <= B.Top) && (A.Bottom >= B.Bottom);
	}

	/**
	* Returns a string of containing the coordinates of the rect
	*
	* @return	A string of the rect coordinates 
	*/
	SLATECORE_API FString ToString() const;

	/**
	* Returns a string of containing the coordinates of the rect
	*
	* @param InSourceString A string containing the values to initialize this rect in format Left=Value Top=Value...
	*
	* @return	True if initialized successfully
	*/
	SLATECORE_API bool InitFromString(const FString& InSourceString);

	
	friend inline uint32 GetTypeHash(const FSlateRect& Key)
	{
		uint32 Hash = 0;
		Hash = HashCombine(Hash, GetTypeHash(Key.Left));
		Hash = HashCombine(Hash, GetTypeHash(Key.Right));
		Hash = HashCombine(Hash, GetTypeHash(Key.Top));
		Hash = HashCombine(Hash, GetTypeHash(Key.Bottom));
		return Hash;
	}
};

/**
 * Transforms a rect by the given transform, ensuring the rect does not get inverted.
 * WARNING: this only really supports scales and offsets. Any skew or rotation that 
 * would turn this into an un-aligned rect will won't work because FSlateRect doesn't support
 * non-axis-alignment. Instead, convert to ta FSlateRotatedRect first and transform that.
 */
template <typename TransformType>
FSlateRect TransformRect(const TransformType& Transform, const FSlateRect& Rect)
{
	FVector2f TopLeftTransformed = TransformPoint(Transform, FVector2f(Rect.Left, Rect.Top));
	FVector2f BottomRightTransformed = TransformPoint(Transform, FVector2f(Rect.Right, Rect.Bottom));

	if (TopLeftTransformed.X > BottomRightTransformed.X)
	{
		Swap(TopLeftTransformed.X, BottomRightTransformed.X);
	}
	if (TopLeftTransformed.Y > BottomRightTransformed.Y)
	{
		Swap(TopLeftTransformed.Y, BottomRightTransformed.Y);
	}
	return FSlateRect(TopLeftTransformed, BottomRightTransformed);
}

template<> struct TIsPODType<FSlateRect> { enum { Value = true }; };
