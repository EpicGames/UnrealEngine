// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/SlateRect.h"
#include "Math/TransformCalculus.h"
#include "Math/TransformCalculus2D.h"
#include "Math/Vector2D.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Rendering/SlateRenderTransform.h"
#include "Types/SlateVector2.h"

/**
 * Stores a rectangle that has been transformed by an arbitrary render transform.
 * We provide a ctor that does the work common to slate drawing, but you could technically
 * create this any way you want.
 */
struct FSlateRotatedRect
{
public:
	/** Default ctor. */
	FSlateRotatedRect() {}

	/** Construct a rotated rect from a given aligned rect. */
	explicit FSlateRotatedRect(const FSlateRect& AlignedRect)
		: TopLeft(AlignedRect.Left, AlignedRect.Top)
		, ExtentX(AlignedRect.Right - AlignedRect.Left, 0.0f)
		, ExtentY(0.0f, AlignedRect.Bottom - AlignedRect.Top)
	{
	}

	/** Per-element constructor. */
	FSlateRotatedRect(const UE::Slate::FDeprecateVector2DParameter& InTopLeft, const UE::Slate::FDeprecateVector2DParameter& InExtentX, const UE::Slate::FDeprecateVector2DParameter& InExtentY)
		: TopLeft(InTopLeft)
		, ExtentX(InExtentX)
		, ExtentY(InExtentY)
	{
	}

public:

	/** transformed Top-left corner. */
	FVector2f TopLeft;
	/** transformed X extent (right-left). */
	FVector2f ExtentX;
	/** transformed Y extent (bottom-top). */
	FVector2f ExtentY;

public:
	bool operator == (const FSlateRotatedRect& Other) const
	{
		return
			TopLeft == Other.TopLeft &&
			ExtentX == Other.ExtentX &&
			ExtentY == Other.ExtentY;
	}

public:

	/** Convert to a bounding, aligned rect. */
	SLATECORE_API FSlateRect ToBoundingRect() const;

	/** Point-in-rect test. */
	SLATECORE_API bool IsUnderLocation(const UE::Slate::FDeprecateVector2DParameter Location) const;

	static FSlateRotatedRect MakeRotatedRect(const FSlateRect& ClipRectInLayoutWindowSpace, const FSlateLayoutTransform& InverseLayoutTransform, const FSlateRenderTransform& RenderTransform)
	{
		return MakeRotatedRect(ClipRectInLayoutWindowSpace, Concatenate(InverseLayoutTransform, RenderTransform));
	}

	static SLATECORE_API FSlateRotatedRect MakeRotatedRect(const FSlateRect& ClipRectInLayoutWindowSpace, const FTransform2f& LayoutToRenderTransform);

	static FSlateRotatedRect MakeSnappedRotatedRect(const FSlateRect& ClipRectInLayoutWindowSpace, const FSlateLayoutTransform& InverseLayoutTransform, const FSlateRenderTransform& RenderTransform)
	{
		return MakeSnappedRotatedRect(ClipRectInLayoutWindowSpace, Concatenate(InverseLayoutTransform, RenderTransform));
	}

	/**
	* Used to construct a rotated rect from an aligned clip rect and a set of layout and render transforms from the geometry, snapped to pixel boundaries. Returns a float or float16 version of the rect based on the typedef.
	*/
	static SLATECORE_API FSlateRotatedRect MakeSnappedRotatedRect(const FSlateRect& ClipRectInLayoutWindowSpace, const FTransform2f& LayoutToRenderTransform);
};

/**
* Transforms a rect by the given transform.
*/
template <typename TransformType>
FSlateRotatedRect TransformRect(const TransformType& Transform, const FSlateRotatedRect& Rect)
{
	return FSlateRotatedRect
	(
		TransformPoint(Transform, Rect.TopLeft),
		TransformVector(Transform, Rect.ExtentX),
		TransformVector(Transform, Rect.ExtentY)
	);
}
