// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/EnumAsByte.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"

/** Is an entity visible? */
struct EVisibility
{
	/** Visible and hit-testable (can interact with cursor). Default value. */
	static SLATECORE_API const EVisibility Visible;

	/** Not visible and takes up no space in the layout (obviously not hit-testable). */
	static SLATECORE_API const EVisibility Collapsed;

	/** Not visible but occupies layout space (obviously not hit-testable). */
	static SLATECORE_API const EVisibility Hidden;

	/** Visible but not hit-testable (cannot interact with cursor) and children in the hierarchy (if any) are also not hit-testable. */
	static SLATECORE_API const EVisibility HitTestInvisible;

	/** Visible but not hit-testable (cannot interact with cursor) and doesn't affect hit-testing on children (if any). */
	static SLATECORE_API const EVisibility SelfHitTestInvisible;

	/** Any visibility will do */
	static SLATECORE_API const EVisibility All;

public:

	/**
	 * Default constructor.
	 *
	 * The default visibility is 'visible'.
	 */
	EVisibility( )
		: Value(VIS_Visible)
	{ }

public:

	bool operator==( const EVisibility& Other ) const
	{
		return this->Value == Other.Value;
	}

	bool operator!=( const EVisibility& Other ) const
	{
		return this->Value != Other.Value;
	}

public:

	bool AreChildrenHitTestVisible( ) const
	{
		return 0 != (Value & VISPRIVATE_ChildrenHitTestVisible);
	}

	bool IsHitTestVisible( ) const
	{
		return 0 != (Value & VISPRIVATE_SelfHitTestVisible);
	}

	bool IsVisible() const
	{
		return 0 != (Value & VIS_Visible);
	}

	static inline bool DoesVisibilityPassFilter(const EVisibility InVisibility, const EVisibility InVisibilityFilter)
	{
		return 0 != (InVisibility.Value & InVisibilityFilter.Value);
	}

	SLATECORE_API FString ToString() const;

private:

	enum Private
	{
		/** Entity is visible */
		VISPRIVATE_Visible = 0x1 << 0,
		/** Entity is invisible and takes up no space */
		VISPRIVATE_Collapsed = 0x1 << 1,
		/** Entity is invisible, but still takes up space (layout pretends it is visible) */
		VISPRIVATE_Hidden = 0x1 << 2,
		/** Can we click on this widget or is it just non-interactive decoration? */
		VISPRIVATE_SelfHitTestVisible = 0x1 << 3,
		/** Can we click on this widget's child widgets? */
		VISPRIVATE_ChildrenHitTestVisible = 0x1 << 4,


		/** Default widget visibility - visible and can interactive with the cursor */
		VIS_Visible = VISPRIVATE_Visible | VISPRIVATE_SelfHitTestVisible | VISPRIVATE_ChildrenHitTestVisible,
		/** Not visible and takes up no space in the layout; can never be clicked on because it takes up no space. */
		VIS_Collapsed = VISPRIVATE_Collapsed,
		/** Not visible, but occupies layout space. Not interactive for obvious reasons. */
		VIS_Hidden = VISPRIVATE_Hidden,
		/** Visible to the user, but only as art. The cursors hit tests will never see this widget. */
		VIS_HitTestInvisible = VISPRIVATE_Visible,
		/** Same as HitTestInvisible, but doesn't apply to child widgets. */
		VIS_SelfHitTestInvisible = VISPRIVATE_Visible | VISPRIVATE_ChildrenHitTestVisible,


		/** Any visibility will do */
		VIS_All = VISPRIVATE_Visible | VISPRIVATE_Hidden | VISPRIVATE_Collapsed | VISPRIVATE_SelfHitTestVisible | VISPRIVATE_ChildrenHitTestVisible
	};

	/**
	 * Private constructor.
	 */
	EVisibility( Private InValue )
		: Value(InValue)
	{ }

public:

	TEnumAsByte<Private> Value;
};
