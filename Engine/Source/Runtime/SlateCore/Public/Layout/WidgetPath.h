// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/SlateEnums.h"
#include "Layout/Visibility.h"
#include "Layout/ArrangedWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

/**
 * Enumerates different purposes for searching through a widget path. Normally, Standard is appropriate.
 */
enum class EWidgetPathSearchPurpose
{
	/**
	 * No specified search purpose. This will be the default and should be used when no other purpose matches
	 */
	Standard,

	/**
	 * We are searching through the WidgetPath to change focus during navigation.
	 */
	FocusHandling
};

/** Matches widgets against InWidget */
struct FWidgetMatcher
{
	FWidgetMatcher( const TSharedRef<const SWidget> InWidget )
		: WidgetToFind( InWidget )
	{}

	bool IsMatch( const TSharedRef<const SWidget>& InWidget ) const
	{
		return WidgetToFind == InWidget;
	}

	TSharedRef<const SWidget> WidgetToFind;
};

/**
 * A widget path is a vertical slice through the tree.
 * The canonical form for widget paths is "leafmost last". The top-level window always resides at index 0.
 * A widget path also contains a reference to a top-level SWindow that contains all the widgets in the path.
 * The window is needed for its ability to determine its own geometry, from which the geometries of the rest
 * of the widget can be determined.
 */
class FWidgetPath
{
public:
	SLATECORE_API FWidgetPath();

	SLATECORE_API FWidgetPath( TSharedPtr<SWindow> InTopLevelWindow, const FArrangedChildren& InWidgetPath );

	SLATECORE_API FWidgetPath( TArrayView<FWidgetAndPointer> InWidgetsAndPointers );

	/**
	 * @param MarkerWidget Copy the path up to and including this widget 
	 *
	 * @return a copy of the widget path down to and including the MarkerWidget. If the MarkerWidget is not found in the path, return an invalid path.
	 */
	SLATECORE_API FWidgetPath GetPathDownTo( TSharedRef<const SWidget> MarkerWidget ) const;

	/** Get the virtual representation of the mouse at each level in the widget path. */
	TOptional<FVirtualPointerPosition> GetVirtualPointerPosition(int32 Index) const
	{
		return VirtualPointerPositions[Index];
	}
	
	/** @return true if the WidgetToFind is in this WidgetPath, false otherwise. */
	SLATECORE_API bool ContainsWidget( const SWidget* WidgetToFind ) const;

	SLATECORE_API TOptional<FArrangedWidget> FindArrangedWidget( TSharedRef<const SWidget> WidgetToFind ) const;

	SLATECORE_API TOptional<FWidgetAndPointer> FindArrangedWidgetAndCursor( TSharedRef<const SWidget> WidgetToFind ) const;

	/**
	 * Get the first (top-most) widget in this path, which is always a window; assumes path is valid.
	 *
	 * @return Window at the top of this path.
	 */
	SLATECORE_API TSharedRef<SWindow> GetWindow() const;

	/**
	 * Get the deepest (bottom-most) window in this path; assumes path is valid.
	 *
	 * @return Window at the bottom of this path.
	 */
	SLATECORE_API TSharedRef<SWindow> GetDeepestWindow() const;

	/** A valid path has at least one widget in it */
	SLATECORE_API bool IsValid() const;
	
	/**
	 * Builds a string representation of the widget path.
	 */
	SLATECORE_API FString ToString() const;

	/**
	 * Extend the current path such that it reaches some widget that qualifies as a Match
	 * The widget to match must be a descendant of the last widget currently in the path.
	 *
	 * @param  Matcher          Some struct that has a "bool IsMatch( const TSharedRef<const SWidget>& InWidget ) const" method
	 * @param  VisibilityFilter	Widgets must have this type of visibility to be included the path
	 * @param  SearchPurpose	What is the purpose for extending the path
	 *
	 * @return true if successful; false otherwise.
	 */
	template<typename MatcherType>
	bool ExtendPathTo( const MatcherType& Matcher, EVisibility VisibilityFilter = EVisibility::Visible, EWidgetPathSearchPurpose SearchPurpose = EWidgetPathSearchPurpose::Standard)
	{
		const FArrangedWidget& LastWidget = Widgets.Last();
		
		FArrangedChildren Extension = GeneratePathToWidget(Matcher, LastWidget, EUINavigation::Next, VisibilityFilter, SearchPurpose);

		for( int32 WidgetIndex=0; WidgetIndex < Extension.Num(); ++WidgetIndex )
		{
			this->Widgets.AddWidget( Extension[WidgetIndex] );
		}

		return Extension.Num() > 0;
	}

	/**
	 * Generate a path from FromWidget to WidgetToFind. The path will not include FromWidget.
	 *
	 * @param  Matcher          Some struct that has a "bool IsMatch( const TSharedRef<const SWidget>& InWidget ) const" method
	 * @param  FromWidget       Widget from which we a building a path.AddItem*
	 * @param  VisibilityFilter	Widgets must have this type of visibility to be included the path
	 * @param  SearchPurpose	What is the purpose for generating the path
	 * 
	 * @return A path from FromWidget to WidgetToFind; will not include FromWidget.
	 */
	template<typename MatcherType>
	FArrangedChildren GeneratePathToWidget(const MatcherType& Matcher, const FArrangedWidget& FromWidget, EUINavigation NavigationType = EUINavigation::Next, EVisibility VisibilityFilter = EVisibility::Visible, EWidgetPathSearchPurpose SearchPurpose = EWidgetPathSearchPurpose::Standard)
	{
		FArrangedChildren PathResult(VisibilityFilter);

		if (NavigationType == EUINavigation::Next)
		{
			SearchForWidgetRecursively( Matcher, FromWidget, PathResult, VisibilityFilter, SearchPurpose);
		}
		else
		{
			SearchForWidgetRecursively_Reverse( Matcher, FromWidget, PathResult, VisibilityFilter, SearchPurpose);
		}
		

		// Reverse the list of widgets we found; canonical form is leafmost last.
		PathResult.Reverse();

		return PathResult;
	}
	
	/**
	 * Move focus either forward on backward in the path level specified by PathLevel.
	 * That is, this movement of focus will modify the subtree under Widgets(PathLevel).
	 *
	 * @param PathLevel					The level in this WidgetPath whose focus to move.
	 * @param MoveDirectin				Move focus forward or backward?
	 * @param bSearchFromPathWidget		if set false the search for the next will simply start at the beginning or end of the list of children dependant on the direction
	 *
	 * @return true if the focus moved successfully, false if we were unable to move focus
	 */
	SLATECORE_API bool MoveFocus(int32 PathLevel, EUINavigation NavigationType, bool bSearchFromPathWidget = true);

	/** Get the last (leaf-most) widget in this path; assumes path is valid */
	TSharedRef< SWidget > GetLastWidget() const
	{
		check(IsValid());
		return Widgets[Widgets.Num() - 1].Widget;
	}

public:

	/** The widgets that make up the widget path, the first item is the root widget, the end is the widget this path was built for. */
	FArrangedChildren Widgets;

	/** The top level window of this widget path. */
	TSharedPtr<SWindow> TopLevelWindow;

private:
	/** The virtual representation of the mouse at each level in the widget path.  Due to 3D widgets, the space you transition to can be completely arbitrary as you traverse the tree. */
	TArray<TOptional<FVirtualPointerPosition>> VirtualPointerPositions;

private:

	/**
	 * Utility function to search recursively through a widget hierarchy for a specific widget
	 *
	 * @param  Matcher          Some struct that has a "bool IsMatch( const TSharedRef<const SWidget>& InWidget ) const" method
	 * @param  InCandidate      The current widget-geometry pair we're testing
	 * @param  OutReversedPath  The resulting path in reversed order (canonical order is Windows @ index 0, Leafmost widget is last.)
	 * @param  VisibilityFilter	Widgets must have this type of visibility to be included the path
	 * @param  SearchPurpose	The purpose for searching for the widget in the path
	 *
	 * @return  true if the child widget was found; false otherwise
	 */
	template<typename MatchRuleType>
	static bool SearchForWidgetRecursively( const MatchRuleType& MatchRule, const FArrangedWidget& InCandidate, FArrangedChildren& OutReversedPath, EVisibility VisibilityFilter = EVisibility::Visible, EWidgetPathSearchPurpose SearchPurpose = EWidgetPathSearchPurpose::Standard);

	/** Identical to SearchForWidgetRecursively, but iterates in reverse order */
	template<typename MatchRuleType>
	static bool SearchForWidgetRecursively_Reverse( const MatchRuleType& MatchRule, const FArrangedWidget& InCandidate, FArrangedChildren& OutReversedPath, EVisibility VisibilityFilter = EVisibility::Visible, EWidgetPathSearchPurpose SearchPurpose = EWidgetPathSearchPurpose::Standard);
};

/**
 * Just like a WidgetPath, but uses weak pointers and does not store geometry.
 */
class FWeakWidgetPath
{
public:
	/** Construct a weak widget path from a widget path. Defaults to an invalid path. */
	SLATECORE_API FWeakWidgetPath( const FWidgetPath& InWidgetPath = FWidgetPath() );

	/** Should interrupted paths truncate or return an invalid path? */
	struct EInterruptedPathHandling
	{
		enum Type
		{
			Truncate,
			ReturnInvalid
		};
	};

	/**
	 * Make a non-weak WidgetPath out of this WeakWidgetPath. Do this by computing all the relevant geometries and converting the weak pointers to TSharedPtr.
	 *
	 * @param InterruptedPathHandling  Should interrupted paths result in a truncated path or an invalid path
	 */
	SLATECORE_API FWidgetPath ToWidgetPath( EInterruptedPathHandling::Type InterruptedPathHandling = EInterruptedPathHandling::Truncate, const FPointerEvent* PointerEvent = nullptr, const EVisibility VisibilityFilter = EVisibility::Visible) const;

	/**
	 * Make a non-weak WidgetPath out of this WeakWidgetPath. Do this by computing all the relevant geometries and converting the weak pointers to TSharedPtr.
	 *
	 * @param InterruptedPathHandling  Should interrupted paths result in a truncated path or an invalid path
	 */
	SLATECORE_API TSharedRef<FWidgetPath> ToWidgetPathRef( EInterruptedPathHandling::Type InterruptedPathHandling = EInterruptedPathHandling::Truncate, const FPointerEvent* PointerEvent = nullptr, const EVisibility VisibilityFilter = EVisibility::Visible) const;

	struct EPathResolutionResult
	{
		enum Result
		{
			Live,
			Truncated
		};
	};

	/** @return true if the WidgetToFind is in this WidgetPath, false otherwise. */
	SLATECORE_API bool ContainsWidget(const SWidget* WidgetToFind) const;

	/**
	 * Make a non-weak WidgetPath out of this WeakWidgetPath. Do this by computing all the relevant geometries and converting the weak pointers to TSharedPtr.
	 *
	 * @param WidgetPath				The non-weak path is returned via this.
	 * @param InterruptedPathHandling	Should interrupted paths result in a truncated path or an invalid path.
	 * @return Whether the path is truncated or live - a live path refers to a widget that is currently active and visible, a widget with a truncated path is not.
	 */
	SLATECORE_API EPathResolutionResult::Result ToWidgetPath( FWidgetPath& WidgetPath, EInterruptedPathHandling::Type InterruptedPathHandling = EInterruptedPathHandling::Truncate, const FPointerEvent* PointerEvent = nullptr, const EVisibility VisibilityFilter = EVisibility::Visible) const;

	/**
	 * @param NavigationType      Direction in which to move the focus (only for use with EUINavigation::Next and EUINavigation::Previous).
	 * 
	 * @return The widget path to the resulting widget
	 */
	SLATECORE_API FWidgetPath ToNextFocusedPath(EUINavigation NavigationType) const;

	/**
	 * @param NavigationType      Direction in which to move the focus (only for use with EUINavigation::Next and EUINavigation::Previous).
	 * @param NavigationReply	  The NavigationReply that the RuleWidget provided during the bubbled navigation event
	 * @param RuleWidget		  The ArrangedWidget or the widget that provided the NavigationReply 
	 * 
	 * @return The widget path to the resulting widget
	 */
	SLATECORE_API FWidgetPath ToNextFocusedPath(EUINavigation NavigationType, const FNavigationReply& NavigationReply, const FArrangedWidget& RuleWidget) const;
	
	/** Get the last (leaf-most) widget in this path; assumes path is valid */
	TWeakPtr< SWidget > GetLastWidget() const
	{
		check( IsValid() );
		return Widgets[Widgets.Num()-1];
	}
	
	/** A valid path has at least one widget in it */
	bool IsValid() const { return Widgets.Num() > 0; }

	TArray< TWeakPtr<SWidget> > Widgets;
	TWeakPtr< SWindow > Window;
};

#include "Layout/WidgetPath.inl" // IWYU pragma: export
