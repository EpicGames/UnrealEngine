// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FArrangedChildren;
class FArrangedWidget;
class FWidgetPath;
struct EVisibility;


/**
 * Utility function to search recursively through a widget hierarchy for a specific widget
 *
 * @param  Matcher          Some struct that has a "bool IsMatch( const TSharedRef<const SWidget>& InWidget ) const" method
 * @param  InCandidate      The current widget-geometry pair we're testing
 * @param  OutReversedPath  The resulting path in reversed order (canonical order is Windows @ index 0, Leafmost widget is last.)
 * @param  VisibilityFilter	Widgets must have this type of visibility to be included the path
 * @param  SearchPurpose	The purpose for searching for the widget in the path
 *
 * @return  true if the child widget was found; false otherwise
 */
template<typename MatchRuleType>
bool FWidgetPath::SearchForWidgetRecursively( const MatchRuleType& MatchRule, const FArrangedWidget& InCandidate, FArrangedChildren& OutReversedPath, EVisibility VisibilityFilter, EWidgetPathSearchPurpose SearchPurpose )
{
	// If we are searching in order to move focus and we want to find a visible widget but the current widget is disabled, we do not want to search through it's children.
	if (SearchPurpose == EWidgetPathSearchPurpose::FocusHandling && VisibilityFilter == EVisibility::Visible && !InCandidate.Widget->IsEnabled())
	{
		return false;
	}

	const bool bAllow3DWidgets = true;
	const bool bUpdateVisibilityAttributes = SearchPurpose == EWidgetPathSearchPurpose::FocusHandling ? false : true;
	FArrangedChildren ArrangedChildren(VisibilityFilter, bAllow3DWidgets);
	InCandidate.Widget->ArrangeChildren( InCandidate.Geometry, ArrangedChildren, bUpdateVisibilityAttributes );

	for( int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex )
	{
		const FArrangedWidget& SomeChild = ArrangedChildren[ChildIndex];
		if ( MatchRule.IsMatch(SomeChild.Widget) )
		{
			OutReversedPath.AddWidget( SomeChild );
			return true;
		}
		else if ( SearchForWidgetRecursively( MatchRule, SomeChild, OutReversedPath, VisibilityFilter, SearchPurpose) )
		{
			OutReversedPath.AddWidget( SomeChild );
			return true;
		}
	}

	return false;
}

/** Identical to SearchForWidgetRecursively, but iterates in reverse order */
template<typename MatchRuleType>
bool FWidgetPath::SearchForWidgetRecursively_Reverse( const MatchRuleType& MatchRule, const FArrangedWidget& InCandidate, FArrangedChildren& OutReversedPath, EVisibility VisibilityFilter, EWidgetPathSearchPurpose SearchPurpose )
{
	// If we are searching in order to move focus and we want to find a visible widget but the current widget is disabled, we do not want to search through it's children.
	if (SearchPurpose == EWidgetPathSearchPurpose::FocusHandling && VisibilityFilter == EVisibility::Visible && !InCandidate.Widget->IsEnabled())
	{
		return false;
	}

	const bool bAllow3DWidgets = true;
	const bool bUpdateVisibilityAttributes = SearchPurpose == EWidgetPathSearchPurpose::FocusHandling ? false : true;
	FArrangedChildren ArrangedChildren(VisibilityFilter, bAllow3DWidgets);
	InCandidate.Widget->ArrangeChildren( InCandidate.Geometry, ArrangedChildren, bUpdateVisibilityAttributes );

	for( int32 ChildIndex = ArrangedChildren.Num()-1; ChildIndex >=0 ; --ChildIndex )
	{
		const FArrangedWidget& SomeChild = ArrangedChildren[ChildIndex];
		if ( MatchRule.IsMatch(SomeChild.Widget) )
		{
			OutReversedPath.AddWidget( SomeChild );
			return true;
		}
		else if ( SearchForWidgetRecursively_Reverse( MatchRule, SomeChild, OutReversedPath, VisibilityFilter, SearchPurpose) )
		{
			OutReversedPath.AddWidget( SomeChild );
			return true;
		}
	}

	return false;
}
