// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlotBase.h"
#include "Layout/ChildrenBase.h"
#include "Types/SlateAttribute.h"
#include "Types/SlateAttributeDescriptor.h"

#include <limits>


/**
 * A base slot that supports TSlateContainedAttribute.
 * The FChildren that own the FSlot also needs to support SlateContainedAttribute.
 * @see FChildren::SupportSlotWithSlateAttribute
 */
template <typename SlotType>
class TWidgetSlotWithAttributeSupport : public TSlotBase<SlotType>, public SlateAttributePrivate::ISlateAttributeContainer
{
private:
	using Super = TSlotBase<SlotType>;
protected:
	using TSlotBase<SlotType>::TSlotBase;

	/**
	 * A SlateAttribute that is member variable of a FSlot.
	 * @usage: TSlateSlotAttribute<int32> MyAttribute1; TSlateSlotAttribute<int32, TSlateAttributeComparePredicate<>> MyAttribute2;
	 */
	template<typename InObjectType, typename InComparePredicate = TSlateAttributeComparePredicate<>>
	struct TSlateSlotAttribute : public ::SlateAttributePrivate::TSlateContainedAttribute<InObjectType, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, InComparePredicate>
	{
		using ::SlateAttributePrivate::TSlateContainedAttribute<InObjectType, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, InComparePredicate>::TSlateContainedAttribute;
	};

public:
	virtual ~TWidgetSlotWithAttributeSupport()
	{
		if (SWidget* WidgetPtr = Super::GetOwnerWidget())
		{
			RemoveContainerWidget(*WidgetPtr);
		}
	}

	void Construct(const FChildren& SlotOwner, typename Super::FSlotArguments&& InArgs)
	{
		checkf(SlotOwner.SupportSlotWithSlateAttribute(), TEXT("The FChildren '%s' does not support SlateSlotAttribute but use a Slot type that does."), *SlotOwner.GetName().ToString());
		checkf(SlotOwner.GetName().IsValid(), TEXT("The FChildren '%s' does not have a valid name."), *SlotOwner.GetName().ToString());
		Super::Construct(SlotOwner, MoveTemp(InArgs));
	}

	static void RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer)
	{
	}

	void RequestSortAttribute()
	{
		SWidget* WidgetPtr = Super::GetOwnerWidget();
		ensureAlwaysMsgf(WidgetPtr, TEXT("FSlot needs to be constructed before we modify the FChildren."));
		if (WidgetPtr)
		{
			UpdateContainerSortOrder(*WidgetPtr);
		}
	}


	//~ Begin ISlateAttributeContainer interface
	virtual SWidget& GetContainerWidget() const override
	{
		SWidget* WidgetPtr = TSlotBase<SlotType>::GetOwnerWidget();
		checkf(WidgetPtr, TEXT("Slot Attributes has to be registered after the FSlot is constructed."));
		return *WidgetPtr;
	}

	virtual FName GetContainerName() const override
	{
		return FSlotBase::GetOwner()->GetName();
	}

	virtual uint32 GetContainerSortOrder() const override
	{
		check(FSlotBase::GetOwner());
		const int32 NumSlot = FSlotBase::GetOwner()->NumSlot();
		for (int32 Index = 0; Index < NumSlot; ++Index)
		{
			if (&FSlotBase::GetOwner()->GetSlotAt(Index) == this)
			{
				return Index;
			}
		}
#pragma push_macro("UNDEF_MAX")
#undef max
		return std::numeric_limits<uint32>::max();
#pragma pop_macro("UNDEF_MAX")
	}
	//~ End ISlateAttributeContainer interface
};
