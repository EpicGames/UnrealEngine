// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "SlateGlobals.h"

/**
 * Note: For types needed even by RenderingCommon.h
 */

struct FSlateBoxElement;
struct FSlateBoxElement; // ET_DebugQuad
struct FSlateTextElement;
struct FSlateShapedTextElement;
struct FSlateSplineElement;
struct FSlateLineElement;
struct FSlateGradientElement;
struct FSlateViewportElement;
struct FSlateBoxElement; // ET_Border
struct FSlateCustomDrawerElement;
struct FSlateCustomVertsElement;
struct FSlatePostProcessElement;
struct FSlateRoundedBoxElement;
class FSlateDrawElement;

DECLARE_MEMORY_STAT_EXTERN(TEXT("Vertex/Index Buffer Pool Memory (CPU)"), STAT_SlateBufferPoolMemory, STATGROUP_SlateMemory, SLATECORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("Cached Draw Element Memory (CPU)"), STAT_SlateCachedDrawElementMemory, STATGROUP_SlateMemory, SLATECORE_API);

/**
 * Slate higher level abstract element types
 */
enum class EElementType : uint8
{
	ET_Box,
	ET_DebugQuad,
	ET_Text,
	ET_ShapedText,
	ET_Spline,
	ET_Line,
	ET_Gradient,
	ET_Viewport,
	ET_Border,
	ET_Custom,
	ET_CustomVerts,
	ET_PostProcessPass,
	ET_RoundedBox,
	/**
	 * We map draw elements by type on add for better cache coherency if possible,
	 * this type is used when that grouping is not possible.
	 * Grouping is also planned to be used for bulk element type processing.
	 */
	ET_NonMapped,
	/** Total number of draw commands */
	ET_Count,
};

#if STATS

struct FRenderingBufferStatTracker
{
	static void MemoryAllocated(int32 SizeBytes)
	{
		INC_DWORD_STAT_BY(STAT_SlateBufferPoolMemory, SizeBytes);
	}

	static void MemoryFreed(int32 SizeBytes)
	{
		DEC_DWORD_STAT_BY(STAT_SlateBufferPoolMemory, SizeBytes);
	}
};

struct FDrawElementStatTracker
{
	static void MemoryAllocated(int32 SizeBytes)
	{
		INC_DWORD_STAT_BY(STAT_SlateCachedDrawElementMemory, SizeBytes);
	}

	static void MemoryFreed(int32 SizeBytes)
	{
		DEC_DWORD_STAT_BY(STAT_SlateCachedDrawElementMemory, SizeBytes);
	}
};

template<typename StatTracker>
class FSlateStatTrackingMemoryAllocator : public FDefaultAllocator
{
public:
	typedef FDefaultAllocator Super;

	class ForAnyElementType : public FDefaultAllocator::ForAnyElementType
	{
	public:
		typedef FDefaultAllocator::ForAnyElementType Super;

		ForAnyElementType()
			: AllocatedSize(0)
		{

		}

		/**
		* Moves the state of another allocator into this one.
		* Assumes that the allocator is currently empty, i.e. memory may be allocated but any existing elements have already been destructed (if necessary).
		* @param Other - The allocator to move the state from.  This allocator should be left in a valid empty state.
		*/
		inline void MoveToEmpty(ForAnyElementType& Other)
		{
			Super::MoveToEmpty(Other);

			AllocatedSize = Other.AllocatedSize;
			Other.AllocatedSize = 0;
		}

		/** Destructor. */
		~ForAnyElementType()
		{
			if (AllocatedSize)
			{
				StatTracker::MemoryFreed(AllocatedSize);
			}
		}

		void ResizeAllocation(int32 PreviousNumElements, int32 NumElements, int32 NumBytesPerElement)
		{
			const int32 NewSize = NumElements * NumBytesPerElement;
			StatTracker::MemoryAllocated(NewSize - AllocatedSize);

			AllocatedSize = NewSize;

			Super::ResizeAllocation(PreviousNumElements, NumElements, NumBytesPerElement);
		}

	private:
		ForAnyElementType(const ForAnyElementType&);
		ForAnyElementType& operator=(const ForAnyElementType&);
	private:
		int32 AllocatedSize;
	};
};

template <typename T>
struct TAllocatorTraits<FSlateStatTrackingMemoryAllocator<T>> : TAllocatorTraitsBase<FSlateStatTrackingMemoryAllocator<T>>
{
	enum { IsZeroConstruct = TAllocatorTraits<FDefaultAllocator>::IsZeroConstruct };
};

#endif // STATS

#if STATS

template <typename DrawElementType>
using FSlateDrawElementArray = TArray<DrawElementType, FSlateStatTrackingMemoryAllocator<FDrawElementStatTracker>>;

/** Tuple / List of element arrays for each element type */
using FSlateDrawElementMap = TTuple<
	FSlateDrawElementArray<FSlateBoxElement>
	, FSlateDrawElementArray<FSlateBoxElement>
	, FSlateDrawElementArray<FSlateTextElement>
	, FSlateDrawElementArray<FSlateShapedTextElement>
	, FSlateDrawElementArray<FSlateSplineElement>
	, FSlateDrawElementArray<FSlateLineElement>
	, FSlateDrawElementArray<FSlateGradientElement>
	, FSlateDrawElementArray<FSlateViewportElement>
	, FSlateDrawElementArray<FSlateBoxElement>
	, FSlateDrawElementArray<FSlateCustomDrawerElement>
	, FSlateDrawElementArray<FSlateCustomVertsElement>
	, FSlateDrawElementArray<FSlatePostProcessElement>
	, FSlateDrawElementArray<FSlateRoundedBoxElement>
	, FSlateDrawElementArray<FSlateDrawElement>>;

#else

template <typename DrawElementType>
using FSlateDrawElementArray = TArray<DrawElementType>;

/** Tuple / List of element arrays for each element type */
using FSlateDrawElementMap = TTuple<
	FSlateDrawElementArray<FSlateBoxElement>
	, FSlateDrawElementArray<FSlateBoxElement>
	, FSlateDrawElementArray<FSlateTextElement>
	, FSlateDrawElementArray<FSlateShapedTextElement>
	, FSlateDrawElementArray<FSlateSplineElement>
	, FSlateDrawElementArray<FSlateLineElement>
	, FSlateDrawElementArray<FSlateGradientElement>
	, FSlateDrawElementArray<FSlateViewportElement>
	, FSlateDrawElementArray<FSlateBoxElement>
	, FSlateDrawElementArray<FSlateCustomDrawerElement>
	, FSlateDrawElementArray<FSlateCustomVertsElement>
	, FSlateDrawElementArray<FSlatePostProcessElement>
	, FSlateDrawElementArray<FSlateRoundedBoxElement>
	, FSlateDrawElementArray<FSlateDrawElement>>;

#endif // STATS

/** Helper template to get draw element type in 'FSlateDrawElementMap' for a given element type enum, used to avoid 'auto' return values */
template<EElementType ElementType>
using TSlateDrawElement = typename TTupleElement<(uint8)ElementType, FSlateDrawElementMap>::Type::ElementType;