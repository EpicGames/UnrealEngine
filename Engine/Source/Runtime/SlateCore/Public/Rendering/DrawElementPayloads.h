// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DrawElementTextOverflowArgs.h"
#include "Fonts/FontCache.h"
#include "Fonts/ShapedTextFwd.h"
#include "SlateRenderBatch.h"
#include "Styling/SlateBrush.h"
#include "Types/SlateVector2.h"

#include "Rendering/RenderingCommon.h"
#include "Rendering/DrawElementTypes.h"

//////////////////////////////////////////////////////////////////////////
// Deprecated payloads
//////////////////////////////////////////////////////////////////////////

PRAGMA_DISABLE_DEPRECATION_WARNINGS

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateDataPayload
{
	virtual ~FSlateDataPayload() {}
	virtual void AddReferencedObjects(FReferenceCollector& Collector) {}
};

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateBoxPayload : public FSlateDataPayload, public FSlateTintableElement
{
	FMargin Margin;
	FBox2f UVRegion;
	const FSlateShaderResourceProxy* ResourceProxy;
	ESlateBrushTileType::Type Tiling;
	ESlateBrushMirrorType::Type Mirroring;
	ESlateBrushDrawType::Type DrawType;

	const FMargin& GetBrushMargin() const { return Margin; }
	const FBox2f& GetBrushUVRegion() const { return UVRegion; }
	ESlateBrushTileType::Type GetBrushTiling() const { return Tiling; }
	ESlateBrushMirrorType::Type GetBrushMirroring() const { return Mirroring; }
	ESlateBrushDrawType::Type GetBrushDrawType() const { return DrawType; }
	const FSlateShaderResourceProxy* GetResourceProxy() const { return ResourceProxy; }

	void SetBrush(const FSlateBrush* InBrush, UE::Slate::FDeprecateVector2DParameter LocalSize, float DrawScale)
	{
		check(InBrush);
		ensureMsgf(InBrush->GetDrawType() != ESlateBrushDrawType::NoDrawType, TEXT("This should have been filtered out earlier in the Make... call."));

		// Note: Do not store the brush.  It is possible brushes are destroyed after an element is enqueued for rendering
		Margin = InBrush->GetMargin();
		UVRegion = InBrush->GetUVRegion();
		Tiling = InBrush->GetTiling();
		Mirroring = InBrush->GetMirroring();
		DrawType = InBrush->GetDrawType();
		const FSlateResourceHandle& Handle = InBrush->GetRenderingResource(LocalSize, DrawScale);
		if (Handle.IsValid())
		{
			ResourceProxy = Handle.GetResourceProxy();
		}
		else
		{
			ResourceProxy = nullptr;
		}
	}

	FORCENOINLINE virtual ~FSlateBoxPayload()
	{
	}
};

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateRoundedBoxPayload : public FSlateBoxPayload
{
	FLinearColor OutlineColor;
	FVector4f Radius;
	float OutlineWeight;

	FORCEINLINE void SetRadius(FVector4f InRadius) { Radius = InRadius; }
	FORCEINLINE FVector4f GetRadius() const { return Radius; }

	FORCEINLINE void SetOutline(const FLinearColor& InOutlineColor, float InOutlineWeight) { OutlineColor = InOutlineColor; OutlineWeight = InOutlineWeight; }
	FORCEINLINE FLinearColor GetOutlineColor() const { return OutlineColor; }
	FORCEINLINE float GetOutlineWeight() const { return OutlineWeight; }
};

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateTextPayload : public FSlateDataPayload, public FSlateTintableElement
{
	// The font to use when rendering
	FSlateFontInfo FontInfo;
	// Basic text data
	FString ImmutableText;

	const FSlateFontInfo& GetFontInfo() const { return FontInfo; }
	const TCHAR* GetText() const { return *ImmutableText; }
	int32 GetTextLength() const { return ImmutableText.Len(); }

	void SetText(const FString& InText, const FSlateFontInfo& InFontInfo, int32 InStartIndex, int32 InEndIndex)
	{
		FontInfo = InFontInfo;
		const int32 StartIndex = FMath::Min<int32>(InStartIndex, InText.Len());
		const int32 EndIndex = FMath::Min<int32>(InEndIndex, InText.Len());
		const int32 TextLength = (EndIndex > StartIndex) ? EndIndex - StartIndex : 0;
		if (TextLength > 0)
		{
			ImmutableText = InText.Mid(StartIndex, TextLength);
		}
	}

	void SetText(const FString& InText, const FSlateFontInfo& InFontInfo)
	{
		FontInfo = InFontInfo;
		ImmutableText = InText;
	}


	virtual void AddReferencedObjects(FReferenceCollector& Collector)
	{
		FontInfo.AddReferencedObjects(Collector);
	}
};

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateShapedTextPayload : public FSlateDataPayload, public FSlateTintableElement
{
	// Shaped text data
	FShapedGlyphSequencePtr ShapedGlyphSequence;

	FLinearColor OutlineTint;

	FTextOverflowArgs OverflowArgs;

	const FShapedGlyphSequencePtr& GetShapedGlyphSequence() const { return ShapedGlyphSequence; }
	FLinearColor GetOutlineTint() const { return OutlineTint; }

	void SetShapedText(FSlateWindowElementList& ElementList, const FShapedGlyphSequencePtr& InShapedGlyphSequence, const FLinearColor& InOutlineTint)
	{
		ShapedGlyphSequence = InShapedGlyphSequence;
		OutlineTint = InOutlineTint;
	}

	void SetOverflowArgs(const FTextOverflowArgs& InArgs)
	{
		OverflowArgs = InArgs;
		check(InArgs.OverflowDirection == ETextOverflowDirection::NoOverflow || InArgs.OverflowTextPtr.IsValid());
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector)
	{
		if (ShapedGlyphSequence.IsValid())
		{
			const_cast<FShapedGlyphSequence*>(ShapedGlyphSequence.Get())->AddReferencedObjects(Collector);
		}

		if (OverflowArgs.OverflowTextPtr.IsValid())
		{
			const_cast<FShapedGlyphSequence*>(OverflowArgs.OverflowTextPtr.Get())->AddReferencedObjects(Collector);
		}
	}
};

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateGradientPayload : public FSlateDataPayload
{
	TArray<FSlateGradientStop> GradientStops;
	EOrientation GradientType;
	FVector4f CornerRadius;

	void SetGradient(TArray<FSlateGradientStop> InGradientStops, EOrientation InGradientType, FVector4f InCornerRadius)
	{
		GradientStops = MoveTemp(InGradientStops);
		GradientType = InGradientType;
		CornerRadius = InCornerRadius;
	}
};

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateSplinePayload : public FSlateDataPayload, public FSlateTintableElement
{
	TArray<FSlateGradientStop> GradientStops;
	// Bezier Spline Data points. E.g.
	//
	//       P1 + - - - - + P2                P1 +
	//         /           \                    / \
	//     P0 *             * P3            P0 *   \   * P3
	//                                              \ /
	//                                               + P2	
	FVector2f P0;
	FVector2f P1;
	FVector2f P2;
	FVector2f P3;

	float Thickness;

	// Thickness
	void SetThickness(float InThickness) { Thickness = InThickness; }
	float GetThickness() const { return Thickness; }

	void SetCubicBezier(const UE::Slate::FDeprecateVector2DParameter InP0, const UE::Slate::FDeprecateVector2DParameter InP1, const UE::Slate::FDeprecateVector2DParameter InP2, const UE::Slate::FDeprecateVector2DParameter InP3, float InThickness, const FLinearColor InTint)
	{
		Tint = InTint;
		P0 = InP0;
		P1 = InP1;
		P2 = InP2;
		P3 = InP3;
		Thickness = InThickness;
	}

	void SetHermiteSpline(const UE::Slate::FDeprecateVector2DParameter InStart, const UE::Slate::FDeprecateVector2DParameter InStartDir, const UE::Slate::FDeprecateVector2DParameter InEnd, const UE::Slate::FDeprecateVector2DParameter InEndDir, float InThickness, const FLinearColor InTint)
	{
		Tint = InTint;
		P0 = InStart;
		P1 = InStart + InStartDir / 3.0f;
		P2 = InEnd - InEndDir / 3.0f;
		P3 = InEnd;
		Thickness = InThickness;
	}

	void SetGradientHermiteSpline(const UE::Slate::FDeprecateVector2DParameter InStart, const UE::Slate::FDeprecateVector2DParameter InStartDir, const UE::Slate::FDeprecateVector2DParameter InEnd, const UE::Slate::FDeprecateVector2DParameter InEndDir, float InThickness, TArray<FSlateGradientStop> InGradientStops)
	{
		P0 = InStart;
		P1 = InStart + InStartDir / 3.0f;
		P2 = InEnd - InEndDir / 3.0f;
		P3 = InEnd;
		Thickness = InThickness;
		GradientStops = MoveTemp(InGradientStops);
	}
};


struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateLinePayload : public FSlateDataPayload, public FSlateTintableElement
{
	TArray<FVector2f> Points;
	TArray<FLinearColor> PointColors;
	float Thickness;

	bool bAntialias;

	bool IsAntialiased() const { return bAntialias; }
	const TArray<FVector2f>& GetPoints() const { return Points; }
	const TArray<FLinearColor>& GetPointColors() const { return PointColors; }
	float GetThickness() const { return Thickness; }

	void SetThickness(float InThickness)
	{
		Thickness = InThickness;
	}

#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS
	void SetLines(const TArray<FVector2D>& InPoints, bool bInAntialias, const TArray<FLinearColor>* InPointColors = nullptr)
	{
		TArray<FVector2f> NewPoints;
		NewPoints.Reserve(InPoints.Num());
		for (FVector2D Vect : InPoints)
		{
			NewPoints.Add(UE::Slate::CastToVector2f(Vect));
		}
		if (InPointColors)
		{
			SetLines(MoveTemp(NewPoints), bInAntialias, *InPointColors);
		}
		else
		{
			SetLines(MoveTemp(NewPoints), bInAntialias);
		}
	}
#endif

	void SetLines(TArray<FVector2f> InPoints, bool bInAntialias)
	{
		bAntialias = bInAntialias;
		Points = MoveTemp(InPoints);
		PointColors.Reset();
	}

	void SetLines(TArray<FVector2f> InPoints, bool bInAntialias, TArray<FLinearColor> InPointColors)
	{
		bAntialias = bInAntialias;
		Points = MoveTemp(InPoints);
		PointColors = MoveTemp(InPointColors);
	}
};

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateViewportPayload : public FSlateDataPayload, public FSlateTintableElement
{
	FSlateShaderResource* RenderTargetResource;
	uint8 bAllowViewportScaling : 1;
	uint8 bViewportTextureAlphaOnly : 1;
	uint8 bRequiresVSync : 1;

	void SetViewport(const TSharedPtr<const ISlateViewport>& InViewport, const FLinearColor& InTint)
	{
		Tint = InTint;
		RenderTargetResource = InViewport->GetViewportRenderTargetTexture();
		bAllowViewportScaling = InViewport->AllowScaling();
		bViewportTextureAlphaOnly = InViewport->IsViewportTextureAlphaOnly();
		bRequiresVSync = InViewport->RequiresVsync();
	}
};

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateCustomDrawerPayload : public FSlateDataPayload
{
	// Custom drawer data
	TWeakPtr<ICustomSlateElement, ESPMode::ThreadSafe> CustomDrawer;

	void SetCustomDrawer(const TSharedPtr<ICustomSlateElement, ESPMode::ThreadSafe>& InCustomDrawer)
	{
		CustomDrawer = InCustomDrawer;
	}
};

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateLayerPayload : public FSlateDataPayload
{
	class FSlateDrawLayerHandle* LayerHandle;

	void SetLayer(FSlateDrawLayerHandle* InLayerHandle)
	{
		LayerHandle = InLayerHandle;
		checkSlow(LayerHandle);
	}

};

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateCachedBufferPayload : public FSlateDataPayload
{
	// Cached render data
	class FSlateRenderDataHandle* CachedRenderData;
	FVector2f CachedRenderDataOffset;

	// Cached Buffers
	void SetCachedBuffer(FSlateRenderDataHandle* InRenderDataHandle, const UE::Slate::FDeprecateVector2DParameter Offset)
	{
		check(InRenderDataHandle);

		CachedRenderData = InRenderDataHandle;
		CachedRenderDataOffset = Offset;
	}
};

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlateCustomVertsPayload : public FSlateDataPayload
{
	const FSlateShaderResourceProxy* ResourceProxy;

	TArray<FSlateVertex> Vertices;
	TArray<SlateIndex> Indices;

	// Instancing support
	ISlateUpdatableInstanceBufferRenderProxy* InstanceData;
	uint32 InstanceOffset;
	uint32 NumInstances;

	void SetCustomVerts(const FSlateShaderResourceProxy* InRenderProxy, TArray<FSlateVertex> InVerts, TArray<SlateIndex> InIndices, ISlateUpdatableInstanceBufferRenderProxy* InInstanceData, uint32 InInstanceOffset, uint32 InNumInstances)
	{
		ResourceProxy = InRenderProxy;

		Vertices = MoveTemp(InVerts);
		Indices = MoveTemp(InIndices);

		InstanceData = InInstanceData;
		InstanceOffset = InInstanceOffset;
		NumInstances = InNumInstances;
	}
};

struct UE_DEPRECATED(5.3, "Draw Element Payloads are no longer used, instead use the equivalent FSlateDrawElement subclass") FSlatePostProcessPayload : public FSlateDataPayload
{
	// Post Process Data
	FVector4f PostProcessData;
	FVector4f CornerRadius;
	int32 DownsampleAmount;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS