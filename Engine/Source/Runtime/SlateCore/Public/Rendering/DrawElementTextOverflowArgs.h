// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Fonts/ShapedTextFwd.h"
#include "Styling/SlateTypes.h"

enum class ETextOverflowDirection : uint8
{
	// No overflow
	NoOverflow,
	// Left justification overflow
	LeftToRight,
	// Right justification overflow
	RightToLeft
};

struct FTextOverflowArgs
{
	FTextOverflowArgs(FShapedGlyphSequencePtr& InOverflowText, ETextOverflowDirection InOverflowDirection, ETextOverflowPolicy InOverflowPolicy)
		: OverflowTextPtr(InOverflowText)
		, OverflowDirection(InOverflowDirection)
		, OverflowPolicy(InOverflowPolicy)
		, bIsLastVisibleBlock(false)
		, bIsNextBlockClipped(false)
	{}

	FTextOverflowArgs(FShapedGlyphSequencePtr& InOverflowText, ETextOverflowDirection InOverflowDirection)
		: OverflowTextPtr(InOverflowText)
		, OverflowDirection(InOverflowDirection)
		, OverflowPolicy(ETextOverflowPolicy::Clip)
		, bIsLastVisibleBlock(false)
		, bIsNextBlockClipped(false)
	{}

	FTextOverflowArgs()
		: OverflowDirection(ETextOverflowDirection::NoOverflow)
		, OverflowPolicy(ETextOverflowPolicy::Clip)
		, bIsLastVisibleBlock(false)
		, bIsNextBlockClipped(false)
	{}

	/** Sequence that represents the ellipsis glyph */
	FShapedGlyphSequencePtr OverflowTextPtr;
	ETextOverflowDirection OverflowDirection;
	ETextOverflowPolicy OverflowPolicy;
	bool bIsLastVisibleBlock;
	bool bIsNextBlockClipped;
};
