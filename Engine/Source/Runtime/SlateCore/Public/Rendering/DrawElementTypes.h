// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DrawElementTextOverflowArgs.h"
#include "Fonts/FontCache.h"
#include "Fonts/ShapedTextFwd.h"
#include "SlateRenderBatch.h"
#include "Styling/SlateBrush.h"
#include "Types/SlateVector2.h"

#include "UObject/GCObject.h"
#include "Fonts/ShapedTextFwd.h"
#include "Misc/MemStack.h"
#include "Styling/WidgetStyle.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/SlateRect.h"
#include "Layout/Clipping.h"
#include "Types/PaintArgs.h"
#include "Types/SlateVector2.h"
#include "Layout/Geometry.h"
#include "Rendering/RenderingCommon.h"
#include "Debugging/SlateDebugging.h"
#include "Rendering/SlateRenderBatch.h"
#include "DrawElementTextOverflowArgs.h"
#include "ElementBatcher.h"
#include "Widgets/WidgetPixelSnapping.h"
#include "Types/SlateVector2.h"

class FSlateRenderBatch;
class FSlateDrawLayerHandle;
class FSlateResourceHandle;
class FSlateWindowElementList;
class SWidget;
class SWindow;
struct FSlateBrush;
struct FSlateDataPayload;
struct FSlateGradientStop;

/**
 * FSlateDrawElement is the building block for Slate's rendering interface.
 * Slate describes its visual output as an ordered list of FSlateDrawElement s
 */
class FSlateDrawElement
{
	friend class FSlateWindowElementList;
public:
	
	enum ERotationSpace
	{
		/** Relative to the element.  (0,0) is the upper left corner of the element */
		RelativeToElement,
		/** Relative to the alloted paint geometry.  (0,0) is the upper left corner of the paint geometry */
		RelativeToWorld,
	};

	/**
	 * Creates a solid quad for debug purposes
	 *
	 * @param ElementList			The list in which to add elements
	 * @param InLayer				The layer to draw the element on
	 * @param PaintGeometry         DrawSpace position and dimensions; see FPaintGeometry
	 * @param Tint					The color to tint the quad
	 */
	SLATECORE_API static void MakeDebugQuad(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, FLinearColor Tint = FLinearColor::White);

	/**
	 * Creates an outline of the specific Paint Geometry
	 *
	 * @param ElementList			The list in which to add elements
	 * @param InLayer				The layer to draw the element on
	 * @param PaintGeometry         DrawSpace position and dimensions; see FPaintGeometry
	 * @param Tint					The color to tint the outline
	 * @param InDrawEffects			Which effects to apply to the outline
	 * @param bAntialias			Whether to anti-alias the outline
	 * @param Thickness				How thick the outline should be
	 */
	SLATECORE_API static void MakeGeometryOutline(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, FLinearColor Tint = FLinearColor::White, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, bool bAntialias = true, float Thickness = 1.0f);

	/**
	 * Creates a box element based on the following diagram.  Allows for this element to be resized while maintain the border of the image
	 * If there are no margins the resulting box is simply a quad
	 *     ___LeftMargin    ___RightMargin
	 *    /                /
	 *  +--+-------------+--+
	 *  |  |c1           |c2| ___TopMargin
	 *  +--o-------------o--+
	 *  |  |             |  |
	 *  |  |c3           |c4|
	 *  +--o-------------o--+
	 *  |  |             |  | ___BottomMargin
	 *  +--+-------------+--+
	 *
	 * @param ElementList			The list in which to add elements
	 * @param InLayer               The layer to draw the element on
	 * @param PaintGeometry         DrawSpace position and dimensions; see FPaintGeometry
	 * @param InBrush               Brush to apply to this element
	 * @param InDrawEffects         Optional draw effects to apply
	 * @param InTint                Color to tint the element
	 */
	SLATECORE_API static void MakeBox( 
		FSlateWindowElementList& ElementList,
		uint32 InLayer,
		const FPaintGeometry& PaintGeometry,
		const FSlateBrush* InBrush,
		ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None,
		const FLinearColor& InTint = FLinearColor::White );

	SLATECORE_API static void MakeRotatedBox(
		FSlateWindowElementList& ElementList,
		uint32 InLayer, 
		const FPaintGeometry& PaintGeometry, 
		const FSlateBrush* InBrush, 
		ESlateDrawEffect,
		float Angle,
		UE::Slate::FDeprecateOptionalVector2DParameter InRotationPoint = TOptional<FVector2f>(),
		ERotationSpace RotationSpace = RelativeToElement,
		const FLinearColor& InTint = FLinearColor::White );

	/**
	 * Creates a text element which displays a string of a rendered in a certain font on the screen
	 *
	 * @param ElementList			The list in which to add elements
	 * @param InLayer               The layer to draw the element on
	 * @param PaintGeometry         DrawSpace position and dimensions; see FPaintGeometry
	 * @param InText                The string to draw
	 * @param StartIndex            Inclusive index to start rendering from on the specified text
	 * @param EndIndex				Exclusive index to stop rendering on the specified text
	 * @param InFontInfo            The font to draw the string with
	 * @param InDrawEffects         Optional draw effects to apply
	 * @param InTint                Color to tint the element
	 */
	SLATECORE_API static void MakeText( FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FString& InText, const int32 StartIndex, const int32 EndIndex, const FSlateFontInfo& InFontInfo, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint = FLinearColor::White );
	
	SLATECORE_API static void MakeText( FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FString& InText, const FSlateFontInfo& InFontInfo, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint = FLinearColor::White );

	inline static void MakeText(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FText& InText, const FSlateFontInfo& InFontInfo, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint = FLinearColor::White)
	{
		MakeText(ElementList, InLayer, PaintGeometry, InText.ToString(), InFontInfo, InDrawEffects, InTint);
	}

	/**
	 * Creates a text element which displays a series of shaped glyphs on the screen
	 *
	 * @param ElementList			The list in which to add elements
	 * @param InLayer               The layer to draw the element on
	 * @param PaintGeometry         DrawSpace position and dimensions; see FPaintGeometry
	 * @param InShapedGlyphSequence The shaped glyph sequence to draw
	 * @param InDrawEffects         Optional draw effects to apply
	 * @param InTint                Color to tint the element
	 */
	SLATECORE_API static void MakeShapedText( FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FShapedGlyphSequenceRef& InShapedGlyphSequence, ESlateDrawEffect InDrawEffects, const FLinearColor& BaseTint, const FLinearColor& OutlineTint, FTextOverflowArgs TextOverflowArgs = FTextOverflowArgs());

	/**
	 * Creates a text element which displays a series of shaped glyphs on the screen
	 *
	 * @param ElementList			The list in which to add elements
	 * @param InLayer               The layer to draw the element on
	 * @param PaintGeometry         DrawSpace position and dimensions; see FPaintGeometry
	 * @param InShapedGlyphSequence The shaped glyph sequence to draw
	 * @param InDrawEffects         Optional draw effects to apply
	 * @param BaseTint              Color to tint the element
	 * @param OutlineTint           Color to tint the outline
	 * @param Angle2D               The rotation of the element in radians
	 */
	SLATECORE_API static void MakeRotatedShapedText(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FShapedGlyphSequenceRef& InShapedGlyphSequence, ESlateDrawEffect InDrawEffects, const FLinearColor& BaseTint, const FLinearColor& OutlineTint, float Angle2D, const UE::Slate::FDeprecateOptionalVector2DParameter& InRotationPoint = TOptional<FVector2f>(), ERotationSpace RotationSpace = RelativeToElement, const FTextOverflowArgs& TextOverflowArgs = FTextOverflowArgs());
	
	/**
	 * Creates a gradient element
	 *
	 * @param ElementList			   The list in which to add elements
	 * @param InLayer                  The layer to draw the element on
	 * @param PaintGeometry            DrawSpace position and dimensions; see FPaintGeometry
	 * @param InGradientStops          List of gradient stops which define the element
	 * @param InGradientType           The type of gradient (I.E Horizontal, vertical)
	 * @param InDrawEffects            Optional draw effects to apply
	 * @param CornerRadius			   Rounds the corners of the box created by the gradient by the specified radius
	 */
	SLATECORE_API static void MakeGradient( FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, TArray<FSlateGradientStop> InGradientStops, EOrientation InGradientType, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, FVector4f CornerRadius = FVector4f(0.0f) );

	/**
	 * Creates a Hermite Spline element
	 *
	 * @param ElementList			The list in which to add elements
	 * @param InLayer               The layer to draw the element on
	 * @param PaintGeometry         DrawSpace position and dimensions; see FPaintGeometry
	 * @param InStart               The start point of the spline (local space)
	 * @param InStartDir            The direction of the spline from the start point
	 * @param InEnd                 The end point of the spline (local space)
	 * @param InEndDir              The direction of the spline to the end point
	 * @param InDrawEffects         Optional draw effects to apply
	 * @param InTint                Color to tint the element
	 */
	SLATECORE_API static void MakeSpline(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const UE::Slate::FDeprecateVector2DParameter InStart, const UE::Slate::FDeprecateVector2DParameter InStartDir, const UE::Slate::FDeprecateVector2DParameter InEnd, const UE::Slate::FDeprecateVector2DParameter InEndDir, float InThickness = 0.0f, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint=FLinearColor::White);

	/**
	 * Creates a Bezier Spline element
	 *
	 * @param ElementList			The list in which to add elements
	 * @param InLayer               The layer to draw the element on
	 * @param PaintGeometry         DrawSpace position and dimensions; see FPaintGeometry
	 * @param InStart               The start point of the spline (local space)
	 * @param InStartDir            The direction of the spline from the start point
	 * @param InEnd                 The end point of the spline (local space)
	 * @param InEndDir              The direction of the spline to the end point
	 * @param InDrawEffects         Optional draw effects to apply
	 * @param InTint                Color to tint the element
	 */
	SLATECORE_API static void MakeCubicBezierSpline(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const UE::Slate::FDeprecateVector2DParameter P0, const UE::Slate::FDeprecateVector2DParameter P1, const UE::Slate::FDeprecateVector2DParameter P2, const UE::Slate::FDeprecateVector2DParameter P3, float InThickness = 0.0f, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint = FLinearColor::White);

	/** Just like MakeSpline but in draw-space coordinates. This is useful for connecting already-transformed widgets together. */
	SLATECORE_API static void MakeDrawSpaceSpline(FSlateWindowElementList& ElementList, uint32 InLayer, const UE::Slate::FDeprecateVector2DParameter InStart, const UE::Slate::FDeprecateVector2DParameter InStartDir, const UE::Slate::FDeprecateVector2DParameter InEnd, const UE::Slate::FDeprecateVector2DParameter InEndDir, float InThickness = 0.0f, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint=FLinearColor::White);

	/**
	 * Creates a line defined by the provided points
	 *
	 * @param ElementList              The list in which to add elements
	 * @param InLayer                  The layer to draw the element on
	 * @param PaintGeometry            DrawSpace position and dimensions; see FPaintGeometry
	 * @param Points                   Points that make up the lines.  The points are joined together. I.E if Points has A,B,C there the line is A-B-C.  To draw non-joining line segments call MakeLines multiple times
	 * @param InDrawEffects            Optional draw effects to apply
	 * @param InTint                   Color to tint the element
	 * @param bAntialias               Should antialiasing be applied to the line?
	 * @param Thickness                The thickness of the line
	 */
#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS
	SLATECORE_API static void MakeLines(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const TArray<FVector2d>& Points, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint=FLinearColor::White, bool bAntialias = true, float Thickness = 1.0f);
#endif
	SLATECORE_API static void MakeLines(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, TArray<FVector2f> Points, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint = FLinearColor::White, bool bAntialias = true, float Thickness = 1.0f);


	/**
	 * Creates a dashed or dotted line defined by the provided points. Such lines are always renderd using the anti-aliased shader method.
	 *
	 * @param ElementList              The list in which to add elements
	 * @param InLayer                  The layer to draw the element on
	 * @param PaintGeometry            DrawSpace position and dimensions; see FPaintGeometry
	 * @param Points                   Points that make up the lines.  The points are joined together. I.E if Points has A,B,C there the line is A-B-C.  To draw non-joining line segments call MakeLines multiple times
	 * @param InDrawEffects            Optional draw effects to apply
	 * @param InTint                   Color to tint the element
	 * @param Thickness                The thickness of the line
	 * @param DashLengthPx             The screen-space length of each dash (and each gap - irregular spacings are not supported)
	 * @param DashScreenOffset         A screen space offset that can be used to 'anchor' dashes to prevent them sliding when moving lines drawn in a virtual space.
	 */
	SLATECORE_API static void MakeDashedLines(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, TArray<FVector2f>&& Points, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint = FLinearColor::White, float Thickness = 1.0f, float DashLengthPx = 10.0f, float DashScreenOffset = 0.f);

	/**
	 * Creates a line defined by the provided points
	 *
	 * @param ElementList              The list in which to add elements
	 * @param InLayer                  The layer to draw the element on
	 * @param PaintGeometry            DrawSpace position and dimensions; see FPaintGeometry
	 * @param Points                   Points that make up the lines.  The points are joined together. I.E if Points has A,B,C there the line is A-B-C.  To draw non-joining line segments call MakeLines multiple times
	 * @param PointColors              Vertex Color for each defined points
	 * @param InDrawEffects            Optional draw effects to apply
	 * @param InTint                   Color to tint the element
	 * @param bAntialias               Should antialiasing be applied to the line?
	 * @param Thickness                The thickness of the line
	 */
#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS
	SLATECORE_API static void MakeLines(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const TArray<FVector2d>& Points, const TArray<FLinearColor>& PointColors, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint=FLinearColor::White, bool bAntialias = true, float Thickness = 1.0f);
#endif
	SLATECORE_API static void MakeLines(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, TArray<FVector2f> Points, TArray<FLinearColor> PointColors, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint=FLinearColor::White, bool bAntialias = true, float Thickness = 1.0f);

	/**
	 * Creates a dashed or dotted line defined by the provided points. Such lines are always renderd using the anti-aliased shader method.
	 *
	 * @param ElementList              The list in which to add elements
	 * @param InLayer                  The layer to draw the element on
	 * @param PaintGeometry            DrawSpace position and dimensions; see FPaintGeometry
	 * @param Points                   Points that make up the lines.  The points are joined together. I.E if Points has A,B,C there the line is A-B-C.  To draw non-joining line segments call MakeLines multiple times
	 * @param PointColors              Vertex Color for each defined points
	 * @param InDrawEffects            Optional draw effects to apply
	 * @param InTint                   Color to tint the element
	 * @param Thickness                The thickness of the line
	 * @param DashLengthPx             The screen-space length of each dash (and each gap - irregular spacings are not supported)
	 * @param DashScreenOffset         A screen space offset that can be used to 'anchor' dashes to prevent them sliding when moving lines drawn in a virtual space.
	 */
	SLATECORE_API static void MakeDashedLines(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, TArray<FVector2f>&& Points, TArray<FLinearColor>&& PointColors, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint = FLinearColor::White, float Thickness = 1.0f, float DashLengthPx = 10.0f, float DashScreenOffset = 0.f);

	/**
	 * Creates a viewport element which is useful for rendering custom data in a texture into Slate
	 *
	 * @param ElementList		   The list in which to add elements
	 * @param InLayer                  The layer to draw the element on
	 * @param PaintGeometry            DrawSpace position and dimensions; see FPaintGeometry
	 * @param Viewport                 Interface for drawing the viewport
	 * @param InScale                  Draw scale to apply to the entire element
	 * @param InDrawEffects            Optional draw effects to apply
	 * @param InTint                   Color to tint the element
	 */
	SLATECORE_API static void MakeViewport( FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, TSharedPtr<const ISlateViewport> Viewport, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, const FLinearColor& InTint=FLinearColor::White );

	/**
	 * Creates a custom element which can be used to manually draw into the Slate render target with graphics API calls rather than Slate elements
	 *
	 * @param ElementList		   The list in which to add elements
	 * @param InLayer                  The layer to draw the element on
	 * @param PaintGeometry            DrawSpace position and dimensions; see FPaintGeometry
	 * @param CustomDrawer		   Interface to a drawer which will be called when Slate renders this element
	 */
	SLATECORE_API static void MakeCustom( FSlateWindowElementList& ElementList, uint32 InLayer, TSharedPtr<ICustomSlateElement, ESPMode::ThreadSafe> CustomDrawer );
	
	SLATECORE_API static void MakeCustomVerts(FSlateWindowElementList& ElementList, uint32 InLayer, const FSlateResourceHandle& InRenderResourceHandle, const TArray<FSlateVertex>& InVerts, const TArray<SlateIndex>& InIndexes, ISlateUpdatableInstanceBuffer* InInstanceData, uint32 InInstanceOffset, uint32 InNumInstances, ESlateDrawEffect InDrawEffects = ESlateDrawEffect::None, ESlateBatchDrawFlag InBatchFlags = ESlateBatchDrawFlag::None);

	UE_DEPRECATED(5.4, "MakePostProcessPass has been deprecated. If you need to make a blur please call MakePostProcessBlur.")
	SLATECORE_API static void MakePostProcessPass(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FVector4f& Params, int32 DownsampleAmount, const FVector4f CornerRadius = FVector4f(0.0f));

	/**
	 * Creates an element that performs a blur pass
	 *
	 * @param ElementList			The list in which to add elements
	 * @param InLayer				The layer to draw the element on
	 * @param PaintGeometry			DrawSpace position and dimensions; see FPaintGeometry
	 * @param Params				Shader params for blur, should be tuple of KernelSize, Strength, Width, & Height
	 * @param DownSampleAmount		Amount we can downsample for the blur based on kernel size / strength
	 * @param CornerRadius			Amount pixels will be weighted in any direction
	 */
	SLATECORE_API static void MakePostProcessBlur(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FVector4f& Params, int32 DownsampleAmount, const FVector4f CornerRadius = FVector4f(0.0f));

	FSlateDrawElement();

	inline int32 GetLayer() const { return LayerId; }

	inline const FSlateRenderTransform& GetRenderTransform() const { return RenderTransform; }
	inline void SetRenderTransform(const FSlateRenderTransform& InRenderTransform) { RenderTransform = InRenderTransform; }

	inline UE::Slate::FDeprecateVector2DResult GetPosition() const
	{
		return UE::Slate::FDeprecateVector2DResult(Position);
	}

	inline void SetPosition(UE::Slate::FDeprecateVector2DParameter InPosition)
	{
		Position = InPosition;
	}

	inline UE::Slate::FDeprecateVector2DResult GetLocalSize() const
	{
		return UE::Slate::FDeprecateVector2DResult(LocalSize);
	}

	inline float GetScale() const { return Scale; }
	inline ESlateDrawEffect GetDrawEffects() const { return DrawEffects; }
	inline ESlateBatchDrawFlag GetBatchFlags() const { return BatchFlags; }
	inline void SetBatchFlags(ESlateBatchDrawFlag InFlags) { BatchFlags = InFlags; }
	inline void AddBatchFlags(ESlateBatchDrawFlag InFlags) { BatchFlags |= InFlags; }
	inline bool IsPixelSnapped() const { return !EnumHasAllFlags(DrawEffects, ESlateDrawEffect::NoPixelSnapping); }

	inline int32 GetPrecachedClippingIndex() const { return ClipStateHandle.GetPrecachedClipIndex(); }
	inline void SetPrecachedClippingIndex(int32 InClippingIndex) { ClipStateHandle.SetPreCachedClipIndex(InClippingIndex); }

	inline void SetCachedClippingState(const FSlateClippingState* CachedState) { ClipStateHandle.SetCachedClipState(CachedState); }
	inline const FClipStateHandle& GetClippingHandle() const { return ClipStateHandle; }
	inline const int8 GetSceneIndex() const { return SceneIndex; }

	inline void SetIsCached(bool bInIsCached) { bIsCached = bInIsCached; }
	inline bool IsCached() const { return bIsCached; }

	inline FSlateLayoutTransform GetInverseLayoutTransform() const
	{
		return Inverse(FSlateLayoutTransform(Scale, Position));
	}

	/**
	 * Update element cached position with an arbitrary offset
	 *
	 * @param Element		   Element to update
	 * @param InOffset         Absolute translation delta
	 */
	void ApplyPositionOffset(UE::Slate::FDeprecateVector2DParameter InOffset);

private:
	void Init(FSlateWindowElementList& ElementList, EElementType InElementType, uint32 InLayer, const FPaintGeometry& PaintGeometry, ESlateDrawEffect InDrawEffects);

	static FVector2f GetRotationPoint( const FPaintGeometry& PaintGeometry, const TOptional<FVector2f>& UserRotationPoint, ERotationSpace RotationSpace );
	static FSlateDrawElement& MakeBoxInternal(FSlateWindowElementList& ElementList, uint32 InLayer, const FPaintGeometry& PaintGeometry, const FSlateBrush* InBrush, ESlateDrawEffect InDrawEffects, const FLinearColor& InTint);
private:
	FSlateRenderTransform RenderTransform;
	FVector2f Position;
	FVector2f LocalSize;
	int32 LayerId;
	float Scale;
	FClipStateHandle ClipStateHandle;
	ESlateBatchDrawFlag BatchFlags;
	int8 SceneIndex;
	ESlateDrawEffect DrawEffects;
	EElementType ElementType;
	// Misc data
	uint8 bIsCached : 1;
};

class FSlateShaderResourceProxy;

struct FSlateGradientStop
{
	FVector2f Position;
	FLinearColor Color;

	/**
	 * Construct a Gradient Stop from a Position and a Color.
	 * @param InPosition - The position in widget space for this stop. Both X and Y are used for a single-axis gradient.
						  A two stop gradient should go from (0,0), to (Width,Height).
	 * @param InColor	- The color to lerp towards at this stop.
	 */
	template<typename VectorType>
	FSlateGradientStop(const VectorType& InPosition, const FLinearColor InColor)
		: Position(UE::Slate::CastToVector2f(InPosition))
		, Color(InColor)
	{
	}
};
template <> struct TIsPODType<FSlateGradientStop> { enum { Value = true }; };

struct FSlateTintableElement
{
	FLinearColor Tint;

	inline void SetTint(const FLinearColor& InTint) { Tint = InTint; }
	inline FLinearColor GetTint() const { return Tint; }
};

struct FSlateBoxElement : public FSlateDrawElement, public FSlateTintableElement
{
	FMargin Margin;
	FBox2f UVRegion;
	const FSlateShaderResourceProxy* ResourceProxy;
	ESlateBrushTileType::Type Tiling;
	ESlateBrushMirrorType::Type Mirroring;
	ESlateBrushDrawType::Type DrawType;

	const FMargin& GetBrushMargin() const { return Margin; }
	const FBox2f& GetBrushUVRegion() const { return UVRegion; }
	ESlateBrushTileType::Type GetBrushTiling() const { return Tiling; }
	ESlateBrushMirrorType::Type GetBrushMirroring() const { return Mirroring; }
	ESlateBrushDrawType::Type GetBrushDrawType() const { return DrawType; }
	const FSlateShaderResourceProxy* GetResourceProxy() const { return ResourceProxy; }

	void SetBrush(const FSlateBrush* InBrush, UE::Slate::FDeprecateVector2DParameter InLocalSize, float DrawScale)
	{
		check(InBrush);
		ensureMsgf(InBrush->GetDrawType() != ESlateBrushDrawType::NoDrawType, TEXT("This should have been filtered out earlier in the Make... call."));

		// Note: Do not store the brush.  It is possible brushes are destroyed after an element is enqueued for rendering
		Margin = InBrush->GetMargin();
		UVRegion = InBrush->GetUVRegion();
		Tiling = InBrush->GetTiling();
		Mirroring = InBrush->GetMirroring();
		DrawType = InBrush->GetDrawType();
		const FSlateResourceHandle& Handle = InBrush->GetRenderingResource(InLocalSize, DrawScale);
		if (Handle.IsValid())
		{
			ResourceProxy = Handle.GetResourceProxy();
		}
		else
		{
			ResourceProxy = nullptr;
		}
	}
};

struct FSlateRoundedBoxElement : public FSlateBoxElement
{
	FLinearColor OutlineColor;
	FVector4f Radius;
	float OutlineWeight;

	inline void SetRadius(FVector4f InRadius) { Radius = InRadius; }
	inline FVector4f GetRadius() const { return Radius; }

	inline void SetOutline(const FLinearColor& InOutlineColor, float InOutlineWeight) { OutlineColor = InOutlineColor; OutlineWeight = InOutlineWeight; }
	inline FLinearColor GetOutlineColor() const { return OutlineColor; }
	inline float GetOutlineWeight() const { return OutlineWeight; }
};

struct FSlateTextElement : public FSlateDrawElement, public FSlateTintableElement
{
	// The font to use when rendering
	FSlateFontInfo FontInfo;
	// Basic text data
	FString ImmutableText;

	const FSlateFontInfo& GetFontInfo() const { return FontInfo; }
	const TCHAR* GetText() const { return *ImmutableText; }
	int32 GetTextLength() const { return ImmutableText.Len(); }

	void SetText(const FString& InText, const FSlateFontInfo& InFontInfo, int32 InStartIndex, int32 InEndIndex)
	{
		FontInfo = InFontInfo;
		const int32 StartIndex = FMath::Min<int32>(InStartIndex, InText.Len());
		const int32 EndIndex = FMath::Min<int32>(InEndIndex, InText.Len());
		const int32 TextLength = (EndIndex > StartIndex) ? EndIndex - StartIndex : 0;
		if (TextLength > 0)
		{
			ImmutableText = InText.Mid(StartIndex, TextLength);
		}
	}

	void SetText(const FString& InText, const FSlateFontInfo& InFontInfo)
	{
		FontInfo = InFontInfo;
		ImmutableText = InText;
	}


	void AddReferencedObjects(FReferenceCollector& Collector)
	{
		FontInfo.AddReferencedObjects(Collector);
	}
};

struct FSlateShapedTextElement : public FSlateDrawElement, public FSlateTintableElement
{ 
	// Shaped text data
	FShapedGlyphSequencePtr ShapedGlyphSequence;

	FLinearColor OutlineTint;

	FTextOverflowArgs OverflowArgs;

	const FShapedGlyphSequencePtr& GetShapedGlyphSequence() const { return ShapedGlyphSequence; }
	FLinearColor GetOutlineTint() const { return OutlineTint; }

	void SetShapedText(const FShapedGlyphSequencePtr& InShapedGlyphSequence, const FLinearColor& InOutlineTint)
	{
		ShapedGlyphSequence = InShapedGlyphSequence;
		OutlineTint = InOutlineTint;
	}

	void SetOverflowArgs(const FTextOverflowArgs& InArgs)
	{
		OverflowArgs = InArgs;
		check(InArgs.OverflowDirection == ETextOverflowDirection::NoOverflow || InArgs.OverflowTextPtr.IsValid());
	}

	void AddReferencedObjects(FReferenceCollector& Collector)
	{
		if (ShapedGlyphSequence.IsValid())
		{
			const_cast<FShapedGlyphSequence*>(ShapedGlyphSequence.Get())->AddReferencedObjects(Collector);
		}

		if (OverflowArgs.OverflowTextPtr.IsValid())
		{
			const_cast<FShapedGlyphSequence*>(OverflowArgs.OverflowTextPtr.Get())->AddReferencedObjects(Collector);
		}
	}
};

struct FSlateGradientElement : public FSlateDrawElement
{
	TArray<FSlateGradientStop> GradientStops;
	EOrientation GradientType;
	FVector4f CornerRadius;

	void SetGradient(TArray<FSlateGradientStop> InGradientStops, EOrientation InGradientType, FVector4f InCornerRadius)
	{
		GradientStops = MoveTemp(InGradientStops);
		GradientType = InGradientType;
		CornerRadius = InCornerRadius;
	}
};

struct FSlateSplineElement : public FSlateDrawElement, public FSlateTintableElement
{
	TArray<FSlateGradientStop> GradientStops;
	// Bezier Spline Data points. E.g.
	//
	//       P1 + - - - - + P2                P1 +
	//         /           \                    / \
	//     P0 *             * P3            P0 *   \   * P3
	//                                              \ /
	//                                               + P2	
	FVector2f P0;
	FVector2f P1;
	FVector2f P2;
	FVector2f P3;

	float Thickness;

	// Thickness
	void SetThickness(float InThickness) { Thickness = InThickness; }
	float GetThickness() const { return Thickness; }

	void SetCubicBezier(const UE::Slate::FDeprecateVector2DParameter InP0, const UE::Slate::FDeprecateVector2DParameter InP1, const UE::Slate::FDeprecateVector2DParameter InP2, const UE::Slate::FDeprecateVector2DParameter InP3, float InThickness, const FLinearColor InTint)
	{
		Tint = InTint;
		P0 = InP0;
		P1 = InP1;
		P2 = InP2;
		P3 = InP3;
		Thickness = InThickness;
	}

	void SetHermiteSpline(const UE::Slate::FDeprecateVector2DParameter InStart, const UE::Slate::FDeprecateVector2DParameter InStartDir, const UE::Slate::FDeprecateVector2DParameter InEnd, const UE::Slate::FDeprecateVector2DParameter InEndDir, float InThickness, const FLinearColor InTint)
	{
		Tint = InTint;
		P0 = InStart;
		P1 = InStart + InStartDir / 3.0f;
		P2 = InEnd - InEndDir / 3.0f;
		P3 = InEnd;
		Thickness = InThickness;
	}

	void SetGradientHermiteSpline(const UE::Slate::FDeprecateVector2DParameter InStart, const UE::Slate::FDeprecateVector2DParameter InStartDir, const UE::Slate::FDeprecateVector2DParameter InEnd, const UE::Slate::FDeprecateVector2DParameter InEndDir, float InThickness, TArray<FSlateGradientStop> InGradientStops)
	{
		P0 = InStart;
		P1 = InStart + InStartDir / 3.0f;
		P2 = InEnd - InEndDir / 3.0f;
		P3 = InEnd;
		Thickness = InThickness;
		GradientStops = MoveTemp(InGradientStops);
	}
};

struct FSlateLineElement : public FSlateDrawElement, public FSlateTintableElement
{ 
	TArray<FVector2f> Points;
	TArray<FLinearColor> PointColors;
	float Thickness;
	float DashLength = 0.f;
	float DashOffset = 0.f;

	bool bAntialias;

	bool IsAntialiased() const { return bAntialias; }
	const TArray<FVector2f>& GetPoints() const { return Points; }
	const TArray<FLinearColor>& GetPointColors() const { return PointColors; }
	float GetThickness() const { return Thickness; }

	void SetThickness(float InThickness)
	{
		Thickness = InThickness;
	}

#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS
	void SetLines(const TArray<FVector2D>& InPoints, bool bInAntialias, const TArray<FLinearColor>* InPointColors = nullptr)
	{
		TArray<FVector2f> NewPoints;
		NewPoints.Reserve(InPoints.Num());
		for (FVector2D Vect : InPoints)
		{
			NewPoints.Add(UE::Slate::CastToVector2f(Vect));
		}
		if (InPointColors)
		{
			SetLines(MoveTemp(NewPoints), bInAntialias, *InPointColors);
		}
		else
		{
			SetLines(MoveTemp(NewPoints), bInAntialias);
		}
	}
#endif

	void SetLines(TArray<FVector2f> InPoints, bool bInAntialias)
	{
		bAntialias = bInAntialias;
		Points = MoveTemp(InPoints);
		PointColors.Reset();
	}

	void SetLines(TArray<FVector2f> InPoints, bool bInAntialias, TArray<FLinearColor> InPointColors)
	{
		bAntialias = bInAntialias;
		Points = MoveTemp(InPoints);
		PointColors = MoveTemp(InPointColors);
	}
};

struct FSlateViewportElement : public FSlateDrawElement, public FSlateTintableElement
{
	FSlateShaderResource* RenderTargetResource;
	uint8 bAllowViewportScaling : 1;
	uint8 bViewportTextureAlphaOnly : 1;
	uint8 bRequiresVSync : 1;

	void SetViewport(const TSharedPtr<const ISlateViewport>& InViewport, const FLinearColor& InTint)
	{
		Tint = InTint;
		RenderTargetResource = InViewport->GetViewportRenderTargetTexture();
		bAllowViewportScaling = InViewport->AllowScaling();
		bViewportTextureAlphaOnly = InViewport->IsViewportTextureAlphaOnly();
		bRequiresVSync = InViewport->RequiresVsync();
	}
};

struct FSlateCustomDrawerElement : public FSlateDrawElement
{
	// Custom drawer data
	TWeakPtr<ICustomSlateElement, ESPMode::ThreadSafe> CustomDrawer;

	void SetCustomDrawer(const TSharedPtr<ICustomSlateElement, ESPMode::ThreadSafe>& InCustomDrawer)
	{
		CustomDrawer = InCustomDrawer;
	}
};

struct FSlateLayerElement : public FSlateDrawElement
{
	class FSlateDrawLayerHandle* LayerHandle;

	void SetLayer(FSlateDrawLayerHandle* InLayerHandle)
	{
		LayerHandle = InLayerHandle;
		checkSlow(LayerHandle);
	}

};

struct FSlateCachedBufferElement : public FSlateDrawElement
{
	// Cached render data
	class FSlateRenderDataHandle* CachedRenderData;
	FVector2f CachedRenderDataOffset;

	// Cached Buffers
	void SetCachedBuffer(FSlateRenderDataHandle* InRenderDataHandle, const UE::Slate::FDeprecateVector2DParameter Offset)
	{
		check(InRenderDataHandle);

		CachedRenderData = InRenderDataHandle;
		CachedRenderDataOffset = Offset;
	}
};

struct FSlateCustomVertsElement : public FSlateDrawElement
{
	const FSlateShaderResourceProxy* ResourceProxy;

	TArray<FSlateVertex> Vertices;
	TArray<SlateIndex> Indices;

	// Instancing support
	ISlateUpdatableInstanceBufferRenderProxy* InstanceData;
	uint32 InstanceOffset;
	uint32 NumInstances;

	void SetCustomVerts(const FSlateShaderResourceProxy* InRenderProxy, TArray<FSlateVertex> InVerts, TArray<SlateIndex> InIndices, ISlateUpdatableInstanceBufferRenderProxy* InInstanceData, uint32 InInstanceOffset, uint32 InNumInstances)
	{
		ResourceProxy = InRenderProxy;

		Vertices = MoveTemp(InVerts);
		Indices = MoveTemp(InIndices);

		InstanceData = InInstanceData;
		InstanceOffset = InInstanceOffset;
		NumInstances = InNumInstances;
	}
};

struct FSlatePostProcessElement : public FSlateDrawElement
{
	// Post Process Data
	FVector4f PostProcessData;
	FVector4f CornerRadius;
	int32 DownsampleAmount;
};
