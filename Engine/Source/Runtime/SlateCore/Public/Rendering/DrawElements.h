// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rendering/DrawElementCoreTypes.h"
#include "Rendering/DrawElementTypes.h"
#include "Rendering/DrawElementPayloads.h"
#include "Tasks/Task.h"

class FSlateRenderBatch;
class FSlateDrawLayerHandle;
class FSlateResourceHandle;
class FSlateWindowElementList;
class SWidget;
class SWindow;
struct FSlateBrush;
struct FSlateDataPayload;
struct FSlateGradientStop;

/**
 * Note: FSlateDrawElement has been moved to DrawElementTypes.h
 */

/**
 * Data holder for info used in cached renderbatches. Primarily clipping state, vertices and indices
 */
struct FSlateCachedFastPathRenderingData
{
	~FSlateCachedFastPathRenderingData()
	{
		CachedClipStates.Reset();
	}

	TArray<FSlateCachedClipState, TInlineAllocator<1>> CachedClipStates;
	FSlateVertexArray Vertices;
	FSlateIndexArray Indices;
};

struct FSlateCachedElementData;

/**
 * Cached list of elements corresponding to a particular widget. 
 * This class is used as an interfaces for renderbatches to interact with cached rendering data with
 * In particular verticies and indicies
 */
struct FSlateCachedElementList
{
	FSlateCachedElementList(FSlateCachedElementData* InParentData, const SWidget* InWidget)
		: OwningWidget(InWidget)
		, ParentData(InParentData)
		, CachedRenderingData(nullptr)
	{
	}

	void Initialize()
	{
		CachedRenderingData = new FSlateCachedFastPathRenderingData;
	}

	SLATECORE_API ~FSlateCachedElementList();

	void ClearCachedElements();

	FSlateCachedElementData* GetOwningData() { return ParentData; }

	FSlateRenderBatch& AddRenderBatch(int32 InLayer, const FShaderParams& InShaderParams, const FSlateShaderResource* InResource, ESlateDrawPrimitive InPrimitiveType, ESlateShader InShaderType, ESlateDrawEffect InDrawEffects, ESlateBatchDrawFlag InDrawFlags, int8 SceneIndex);

	void AddCachedClipState(FSlateCachedClipState& ClipStateToCache);

	void AddReferencedObjects(FReferenceCollector& Collector);

	/**
	 * Returns true if all typed containers are empty, else false
	 */
	SLATECORE_API bool IsEmpty();

	/**
	 * Returns number of elements in all containers summed
	 */
	SLATECORE_API int32 NumElements();
private:
	SLATECORE_API void DestroyCachedData();

public:
	/** List of source draw elements to create batches from */
	FSlateDrawElementMap DrawElements;

	TArray<int32> CachedRenderBatchIndices;

	/** The widget whose draw elements are in this list */
	const SWidget* OwningWidget;

	FSlateCachedElementData* ParentData;

	FSlateCachedFastPathRenderingData* CachedRenderingData;
};


/**
 * Handle used to uniquely identify a cached element list with some widget
 * Used when a widget gets invalidated / additional elements added
 * So we can modify that widget's existing cached element list
 */
struct FSlateCachedElementsHandle
{
	friend struct FSlateCachedElementData;

	static FSlateCachedElementsHandle Invalid;
	void ClearCachedElements();
	void RemoveFromCache();

	bool IsOwnedByWidget(const SWidget* Widget) const;

	bool IsValid() const { return Ptr.IsValid(); }
	bool HasCachedElements() const;

	bool operator!=(FSlateCachedElementsHandle& Other) const { return Ptr != Other.Ptr; }

	FSlateCachedElementsHandle() {}
private:
	FSlateCachedElementsHandle(TSharedRef<FSlateCachedElementList>& DataPtr)
		: Ptr(DataPtr)
	{
	}

private:
	TWeakPtr<FSlateCachedElementList> Ptr;
};

/**
 * Top level class responsible for cached elements within a particular invalidation root.
 * Equivalent to the plain FSlateBatchData in that it manages multiple renderbatches,
 * element lists (Per widget), & new element lists (Per invalidated widgets).
 * 
 * Note: Just as each window may have multiple invalidation roots, each window element list 
 * may have multiple 'FSlateCachedElementData' for each root
 */
struct FSlateCachedElementData
{
	friend class FSlateElementBatcher;
	friend struct FSlateCachedElementsHandle;

	/** Reset all cached data, except num elements, call EmptyCachedNumElements for that */
	void Empty();

	/** Create a new CachedElementList, occurs when adding a cached element whose top level draw widget is not already part of some cache */
	FSlateCachedElementsHandle AddCache(const SWidget* Widget);

	static const FSlateClippingState* GetClipStateFromParent(const FSlateClippingManager& ParentClipManager);

	SLATECORE_API void ValidateWidgetOwner(TSharedPtr<FSlateCachedElementList> List, const SWidget* CurrentWidget);

	template<EElementType ElementType>
	TSlateDrawElement<ElementType>& AddCachedElement(FSlateCachedElementsHandle& CacheHandle, const FSlateClippingManager& ParentClipManager, const SWidget* CurrentWidget)
	{
		using FSlateElementType = TSlateDrawElement<ElementType>;

		TSharedPtr<FSlateCachedElementList> List = CacheHandle.Ptr.Pin(); 

#if WITH_SLATE_DEBUGGING
		ValidateWidgetOwner(List, CurrentWidget);
#endif

		FSlateDrawElementArray<FSlateElementType>& Container = List->DrawElements.Get<(uint8)ElementType>();
		FSlateElementType& NewElement = Container.AddDefaulted_GetRef();
		NewElement.SetIsCached(true);

		// Check if slow vs checking a flag on the list to see if it contains new data.
		ListsWithNewData.AddUnique(List.Get());

		const FSlateClippingState* ExistingClipState = FSlateCachedElementData::GetClipStateFromParent(ParentClipManager);

		if (ExistingClipState)
		{
			// We need to cache this clip state for the next time the element draws
			FSlateCachedClipState& CachedClipState = FindOrAddCachedClipState(ExistingClipState);
			List->AddCachedClipState(CachedClipState);
			NewElement.SetCachedClippingState(&CachedClipState.ClippingState.Get());
		}

		return NewElement;
	}

	FSlateRenderBatch& AddCachedRenderBatch(FSlateRenderBatch&& NewBatch, int32& OutIndex);
	void RemoveCachedRenderBatches(const TArray<int32>& CachedRenderBatchIndices);

	FSlateCachedClipState& FindOrAddCachedClipState(const FSlateClippingState* RefClipState);
	void CleanupUnusedClipStates();

	const TSparseArray<FSlateRenderBatch>& GetCachedBatches() const { return CachedBatches; }
	const TArray<TSharedPtr<FSlateCachedElementList>>& GetCachedElementLists() const { return CachedElementLists; }

	void AddReferencedObjects(FReferenceCollector& Collector);

	TArrayView<FSlateCachedElementList* const> GetListsWithNewData() const { return MakeArrayView(ListsWithNewData.GetData(), ListsWithNewData.Num()); }
private:
	/** Removes a cache node completely from the cache */
	void RemoveList(FSlateCachedElementsHandle& CacheHandle);

private:

	/** List of cached batches to submit for drawing */
	TSparseArray<FSlateRenderBatch> CachedBatches;

	/** List of cached element lists used to redraw when no invalidation occurs, each list corresponds to a particular widget. See 'FSlateCachedElementsHandle' */
	TArray<TSharedPtr<FSlateCachedElementList>> CachedElementLists;

	/** List of pointers to instances of the element lists above which have new data or have been invalidated */
	TArray<FSlateCachedElementList*, TInlineAllocator<50>> ListsWithNewData;

	/** List of clip states used later when rendering */
	TArray<FSlateCachedClipState> CachedClipStates;
};

/**
 * Represents a top level window and its draw elements.
 */
class FSlateWindowElementList : public FNoncopyable
{
	friend class FSlateElementBatcher;
public:
	/**
	 * Construct a new list of elements with which to paint a window.
	 *
	 * @param InPaintWindow		The window that owns the widgets being painted.  This is almost most always the same window that is being rendered to
	 * @param InRenderWindow	The window that we will be rendering to.
	 */
	SLATECORE_API explicit FSlateWindowElementList(const TSharedPtr<SWindow>& InPaintWindow);

	SLATECORE_API ~FSlateWindowElementList();

	/** @return Get the window that we will be painting */
	SWindow* GetPaintWindow() const
	{
		check(IsInGameThread() || IsInSlateThread());
		return WeakPaintWindow.IsValid() ? RawPaintWindow : nullptr;
	}

	/** @return Get the window that we will be rendering to.  Unless you are a slate renderer you probably want to use GetPaintWindow() */
	SWindow* GetRenderWindow() const
	{
		check(IsInGameThread() || IsInSlateThread());
		// Note: This assumes that the PaintWindow is safe to pin and is not accessed by another thread
		return RenderTargetWindow != nullptr ? RenderTargetWindow : GetPaintWindow();
	}

	/** @return Get the draw elements that we want to render into this window */
	const FSlateDrawElementMap& GetUncachedDrawElements() const
	{
		return UncachedDrawElements;
	}

	/** @return Get the window size that we will be painting */
	UE::Slate::FDeprecateVector2DResult GetWindowSize() const
	{
		return UE::Slate::FDeprecateVector2DResult(WindowSize);
	}
	
	/**
	 * Creates an uninitialized draw element if using caching will create a new cached draw list
	 * if needed (Whenever a top level draw widget's cache handle doesn't match the current cached handle).
	 */
	template<EElementType ElementType = EElementType::ET_NonMapped>
	TSlateDrawElement<ElementType>& AddUninitialized()
	{
		using FSlateElementType = TSlateDrawElement<ElementType>;

		const bool bAllowCache = CachedElementDataListStack.Num() > 0 && WidgetDrawStack.Num() && !WidgetDrawStack.Top().bIsVolatile;

		if (bAllowCache)
		{
			// @todo get working with slate debugging
			return AddCachedElement<ElementType>();
		}
		else
		{
			FSlateDrawElementMap& Elements = UncachedDrawElements;
			FSlateDrawElementArray<FSlateElementType>& Container = Elements.Get<(uint8)ElementType>();
			const int32 InsertIdx = Container.AddDefaulted();

#if WITH_SLATE_DEBUGGING
			FSlateDebuggingElementTypeAddedEventArgs ElementTypeAddedArgs{*this, InsertIdx, ElementType};
			FSlateDebugging::ElementTypeAdded.Broadcast(ElementTypeAddedArgs);
#endif

			FSlateElementType& NewElement = Container[InsertIdx];
			return NewElement;
		}
	}

	/**
	 * Calls AddUninitialized, resolving template version based on enum
	 */
	SLATECORE_API FSlateDrawElement& AddUninitializedLookup(EElementType InElementType = EElementType::ET_NonMapped);


	//--------------------------------------------------------------------------
	// CLIPPING
	//--------------------------------------------------------------------------

	SLATECORE_API int32 PushClip(const FSlateClippingZone& InClipZone);
	SLATECORE_API int32 GetClippingIndex() const;
	int32 GetClippingStackDepth() const { return ClippingManager.GetStackDepth(); }
	SLATECORE_API TOptional<FSlateClippingState> GetClippingState() const;
	SLATECORE_API void PopClip();
	SLATECORE_API void PopClipToStackIndex(int32 Index);


	FSlateClippingManager& GetClippingManager() { return ClippingManager; }
	const FSlateClippingManager& GetClippingManager() const { return ClippingManager; }

	//--------------------------------------------------------------------------
	// PIXEL SNAPPING
	//--------------------------------------------------------------------------

	SLATECORE_API int32 PushPixelSnappingMethod(EWidgetPixelSnapping InPixelSnappingMethod);
	SLATECORE_API void PopPixelSnappingMethod();
	SLATECORE_API EWidgetPixelSnapping GetPixelSnappingMethod() const;
	
	//--------------------------------------------------------------------------
	// DEFERRED PAINTING
	//--------------------------------------------------------------------------

	/**
	 * Some widgets may want to paint their children after after another, loosely-related widget finished painting.
	 * Or they may want to paint "after everyone".
	 */
	struct FDeferredPaint
	{
	public:
		SLATECORE_API FDeferredPaint(const TSharedRef<const SWidget>& InWidgetToPaint, const FPaintArgs& InArgs, const FGeometry InAllottedGeometry, const FWidgetStyle& InWidgetStyle, bool InParentEnabled);

		SLATECORE_API int32 ExecutePaint(int32 LayerId, FSlateWindowElementList& OutDrawElements, const FSlateRect& MyCullingRect) const;

		SLATECORE_API FDeferredPaint Copy(const FPaintArgs& InArgs);

	private:
		// Used for making copies.
		FDeferredPaint(const FDeferredPaint& Copy, const FPaintArgs& InArgs);

		const TWeakPtr<const SWidget> WidgetToPaintPtr;
		const FPaintArgs Args;
		const FGeometry AllottedGeometry;
		const FWidgetStyle WidgetStyle;
		const bool bParentEnabled;
	};

	SLATECORE_API void QueueDeferredPainting(const FDeferredPaint& InDeferredPaint);

	int32 PaintDeferred(int32 LayerId, const FSlateRect& MyCullingRect);

	bool ShouldResolveDeferred() const { return bNeedsDeferredResolve; }

	SLATECORE_API void BeginDeferredGroup();
	SLATECORE_API void EndDeferredGroup();

	TArray< TSharedPtr<FDeferredPaint> > GetDeferredPaintList() const { return DeferredPaintList; }


	//--------------------------------------------------------------------------
	// FAST PATH
	//--------------------------------------------------------------------------

	/**
	 * Pushes the current widget that is painting onto the widget stack so we know what elements belong to each widget
	 * This information is used for caching later.
	 *
	 */
	SLATECORE_API void PushPaintingWidget(const SWidget& CurrentWidget, int32 StartingLayerId, FSlateCachedElementsHandle& CurrentCacheHandle);

	/**
	 * Pops the current painted widget off the stack
	 * @return true if an element was added while the widget was pushed
	 */
	SLATECORE_API FSlateCachedElementsHandle PopPaintingWidget(const SWidget& CurrentWidget);

	/** Pushes cached element data onto the stack.  Any draw elements cached after will use this cached element data until popped */
	void PushCachedElementData(FSlateCachedElementData& CachedElementData);
	void PopCachedElementData();

	//--------------------------------------------------------------------------
	// OTHER
	//--------------------------------------------------------------------------
	
	/**
	 * Remove all the elements from this draw list.
	 */
	SLATECORE_API void ResetElementList();


	FSlateBatchData& GetBatchData()
	{
		FinishMergeRenderBatches();
		return BatchData;
	}

	FSlateBatchData& GetBatchDataHDR()
	{
		FinishMergeRenderBatches();
		return BatchDataHDR;
	}

	SLATECORE_API void SetRenderTargetWindow(SWindow* InRenderTargetWindow);

	void AddReferencedObjects(FReferenceCollector& Collector);
	
	UE_DEPRECATED(5.8, "Use SetIsProjectContent from CustomPrepass instead")
	SLATECORE_API void SetIsInGameLayer(bool bInGameLayer);

	UE_DEPRECATED(5.8, "Renamed to IsInProjectContent")
	SLATECORE_API bool GetIsInGameLayer();

	void SetIsInProjectContent(bool bIsInProjectContent);

	/** Whether the stack frame is drawing the project's content (as opposed to editor, chrome, diagnostic output). */
	bool IsInProjectContent() const
	{
		return bIsProjectContent;
	}

	TArrayView<FSlateCachedElementList* const> GetCurrentCachedElementWithNewData() const
	{ 
		if (GetCurrentCachedElementData() != nullptr)
		{
			return CachedElementDataList[CachedElementDataListStack.Top()]->GetListsWithNewData();
		}
		else
		{
			return TArrayView<FSlateCachedElementList* const>();
		}
	}
private:
	/** Adds a cached element, generating a new cached list for the widget at the top of the cache if needed */
	template<EElementType ElementType>
	TSlateDrawElement<ElementType>& AddCachedElement()
	{
		FSlateCachedElementData* CurrentCachedElementData = GetCurrentCachedElementData();
		check(CurrentCachedElementData);

		FWidgetDrawElementState& CurrentWidgetState = WidgetDrawStack.Top();
		check(!CurrentWidgetState.bIsVolatile);

		if (!CurrentWidgetState.CacheHandle.IsValid())
		{
			CurrentWidgetState.CacheHandle = CurrentCachedElementData->AddCache(CurrentWidgetState.Widget);
		}

		return CurrentCachedElementData->AddCachedElement<ElementType>(CurrentWidgetState.CacheHandle, GetClippingManager(), CurrentWidgetState.Widget);
	}

	TArrayView<FSlateCachedElementData* const> GetCachedElementDataList() const { return MakeArrayView(CachedElementDataList.GetData(), CachedElementDataList.Num()); }

	FSlateCachedElementData* GetCurrentCachedElementData() const { return CachedElementDataListStack.Num() ? CachedElementDataList[CachedElementDataListStack.Top()] : nullptr; }

	SLATECORE_API void StartMergeRenderBatches();
	SLATECORE_API void FinishMergeRenderBatches();

private:
	/**
	* Window which owns the widgets that are being painted but not necessarily rendered to
	* Widgets are always rendered to the RenderTargetWindow
	*/
	TWeakPtr<SWindow> WeakPaintWindow;
	SWindow* RawPaintWindow;

	FMemStackBase MemManager;
	STAT(int32 MemManagerAllocatedMemory;)
	/**
	 * The window to render to.  This may be different from the paint window if we are displaying the contents of a window (or virtual window) onto another window
	 * The primary use case of this is thread safe rendering of widgets during times when the main thread is blocked (e.g loading movies)
	 * If this is null, the paint window is used
 	 */
	SWindow* RenderTargetWindow;

	/** Batched data used for rendering */
	FSlateBatchData BatchData;

	/** Batched data used for rendering */
	FSlateBatchData BatchDataHDR;

	UE::Tasks::FTask MergeBatchDataTask;

	/**  */
	FSlateClippingManager ClippingManager;

	/** Manages what pixel snapping method should be applied */
	TArray<EWidgetPixelSnapping, TInlineAllocator<4>> PixelSnappingMethodStack;
	
	/**
	 * Some widgets want their logical children to appear at a different "layer" in the physical hierarchy.
	 * We accomplish this by deferring their painting.
	 */
	TArray< TSharedPtr<FDeferredPaint> > DeferredPaintList;

	/** Caches whether the current widget constitutes "content" (as opposed to editor, chrome, diagnostic output). */
	bool bIsProjectContent;

	bool bNeedsDeferredResolve;
	TArray<int32> ResolveToDeferredIndex;

	// Begin Fast Path

	/** State of the current widget that is adding draw elements */
	struct FWidgetDrawElementState
	{
		FWidgetDrawElementState(FSlateCachedElementsHandle& InCurrentHandle, bool bInIsVolatile, const SWidget* InWidget)
			: CacheHandle(InCurrentHandle)
			, Widget(InWidget)
			, bIsVolatile(bInIsVolatile)
		{
		}

		FSlateCachedElementsHandle CacheHandle;
		const SWidget* Widget;
		bool bIsVolatile;
	};

	/** The uncached draw elements to be processed */
	FSlateDrawElementMap UncachedDrawElements;

	TArray<FWidgetDrawElementState, TInlineAllocator<50>> WidgetDrawStack;
	TArray<FSlateCachedElementData*, TInlineAllocator<4>> CachedElementDataList;
	TArray<int32, TInlineAllocator<4>> CachedElementDataListStack;
	// End Fast Path

	/** Store the size of the window being used to paint */
	FVector2f WindowSize;
};
