// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/SlateRendererTypes.h"
#include "Layout/Clipping.h"
#include "Stats/Stats.h"
#include "SlateGlobals.h"
#include "Styling/SlateTypes.h"
#include "Containers/StaticArray.h"

class FSlateBatchData;
class FSlateDrawElement;
class FSlateRenderBatch;
class FSlateRenderingPolicy;
class FSlateShaderResource;
class FSlateWindowElementList;
struct FShaderParams;
struct FSlateCachedElementData;
struct FSlateCachedElementList;
enum class ETextOverflowDirection : uint8;

DECLARE_CYCLE_STAT_EXTERN(TEXT("Add Elements Time"), STAT_SlateAddElements, STATGROUP_Slate, SLATECORE_API);

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Elements"), STAT_SlateElements, STATGROUP_Slate, SLATECORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Elements (Box)"), STAT_SlateElements_Box, STATGROUP_Slate, SLATECORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Elements (Border)"), STAT_SlateElements_Border, STATGROUP_Slate, SLATECORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Elements (Text)"), STAT_SlateElements_Text, STATGROUP_Slate, SLATECORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Elements (ShapedText)"), STAT_SlateElements_ShapedText, STATGROUP_Slate, SLATECORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Elements (Line)"), STAT_SlateElements_Line, STATGROUP_Slate, SLATECORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Elements (Other)"), STAT_SlateElements_Other, STATGROUP_Slate, SLATECORE_API);

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Invalidation: Recached Elements"), STAT_SlateInvalidation_RecachedElements, STATGROUP_Slate, SLATECORE_API);

extern int32 GSlateFeathering;

/**
* Represents an  a set of slate draw elements that are batched together
* These later get converted into FSlateRenderBatches for final rendering
*/
class FSlateElementBatch
{
public:
	FSlateElementBatch(const FSlateShaderResource* InShaderResource, const FShaderParams& InShaderParams, ESlateShader ShaderType, ESlateDrawPrimitive PrimitiveType, ESlateDrawEffect InDrawEffects, ESlateBatchDrawFlag InBatchFlags, const FSlateDrawElement& InDrawElement, int32 InstanceCount = 0, uint32 InstanceOffset = 0, ISlateUpdatableInstanceBuffer* InstanceData = nullptr);
	FSlateElementBatch(TWeakPtr<ICustomSlateElement, ESPMode::ThreadSafe> InCustomDrawer, const FSlateDrawElement& InDrawElement);

	bool operator==(const FSlateElementBatch& Other) const
	{
		return BatchKey == Other.BatchKey && ShaderResource == Other.ShaderResource;
	}

	const FSlateShaderResource* GetShaderResource() const { return ShaderResource; }
	const FShaderParams& GetShaderParams() const { return BatchKey.ShaderParams; }
	ESlateBatchDrawFlag GetDrawFlags() const { return BatchKey.DrawFlags; }
	ESlateDrawPrimitive GetPrimitiveType() const { return BatchKey.DrawPrimitiveType; }
	ESlateShader GetShaderType() const { return BatchKey.ShaderType; }
	ESlateDrawEffect GetDrawEffects() const { return BatchKey.DrawEffects; }
	const TWeakPtr<ICustomSlateElement, ESPMode::ThreadSafe> GetCustomDrawer() const { return BatchKey.CustomDrawer; }
	int32 GetInstanceCount() const { return BatchKey.InstanceCount; }
	uint32 GetInstanceOffset() const { return BatchKey.InstanceOffset; }
	const ISlateUpdatableInstanceBuffer* GetInstanceData() const { return BatchKey.InstanceData; }
	int8 GetSceneIndex() const { return BatchKey.SceneIndex; }
private:
	struct FBatchKey
	{
		const TWeakPtr<ICustomSlateElement, ESPMode::ThreadSafe> CustomDrawer;
		const FShaderParams ShaderParams;
		const ESlateBatchDrawFlag DrawFlags;
		const ESlateShader ShaderType;
		const ESlateDrawPrimitive DrawPrimitiveType;
		const ESlateDrawEffect DrawEffects;
		const FClipStateHandle ClipStateHandle;
		const int32 InstanceCount;
		const uint32 InstanceOffset;
		const ISlateUpdatableInstanceBuffer* InstanceData;
		const int8 SceneIndex;

		FBatchKey(const FShaderParams& InShaderParams, ESlateShader InShaderType, ESlateDrawPrimitive InDrawPrimitiveType, ESlateDrawEffect InDrawEffects, ESlateBatchDrawFlag InDrawFlags, const FClipStateHandle InClipStateHandle, int32 InInstanceCount, uint32 InInstanceOffset, ISlateUpdatableInstanceBuffer* InInstanceBuffer, int8 InSceneIndex)
			: ShaderParams(InShaderParams)
			, DrawFlags(InDrawFlags)
			, ShaderType(InShaderType)
			, DrawPrimitiveType(InDrawPrimitiveType)
			, DrawEffects(InDrawEffects)
			, ClipStateHandle(InClipStateHandle)
			, InstanceCount(InInstanceCount)
			, InstanceOffset(InInstanceOffset)
			, InstanceData(InInstanceBuffer)
			, SceneIndex(InSceneIndex)
		{
		}

		FBatchKey(const FShaderParams& InShaderParams, ESlateShader InShaderType, ESlateDrawPrimitive InDrawPrimitiveType, ESlateDrawEffect InDrawEffects, ESlateBatchDrawFlag InDrawFlags, const FClipStateHandle InClipStateHandle, int32 InInstanceCount, uint32 InInstanceOffset, ISlateUpdatableInstanceBuffer* InInstanceBuffer)
			: ShaderParams(InShaderParams)
			, DrawFlags(InDrawFlags)
			, ShaderType(InShaderType)
			, DrawPrimitiveType(InDrawPrimitiveType)
			, DrawEffects(InDrawEffects)
			, ClipStateHandle(InClipStateHandle)
			, InstanceCount(InInstanceCount)
			, InstanceOffset(InInstanceOffset)
			, InstanceData(InInstanceBuffer)
			, SceneIndex(-1)
		{
		}

		FBatchKey(TWeakPtr<ICustomSlateElement, ESPMode::ThreadSafe> InCustomDrawer, const FClipStateHandle InClipStateHandle)
			: CustomDrawer(InCustomDrawer)
			, ShaderParams()
			, DrawFlags(ESlateBatchDrawFlag::None)
			, ShaderType(ESlateShader::Default)
			, DrawPrimitiveType(ESlateDrawPrimitive::TriangleList)
			, DrawEffects(ESlateDrawEffect::None)
			, ClipStateHandle(InClipStateHandle)
			, InstanceCount(0)
			, InstanceOffset(0)
			, SceneIndex(-1)
		{}

		bool operator==(const FBatchKey& Other) const
		{
			return DrawFlags == Other.DrawFlags
				&& ShaderType == Other.ShaderType
				&& DrawPrimitiveType == Other.DrawPrimitiveType
				&& DrawEffects == Other.DrawEffects
				&& ShaderParams == Other.ShaderParams
				&& ClipStateHandle == Other.ClipStateHandle
				&& CustomDrawer == Other.CustomDrawer
				&& InstanceCount == Other.InstanceCount
				&& InstanceOffset == Other.InstanceOffset
				&& InstanceData == Other.InstanceData
				&& SceneIndex == Other.SceneIndex;
		}
	};

	/** A secondary key which represents elements needed to make a batch unique */
	FBatchKey BatchKey;

	/** Shader resource to use with this batch.  Used as a primary key.  No batch can have multiple textures */
	const FSlateShaderResource* ShaderResource;

public:
	/** Number of elements in the batch */
	uint32 NumElementsInBatch;
	/** Index into an array of vertex arrays where this batches vertices are found (before submitting to the vertex buffer)*/
	int32 VertexArrayIndex;
	/** Index into an array of index arrays where this batches indices are found (before submitting to the index buffer) */
	int32 IndexArrayIndex;
};


class FSlateBatchData
{
public:
	SLATECORE_API FSlateBatchData();
	SLATECORE_API ~FSlateBatchData();

	SLATECORE_API void ResetData();

	/** Returns a list of element batches for this window */
	const TArray<FSlateRenderBatch>& GetRenderBatches() const { return RenderBatches; }

	/** True if stencil buffer / clipping is needed. */
	SLATECORE_API bool IsStencilClippingRequired() const;

	/** True if virtual texture feedback is needed. */
	SLATECORE_API bool IsVirtualTextureFeedbackRequired() const;

	int32 GetFirstRenderBatchIndex() const { return FirstRenderBatchIndex; }
	/** @return Total number of batched layers */
	int32 GetNumLayers() const { return NumLayers; }

	int32 GetNumFinalBatches() const { return NumBatches; }

	int32 GetMaxNumFinalVertices() const { return MaxNumFinalVertices; }
	int32 GetMaxNumFinalIndices() const { return MaxNumFinalIndices; }

	const FSlateVertexArray& GetFinalVertexData() const { return FinalVertexData; }
	const FSlateIndexArray& GetFinalIndexData() const { return FinalIndexData; }

	/**
	 * Fills batch data into the actual vertex and index buffer
	 *
	 * @param VertexBuffer	Pointer to the actual memory for the vertex buffer
	 * @param IndexBuffer	Pointer to the actual memory for an index buffer
	 * @param bAbsoluteIndices	Whether to write absolute indices (simplifies draw call setup on RHIs that do not support BaseVertex)
	 */
	SLATECORE_API void FillVertexAndIndexBuffer(uint8* VertexBuffer, uint8* IndexBuffer, bool bAbsoluteIndices);

	/** Merges render batches across all elements where possible for final submit to GPU */
	SLATECORE_API void MergeRenderBatches();

	/** Adds a new render batch to list of batches */
	FSlateRenderBatch& AddRenderBatch(
		int32 InLayer,
		const FShaderParams& InShaderParams,
		const FSlateShaderResource* InResource,
		ESlateDrawPrimitive InPrimitiveType,
		ESlateShader InShaderType,
		ESlateDrawEffect InDrawEffects,
		ESlateBatchDrawFlag InDrawFlags,
		int8 SceneIndex);

public:
	friend FSlateElementBatcher;

private:
	void FillBuffersFromNewBatch(FSlateRenderBatch& Batch, FSlateVertexArray& FinalVertices, FSlateIndexArray& FinalIndices);
	void CombineBatches(FSlateRenderBatch& FirstBatch, FSlateRenderBatch& SecondBatch, FSlateVertexArray& FinalVertices, FSlateIndexArray& FinalIndices);
private:

	/** List of element batches sorted by later for use in rendering*/
	// todo allocator?
	TArray<FSlateRenderBatch> RenderBatches;

	/** Uncached source vertices and indices from volatile elements */
	FSlateVertexArray UncachedSourceBatchVertices;
	FSlateIndexArray UncachedSourceBatchIndices;

	FSlateVertexArray FinalVertexData;
	FSlateIndexArray  FinalIndexData;

	int32 MaxNumFinalVertices = 0;
	int32 MaxNumFinalIndices = 0;

	int32 FirstRenderBatchIndex;

	/** */
	int32 NumLayers;

	/** Number of final render batches.  it is not the same as RenderBatches.Num(); */
	int32 NumBatches;

	/** true if slate rendering pass requires a stencil buffer */
	bool bIsStencilBufferRequired;
	/** true if slate rendering pass requires a virtual texture feedback buffer */
	bool bRequiresVirtualTextureFeedback;
};


/**
 * A class which batches Slate elements for rendering
 */
class FSlateElementBatcher
{
public:

	SLATECORE_API FSlateElementBatcher( TSharedRef<FSlateRenderingPolicy> InRenderingPolicy );
	SLATECORE_API ~FSlateElementBatcher();

	/** 
	 * Batches elements to be rendered 
	 * 
	 * @param DrawElements	The elements to batch
	 */
	SLATECORE_API void AddElements( FSlateWindowElementList& ElementList );

	/**
	 * Returns true if the elements in this batcher require v-sync.
	 */
	bool RequiresVsync() const { return bRequiresVsync; }

	/** Whether or not any post process passes were batched */
	bool HasFXPassses() const { return NumPostProcessPasses > 0;}

	bool CompositeHDRViewports() const { return bCompositeHDRViewports; }

	ESlatePostRT GetUsedSlatePostBuffers() const { return UsedSlatePostBuffers; }

	ESlatePostRT GetResourceUpdatingPostBuffers() const { return ResourceUpdatingPostBuffers; }

	void SetResourceUpdatingPostBuffers(ESlatePostRT InResourceUpdatingPostBuffers) { ResourceUpdatingPostBuffers = InResourceUpdatingPostBuffers; }

	void SetCompositeHDRViewports(bool bInCompositeHDRViewports) { bCompositeHDRViewports = bInCompositeHDRViewports; }

	/** 
	 * Resets all stored data accumulated during the batching process
	 */
	SLATECORE_API void ResetBatches();

	inline FColor PackVertexColor(const FLinearColor& InLinearColor) const
	{
		//NOTE: Using pow(x,2) instead of a full sRGB conversion has been tried, but it ended up
		// causing too much loss of data in the lower levels of black.
		return InLinearColor.ToFColor(bSRGBVertexColor);
	}

private:
	void AddElementsInternal(const FSlateDrawElementMap& DrawElements, FVector2f ViewportSize);
	void AddCachedElements(FSlateCachedElementData& CachedElementData, FVector2f ViewportSize);

	/**
	 * Generates Vertices, Indices, Renderbatches, & associates each of these together correctly
	 * Attempts to reuse renderbatches across elements if possible. 
	 * 
	 * Note: Future more efficient and less generic reuse is something we may consider.
	 * 
	 * @param DrawElements - Elements to iterate over
	 * @param InElementAdder - Functor to add an individual slate draw element
	 * @param InElementBatchParamCreator - Functor that generates batch params given a slate draw element, used during batch-reuse
	 * @param InElementBatchReserver - Functor that reserves vertexes and indicies given an element range and list of elements
	 */
	template<typename ElementType, typename ElementAdder, typename ElementBatchParamCreator, typename ElementBatchReserver>
	inline void GenerateIndexedVertexBatches(const FSlateDrawElementArray<ElementType>& DrawElements
		, ElementAdder&& InElementAdder
		, ElementBatchParamCreator&& InElementBatchParamCreator
		, ElementBatchReserver&& InElementBatchReserver);

	/** 
	 * Creates vertices necessary to draw a Quad element 
	 */
	template<ESlateVertexRounding Rounding>
	void AddDebugQuadElement( const FSlateBoxElement& DrawElement);

	/** 
	 * Creates vertices necessary to draw multiple 3x3 elements
	 */
	template<typename ElementType>
	void AddBoxElements( const FSlateDrawElementArray<ElementType>& DrawElement );

	/** 
	 * Creates vertices necessary to draw a string (one quad per character)
	 */
	template<ESlateVertexRounding Rounding>
	void AddTextElement( const FSlateTextElement& DrawElement );

	/** 
	 * Creates vertices necessary to draw a shaped glyph sequence (one quad per glyph)
	 */
	template<ESlateVertexRounding Rounding>
	void AddShapedTextElement( const FSlateShapedTextElement& DrawElement );

	/** 
	 * Creates vertices necessary to draw a gradient box (horizontal or vertical)
	 */
	template<ESlateVertexRounding Rounding>
	void AddGradientElement( const FSlateGradientElement& DrawElement );

	/** 
	 * Creates vertices necessary to draw a spline (Bezier curve)
	 */
	void AddSplineElement( const FSlateSplineElement& DrawElement );

	/** 
	 * Creates vertices necessary to draw a multiple attached line segments
	 */
	void AddLineElements(const FSlateDrawElementArray<FSlateLineElement>& DrawElements);
	
	/** 
	 * Creates vertices necessary to draw a viewport (just a textured quad)
	 */
	template<ESlateVertexRounding Rounding>
	void AddViewportElement( const FSlateViewportElement& DrawElement );

	/** 
	 * Creates vertices necessary to draw a border element
	 */
	template<ESlateVertexRounding Rounding>
	void AddBorderElement( const FSlateBoxElement& DrawElement );

	void AddCustomElement( const FSlateCustomDrawerElement& DrawElement );

	void AddCustomVerts( const FSlateCustomVertsElement& DrawElement );

	void AddPostProcessPass(const FSlatePostProcessElement& DrawElement, FVector2f WindowSize);

	FSlateRenderBatch& CreateRenderBatch(
		int32 Layer,
		const FShaderParams& ShaderParams,
		const FSlateShaderResource* InResource,
		ESlateDrawPrimitive PrimitiveType,
		ESlateShader ShaderType,
		ESlateDrawEffect DrawEffects,
		ESlateBatchDrawFlag DrawFlags,
		int8 SceneIndex,
		const FSlateClippingState* ClippingState)
	{
		return CreateRenderBatch(BatchData, Layer, ShaderParams, InResource, PrimitiveType, ShaderType, DrawEffects, DrawFlags, SceneIndex, ClippingState);
	}

	FSlateRenderBatch& CreateRenderBatch(
		int32 Layer,
		const FShaderParams& ShaderParams,
		const FSlateShaderResource* InResource,
		ESlateDrawPrimitive PrimitiveType,
		ESlateShader ShaderType,
		ESlateDrawEffect DrawEffects,
		ESlateBatchDrawFlag DrawFlags,
		const FSlateDrawElement& DrawElement)
	{
		return CreateRenderBatch(BatchData, Layer, ShaderParams, InResource, PrimitiveType, ShaderType, DrawEffects, DrawFlags, DrawElement);
	}

	FSlateRenderBatch& CreateRenderBatch(
		FSlateBatchData* SlateBatchData,
		int32 Layer,
		const FShaderParams& ShaderParams,
		const FSlateShaderResource* InResource,
		ESlateDrawPrimitive PrimitiveType,
		ESlateShader ShaderType,
		ESlateDrawEffect DrawEffects,
		ESlateBatchDrawFlag DrawFlags,
		int8 SceneIndex, 
		const FSlateClippingState* ClippingState);

	FSlateRenderBatch& CreateRenderBatch(
		FSlateBatchData* SlateBatchData,
		int32 Layer,
		const FShaderParams& ShaderParams,
		const FSlateShaderResource* InResource,
		ESlateDrawPrimitive PrimitiveType,
		ESlateShader ShaderType,
		ESlateDrawEffect DrawEffects,
		ESlateBatchDrawFlag DrawFlags,
		const FSlateDrawElement& DrawElement);

	const FSlateClippingState* ResolveClippingState(const FSlateDrawElement& DrawElement) const;

	struct FShapedTextBuildContext
	{
		const class FShapedGlyphSequence* ShapedGlyphSequence;
		const class FShapedGlyphSequence* OverflowGlyphSequence;
		const UObject* FontMaterial;
		const struct FFontOutlineSettings* OutlineSettings;
		const FSlateDrawElement* DrawElement;
		class FSlateFontCache* FontCache;
		const FSlateRenderTransform* RenderTransform;
		float TextBaseline;
		float MaxHeight;
		float StartLineX;
		float StartLineY;
		float LocalClipBoundingBoxLeft = 0;
		float LocalClipBoundingBoxRight = 0;
		int32 MaxGlyphCountToRender = -1;
		int32 LayerId;
		FColor FontTint;
		ETextOverflowDirection OverflowDirection;
		ETextOverflowPolicy OverflowPolicy;
		bool bEnableCulling : 1;
		bool bForceEllipsis : 1;
		
	};

	template<ESlateVertexRounding Rounding>
	int32 BuildShapedTextSequence(const FShapedTextBuildContext& Context);
private:
	/** struct containing info to use for the middle ellipsis policy */
	struct FMiddleEllipsisOverflowData
	{
		FMiddleEllipsisOverflowData()
			: SkipIndexStart(INDEX_NONE)
			, SkipIndexEnd(INDEX_NONE)
			, LineX(0.f)
			, LineY(0.f)
			, EllipsisLineX(0.f)
			, EllipsisLineY(0.f)
		{}

		int32 SkipIndexStart;
		int32 SkipIndexEnd;
		float LineX;
		float LineY;
		float EllipsisLineX;
		float EllipsisLineY;
	};

	/** Retrieve information for the middle ellipsis, like offset to use and Glyphs to skip */
	void CalculateMiddleEllipsisSkipIndexAndOffset(const FShapedTextBuildContext& InContext, ETextOverflowDirection InOverflowDirection, FMiddleEllipsisOverflowData& OutMiddleEllipsisData);

	/** Uncached Batch data currently being filled in */
	FSlateBatchData* BatchData;
	FSlateBatchData* BatchDataHDR;

	/** Cached batches currently being filled in */
	FSlateCachedElementList* CurrentCachedElementList;

	/** Clipping states currently being used */
	const TArray<FSlateClippingState>* PrecachedClippingStates;

	/** Rendering policy we were created from */
	FSlateRenderingPolicy* RenderingPolicy;

#if STATS
	/** Track the number of drawn boxes from the previous frame to report to stats. */
	int32 ElementStat_Boxes;

	/** Track the number of drawn borders from the previous frame to report to stats. */
	int32 ElementStat_Borders;

	/** Track the number of drawn text from the previous frame to report to stats. */
	int32 ElementStat_Text;

	/** Track the number of drawn shaped text from the previous frame to report to stats. */
	int32 ElementStat_ShapedText;

	/** Track the number of drawn lines from the previous frame to report to stats. */
	int32 ElementStat_Line;

	/** Track the number of drawn batches from the previous frame to report to stats. */
	int32 ElementStat_Other;

	int32 ElementStat_RecachedElements;
#endif

	/** How many post process passes are needed */
	int32 NumPostProcessPasses;

	/** Offset to use when supporting 1:1 texture to pixel snapping */
	const float PixelCenterOffset;

	/** Are the vertex colors expected to be in sRGB space? */
	const bool bSRGBVertexColor;

	// true if any element in the batch requires vsync.
	bool bRequiresVsync;

	// true if viewports get composited as a separate pass, instead of being rendered directly to the render target. Useful for HDR displays
	bool bCompositeHDRViewports;

	// true if we added a resource that is using a slate post buffer 
	ESlatePostRT UsedSlatePostBuffers;

	// true if a resource is updating a slate post buffer, if true we need to add a fence for the scene draw to complete before clearing unused post buffers
	ESlatePostRT ResourceUpdatingPostBuffers;
};
