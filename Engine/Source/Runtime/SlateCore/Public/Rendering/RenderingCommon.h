// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Layout/SlateRect.h"
#include "Layout/SlateRotatedRect.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Input/NavigationReply.h"
#include "Input/PopupMethodReply.h"
#include "PixelFormat.h"
#include "Rendering/DrawElementCoreTypes.h"
#include "Rendering/SlateRendererTypes.h"
#include "SlateGlobals.h"
#include <type_traits>
#include <utility>

#include "RenderingCommon.generated.h"

class FRHICommandListImmediate;
class FRHIBuffer;
class FRDGBuilder;
class FRDGTexture;
class FSlateElementBatcher;
class FSlateInstanceBufferUpdate;
class FWidgetStyle;
class SWidget;

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num Cached Element Lists"), STAT_SlateNumCachedElementLists, STATGROUP_Slate, SLATECORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Num Cached Elements"), STAT_SlateNumCachedElements, STATGROUP_Slate, SLATECORE_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("PreFill Buffers"), STAT_SlatePreFullBufferTime, STATGROUP_Slate, SLATECORE_API);

#define UE_SLATE_VERIFY_PIXELSIZE UE_BUILD_DEBUG

#define SLATE_USE_32BIT_INDICES !PLATFORM_USES_GLES

#if SLATE_USE_32BIT_INDICES
typedef uint32 SlateIndex;
#else
typedef uint16 SlateIndex;
#endif
/**
 * Draw primitive types                   
 */
enum class ESlateDrawPrimitive : uint8
{
	None,
	LineList,
	TriangleList,
};

/**
 * Shader types. NOTE: mirrored in the shader file   
 * If you add a type here you must also implement the proper shader type (TSlateElementPS).  See SlateShaders.h
 */
enum class ESlateShader : uint8
{
	/** The default shader type.  Simple texture lookup */
	Default = 0,
	/** Border shader */
	Border = 1,
	/** Grayscale font shader. Uses an alpha only texture */
	GrayscaleFont = 2,
	/** Color font shader. Uses an sRGB texture */
	ColorFont = 3,
	/** Line segment shader. For drawing anti-aliased lines */
	LineSegment = 4,
	/** For completely customized materials.  Makes no assumptions on use*/
	Custom = 5,
	/** For post processing passes */
	PostProcess = 6,
	/** Rounded Box shader. **/
	RoundedBox = 7,
	/** Signed distance field font shader */
	SdfFont = 8,
	/** Multi-channel signed distance field font shader */
	MsdfFont = 9,
	/** Dynamic branching version of all the shader types above */
	Dynamic = 10,
};

/**
 * Effects that can be applied to elements when rendered.
 * Note: New effects added should be in bit mask form
 * If you add a type here you must also implement the proper shader type (TSlateElementPS).  See SlateShaders.h
 */
enum class ESlateDrawEffect : uint8
{
	/** No effect applied */
	None					= 0,
	/** Advanced: Draw the element with no blending */
	NoBlending			= 1 << 0,
	/** Advanced: Blend using pre-multiplied alpha. Ignored if NoBlending is set. */
	PreMultipliedAlpha	= 1 << 1,
	/** Advanced: No gamma correction should be done */
	NoGamma				= 1 << 2,
	/** Advanced: Change the alpha value to 1 - Alpha. */
	InvertAlpha			= 1 << 3,

	// ^^ These Match ESlateBatchDrawFlag ^^

	/** Disables pixel snapping */
	NoPixelSnapping		= 1 << 4,
	/** Draw the element with a disabled effect */
	DisabledEffect		= 1 << 5,
	/** Advanced: Don't read from texture alpha channel */
	IgnoreTextureAlpha	= 1 << 6,

	/** Advanced: Existing Gamma correction should be reversed */
	ReverseGamma			= 1 << 7
};

ENUM_CLASS_FLAGS(ESlateDrawEffect);

/** Flags for drawing a batch */
enum class ESlateBatchDrawFlag : uint16
{
	/** No draw flags */
	None					= 0,
	/** Draw the element with no blending */
	NoBlending			= 1 << 0,
	/** Blend using pre-multiplied alpha. Ignored if NoBlending is set. */
	PreMultipliedAlpha	= 1 << 1,
	/** No gamma correction should be done */
	NoGamma				= 1 << 2,
	/** Change the alpha value to 1 - Alpha */
	InvertAlpha			= 1 << 3,

	// ^^ These Match ESlateDrawEffect ^^

	/** Draw the element as wireframe */
	Wireframe			= 1 << 4,
	/** The element should be tiled horizontally */
	TileU				= 1 << 5,
	/** The element should be tiled vertically */
	TileV				= 1 << 6,
	/** Reverse gamma correction */
	ReverseGamma		= 1 << 7,
	/** Potentially apply to HDR batch when composition is active*/
	HDR					= 1 << 8
};

ENUM_CLASS_FLAGS(ESlateBatchDrawFlag);

enum class ESlateLineJoinType : uint8
{
	// Joins line segments with a sharp edge (miter)
	Sharp =	0,
	// Simply stitches together line segments
	Simple = 1,
};


/**
 * Enumerates color vision deficiency types.
 */
UENUM()
enum class EColorVisionDeficiency : uint8
{
	NormalVision UMETA(DisplayName="Normal Vision"),
	Deuteranope UMETA(DisplayName="Deuteranope (green weak/blind) (7% of males, 0.4% of females)"),
	Protanope UMETA(DisplayName="Protanope (red weak/blind) (2% of males, 0.01% of females)"),
	Tritanope UMETA(DisplayName="Tritanope (blue weak/blind) (0.0003% of males)"),
};


enum class ESlateVertexRounding : uint8
{
	Disabled,
	Enabled
};

enum class ESlateViewportDynamicRange : uint8
{
	SDR,
	HDR
};

class FSlateRenderBatch;


/**
* Shader parameters for slate
*/
struct FShaderParams
{
	/** Pixel shader parameters */
	FVector4f PixelParams;
	FVector4f PixelParams2;
	FVector4f PixelParams3;

	FShaderParams()
		: PixelParams(0, 0, 0, 0)
		, PixelParams2(0, 0, 0, 0)
		, PixelParams3(0, 0, 0, 0)
	{}

	FShaderParams(const FVector4f& InPixelParams, const FVector4f& InPixelParams2 = FVector4f(0), const FVector4f& InPixelParams3 = FVector4f(0))
		: PixelParams(InPixelParams)
		, PixelParams2(InPixelParams2)
		, PixelParams3(InPixelParams3)
	{}

	bool operator==(const FShaderParams& Other) const
	{
		return PixelParams == Other.PixelParams && PixelParams2 == Other.PixelParams2 && PixelParams3 == Other.PixelParams3;
	}

	static FShaderParams MakePixelShaderParams(const FVector4f& PixelShaderParams, const FVector4f& InPixelShaderParams2 = FVector4f(0), const FVector4f& InPixelShaderParams3 = FVector4f(0))
	{
		return FShaderParams(PixelShaderParams, InPixelShaderParams2, InPixelShaderParams3);
	}
};


/** 
 * A struct which defines a basic vertex seen by the Slate vertex buffers and shaders
 */
struct FSlateVertex
{
	/** Texture coordinates.  The first 2 are in xy and the 2nd are in zw */
	float TexCoords[4]; 

	/** Texture coordinates used as pass through to materials for custom texturing. */
	FVector2f MaterialTexCoords;

	/** Position of the vertex in window space */
	FVector2f Position;

	/** Vertex color */
	FColor Color;
	
	/** Secondary vertex color. Generally used for outlines */
	FColor SecondaryColor;

	/** Local size of the element */
	uint16 PixelSize[2];

	FSlateVertex() 
	{}

public:

	template<ESlateVertexRounding Rounding>
	inline static FSlateVertex Make(const FSlateRenderTransform& RenderTransform, const FVector2f InLocalPosition, const FVector2f InTexCoord, const FVector2f InTexCoord2, const FColor InColor, const FColor SecondaryColor = FColor())
	{
		FSlateVertex Vertex;
		Vertex.TexCoords[0] = InTexCoord.X;
		Vertex.TexCoords[1] = InTexCoord.Y;
		Vertex.TexCoords[2] = InTexCoord2.X;
		Vertex.TexCoords[3] = InTexCoord2.Y;
		Vertex.InitCommon<Rounding>(RenderTransform, InLocalPosition, InColor, SecondaryColor);

		return Vertex;
	}

	template<ESlateVertexRounding Rounding>
	inline static FSlateVertex Make(const FSlateRenderTransform& RenderTransform, const FVector2f InLocalPosition, const FVector2f InTexCoord, const FColor& InColor, const FColor SecondaryColor = FColor())
	{
		FSlateVertex Vertex;
		Vertex.TexCoords[0] = InTexCoord.X;
		Vertex.TexCoords[1] = InTexCoord.Y;
		Vertex.TexCoords[2] = 1.0f;
		Vertex.TexCoords[3] = 1.0f;
		Vertex.InitCommon<Rounding>(RenderTransform, InLocalPosition, InColor, SecondaryColor);

		return Vertex;
	}

	template<ESlateVertexRounding Rounding>
	inline static FSlateVertex Make(const FSlateRenderTransform& RenderTransform, const FVector2f InLocalPosition, const FVector4f InTexCoords, const FVector2f InMaterialTexCoords, const FColor InColor, const FColor SecondaryColor = FColor())
	{
		FSlateVertex Vertex;
		Vertex.TexCoords[0] = InTexCoords.X;
		Vertex.TexCoords[1] = InTexCoords.Y;
		Vertex.TexCoords[2] = InTexCoords.Z;
		Vertex.TexCoords[3] = InTexCoords.W;
		Vertex.MaterialTexCoords = InMaterialTexCoords;
		Vertex.InitCommon<Rounding>(RenderTransform, InLocalPosition, InColor, SecondaryColor);

		return Vertex;
	}

	template<ESlateVertexRounding Rounding>
	inline static FSlateVertex Make(const FSlateRenderTransform& RenderTransform, const FVector2f InLocalPosition, const FVector2f InLocalSize, float Scale, const FVector4f InTexCoords, const FColor InColor, const FColor SecondaryColor = FColor())
	{
		FSlateVertex Vertex;
		Vertex.TexCoords[0] = InTexCoords.X;
		Vertex.TexCoords[1] = InTexCoords.Y;
		Vertex.TexCoords[2] = InTexCoords.Z;
		Vertex.TexCoords[3] = InTexCoords.W;
		Vertex.MaterialTexCoords = FVector2f(InLocalPosition.X / InLocalSize.X, InLocalPosition.Y / InLocalSize.Y);
		Vertex.InitCommon<Rounding>(RenderTransform, InLocalPosition, InColor, SecondaryColor);

		const int32 PixelSizeX = FMath::RoundToInt(InLocalSize.X * Scale);
		const int32 PixelSizeY = FMath::RoundToInt(InLocalSize.Y * Scale);
		Vertex.PixelSize[0] = (uint16)PixelSizeX;
		Vertex.PixelSize[1] = (uint16)PixelSizeY;

#if UE_SLATE_VERIFY_PIXELSIZE
		ensureMsgf((int32)Vertex.PixelSize[0] == PixelSizeX, TEXT("Conversion of PixelSizeX is bigger than 16. Cast:%d, int16:%d, int32:%d")
			, (int32)Vertex.PixelSize[0], Vertex.PixelSize[0], PixelSizeX);
		ensureMsgf((int32)Vertex.PixelSize[1] == PixelSizeY, TEXT("Conversion of PixelSizeY is bigger than 16. Cast:%d, int16:%d, int32:%d")
			, (int32)Vertex.PixelSize[1], Vertex.PixelSize[1], PixelSizeY);
#endif

		return Vertex;
	}
	
	inline static FSlateVertex Make(const FSlateRenderTransform& RenderTransform, const FVector2f InLocalPosition, const FVector2f InTexCoord, const FVector2f InTexCoord2, const FColor InColor, const FColor SecondaryColor = FColor(), const ESlateVertexRounding InRounding = ESlateVertexRounding::Disabled)
	{
		return InRounding == ESlateVertexRounding::Enabled
			? FSlateVertex::Make<ESlateVertexRounding::Enabled>(RenderTransform, InLocalPosition, InTexCoord, InTexCoord2, InColor, SecondaryColor)
			: FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, InLocalPosition, InTexCoord, InTexCoord2, InColor, SecondaryColor);
	}

	
	inline static FSlateVertex Make(const FSlateRenderTransform& RenderTransform, const FVector2f InLocalPosition, const FVector2f InTexCoord, const FColor& InColor, const FColor SecondaryColor = FColor(), const ESlateVertexRounding InRounding = ESlateVertexRounding::Disabled)
	{
		return InRounding == ESlateVertexRounding::Enabled
			? FSlateVertex::Make<ESlateVertexRounding::Enabled>(RenderTransform, InLocalPosition, InTexCoord, InColor, SecondaryColor)
			: FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, InLocalPosition, InTexCoord, InColor, SecondaryColor);
	}

	
	inline static FSlateVertex Make(const FSlateRenderTransform& RenderTransform, const FVector2f InLocalPosition, const FVector4f InTexCoords, const FVector2f InMaterialTexCoords, const FColor InColor, const FColor SecondaryColor = FColor(), const ESlateVertexRounding InRounding = ESlateVertexRounding::Disabled)
	{
		return InRounding == ESlateVertexRounding::Enabled
			? FSlateVertex::Make<ESlateVertexRounding::Enabled>(RenderTransform, InLocalPosition, InTexCoords, InMaterialTexCoords, InColor, SecondaryColor)
			: FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, InLocalPosition, InTexCoords, InMaterialTexCoords, InColor, SecondaryColor);
	}

	
	inline static FSlateVertex Make(const FSlateRenderTransform& RenderTransform, const FVector2f InLocalPosition, const FVector2f InLocalSize, float Scale, const FVector4f InTexCoords, const FColor InColor, const FColor SecondaryColor = FColor(), const ESlateVertexRounding InRounding = ESlateVertexRounding::Disabled)
	{
		return InRounding == ESlateVertexRounding::Enabled
			? FSlateVertex::Make<ESlateVertexRounding::Enabled>(RenderTransform, InLocalPosition, InLocalSize, Scale, InTexCoords, InColor, SecondaryColor)
			: FSlateVertex::Make<ESlateVertexRounding::Disabled>(RenderTransform, InLocalPosition, InLocalSize, Scale, InTexCoords, InColor, SecondaryColor);
	}

	inline void SetTexCoords(const FVector4f InTexCoords)
	{
		TexCoords[0] = InTexCoords.X;
		TexCoords[1] = InTexCoords.Y;
		TexCoords[2] = InTexCoords.Z;
		TexCoords[3] = InTexCoords.W;
	}

	inline void SetPosition(const FVector2f InPosition)
	{
		Position = InPosition;
	}

private:

	template<ESlateVertexRounding Rounding>
	inline void InitCommon(const FSlateRenderTransform& RenderTransform, const FVector2f InLocalPosition, const FColor InColor, const FColor InSecondaryColor)
	{
		Position = TransformPoint(RenderTransform, InLocalPosition);

		if constexpr ( Rounding == ESlateVertexRounding::Enabled )
		{
			Position.X = FMath::RoundToFloat(Position.X);
			Position.Y = FMath::RoundToFloat(Position.Y);
		}

		Color = InColor;
		SecondaryColor = InSecondaryColor;
	}
};

template<> struct TIsPODType<FSlateVertex> { enum { Value = true }; };
static_assert(std::is_trivially_destructible_v<FSlateVertex>, "FSlateVertex should be trivially destructible");
static_assert(std::is_trivially_copyable_v<FSlateVertex> == true, "FSlateVertex should be trivially copyable");

/** Stores an aligned rect as shorts. */
struct FShortRect
{
	FShortRect()
		: Left(0)
		, Top(0)
		, Right(0)
		, Bottom(0)
	{
	}
	
	FShortRect(uint16 InLeft, uint16 InTop, uint16 InRight, uint16 InBottom)
		: Left(InLeft)
		, Top(InTop)
		, Right(InRight)
		, Bottom(InBottom)
	{
	}
	
	explicit FShortRect(const FSlateRect& Rect)
		: Left((uint16)FMath::Clamp(Rect.Left, 0.0f, 65535.0f))
		, Top((uint16)FMath::Clamp(Rect.Top, 0.0f, 65535.0f))
		, Right((uint16)FMath::Clamp(Rect.Right, 0.0f, 65535.0f))
		, Bottom((uint16)FMath::Clamp(Rect.Bottom, 0.0f, 65535.0f))
	{
	}

	bool operator==(const FShortRect& RHS) const { return Left == RHS.Left && Top == RHS.Top && Right == RHS.Right && Bottom == RHS.Bottom; }
	bool operator!=(const FShortRect& RHS) const { return !(*this == RHS); }
	bool DoesIntersect( const FShortRect& B ) const
	{
		const bool bDoNotOverlap =
			B.Right < Left || Right < B.Left ||
			B.Bottom < Top || Bottom < B.Top;

		return ! bDoNotOverlap;
	}

	bool DoesIntersect(const FSlateRect& B) const
	{
		const bool bDoNotOverlap =
			B.Right < Left || Right < B.Left ||
			B.Bottom < Top || Bottom < B.Top;

		return !bDoNotOverlap;
	}

	FVector2f GetTopLeft() const { return FVector2f(Left, Top); }
	FVector2f GetBottomRight() const { return FVector2f(Right, Bottom); }

	uint16 Left;
	uint16 Top;
	uint16 Right;
	uint16 Bottom;
};

template<> struct TIsPODType<FShortRect> { enum { Value = true }; };
static_assert(std::is_trivially_destructible_v<FShortRect>, "FShortRect should be trivially destructible");

namespace UE::Slate
{
	template<typename IndexType, IndexType...Indices>
	auto MakeTupleIndiciesInner(std::integer_sequence<IndexType, Indices...>)
	{
		return MakeTuple<IndexType>(Indices...);
	};

	template<typename IndexType, std::size_t Num, typename Indices = std::make_integer_sequence<IndexType, Num>>
	auto MakeTupleIndicies()
	{
		return MakeTupleIndiciesInner(Indices{});
	};
};

/**
 * Note: FRenderingBufferStatTracker & FSlateDrawElementArray have been moved to DrawElementCoreTypes.h
 */

#if STATS

typedef TArray<FSlateVertex, FSlateStatTrackingMemoryAllocator<FRenderingBufferStatTracker>> FSlateVertexArray;
typedef TArray<SlateIndex, FSlateStatTrackingMemoryAllocator<FRenderingBufferStatTracker>> FSlateIndexArray;

#else

typedef TArray<FSlateVertex> FSlateVertexArray;
typedef TArray<SlateIndex> FSlateIndexArray;

#endif // STATS

namespace UE::Slate
{
	/**
	 * ISlateViewport::Paint must return this interface.  By default, it is a no-op.  If a Slate Viewport needs to establish some global state while
	 * its host SWidget and while that SWidget's child widget(s) are painted, then it should clean up that state in the destructor of the PaintScope
	 * that it returns.
	 */
	class FPaintScope
	{
	public:
		virtual ~FPaintScope() = default;
	};
}

/**
 * Viewport implementation interface that is used by SViewport when it needs to draw and processes input.                   
 */
class ISlateViewport
{
public:
	virtual ~ISlateViewport() = default;

	/**
	 * Called by Slate when the viewport widget is drawn
	 * Implementers of this interface can use this method to perform custom
	 * per draw functionality.  This is only called if the widget is visible
	 *
	 * @param AllottedGeometry	The geometry of the viewport widget
	 */
	UE_DEPRECATED(5.8, "Override Paint instead; perform any teardown in the returned scope's destructor so it runs after child widgets paint.")
	virtual void OnDrawViewport( const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, class FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) final { }

	/**
	 * Called by SViewport at the start of its OnPaint. The returned scope object lives until SViewport::OnPaint
	 * returns.
	 */
	virtual TUniquePtr<UE::Slate::FPaintScope> Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, const int32 LayerId, const FWidgetStyle& InWidgetStyle, const bool bParentEnabled)
	{
		return nullptr;
	}

	/**
	 * Returns the size of the viewport                   
	 */
	virtual FIntPoint GetSize() const = 0;

	/**
	 * Returns a slate texture used to draw the rendered viewport in Slate.                   
	 */
	virtual class FSlateShaderResource* GetViewportRenderTargetTexture() const = 0;

	/**
	 * Returns format for the scene of this viewport, only valid if this viewport actually has a scene, else PF_Unknown.
	 */
	virtual EPixelFormat GetSceneTargetFormat() const { return EPixelFormat::PF_Unknown; };

	/**
	 * Does the texture returned by GetViewportRenderTargetTexture only have an alpha channel?
	 */
	virtual bool IsViewportTextureAlphaOnly() const
	{
		return false;
	}

	/**
	 * Does the texture returned by GetViewportRenderTargetTexture represent a stereoscopic 3D target?
	 */
	virtual bool IsStereoscopic3D() const
	{
		return false;
	}

	/**
	* Does this frame identify as a Camera Cut ?
	*/
	virtual bool IsCameraCut() const
	{
		return false;
	}

	
	/**
	 * Is the viewport expected to render to GetViewportRenderTargetTexture? Or to the swapchain?
	 */
	virtual bool UseSeparateRenderTarget() const
	{
		return false;
	}

	/**
	 * Does the texture contain SDR/HDR information
	 */
	virtual ESlateViewportDynamicRange GetViewportDynamicRange() const
	{
		return ESlateViewportDynamicRange::SDR;
	}

	/**
	 * Performs any ticking necessary by this handle                   
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, double InCurrentTime, float DeltaTime )
	{
	}

	/**
	 * Returns true if the viewport should be vsynced.
	 */
	virtual bool RequiresVsync() const = 0;

	/**
	 * Whether the viewport contents should be scaled or not. Defaults to true.
	 */
	virtual bool AllowScaling() const
	{
		return true;
	}

	/**
	 * Called when Slate needs to know what the mouse cursor should be.
	 * 
	 * @return FCursorReply::Unhandled() if the event is not handled; FCursorReply::Cursor() otherwise.
	 */
	virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent )
	{
		return FCursorReply::Unhandled();
	}

	/**
	 * After OnCursorQuery has specified a cursor type the system asks each widget under the mouse to map that cursor to a widget. This event is bubbled.
	 * 
	 * @return TOptional<TSharedRef<SWidget>>() if you don't have a mapping otherwise return the Widget to show.
	 */
	virtual TOptional<TSharedRef<SWidget>> OnMapCursor(const FCursorReply& CursorReply)
	{
		return TOptional<TSharedRef<SWidget>>();
	}

	/**
	 *	Returns whether the software cursor is currently visible
	 */
	virtual bool IsSoftwareCursorVisible() const
	{
		return false;
	}

	/**
	 *	Returns the current position of the software cursor
	 */
	virtual FVector2D GetSoftwareCursorPosition() const
	{
		return FVector2D::ZeroVector;
	}

	/**
	 * Called by Slate when a mouse button is pressed inside the viewport
	 *
	 * @param MyGeometry	Information about the location and size of the viewport
	 * @param MouseEvent	Information about the mouse event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called by Slate when a mouse button is released inside the viewport
	 *
	 * @param MyGeometry	Information about the location and size of the viewport
	 * @param MouseEvent	Information about the mouse event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
	{
		return FReply::Unhandled();
	}


	virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
	{

	}

	virtual void OnMouseLeave( const FPointerEvent& MouseEvent )
	{
		
	}

	/**
	 * Called by Slate when a mouse button is released inside the viewport
	 *
	 * @param MyGeometry	Information about the location and size of the viewport
	 * @param MouseEvent	Information about the mouse event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called by Slate when the mouse wheel is used inside the viewport
	 *
	 * @param MyGeometry	Information about the location and size of the viewport
	 * @param MouseEvent	Information about the mouse event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called by Slate when the mouse wheel is used inside the viewport
	 *
	 * @param MyGeometry	Information about the location and size of the viewport
	 * @param MouseEvent	Information about the mouse event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called by Slate when a key is pressed inside the viewport
	 *
	 * @param MyGeometry	Information about the location and size of the viewport
	 * @param MouseEvent	Information about the key event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called by Slate when a key is released inside the viewport
	 *
	 * @param MyGeometry	Information about the location and size of the viewport
	 * @param MouseEvent	Information about the key event
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called when an analog value changes on a button that supports analog
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param InAnalogInputEvent Analog input event
	 * @return Returns whether the event was handled, along with other possible actions
	 */
	virtual FReply OnAnalogValueChanged( const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called by Slate when a character key is pressed while the viewport has focus
	 *
	 * @param MyGeometry	Information about the location and size of the viewport
	 * @param MouseEvent	Information about the character that was pressed
	 *
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	virtual FReply OnKeyChar( const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called when the viewport gains keyboard focus.  
	 *
	 * @param InFocusEvent	Information about what caused the viewport to gain focus
	 */
	virtual FReply OnFocusReceived( const FFocusEvent& InFocusEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called when a touchpad touch is started (finger down)
	 * 
	 * @param ControllerEvent	The controller event generated
	 */
	virtual FReply OnTouchStarted( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called when a touchpad touch is moved  (finger moved)
	 * 
	 * @param ControllerEvent	The controller event generated
	 */
	virtual FReply OnTouchMoved( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called when a touchpad touch is ended (finger lifted)
	 * 
	 * @param ControllerEvent	The controller event generated
	 */
	virtual FReply OnTouchEnded( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called when a touchpad touch force changes, but may or may not have moved
	 * 
	 * @param ControllerEvent	The controller event generated
	 */
	virtual FReply OnTouchForceChanged( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called when a touchpad touch has first moved after initial press
	 * 
	 * @param ControllerEvent	The controller event generated
	 */
	virtual FReply OnTouchFirstMove( const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called on a touchpad gesture event
	 *
	 * @param InGestureEvent	The touch event generated
	 */
	virtual FReply OnTouchGesture( const FGeometry& MyGeometry, const FPointerEvent& InGestureEvent )
	{
		return FReply::Unhandled();
	}
	
	/**
	 * Called when motion is detected (controller or device)
	 * e.g. Someone tilts or shakes their controller.
	 * 
	 * @param InMotionEvent	The motion event generated
	 */
	virtual FReply OnMotionDetected( const FGeometry& MyGeometry, const FMotionEvent& InMotionEvent )
	{
		return FReply::Unhandled();
	}

	/**
	 * Called to determine if we should render the focus brush.
	 *
	 * @param InFocusCause	The cause of focus
	 */
	virtual TOptional<bool> OnQueryShowFocus(const EFocusCause InFocusCause) const
	{
		return TOptional<bool>();
	}

	/**
	 * Called after all input for this frame is processed.
	 */
	virtual void OnFinishedPointerInput()
	{
	}

	/**
	 * Called to figure out whether we can make new windows for popups within this viewport.
	 * Making windows allows us to have popups that go outside the parent window, but cannot
	 * be used in fullscreen and do not have per-pixel alpha.
	 */
	virtual FPopupMethodReply OnQueryPopupMethod() const
	{
		return FPopupMethodReply::Unhandled();
	}

	/**
	 * Called when navigation is requested
	 * e.g. Left Joystick, Direction Pad, Arrow Keys can generate navigation events.
	 * 
	 * @param InNavigationEvent	The navigation event generated
	 */
	virtual FNavigationReply OnNavigation( const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent )
	{
		return FNavigationReply::Stop();
	}

	/**
	 * Give the viewport an opportunity to override the navigation behavior.
	 * This is called after all the navigation event bubbling is complete and we know a destination.
	 *
	 * @param InDestination	The destination widget
	 * @return whether we handled the navigation
	 */
	virtual bool HandleNavigation(const uint32 InUserIndex, TSharedPtr<SWidget> InDestination)
	{
		return false;
	}

	/**
	 * Called when the viewport loses keyboard focus.  
	 *
	 * @param InFocusEvent	Information about what caused the viewport to lose focus
	 */
	virtual void OnFocusLost( const FFocusEvent& InFocusEvent )
	{
	}

	/**
	 * Called when the top level window associated with the viewport has been requested to close.
	 * At this point, the viewport has not been closed and the operation may be canceled.
	 * This may not called from PIE, Editor Windows, on consoles, or before the game ends
 	 * from other methods.
	 * This is only when the platform specific window is closed.
	 *
	 * @return FReply::Handled if the close event was consumed (and the window should remain open).
	 */
	virtual FReply OnRequestWindowClose()
	{
		return FReply::Unhandled();
	}

	/**
	 * Called when the viewport has been requested to close.
	 */
	virtual void OnViewportClosed()
	{
	}

	/**
	 * Gets the SWidget associated with this viewport
	 */
	virtual TWeakPtr<SWidget> GetWidget()
	{
		return nullptr;
	}

	/**
	 * Called when the viewports top level window is being Activated
	 */
	virtual FReply OnViewportActivated(const FWindowActivateEvent& InActivateEvent)
	{
		return FReply::Unhandled();
	}

	/**
	 * Called when the viewports top level window is being Deactivated
	 */
	virtual void OnViewportDeactivated(const FWindowActivateEvent& InActivateEvent)
	{
	}
};

/**
 * An interface for a custom slate elements. Lifetime must support pipelining to the render thread.
 */
class ICustomSlateElement
{
public:
	virtual ~ICustomSlateElement() = default;

	/** Called from the game thread during element batching. */
	virtual void PostCustomElementAdded(FSlateElementBatcher& ElementBatcher) const {}

	struct FDrawPassInputs
	{
		FRDGTexture* OutputTexture = nullptr;
		FIntRect SceneViewRect;
		FMatrix44f ElementsMatrix;
		FVector2f ElementsOffset;
		EDisplayColorGamut HDRDisplayColorGamut = EDisplayColorGamut::sRGB_D65;
		ESlatePostRT UsedSlatePostBuffers = ESlatePostRT::None;
		bool bOutputIsHDRDisplay = false;
		bool bWireFrame = false;
	};

	/** Called from the render thread during element drawing. */
	virtual void Draw_RenderThread(FRDGBuilder& GraphBuilder, const FDrawPassInputs& Inputs) {};

	//////////////////////////////////////////////////////////////////////////
	// Deprecated API

	struct UE_DEPRECATED(5.5, "Use ICustomSlateElement::Draw instead") FSlateCustomDrawParams
	{
		FMatrix44f ViewProjectionMatrix;
		FVector2f ViewOffset;
		FIntRect ViewRect;
		EDisplayColorGamut HDRDisplayColorGamut = EDisplayColorGamut::sRGB_D65;
		ESlatePostRT UsedSlatePostBuffers = ESlatePostRT::None;
		bool bWireFrame = false;
		bool bIsHDR = false;
	};

	UE_DEPRECATED(5.4, "Use ICustomSlateElement::Draw instead.")
	virtual void DrawRenderThread(FRHICommandListImmediate& RHICmdList, const void* RenderTarget) {}

	UE_DEPRECATED(5.5, "Use ICustomSlateElement::Draw instead.")
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual void Draw_RenderThread(FRHICommandListImmediate& RHICmdList, const void* RenderTarget, const FSlateCustomDrawParams& Params) {}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.5, "Use ICustomSlateElement::Draw instead.")
	virtual bool UsesAdditionalRHIParams() const { return false; }

	//////////////////////////////////////////////////////////////////////////
};

/*
 * A proxy object used to access a slate per-instance data buffer on the render thread.
 */
class ISlateUpdatableInstanceBufferRenderProxy
{
protected:
	virtual ~ISlateUpdatableInstanceBufferRenderProxy() = default;

public:
	virtual FRHIBuffer* GetRHI() const = 0;

	UE_DEPRECATED(5.5, "Use GetStreamSource instead")
	virtual void BindStreamSource(class FRHICommandList& RHICmdList, int32 StreamIndex, uint32 InstanceOffset) {}
};

typedef TArray<FVector4f> FSlateInstanceBufferData;

/**
 * Represents a per instance data buffer for a custom Slate mesh element.
 */
class ISlateUpdatableInstanceBuffer
{
public:
	virtual ~ISlateUpdatableInstanceBuffer() = default;
	friend class FSlateInstanceBufferUpdate;

	/** How many instances should we draw? */
	virtual uint32 GetNumInstances() const = 0;

	/** Returns the pointer to the render proxy, to be forwarded to the render thread. */
	virtual ISlateUpdatableInstanceBufferRenderProxy* GetRenderProxy() const = 0;

	/** Updates the buffer with the provided data. */
	virtual void Update(FSlateInstanceBufferData& Data) = 0;
};
