// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


class FSlateFontServices;
class FSlateElementBatch;
class FSlateShaderResource;
struct FSlateVertex;
class FSceneInterface;


/**
 * Abstract base class for Slate rendering policies.
 */
class FSlateRenderingPolicy
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InPixelCenterOffset
	 */
	FSlateRenderingPolicy(const TSharedRef<class FSlateFontServices>& InFontServices, float InPixelCenterOffset)
		: FontServices(InFontServices)
		, PixelCenterOffset(InPixelCenterOffset)
	{ }

	/** Virtual destructor. */
	virtual ~FSlateRenderingPolicy() = default;

	TSharedRef<class FSlateFontCache> GetFontCache() const;
	TSharedRef<class FSlateFontServices> GetFontServices() const;

	virtual TSharedRef<class FSlateShaderResourceManager> GetResourceManager() const = 0;

	virtual bool IsVertexColorInLinearSpace() const = 0;

	float GetPixelCenterOffset() const
	{
		return PixelCenterOffset;
	}


	virtual void AddSceneAt(FSceneInterface* Scene, int32 Index){};
	virtual void ClearScenes() {};

private:

	FSlateRenderingPolicy(const FSlateRenderingPolicy&);
	FSlateRenderingPolicy& operator=(const FSlateRenderingPolicy&);

private:

	TSharedRef<FSlateFontServices> FontServices;
	float PixelCenterOffset;

};
