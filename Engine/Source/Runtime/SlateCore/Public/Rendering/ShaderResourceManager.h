// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateBrush.h"
#include "Textures/SlateShaderResource.h"
#include "Textures/SlateTextureData.h"
#include "Textures/TextureAtlas.h"
#include "Types/SlateVector2.h"

/**
 * Represents a new texture that has been loaded but no resource created for it
 */
struct FNewTextureInfo
{
	/** Raw data */
	FSlateTextureDataPtr TextureData;
	/** Whether or not the texture should be atlased */
	bool bShouldAtlas;
	/** Whether or not the texture is in srgb space */
	bool bSrgb;
	FNewTextureInfo()
		: bShouldAtlas(true)
		, bSrgb(true)
	{

	}
};

struct FCompareFNewTextureInfoByTextureSize
{
	inline bool operator()( const FNewTextureInfo& A, const FNewTextureInfo& B ) const
	{
		return (B.TextureData->GetWidth()+B.TextureData->GetHeight()) < (A.TextureData->GetWidth()+A.TextureData->GetHeight());
	}
};


/** 
 * Base texture manager class used by a Slate renderer to manage texture resources
 */
class FSlateShaderResourceManager
{
public:
	SLATECORE_API FSlateShaderResourceManager();
	SLATECORE_API virtual ~FSlateShaderResourceManager();

	/** 
	 * Returns a texture associated with the passed in name.  Should return nullptr if not found 
	 */
	virtual FSlateShaderResourceProxy* GetShaderResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale) = 0;

	/**
	 * Creates a handle to a Slate resource
	 * A handle is used as fast path for looking up a rendering resource for a given brush when adding Slate draw elements
	 * This can be cached and stored safely in code.  It will become invalid when a resource is destroyed
	 * It is expensive to create a resource so do not do it in time sensitive areas
	 *
	 * @param	Brush		The brush to get a rendering resource handle 
	 * @param	LocalSize	The unscaled size of the draw element using this brush.  This size is used to rescale vector graphics only
	 * @param	DrawScale	The scale of the draw element using this brush.  This scale is used to rescale vector graphics only
	 * @return	The created resource handle.  
	 */
	SLATECORE_API virtual FSlateResourceHandle GetResourceHandle(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale);
	SLATECORE_API virtual FSlateResourceHandle GetResourceHandle(const FSlateBrush& Brush);

	virtual FSlateShaderResource* GetFontShaderResource( int32 InTextureAtlasIndex, FSlateShaderResource* FontTextureAtlas, const class UObject* FontMaterial ) { return FontTextureAtlas; }

	/**
	 * Returns the way to access the texture atlas information from this resource manager
	 */
	virtual ISlateAtlasProvider* GetTextureAtlasProvider() = 0;

protected:

	void ClearTextureMap()
	{
		// delete all allocated textures
		for( TMap<FName,FSlateShaderResourceProxy*>::TIterator It(ResourceMap); It; ++It )
		{
			delete It.Value();
		}
		ResourceMap.Empty();
#if WITH_ATLAS_DEBUGGING
		AtlasDebugData.Empty();
#endif
	}

	FString GetResourcePath( const FSlateBrush& InBrush ) const
	{
		// assume the brush name contains the whole path
		return InBrush.GetResourceName().ToString();
	}

	/** Mapping of names to texture pointers */
	TMap<FName,FSlateShaderResourceProxy*> ResourceMap;

#if WITH_ATLAS_DEBUGGING
	TMap<const FAtlasedTextureSlot*, FName> AtlasDebugData;
#endif
private:
	// Non-copyable
	SLATECORE_API FSlateShaderResourceManager(const FSlateShaderResourceManager&);
	SLATECORE_API FSlateShaderResourceManager& operator=(const FSlateShaderResourceManager&);

};
