// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "HAL/CriticalSection.h"
#include "Types/SlateVector2.h"
#include "Tasks/Task.h"

#include <atomic>

class FSlateWindowElementList;
class SWindow;

/**
 * Implements a draw buffer for Slate.
 */
class FSlateDrawBuffer : public FGCObject
{
public:
	/** Default constructor. */
	SLATECORE_API FSlateDrawBuffer();
	SLATECORE_API ~FSlateDrawBuffer();

	/** Removes all data from the buffer. */
	SLATECORE_API void ClearBuffer();

	/** Updates renderer resource version to allow the draw buffer to clean up cached resources */
	SLATECORE_API void UpdateResourceVersion(uint32 NewResourceVersion);

	/**
	 * Creates a new FSlateWindowElementList and returns a reference to it so it can have draw elements added to it
	 *
	 * @param ForWindow    The window for which we are creating a list of paint elements.
	 */
	SLATECORE_API FSlateWindowElementList& AddWindowElementList(TSharedRef<SWindow> ForWindow);

	/** Removes any window from the draw buffer that's not in this list or whose window has become invalid. */
	SLATECORE_API void RemoveUnusedWindowElement(const TArray<SWindow*>& AllWindows);

	/**
	 * Gets all window element lists in this buffer.
	 */
	const TArray< TSharedRef<FSlateWindowElementList> >& GetWindowElementLists()
	{
		return WindowElementLists;
	}

	/** 
	 * Locks the draw buffer.  Indicates that the viewport is in use.
	 *
	 * @return true if the buffer could be locked.  False otherwise.
	 * @see Unlock
	 */
	SLATECORE_API bool Lock();

	/**
	 * Unlocks the buffer.  Indicates that the buffer is free.
	 *
	 * @see Lock
	 */
	SLATECORE_API void Unlock(const UE::Tasks::FTask& PrerequisiteTask);

	void Unlock()
	{
		Unlock(UE::Tasks::FTask{});
	}

	/** @return true if the buffer is locked. */
	bool IsLocked() const
	{
		return bIsLocked;
	}

	/** FGCObject Interface */
	SLATECORE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const
	{
		return TEXT("FSlateDrawBuffer for Uncached Elements");
	}

private:
	// List of window element lists
	TArray< TSharedRef<FSlateWindowElementList> > WindowElementLists;

	// List of window element lists that we store from the previous frame 
	// that we restore if they're requested again.
	TArray< TSharedRef<FSlateWindowElementList> > WindowElementListsPool;

	FCriticalSection GCLock;
	std::atomic<bool> bIsLocked;
	bool bIsLockedBySlateThread;

	// Last recorded version from the render. The WindowElementListsPool is emptied when this changes.
	uint32 ResourceVersion;

public:
	FDeprecateSlateVector2D ViewOffset;
};
