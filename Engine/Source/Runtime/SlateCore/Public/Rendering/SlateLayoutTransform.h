// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Math/TransformCalculus.h"
#include "Math/TransformCalculus2D.h"
#include "Types/SlateVector2.h"

/**
 * Represents a 2D transformation in the following order: scale then translate.
 * Used by FGeometry for it's layout transformations.
 *
 * Matrix form looks like:
 *   [Vx Vy 1] * [ S   0   0 ]
 *               [ 0   S   0 ]
 *               [ Tx  Ty  1 ]
 *
 */
class FSlateLayoutTransform
{
public:
	FSlateLayoutTransform(float InScale = 1.0f)
		: Scale(InScale)
		, Translation(FVector2f(ForceInit))
	{
	}

	/** Ctor from a scale followed by translate. Shortcut to Concatenate(InScale, InTranslation). */
	explicit FSlateLayoutTransform(float InScale, const UE::Slate::FDeprecateVector2DParameter& InTranslation)
		: Scale(InScale)
		, Translation(FVector2f(InTranslation))
	{
	}

	explicit FSlateLayoutTransform(const UE::Slate::FDeprecateVector2DParameter& InTranslation)
		:Scale(1.0f)
		,Translation(FVector2f(InTranslation))
	{
	}

	/** Access to the 2D translation */
	UE::Slate::FDeprecateVector2DResult GetTranslation() const
	{
		return UE::Slate::FDeprecateVector2DResult(Translation);
	}

	/** Access to the scale. */
	float GetScale() const
	{
		return Scale;
	}

	/** Support for converting to an FMatrix. */
	FMatrix ToMatrix() const
	{
		FMatrix Matrix = FScaleMatrix(GetScale());
		Matrix.SetOrigin(FVector(FVector2d(Translation), 0.0f));
		return Matrix;
	}

	inline UE::Slate::FDeprecateVector2DResult TransformPoint(const UE::Slate::FDeprecateVector2DParameter& Point) const
	{
		return UE::Slate::FDeprecateVector2DResult(::TransformPoint(Translation, ::TransformPoint(Scale, FVector2f(Point))));
	}

	inline UE::Slate::FDeprecateVector2DResult TransformVector(const UE::Slate::FDeprecateVector2DParameter& Vector) const
	{
		return UE::Slate::FDeprecateVector2DResult(::TransformVector(Translation, ::TransformVector(Scale, FVector2f(Vector))));
	}

	/**
	 * This works by transforming the origin through LHS then RHS.
	 * In matrix form, looks like this:
	 * [ Sa  0   0 ]   [ Sb  0   0 ]
	 * [ 0   Sa  0 ] * [ 0   Sb  0 ] 
	 * [ Tax Tay 1 ]   [ Tbx Tby 1 ]
	 */
	FSlateLayoutTransform Concatenate(const FSlateLayoutTransform& RHS) const
	{
		// New Translation is essentially: RHS.TransformPoint(TransformPoint(FVector2D::ZeroVector))
		// Since Zero through LHS -> Translation we optimize it slightly to skip the zero multiplies.
		return FSlateLayoutTransform(::Concatenate(Scale, RHS.Scale), RHS.TransformPoint(Translation));
	}

	/** Invert the transform/scale. */
	FSlateLayoutTransform Inverse() const
	{
		return FSlateLayoutTransform(::Inverse(Scale), ::Inverse(Translation) * ::Inverse(Scale));
	}

	/** Equality. */
	bool operator==(const FSlateLayoutTransform& Other) const
	{
		return Scale == Other.Scale && Translation == Other.Translation;
	}
	
	/** Inequality. */
	bool operator!=(const FSlateLayoutTransform& Other) const
	{
		return !operator==(Other);
	}

private:
	float Scale;
	FVector2f Translation;
};

/** Specialization for concatenating a uniform scale and 2D Translation. */
template<typename T>
inline FSlateLayoutTransform Concatenate(float Scale, const UE::Math::TVector2<T>& Translation)
{
	return FSlateLayoutTransform(Scale, Translation);
}

template<typename T>
inline FSlateLayoutTransform Concatenate(double Scale, const UE::Math::TVector2<T>& Translation)
{
	return FSlateLayoutTransform((float)Scale, Translation);
}

/** Specialization for concatenating a 2D Translation and uniform scale. */
template<typename T>
inline FSlateLayoutTransform Concatenate(const UE::Math::TVector2<T>& Translation, float Scale)
{
	return FSlateLayoutTransform(Scale, TransformPoint(Scale, Translation));
}

template<typename T>
inline FSlateLayoutTransform Concatenate(const UE::Math::TVector2<T>& Translation, double Scale)
{
	return FSlateLayoutTransform((float)Scale, TransformPoint(Scale, Translation));
}

/** concatenation rules for LayoutTransform. */
template<> struct ConcatenateRules<FSlateLayoutTransform, float					> { typedef FSlateLayoutTransform ResultType; };
template<> struct ConcatenateRules<FSlateLayoutTransform, double				> { typedef FSlateLayoutTransform ResultType; };
template<> struct ConcatenateRules<float				, FSlateLayoutTransform	> { typedef FSlateLayoutTransform ResultType; };
template<> struct ConcatenateRules<double				, FSlateLayoutTransform	> { typedef FSlateLayoutTransform ResultType; };
template<typename T> struct ConcatenateRules<FSlateLayoutTransform, UE::Math::TVector2<T>	> { typedef FSlateLayoutTransform ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TVector2<T>, FSlateLayoutTransform	> { typedef FSlateLayoutTransform ResultType; };
template<typename T> struct ConcatenateRules<FSlateLayoutTransform	, UE::Math::TMatrix<T>	> { typedef UE::Math::TMatrix<T> ResultType; };
template<typename T> struct ConcatenateRules<UE::Math::TMatrix<T>	, FSlateLayoutTransform	> { typedef UE::Math::TMatrix<T> ResultType; };

/** concatenation rules for layout transforms and 2x2 generalized transforms. Need to be upcast to FTransform2D. */
template<typename T> struct ConcatenateRules<TScale2<T>				, FSlateLayoutTransform	> { typedef TTransform2<T> ResultType; };
template<typename T> struct ConcatenateRules<TShear2<T>				, FSlateLayoutTransform	> { typedef TTransform2<T> ResultType; };
template<typename T> struct ConcatenateRules<TQuat2<T>				, FSlateLayoutTransform	> { typedef TTransform2<T> ResultType; };
template<typename T> struct ConcatenateRules<TMatrix2x2<T>			, FSlateLayoutTransform	> { typedef TTransform2<T> ResultType; };
template<typename T> struct ConcatenateRules<FSlateLayoutTransform	, TScale2<T>			> { typedef TTransform2<T> ResultType; };
template<typename T> struct ConcatenateRules<FSlateLayoutTransform	, TShear2<T>			> { typedef TTransform2<T> ResultType; };
template<typename T> struct ConcatenateRules<FSlateLayoutTransform	, TQuat2<T>				> { typedef TTransform2<T> ResultType; };
template<typename T> struct ConcatenateRules<FSlateLayoutTransform	, TMatrix2x2<T>			> { typedef TTransform2<T> ResultType; };

//////////////////////////////////////////////////////////////////////////
// FSlateLayoutTransform adapters.
// 
// Adapt FTransform2D to accept FSlateLayoutTransforms as well.
//////////////////////////////////////////////////////////////////////////
template<> template<> inline TTransform2<float> TransformConverter<TTransform2<float>>::Convert<FSlateLayoutTransform>(const FSlateLayoutTransform& Transform)
{
	return TTransform2<float>(TScale2<float>((float)Transform.GetScale()), Transform.GetTranslation());
}

template<> template<> inline TTransform2<double> TransformConverter<TTransform2<double>>::Convert<FSlateLayoutTransform>(const FSlateLayoutTransform& Transform)
{
	return TTransform2<double>(TScale2<double>(Transform.GetScale()), UE::Math::TVector2<double>(UE::Slate::CastToVector2f(Transform.GetTranslation())));
}

