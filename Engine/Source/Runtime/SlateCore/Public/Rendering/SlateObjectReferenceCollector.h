// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UObjectGlobals.h"

/**
 * Used to gather all the referenced UObjects used by Slate when rendering a frame.
 * We're forced to do this because several draw calls only use resources for a frame
 * and then allow them to be garbage collected, so Slate needs to keep those objects
 * alive for the duration that they are used.
 */
class FSlateObjectReferenceCollector : public FReferenceCollector
{
public:
	FSlateObjectReferenceCollector(TArray<UObject*>& InReferencedSet)
		: ReferencedObjects(InReferencedSet)
	{
	}

	virtual bool IsIgnoringArchetypeRef() const override
	{
		return true;
	}

	virtual bool IsIgnoringTransient() const override
	{
		return false;
	}

	virtual void HandleObjectReference(UObject*& InObject, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
	{
		if (InObject)
		{
			ReferencedObjects.Add(InObject);
		}
	}

	TArray<UObject*>& ReferencedObjects;
};