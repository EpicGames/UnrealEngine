// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderingCommon.h"
#include "Layout/Clipping.h"
#include "Templates/UnrealTemplate.h"
#include <type_traits>

class FSlateElementBatch;
class FSlateDrawLayerHandle;
class ICustomSlateElement;
class FSlateRenderDataHandle;

struct FSlateDataPayload;


struct FSlateRenderBatchParams
{
	int32 Layer;
	FShaderParams ShaderParams;
	const FSlateShaderResource* Resource = nullptr;
	ESlateDrawPrimitive PrimitiveType;
	ESlateShader ShaderType;
	ESlateDrawEffect DrawEffects;
	ESlateBatchDrawFlag DrawFlags;
	int8 SceneIndex;
	const FSlateClippingState* ClippingState = nullptr;

	bool IsBatchableWith(const FSlateRenderBatchParams& Other) const
	{
		return Layer == Other.Layer 
			&& ShaderParams == Other.ShaderParams 
			&& Resource == Other.Resource 
			&& PrimitiveType == Other.PrimitiveType 
			&& ShaderType == Other.ShaderType 
			&& DrawEffects == Other.DrawEffects 
			&& DrawFlags == Other.DrawFlags 
			&& SceneIndex == Other.SceneIndex
			&& ClippingState == Other.ClippingState;
	}
};

class FSlateRenderBatch
{
public:
	FSlateRenderBatch(
		int32 InLayer,
		const FShaderParams& InShaderParams,
		const FSlateShaderResource* InResource,
		ESlateDrawPrimitive InPrimitiveType,
		ESlateShader InShaderType,
		ESlateDrawEffect InDrawEffects,
		ESlateBatchDrawFlag InDrawFlags,
		int8 InSceneIndex,
		FSlateVertexArray* InSourceVertArray,
		FSlateIndexArray* InSourceIndexArray,
		int32 InVertexOffset,
		int32 InIndexOffset);

	void ReserveVertices(uint32 Num)
	{
		SourceVertices->Reserve(SourceVertices->Num() + Num);
	}

	void ReserveIndices(uint32 Num)
	{
		SourceIndices->Reserve(SourceIndices->Num() + Num);
	}

	void AddVertex(FSlateVertex&& Vertex)
	{
		const int32 Index = SourceVertices->AddUninitialized(1);
		new(SourceVertices->GetData()+Index) FSlateVertex(Forward<FSlateVertex>(Vertex));
		++NumVertices;
	}

	void AddIndex(SlateIndex Index)
	{
		SourceIndices->Add(Index);
		++NumIndices;
	}

	void EmplaceVertex(FSlateVertex&& Vertex)
	{
		SourceVertices->Emplace(Vertex);
		++NumVertices;
	}

	void EmplaceIndex(SlateIndex Index)
	{
		SourceIndices->Emplace(Index);
		++NumIndices;
	}

	void AddVertices(const TArray<FSlateVertex>& InVertices)
	{
		SourceVertices->Append(InVertices);
		NumVertices += InVertices.Num();
	}

	void AddIndices(const TArray<SlateIndex>& InIndices)
	{
		SourceIndices->Append(InIndices);
		NumIndices += InIndices.Num();
	}

	void AddVertices(TArray<FSlateVertex>&& InVertices)
	{
		SourceVertices->Append(InVertices);
		NumVertices += InVertices.Num();
	}

	void AddIndices(TArray<SlateIndex>&& InIndices)
	{
		SourceIndices->Append(InIndices);
		NumIndices += InIndices.Num();
	}

	uint32 GetNumVertices() const
	{
		return NumVertices;
	}
	
	uint32 GetNumIndices() const
	{
		return NumIndices;
	}

	uint32 GetVertexOffset() const
	{
		return VertexOffset;
	}

	uint32 GetIndexOffset() const
	{
		return IndexOffset;
	}

	bool HasVertexData() const
	{
		return NumVertices > 0 && NumIndices > 0;
	}

	bool IsBatchableWith(const FSlateRenderBatch& Other) const
	{
		return
			ShaderResource == Other.ShaderResource
			&& DrawFlags == Other.DrawFlags
			&& ShaderType == Other.ShaderType
			&& DrawPrimitiveType == Other.DrawPrimitiveType
			&& DrawEffects == Other.DrawEffects
			&& ShaderParams == Other.ShaderParams
			&& InstanceData == Other.InstanceData
			&& InstanceCount == Other.InstanceCount
			&& InstanceOffset == Other.InstanceOffset
			&& DynamicOffset == Other.DynamicOffset
			&& CustomDrawer == Other.CustomDrawer
			&& SceneIndex == Other.SceneIndex
			&& ClippingState == Other.ClippingState;
	}

	int32 GetLayer() const { return LayerId; }

	const FSlateClippingState* GetClippingState() const { return ClippingState; }

	const FSlateShaderResource* GetShaderResource() const { return ShaderResource; }

	ESlateDrawPrimitive GetDrawPrimitiveType() const { return DrawPrimitiveType; }

	ESlateBatchDrawFlag GetDrawFlags() const { return DrawFlags; }

	ESlateDrawEffect GetDrawEffects() const { return DrawEffects; }

	ESlateShader GetShaderType() const { return ShaderType; }

	const FShaderParams& GetShaderParams() const { return ShaderParams; }

	bool IsValidForRendering() const
	{
		return (NumVertices > 0 && NumIndices > 0) || CustomDrawer != nullptr || ShaderType == ESlateShader::PostProcess;
	}
public:
	FShaderParams ShaderParams;

	/** Dynamically modified offset that occurs when we have relative position stored render batches. */
	FVector2f DynamicOffset;

	/** The Stored clipping state for the corresponding clipping state index.  The indices are not directly comparable later, so we need to expand it to the full state to be compared. */
	const FSlateClippingState* ClippingState;

	/** Shader Resource to use with this batch.  */
	const FSlateShaderResource* ShaderResource;

	ISlateUpdatableInstanceBufferRenderProxy* InstanceData;

	// Source Data
	FSlateVertexArray* SourceVertices;

	FSlateIndexArray* SourceIndices;

	ICustomSlateElement* CustomDrawer;

	/** The layer we need to sort by  */
	int32 LayerId;

	int32 VertexOffset;

	int32 IndexOffset;

	/** Number of vertices in the batch */
	int32 NumVertices;

	/** Number of indices in the batch */
	int32 NumIndices;

	int32 NextBatchIndex;

	int32 InstanceCount;

	int32 InstanceOffset;

	ESlateBatchDrawFlag DrawFlags;

	ESlateShader ShaderType;

	ESlateDrawPrimitive DrawPrimitiveType;

	ESlateDrawEffect DrawEffects;

	ESlatePostRT CachedUsedPostBuffers;

	int8 SceneIndex;

	/** Whether or not the batch can be merged with others in the same layer */
	uint8 bIsMergable : 1;

	uint8 bIsMerged : 1;
};

static_assert(TIsTriviallyCopyConstructible<FSlateRenderBatch>::Value == true, "FSlateRenderBatch must be mem copyable");
static_assert(std::is_trivially_destructible_v<FSlateRenderBatch>, "FSlateRenderBatch must be trivially destructible");
