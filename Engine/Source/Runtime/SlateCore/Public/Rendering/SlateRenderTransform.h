// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Math/TransformCalculus2D.h"
#include "Types/SlateVector2.h"

/** Typecast so it's more clear to the reader when we are dealing with a render transform vs. a layout transform. */
typedef FTransform2D FSlateRenderTransform;


#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS

inline UE::Slate::FDeprecateVector2DResult TransformVector(const FSlateRenderTransform& Transform, const UE::Slate::FDeprecateVector2DParameter& Vector)
{
	return UE::Slate::FDeprecateVector2DResult(TransformVector(Transform, FVector2f(Vector)));
}
inline UE::Slate::FDeprecateVector2DResult TransformVector(const FSlateRenderTransform& Transform, const UE::Slate::FDeprecateVector2DResult& Vector)
{
	return UE::Slate::FDeprecateVector2DResult(TransformVector(Transform, FVector2f(Vector)));
}

#endif