// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateShaderResource.h"
#include "Textures/SlateTextureData.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Rendering/DrawElements.h"
#include "Templates/RefCounting.h"
#include "Fonts/FontTypes.h"
#include "Types/SlateVector2.h"
#include "PixelFormat.h"

class FRenderTarget;
class FSlateDrawBuffer;
class FSlateUpdatableTexture;
class ISlate3DRenderer;
class ISlateAtlasProvider;
class ISlateStyle;
class SWindow;
struct Rect;
class FSceneInterface;
class FSlateRenderer;
struct FSlateBrush;

namespace UE::Slate
{
	/**
	 * Captures the previously registered scene index on construction and restores it on destruction so that nested registrations compose correctly
	 * even across early returns.
	 */
	class FSceneRegistrationScope final
	{
	public:
		FSceneRegistrationScope()
			: Renderer(nullptr)
			, CurrentSceneIndex(INDEX_NONE)
			, PreviousSceneIndex(INDEX_NONE)
		{
		}

		FSceneRegistrationScope(FSceneRegistrationScope&& Other)
			: Renderer(Other.Renderer)
			, CurrentSceneIndex(Other.CurrentSceneIndex)
			, PreviousSceneIndex(Other.PreviousSceneIndex)
		{
			Other.Renderer = nullptr;
		}

		FSceneRegistrationScope& operator=(FSceneRegistrationScope&& Other);

		FSceneRegistrationScope(const FSceneRegistrationScope&) = delete;
		FSceneRegistrationScope& operator=(const FSceneRegistrationScope&) = delete;

		~FSceneRegistrationScope();

		UE_DEPRECATED(5.8, "RegisterScene now returns an FSceneRegistrationScope instead of int32 index. If you need the index, call GetCurrentSceneIndex after registering.")
		operator int32() const
		{
			return CurrentSceneIndex;
		}

	private:
		friend class ::FSlateRenderer;

		FSceneRegistrationScope(FSlateRenderer& InRenderer, int32 InCurrentSceneIndex, int32 InPreviousSceneIndex)
			: Renderer(&InRenderer)
			, CurrentSceneIndex(InCurrentSceneIndex)
			, PreviousSceneIndex(InPreviousSceneIndex)
		{
		}

		FSlateRenderer* Renderer;
		int32 CurrentSceneIndex;
		int32 PreviousSceneIndex;
	};
}

/**
 * Update context for deferred drawing of widgets to render targets
 */
struct FRenderThreadUpdateContext
{
	class FSlateDrawBuffer* WindowDrawBuffer;
	double WorldTimeSeconds;
	float DeltaTimeSeconds;
	double RealTimeSeconds;
	float DeltaRealTimeSeconds;
	FRenderTarget* RenderTarget;
	ISlate3DRenderer* Renderer;
	bool bClearTarget;
};

/**
 * Provides access to the game and render thread font caches that Slate should use
 */
class FSlateFontServices
{
public:
	/**
	 * Construct the font services from the font caches (we'll create corresponding measure services ourselves)
	 * These pointers may be the same if your renderer doesn't need a separate render thread font cache
	 */
	SLATECORE_API FSlateFontServices(TSharedRef<class FSlateFontCache> InGameThreadFontCache, TSharedRef<class FSlateFontCache> InRenderThreadFontCache);

	/**
	 * Destruct the font services
	 */
	SLATECORE_API ~FSlateFontServices();

	/**
	 * Get the font cache to use for the current thread
	 */
	SLATECORE_API TSharedRef<class FSlateFontCache> GetFontCache() const;

	/**
	 * Get the font cache to use for the game thread
	 */
	TSharedRef<class FSlateFontCache> GetGameThreadFontCache() const
	{
		return GameThreadFontCache;
	}

	/**
	 * Get the font cache to use for the render thread
	 */
	TSharedRef<class FSlateFontCache> GetRenderThreadFontCache() const
	{
		return RenderThreadFontCache;
	}

	/**
	 * Get access to the font measure service for the current thread
	 */
	SLATECORE_API TSharedRef<class FSlateFontMeasure> GetFontMeasureService() const;

	/**
	 * Get access to the font measure service for the current thread
	 */
	TSharedRef<class FSlateFontMeasure> GetGameThreadFontMeasureService() const 
	{
		return GameThreadFontMeasure;
	}

	/**
	 * Get access to the font measure service for the current thread
	 */
	TSharedRef<class FSlateFontMeasure> GetRenderThreadFontMeasureService() const 
	{
		return RenderThreadFontMeasure;
	}

	/**
	 * Flushes all cached data from the font cache for the current thread
	 */
	SLATECORE_API void FlushFontCache(const FString& FlushReason);

	/**
	 * Flushes all cached data from the font cache for the game thread
	 */
	SLATECORE_API void FlushGameThreadFontCache(const FString& FlushReason);

	/**
	 * Flushes all cached data from the font cache for the render thread
	 */
	SLATECORE_API void FlushRenderThreadFontCache(const FString& FlushReason);

	/**
	 * Release any rendering resources owned by this font service
	 */
	SLATECORE_API void ReleaseResources();

	/**
	 * Delegate called after releasing the rendering resources used by this font service
	 */
	SLATECORE_API FOnReleaseFontResources& OnReleaseResources();

private:
	SLATECORE_API void HandleFontCacheReleaseResources(const class FSlateFontCache& InFontCache);

	TSharedRef<class FSlateFontCache> GameThreadFontCache;
	TSharedRef<class FSlateFontCache> RenderThreadFontCache;

	TSharedRef<class FSlateFontMeasure> GameThreadFontMeasure;
	TSharedRef<class FSlateFontMeasure> RenderThreadFontMeasure;

	FOnReleaseFontResources OnReleaseResourcesDelegate;
};


struct FMappedTextureBuffer
{
	void* Data;
	int32 Width;
	int32 Height;

	FMappedTextureBuffer()
		: Data(nullptr)
		, Width(0)
		, Height(0)
	{
	}

	bool IsValid() const
	{
		return Data && Width > 0 && Height > 0;
	}

	void Reset()
	{
		Data = nullptr;
		Width = Height = 0;
	}
};


/**
 * Abstract base class for Slate renderers.
 */
class FSlateRenderer
{
public:

	/** Constructor. */
	SLATECORE_API explicit FSlateRenderer(const TSharedRef<FSlateFontServices>& InSlateFontServices);

	/** Virtual destructor. */
	SLATECORE_API virtual ~FSlateRenderer();

public:
	/** Acquire the draw buffer and release it at the end of the scope. */
	struct FScopedAcquireDrawBuffer
	{
		FScopedAcquireDrawBuffer(FSlateRenderer& InSlateRenderer)
			: SlateRenderer(InSlateRenderer)
			, DrawBuffer(InSlateRenderer.AcquireDrawBuffer())
		{
		}
		~FScopedAcquireDrawBuffer()
		{
			SlateRenderer.ReleaseDrawBuffer(DrawBuffer);
		}
		FScopedAcquireDrawBuffer(const FScopedAcquireDrawBuffer&) = delete;
		FScopedAcquireDrawBuffer& operator=(const FScopedAcquireDrawBuffer&) = delete;

		FSlateDrawBuffer& GetDrawBuffer()
		{
			return DrawBuffer;
		}

	private:
		FSlateRenderer& SlateRenderer;
		FSlateDrawBuffer& DrawBuffer;
	};

public:
	/** Returns a draw buffer that can be used by Slate windows to draw window elements */
	UE_DEPRECATED(5.1, "Use FSlateRenderer::AcquireDrawBuffer instead and release the draw buffer.")
	virtual FSlateDrawBuffer& GetDrawBuffer()
	{
		return AcquireDrawBuffer();
	}

	/** Returns a draw buffer that can be used by Slate windows to draw window elements */
	virtual FSlateDrawBuffer& AcquireDrawBuffer() = 0;

	/** Return the previously acquired buffer. */
	virtual void ReleaseDrawBuffer( FSlateDrawBuffer& InWindowDrawBuffer ) = 0;

	virtual bool Initialize() = 0;

	virtual void Destroy() = 0;

	/**
	 * Creates a rendering viewport
	 *
	 * @param InWindow	The window to create the viewport for
	 */ 
	virtual void CreateViewport( const TSharedRef<SWindow> InWindow ) = 0;

	/**
	 * Requests that a rendering viewport be resized
	 *
	 * @param Window		The window to resize
	 * @param Width			The new width of the window
	 * @param Height		The new height of the window
	 */
	virtual void RequestResize( const TSharedPtr<SWindow>& InWindow, uint32 NewSizeX, uint32 NewSizeY ) = 0;

	/**
	 * Sets fullscreen state on the window's rendering viewport
	 *
	 * @param	InWindow		The window to update fullscreen state on
	 * @param	OverrideResX	0 if no override
	 * @param	OverrideResY	0 if no override
	 */
	virtual void UpdateFullscreenState( const TSharedRef<SWindow> InWindow, uint32 OverrideResX = 0, uint32 OverrideResY = 0 ) = 0;


	/**
	 * Set the resolution cached by the engine
	 *
	 * @param	Width			Width of the system resolution
	 * @param	Height			Height of the system resolution
	 */
	virtual void SetSystemResolution(uint32 Width, uint32 Height) = 0;
	
	
	/**
	 * Restore the given window to the resolution settings currently cached by the engine
	 * 
	 * @param InWindow    -> The window to restore to the cached settings
	 */
	virtual void RestoreSystemResolution(const TSharedRef<SWindow> InWindow) = 0;

	/** 
	 * Creates necessary resources to render a window and sends draw commands to the rendering thread
	 *
	 * @param WindowDrawBuffer	The buffer containing elements to draw 
	 */
	virtual void DrawWindows( FSlateDrawBuffer& InWindowDrawBuffer ) = 0;
	
	/** Callback that fires after Slate has rendered each window, each frame */
	DECLARE_MULTICAST_DELEGATE_OneParam( FOnSlateWindowRendered, SWindow& );
	FOnSlateWindowRendered& OnSlateWindowRendered() { return SlateWindowRendered; }

	/** Callback on the render thread after slate rendering finishes and right before present is called. */
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnBackBufferReadyToPresent, SWindow&, class ISlateViewportProvider& );
	FOnBackBufferReadyToPresent& OnBackBufferReadyToPresent() { return OnBackBufferReadyToPresentDelegate; }

	/** 
	 * Sets which color vision filter to use
	 */
	virtual void SetColorVisionDeficiencyType(EColorVisionDeficiency Type, int32 Severity, bool bCorrectDeficiency, bool bShowCorrectionWithDeficiency) { }

	/** 
	 * Creates a dynamic image resource and returns its size
	 *
	 * @param InTextureName The name of the texture resource
	 * @return The size of the loaded texture
	 */
	virtual FIntPoint GenerateDynamicImageResource(const FName InTextureName) {check(0); return FIntPoint( 0, 0 );}

	/**
	 * Creates a dynamic image resource
	 *
	 * @param ResourceName		The name of the texture resource
	 * @param Width				The width of the resource
	 * @param Height			The height of the image
	 * @param Bytes				The payload for the resource
	 * @return					true if the resource was successfully generated, otherwise false
	 */
	virtual bool GenerateDynamicImageResource( FName ResourceName, uint32 Width, uint32 Height, const TArray< uint8 >& Bytes ) { return false; }

	virtual bool GenerateDynamicImageResource(FName ResourceName, FSlateTextureDataRef TextureData) { return false; }

	virtual FSlateResourceHandle GetResourceHandle(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale) = 0;

	/**
	 * Creates a handle to a Slate resource
	 * A handle is used as fast path for looking up a rendering resource for a given brush when adding Slate draw elements
	 * This can be cached and stored safely in code.  It will become invalid when a resource is destroyed
	 * It is expensive to create a resource so do not do it in time sensitive areas
	 *
	 * @param	Brush		The brush to get a rendering resource handle 
	 * @return	The created resource handle.  
	 */
	virtual FSlateResourceHandle GetResourceHandle(const FSlateBrush& Brush)
	{
		return GetResourceHandle(Brush, FVector2f::ZeroVector, 1.0f);
	}


	/** The default implementation assumes all things are renderable. */
	virtual bool CanRenderResource(UObject& InResourceObject) const { return true; }

	/**
	 * Queues a dynamic brush for removal when it is safe.  The brush is not immediately released but you should consider the brush destroyed and no longer usable
	 *
	 * @param BrushToRemove	The brush to queue for removal which is no longer valid to use
	 */
	virtual void RemoveDynamicBrushResource( TSharedPtr<FSlateDynamicImageBrush> BrushToRemove ) = 0;

	/** 
	 * Releases a specific resource.  
	 * It is unlikely that you want to call this directly.  Use RemoveDynamicBrushResource instead
	 */
	virtual void ReleaseDynamicResource( const FSlateBrush& Brush ) = 0;


	/** Called when a window is destroyed to give the renderer a chance to free resources */
	virtual void OnWindowDestroyed( const TSharedRef<SWindow>& InWindow ) = 0;

	/**
	 * When implemented, returns the viewport rendering resource for the provided window.
	 *
	 * @param Window	The window to get the viewport from 
	 */
	virtual void* GetViewportResource( const SWindow& Window ) { return nullptr; }
	
	/**
	 * When implemented, returns an interface that can be used to retrieve the back buffer texture for the given SWindow.
	 * @param Window	The window to get the viewport back buffer from.
	 */
	virtual class ISlateViewportProvider* GetViewportProvider( const SWindow& Window ) { return nullptr; }

	/**
	 * Get access to the font services used by this renderer
	 */
	TSharedRef<FSlateFontServices> GetFontServices() const 
	{
		return SlateFontServices.ToSharedRef();
	}

	/**
	 * Get access to the font measure service (game thread only!)
	 */
	TSharedRef<class FSlateFontMeasure> GetFontMeasureService() const 
	{
		return SlateFontServices->GetFontMeasureService();
	}

	/**
	 * Get the font cache to use for the current thread
	 */
	TSharedRef<class FSlateFontCache> GetFontCache() const
	{
		return SlateFontServices->GetFontCache();
	}

	/**
	 * Flushes all cached data from the font cache for the current thread
	 */
	void FlushFontCache(const FString& FlushReason)
	{
		SlateFontServices->FlushFontCache(FlushReason);
	}

	/**
	 * Gives the renderer a chance to wait for any render commands to be completed before returning/
	 */
	virtual void FlushCommands() const {};

	/**
	 * Gives the renderer a chance to synchronize with another thread in the event that the renderer runs 
	 * in a multi-threaded environment.  This function does not return until the sync is complete
	 */
	virtual void Sync() const {};
	
	/**
	 * Indicates the end of the current frame to the Renderer. This is usually handled by the engine loop
	 * but certain situations (ie, when the main loop is paused) may require manual calls.
	 */
	virtual void EndFrame() const {};

	/**
	 * Reloads all texture resources from disk
	 */
	virtual void ReloadTextureResources() {}

	/**
	 * Loads all the resources used by the specified SlateStyle
	 */
	virtual void LoadStyleResources( const ISlateStyle& Style ) {}

	/**
	 * Returns whether or not a viewport should be in  fullscreen
	 *
	 * @Window	The window to check for fullscreen
	 * @return true if the window's viewport should be fullscreen
	 */
	SLATECORE_API bool IsViewportFullscreen( const SWindow& Window ) const;

	/** Returns whether shaders that Slate depends on have been compiled. */
	virtual bool AreShadersInitialized() const { return true; }

	/**
	 * A renderer may need to keep a cache of accessed garbage collectible objects alive for the duration of their
	 * usage.  During some operations like ending a game.  It becomes important to immediately release game related
	 * resources.  This should flush any buffer holding onto those referenced objects.
	 */
	virtual void ReleaseAccessedResources(bool bImmediatelyFlush) {}

	/** 
	 * Prepares the renderer to take a screenshot of the UI.  The Rect is portion of the rendered output
	 * that will be stored into the TArray of FColors.
	 */
	virtual void PrepareToTakeScreenshot(const FIntRect& Rect, TArray<FColor>* OutColorData, SWindow* InScreenshotWindow) {}

	/** 
	 * Prepares the renderer to take a screenshot of the UI.  The Rect is portion of the rendered output
	 * that will be stored into the TArray of FColors.
	 */
	virtual void PrepareToTakeHDRScreenshot(const FIntRect& Rect, TArray<FLinearColor>* OutColorData, SWindow* InScreenshotWindow) {}

	/**
	 * Create an updatable texture that can receive new data dynamically
	 *
	 * @param	Width	Initial width of the texture
	 * @param	Height	Initial height of the texture
	 *
	 * @return	Newly created updatable texture
	 */
	virtual FSlateUpdatableTexture* CreateUpdatableTexture(uint32 Width, uint32 Height) = 0;

	/**
	 * Create an updatable texture that can receive new data via a shared handle
	 *
	 * @param	SharedHandle	The OS dependant handle that backs the texture data
	 *
	 * @return	Newly created updatable texture
	 */
	virtual FSlateUpdatableTexture* CreateSharedHandleTexture(void *SharedHandle) = 0;
	
	virtual void CreateNativeLayer(int32 NewNativeLayer, SWindow& InWindow, void* NativeViewHandle) {};
	
	virtual void DeleteNativeLayer(int32 OldNativeLayer, SWindow& InWindow) {};

	/**
	 * Return an updatable texture to the renderer for release
	 *
	 * @param	Texture	The texture we are releasing (should not use this pointer after calling)
	 */
	virtual void ReleaseUpdatableTexture(FSlateUpdatableTexture* Texture) = 0;

	/**
	 * Returns the way to access the texture atlas information for this renderer
	 */
	SLATECORE_API virtual ISlateAtlasProvider* GetTextureAtlasProvider();

	/**
	 * Returns the way to access the font atlas information for this renderer
	 */
	SLATECORE_API virtual ISlateAtlasProvider* GetFontAtlasProvider();

	/**
	 * Copies all slate windows out to a buffer at half resolution with debug information
	 * like the mouse cursor and any keypresses.
	 */
	virtual void CopyWindowsToVirtualScreenBuffer(const TArray<FString>& KeypressBuffer) {}
	
	/** Allows and disallows access to the crash tracker buffer data on the CPU */
	virtual void MapVirtualScreenBuffer(FMappedTextureBuffer* OutImageData) {}
	virtual void UnmapVirtualScreenBuffer() {}

	/**
	 * Necessary to grab before flushing the resource pool, as it may be being 
	 * accessed by multiple threads when loading.
	 */
	virtual FCriticalSection* GetResourceCriticalSection() = 0;

	/**
	 * Register the active scene pointer with the renderer.  The returned scope captures the previously registered scene index and restores it when
	 * destroyed.
	 */
	[[nodiscard]] UE::Slate::FSceneRegistrationScope RegisterCurrentScene(FSceneInterface* Scene);

	/** Get the currently registered scene index (set by RegisterCurrentScene)*/
	virtual int32 GetCurrentSceneIndex() const  = 0;

	/** Set currently registered scene index */
	virtual void SetCurrentSceneIndex(int32 InIndex) = 0;

	/** Reset the internal Scene tracking.*/
	virtual void ClearScenes() = 0;

protected:
	virtual void RegisterCurrentScene_Impl(FSceneInterface* Scene) = 0;

public:

	SLATECORE_API virtual void DestroyCachedFastPathRenderingData(struct FSlateCachedFastPathRenderingData* VertexData);
	SLATECORE_API virtual void DestroyCachedFastPathElementData(struct FSlateCachedElementData* ElementData);

	virtual bool HasLostDevice() const { return false; }

	/**
	 * Lets the renderer know that we need to render some widgets to a render target.
	 * 
	 * @param Context						The context that describes what we're rendering to
	 * @param bDeferredRenderTargetUpdate	Whether or not the update is deferred until the end of the frame when it is potentially less expensive to update the render target. 
											See GDeferRetainedRenderingRenderThread for more info.
											Care must be taken to destroy anything referenced in the context when it is safe to do so.
	 */
	virtual void AddWidgetRendererUpdate(const struct FRenderThreadUpdateContext& Context, bool bDeferredRenderTargetUpdate) {}

	virtual EPixelFormat GetSlateRecommendedColorFormat() { return PF_B8G8R8A8; }

	virtual void OnVirtualDesktopSizeChanged(const FDisplayMetrics& NewDisplayMetric) {}
private:

	// Non-copyable
	SLATECORE_API FSlateRenderer(const FSlateRenderer&);
	SLATECORE_API FSlateRenderer& operator=(const FSlateRenderer&);

	SLATECORE_API void HandleFontCacheReleaseResources(const class FSlateFontCache& InFontCache);

protected:

	/** The font services used by this renderer when drawing text */
	TSharedPtr<FSlateFontServices> SlateFontServices;

	/** Callback that fires after Slate has rendered each window, each frame */
	FOnSlateWindowRendered SlateWindowRendered;

	FOnBackBufferReadyToPresent OnBackBufferReadyToPresentDelegate;

	/**
	 * Necessary to grab before flushing the resource pool, as it may be being 
	 * accessed by multiple threads when loading.
	 */
	FCriticalSection ResourceCriticalSection;

	friend class FSlateRenderDataHandle;
};

inline UE::Slate::FSceneRegistrationScope FSlateRenderer::RegisterCurrentScene(FSceneInterface* Scene)
{
	const int32 PreviousIndex = GetCurrentSceneIndex();
	RegisterCurrentScene_Impl(Scene);
	return UE::Slate::FSceneRegistrationScope(*this, GetCurrentSceneIndex(), PreviousIndex);
}

namespace UE::Slate
{
	inline FSceneRegistrationScope::~FSceneRegistrationScope()
	{
		if (Renderer)
		{
			Renderer->SetCurrentSceneIndex(PreviousSceneIndex);
		}
	}

	inline FSceneRegistrationScope& FSceneRegistrationScope::operator=(FSceneRegistrationScope&& Other)
	{
		if (this != &Other)
		{
			if (Renderer)
			{
				Renderer->SetCurrentSceneIndex(PreviousSceneIndex);
			}
			Renderer = Other.Renderer;
			CurrentSceneIndex = Other.CurrentSceneIndex;
			PreviousSceneIndex = Other.PreviousSceneIndex;
			Other.Renderer = nullptr;
		}
		return *this;
	}
}


/**
 * Is this thread valid for sending out rendering commands?
 * If the slate loading thread exists, then yes, it is always safe
 * Otherwise, we have to be on the game thread
 */
bool SLATECORE_API IsThreadSafeForSlateRendering();

/**
 * If it's the game thread, and there's no loading thread, then it owns slate rendering.
 * However if there's a loading thread, it is the exlusive owner of slate rendering.
 */
bool SLATECORE_API DoesThreadOwnSlateRendering();
