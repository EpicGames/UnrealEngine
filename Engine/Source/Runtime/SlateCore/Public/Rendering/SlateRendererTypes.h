// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumRange.h"

#include "SlateRendererTypes.generated.h"

/**
 * Bitfield used to mark if a slate post RT is used or not
 */
UENUM(BlueprintType)
enum class ESlatePostRT : uint8
{
	None = 0 << 0,
	ESlatePostRT_0 = 1 << 0,
	ESlatePostRT_1 = 1 << 1,
	ESlatePostRT_2 = 1 << 2,
	ESlatePostRT_3 = 1 << 3,
	ESlatePostRT_4 = 1 << 4,
	All = ESlatePostRT_0 | ESlatePostRT_1 | ESlatePostRT_2 | ESlatePostRT_3 | ESlatePostRT_4,
	Num = 5
};

ENUM_CLASS_FLAGS(ESlatePostRT);

ENUM_RANGE_BY_VALUES(ESlatePostRT, ESlatePostRT::ESlatePostRT_0, ESlatePostRT::ESlatePostRT_1, ESlatePostRT::ESlatePostRT_2, ESlatePostRT::ESlatePostRT_3, ESlatePostRT::ESlatePostRT_4);
