// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

class FSlateSharedHandleData;
class FSlateShaderResourceProxy;

/**
 * A SlateResourceHandle is used as fast path for looking up a rendering resource for a given brush when adding Slate draw elements
 * This can be cached and stored safely in code.  It will become invalid when a resource is destroyed
*/
class FSlateResourceHandle
{
	friend class FSlateShaderResourceManager;
	friend class FSlateNullShaderResourceManager;
public:
	SLATECORE_API FSlateResourceHandle();

	/**
	 * @return true if the handle still points to a valid rendering resource
	 */
	SLATECORE_API bool IsValid() const;

	/**
	 * @return the resource proxy used to render.
	 */
	SLATECORE_API const FSlateShaderResourceProxy* GetResourceProxy() const;

private:
	SLATECORE_API FSlateResourceHandle(const TSharedPtr<FSlateSharedHandleData>& InData);

	/** Internal data to pair the handle to the resource */
	TSharedPtr<FSlateSharedHandleData> Data;
};
