// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FSlateBrush;

class FSlateSVGRasterizer
{
public:
	static TArray<uint8> RasterizeSVGFromFile(const FString& Filename, FIntPoint PixelSize);
}; 