// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateBrush.h"
#include "Textures/TextureAtlas.h"
#include "Types/SlateVector2.h"

class ISlateTextureAtlasFactory;
class FSlateTextureAtlas;

//typedef TFunctionRef<TUniquePtr<FSlateTextureAtlas> (int32 AtlasSize, int32 AtlasStride, ESlateTextureAtlasPaddingStyle PaddingStyle)> FCreateAtlasFunction;

class FSlateVectorGraphicsCache : public FSlateFlushableAtlasCache
{
public:
	SLATECORE_API FSlateVectorGraphicsCache(TSharedPtr<ISlateTextureAtlasFactory> InAtlasFactory);

	SLATECORE_API FSlateShaderResourceProxy* GetShaderResource(const FSlateBrush& Brush, FVector2f LocalSize, float DrawScale);

	SLATECORE_API void UpdateCache();
	SLATECORE_API void ConditionalFlushCache();

	SLATECORE_API void ReleaseResources(bool bWaitForRelease = false);
	SLATECORE_API void DeleteResources();

	virtual void RequestFlushCache(const FString& Reason) override;


	int32 GetNumAtlasPages() const { return Atlases.Num(); }
	FSlateShaderResource* GetAtlasPageResource(const int32 InIndex) const { return Atlases[InIndex]->GetAtlasTexture(); }
	bool IsAtlasPageResourceAlphaOnly(const int32 InIndex) const { return false; }
	const FSlateTextureAtlas* GetAtlas(const int32 InIndex) const { return Atlases[InIndex].Get(); }

#if WITH_ATLAS_DEBUGGING
	FName GetAtlasDebugData(const FAtlasedTextureSlot* InSlot) const { return AtlasDebugData.FindRef(InSlot);  }
#endif

private:
	void FlushCache();

private:
	struct FVectorCacheKey
	{
		FName BrushName;
		FIntPoint PixelSize;

		FVectorCacheKey(FName InBrushName, FVector2f LocalSize, float DrawScale)
			: BrushName(InBrushName)
			, PixelSize((LocalSize*DrawScale).IntPoint())
		{
			KeyHash = HashCombine(GetTypeHash(BrushName), GetTypeHash(PixelSize));
		}

		bool operator==(const FVectorCacheKey& Other) const
		{
			return BrushName == Other.BrushName
				&& PixelSize == Other.PixelSize;
		}

		friend inline uint32 GetTypeHash(const FVectorCacheKey& Key)
		{
			return Key.KeyHash;
		}

	private:
		uint32 KeyHash;
	};

	struct FRasterRequest
	{
		FVectorCacheKey Key;
		TArray<uint8> PixelData;

		FRasterRequest(FName InBrushName, FVector2f InLocalSize, float InDrawScale)
			: Key(InBrushName, InLocalSize, InDrawScale)
		{}
	};

#if WITH_ATLAS_DEBUGGING
	TMap<const FAtlasedTextureSlot*, FName> AtlasDebugData;
#endif
	TMap<FVectorCacheKey, TUniquePtr<FSlateShaderResourceProxy>> ResourceMap;
	TArray<TUniquePtr<FSlateTextureAtlas>> Atlases;
	TArray<TUniquePtr<FSlateShaderResource>> NonAtlasedTextures;
	TSharedPtr<ISlateTextureAtlasFactory> AtlasFactory;

	TArray<FRasterRequest> PendingRequests;

	bool bFlushRequested;
};