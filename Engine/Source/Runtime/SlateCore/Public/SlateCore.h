// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()

/* Dependencies
 *****************************************************************************/

#include "CoreUObject.h"
#include "InputCore.h"
#include "Modules/ModuleManager.h"

/* Globals
 *****************************************************************************/

#include "SlateGlobals.h"

/* Delegates
 *****************************************************************************/

#include "Types/WidgetActiveTimerDelegate.h"

/* Includes
 *****************************************************************************/

// Types
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Types/PaintArgs.h"
#include "Types/ISlateMetaData.h"

// Layout
#include "SlotBase.h"
#include "Layout/Margin.h"
#include "Layout/SlateRect.h"
#include "Rendering/SlateRenderTransform.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Layout/PaintGeometry.h"
#include "Layout/Geometry.h"
#include "Layout/Visibility.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/ArrangedChildren.h"
#include "Widgets/SNullWidget.h"
#include "Layout/Children.h"
#include "Layout/LayoutUtils.h"

// Animation
#include "Animation/CurveHandle.h"
#include "Animation/CurveSequence.h"
#include "Animation/SlateSprings.h"

// Sound
#include "Sound/SlateSound.h"
#include "Sound/ISlateSoundDevice.h"
#include "Sound/NullSlateSoundDevice.h"

// Styling
#include "Styling/WidgetStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateWidgetStyleContainerInterface.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "Styling/SlateWidgetStyleAsset.h"

// Textures
#include "Textures/SlateShaderResource.h"
#include "Textures/SlateTextureData.h"
#include "Textures/SlateUpdatableTexture.h"
#include "Textures/TextureAtlas.h"
#include "Rendering/ShaderResourceManager.h"

// Fonts
#include "Fonts/FontBulkData.h"
#include "Fonts/CompositeFont.h"
#include "Fonts/SlateFontInfo.h"
#include "Fonts/FontTypes.h"
#include "Fonts/ShapedTextFwd.h"
#include "Fonts/FontCache.h"
#include "Fonts/FontMeasure.h"

// Brushes
#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateNoResource.h"

// Styling (continued)
#include "Styling/StyleDefaults.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyle.h"

// Input
#include "Input/ReplyBase.h"
#include "Input/CursorReply.h"
#include "Input/Events.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Input/NavigationReply.h"
#include "Types/NavigationMetaData.h"

// Rendering
#include "Input/PopupMethodReply.h"
#include "Rendering/RenderingCommon.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingPolicy.h"
#include "Rendering/SlateDrawBuffer.h"
#include "Rendering/SlateRenderer.h"

// Application
#include "Widgets/IToolTip.h"
#include "Application/SlateWindowHelper.h"
#include "Application/SlateApplicationBase.h"
#include "Application/ThrottleManager.h"

// Widgets
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/LayoutGeometry.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SUserWidget.h"
#include "Widgets/SLeafWidget.h"
#include "Widgets/SPanel.h"
#include "Layout/WidgetPath.h"
////
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
////
#include "Widgets/SWindow.h"

