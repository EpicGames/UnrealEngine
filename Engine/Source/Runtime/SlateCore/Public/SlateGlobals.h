// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Debugging/SlateDebugging.h"
#include "Trace/SlateTrace.h"

// Enabled cvar GSlateCheckUObjectRenderResources that will check for invalid reference in the slate resources manager
#define SLATE_CHECK_UOBJECT_RENDER_RESOURCES !UE_BUILD_SHIPPING

// Enabled cvar GSlateCheckUObjectShapedGlyphSequence that will check for invalid reference before using them
#define SLATE_CHECK_UOBJECT_SHAPED_GLYPH_SEQUENCE !UE_BUILD_SHIPPING

#ifndef SLATE_CULL_WIDGETS
	#define SLATE_CULL_WIDGETS 1
#endif

/* Globals
 *****************************************************************************/

 // Compile all the RichText and MultiLine editable text?
#define WITH_FANCY_TEXT 1

 // If you want to get really verbose stats out of Slate to get a really in-depth
 // view of what widgets are causing you the greatest problems, set this define to 1.
#ifndef WITH_VERY_VERBOSE_SLATE_STATS
	#define WITH_VERY_VERBOSE_SLATE_STATS 0
#endif

#ifndef SLATE_VERBOSE_NAMED_EVENTS
	#define SLATE_VERBOSE_NAMED_EVENTS !UE_BUILD_SHIPPING
#endif

/** Generate an unique identifier  */
#ifndef UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
	#define UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER UE_SLATE_TRACE_ENABLED || WITH_SLATE_DEBUGGING
#endif

/** Enables Dynamic Invalidation for SInvalidationPanels. Currently Editor Only  */
#ifndef UE_SLATE_WITH_DYNAMIC_INVALIDATION
	#define UE_SLATE_WITH_DYNAMIC_INVALIDATION WITH_EDITOR
#endif

// HOW TO GET AN IN-DEPTH PERFORMANCE ANALYSIS OF SLATE
//
// Step 1)
//    Set WITH_VERY_VERBOSE_SLATE_STATS to 1.
//
// Step 2)
//    When running the game (outside of the editor), run these commandline options
//    in order and you'll get a large dump of where all the time is going in Slate.
//    
//    stat group enable slateverbose
//    stat group enable slateveryverbose
//    stat dumpave -root=stat_slate -num=120 -ms=0

SLATECORE_API DECLARE_LOG_CATEGORY_EXTERN(LogSlate, Log, All);
SLATECORE_API DECLARE_LOG_CATEGORY_EXTERN(LogSlateStyles, Log, All);

DECLARE_STATS_GROUP(TEXT("Slate Memory"), STATGROUP_SlateMemory, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("Slate"), STATGROUP_Slate, STATCAT_Advanced);
DECLARE_STATS_GROUP_VERBOSE(TEXT("SlateVerbose"), STATGROUP_SlateVerbose, STATCAT_Advanced);
DECLARE_STATS_GROUP_MAYBE_COMPILED_OUT(TEXT("SlateVeryVerbose"), STATGROUP_SlateVeryVerbose, STATCAT_Advanced, WITH_VERY_VERBOSE_SLATE_STATS);

/** Whether or not we've enabled fast widget pathing which validates paths to widgets without arranging children. */
extern SLATECORE_API bool GSlateFastWidgetPath;

/** Whether or not the SWindow can be an Invalidation Panel (use the fast path update). Normal Invalidation Panel will be deactivated. */
extern SLATECORE_API bool GSlateEnableGlobalInvalidation;

/** Whether or not we currently Painting/Updating the widget from the FastUpdate path (global invalidation). */
extern SLATECORE_API bool GSlateIsOnFastUpdatePath;

/** Whether or not we currently processing the widget invalidation from the InvalidationRoot (global invalidation). */
extern SLATECORE_API bool GSlateIsOnFastProcessInvalidation;

/** Whether or not we are currently running building the list of widget in slow path (global invalidation). */
extern SLATECORE_API bool GSlateIsInInvalidationSlowPath;

extern SLATECORE_API int32 GSlateLayoutGeneration;

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES 
extern SLATECORE_API bool GSlateCheckUObjectRenderResources;
// When we detect a none valid resource, should we log a fatal error (crash) or log it (ensure).
extern SLATECORE_API bool GSlateCheckUObjectRenderResourcesShouldLogFatal;
#endif

#if SLATE_CHECK_UOBJECT_SHAPED_GLYPH_SEQUENCE
extern SLATECORE_API bool GSlateCheckUObjectShapedGlyphSequence;
#endif

#if WITH_SLATE_DEBUGGING
extern SLATECORE_API bool GSlateHitTestGridDebugging;
#endif

#if WITH_SLATE_DEBUGGING || WITH_SLATE_NAVIGATION_DEBUGGING
extern SLATECORE_API bool GSlateNavigationDebugging;
#endif
/* Forward declarations
*****************************************************************************/
class FActiveTimerHandle;
enum class EActiveTimerReturnType : uint8;

extern SLATECORE_API bool GSlateUseSharedBreakIterator;

/** Used to guard access across slate to specific threads */
#define SLATE_CROSS_THREAD_CHECK() checkf(IsInGameThread() || IsInSlateThread(), TEXT("Slate can only be accessed from the GameThread or the SlateLoadingThread!"));