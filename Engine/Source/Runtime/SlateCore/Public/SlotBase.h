// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/InvalidateWidgetReason.h"

class FChildren;
class SWidget;

/** Slot are a container of a SWidget used by the FChildren. */
class FSlotBase
{
public:
	SLATECORE_API FSlotBase();
	SLATECORE_API FSlotBase(const FChildren& InParent);
	SLATECORE_API FSlotBase(const TSharedRef<SWidget>& InWidget);
	FSlotBase& operator=(const FSlotBase&) = delete;
	FSlotBase(const FSlotBase&) = delete;

	SLATECORE_API virtual ~FSlotBase();

public:
	struct FSlotArguments {};

public:
	/**
	 * Access the FChildren that own the slot.
	 * The owner can be invalid when the slot is not attached.
	 */
	const FChildren* GetOwner() const { return Owner; }

	/**
	 * Access the widget that own the slot.
	 * The owner can be invalid when the slot is not attached.
	 */
	SLATECORE_API SWidget* GetOwnerWidget() const;

	/**
	 * Set the owner of the slot.
	 * Slots cannot be reassigned to different parents.
	 */
	SLATECORE_API void SetOwner(const FChildren& Children);

	/** Attach the child widget the slot now owns. */
	inline void AttachWidget(TSharedRef<SWidget>&& InWidget)
	{
		DetatchParentFromContent();
		Widget = MoveTemp(InWidget);
		AfterContentOrOwnerAssigned();
	}
	inline void AttachWidget( const TSharedRef<SWidget>& InWidget )
	{
		DetatchParentFromContent();
		Widget = InWidget;
		AfterContentOrOwnerAssigned();
	}

	/**
	 * Access the widget in the current slot.
	 * There will always be a widget in the slot; sometimes it is
	 * the SNullWidget instance.
	 */
	inline const TSharedRef<SWidget>& GetWidget() const
	{
		return Widget;
	}

	/**
	 * Remove the widget from its current slot.
	 * The removed widget is returned so that operations could be performed on it.
	 * If the null widget was being stored, an invalid shared ptr is returned instead.
	 */
	SLATECORE_API const TSharedPtr<SWidget> DetachWidget();

	/** Invalidate the widget's owner. */
	SLATECORE_API void Invalidate(EInvalidateWidgetReason InvalidateReason);

protected:
	/**
	 * Performs the attribute assignment and invalidates the widget minimally based on what actually changed.  So if the boundness of the attribute didn't change
	 * volatility won't need to be recalculated.  Returns true if the value changed.
	 */
	template<typename TargetValueType, typename SourceValueType>
	bool SetAttribute(TAttribute<TargetValueType>& TargetValue, const TAttribute<SourceValueType>& SourceValue, EInvalidateWidgetReason BaseInvalidationReason)
	{
		if (!TargetValue.IdenticalTo(SourceValue))
		{
			const bool bWasBound = TargetValue.IsBound();
			const bool bBoundnessChanged = bWasBound != SourceValue.IsBound();
			TargetValue = SourceValue;

			EInvalidateWidgetReason InvalidateReason = BaseInvalidationReason;
			if (bBoundnessChanged)
			{
				InvalidateReason |= EInvalidateWidgetReason::Volatility;
			}

			Invalidate(InvalidateReason);
			return true;
		}

		return false;
	}

private:
	SLATECORE_API void DetatchParentFromContent();
	SLATECORE_API void AfterContentOrOwnerAssigned();

private:
	/** The children that own the slot. */
	const FChildren* Owner;
	/** The content widget of the slot. */
	TSharedRef<SWidget> Widget;
};


/** A slot that can be used by the declarative syntax. */
template <typename SlotType>
class TSlotBase : public FSlotBase
{
public:
	using FSlotBase::FSlotBase;

	SlotType& operator[](TSharedRef<SWidget>&& InChildWidget)
	{
		this->AttachWidget(MoveTemp(InChildWidget));
		return static_cast<SlotType&>(*this);
	}
	SlotType& operator[]( const TSharedRef<SWidget>& InChildWidget )
	{
		this->AttachWidget(InChildWidget);
		return static_cast<SlotType&>(*this);
	}

	SlotType& Expose(SlotType*& OutVarToInit)
	{
		OutVarToInit = static_cast<SlotType*>(this);
		return static_cast<SlotType&>(*this);
	}


	/** Argument to indicate the Slot is also its owner. */
	enum EConstructSlotIsFChildren { ConstructSlotIsFChildren };

	/** Struct to construct a slot. */
	struct FSlotArguments : public FSlotBase::FSlotArguments
	{
	public:
		FSlotArguments(EConstructSlotIsFChildren) {}
		FSlotArguments(TUniquePtr<SlotType> InSlot)
			: Slot(MoveTemp(InSlot))
		{
			check(Slot.Get());
		}
		FSlotArguments(const FSlotArguments&) = delete;
		FSlotArguments& operator=(const FSlotArguments&) = delete;
		FSlotArguments(FSlotArguments&&) = default;
		FSlotArguments& operator=(FSlotArguments&&) = default;


	public:
		/** Attach the child widget the slot will own. */
		typename SlotType::FSlotArguments& operator[](TSharedRef<SWidget>&& InChildWidget)
		{
			ChildWidget = MoveTemp(InChildWidget);
			return Me();
		}
		typename SlotType::FSlotArguments& operator[](const TSharedRef<SWidget>& InChildWidget)
		{
			ChildWidget = InChildWidget;
			return Me();
		}

		/** Initialize OutVarToInit with the slot that is being constructed. */
		typename SlotType::FSlotArguments& Expose(SlotType*& OutVarToInit)
		{
			OutVarToInit = Slot.Get();
			return Me();
		}

		/** Attach the child widget the slot will own. */
		void AttachWidget(const TSharedRef<SWidget>& InChildWidget)
		{
			ChildWidget = InChildWidget;
		}

		/** @return the child widget that will be owned by the slot. */
		const TSharedPtr<SWidget>& GetAttachedWidget() const { return ChildWidget; }

		/** @return the slot that is being constructed. */
		SlotType* GetSlot() const { return Slot.Get(); }

		/** Steal the slot that is being constructed from the FSlotArguments. */
		TUniquePtr<SlotType> StealSlot()
		{
			return MoveTemp(Slot);
		}

		/** Used by the named argument pattern as a safe way to 'return *this' for call-chaining purposes. */
		typename SlotType::FSlotArguments& Me()
		{
			return static_cast<typename SlotType::FSlotArguments&>(*this);
		}

	private:
		TUniquePtr<SlotType> Slot;
		TSharedPtr<SWidget> ChildWidget;
	};

	void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
	{
		if (InArgs.GetAttachedWidget())
		{
			AttachWidget(InArgs.GetAttachedWidget().ToSharedRef());
		}
		SetOwner(SlotOwner);
	}
};
