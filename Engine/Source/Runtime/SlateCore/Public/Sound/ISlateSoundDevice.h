// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

struct FSlateSound;

/** Interface that Slate uses to play sounds. */
class ISlateSoundDevice
{
public:
	/** Play the sound resource within the given Slate sound object */
	virtual void PlaySound(const FSlateSound& Sound, int32 UserIndex = 0) const = 0;

	/** @return The duration of the given sound resource */
	virtual float GetSoundDuration(const FSlateSound& Sound) const = 0;

	/** Virtual destructor: because it is an interface / pure virtual base class. */
	virtual ~ISlateSoundDevice()= default;
};
