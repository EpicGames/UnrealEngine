// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Sound/ISlateSoundDevice.h"

struct FSlateSound;

/** Silent implementation of ISlateSoundDevice; it plays nothing. */
class FNullSlateSoundDevice : public ISlateSoundDevice
{
public:
	SLATECORE_API virtual void PlaySound(const FSlateSound&, int32) const override;
	SLATECORE_API virtual float GetSoundDuration(const FSlateSound& Sound) const override;
	SLATECORE_API virtual ~FNullSlateSoundDevice();
};
