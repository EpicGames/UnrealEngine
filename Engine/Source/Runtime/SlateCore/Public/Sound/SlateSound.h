// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "UObject/WeakObjectPtr.h"
#include "SlateSound.generated.h"

/**
 * An intermediary to make UBaseSound available for Slate to play sounds
 */
USTRUCT(BlueprintType)
struct FSlateSound
{
	GENERATED_USTRUCT_BODY()

public:

	/** Default constructor. */
	SLATECORE_API FSlateSound( );

	/** Default copy constructor. */
	SLATECORE_API FSlateSound(const FSlateSound&);

	/** Default copy-assignment operator. */
	SLATECORE_API FSlateSound& operator=(const FSlateSound&);

public:

	/** Get the resource object associated with this sound
	 *
	 * @note Ensure that you only access the resource as a USoundBase 
	 * @return The resource object, or null if this sound resource is empty
	 */
	SLATECORE_API UObject* GetResourceObject( ) const;

	/**
	* Sets the sound that is supposed to be played.
	*/
	void SetResourceObject(class UObject* InResourceObject)
	{
		ResourceObject = InResourceObject;
	}

	/**
	 * Used when updating the ResourceObject within an FSlateSound from the editor to clear out any legacy data that may be set.
	 */
	SLATECORE_API void StripLegacyData_DEPRECATED( );

	/**
	 * Used to upgrade a FName property to a FSlateSound property
	 */
	SLATECORE_API bool SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);

public:

	/**
	 * Construct a FSlateSound from a FName
	 *
	 * @note This functionality is deprecated and should only be used for upgrading old data
	 * @param SoundName The name of the sound to load, eg) SoundCue'/Game/QA_Assets/Sound/TEST_Music_Ambient.TEST_Music_Ambient'
	 * @return An FSlateSound object with a ResourceObject pointer that's either set the given sound, or points to null if the sound could not be found
	 */
	static SLATECORE_API FSlateSound FromName_DEPRECATED( const FName& SoundName );

protected:

	/**
	 * Pointer to the USoundBase. Holding onto it as a UObject because USoundBase is not available in Slate core.
	 * Edited via FSlateSoundStructCustomization to ensure you can only set USoundBase assets on it.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sound, meta=(DisplayName="Sound", AllowedClasses="/Script/Engine.SoundBase"))
	TObjectPtr<UObject> ResourceObject;

	/** The legacy resource name; only used by sounds that have been set-up in code, or otherwise upgraded from old FName properties, set to NAME_None in non-legacy instances */
	FName LegacyResourceName_DEPRECATED;

	/** A weak pointer the the resource loaded from the legacy resource name; may be null if the resource needs (re)loading */
	mutable TWeakObjectPtr<UObject> LegacyResourceObject_DEPRECATED;
};


template<>
struct TStructOpsTypeTraits<FSlateSound>
	: public TStructOpsTypeTraitsBase2<FSlateSound>
{
	enum 
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};
