// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"

/**
 * AppStyle class
 *
 * AppStyle is a Singleton accessor to a named SlateStyle to be used as an Application wide
 * base style definition.
 * 
 * Moving forward, all references in any core Slate Application Widgets should use FAppStyle::Get()
 *
 * FEditorStyle::Get accessors and FCoreStyle::Get accessors should be replaced with FAppStyle::Get()
 *
 * Currently, this code defaults to use FCoreStyle::GetCoreStyle() if the named style is not 
 * found.
 *
 */

class FAppStyle
{

public:

	static SLATECORE_API const ISlateStyle& Get();
	
	static SLATECORE_API const FName GetAppStyleSetName();
	static SLATECORE_API void SetAppStyleSetName(const FName& InName);

	static SLATECORE_API void SetAppStyleSet(const ISlateStyle& InStyle);

	template< class T >
	static const T& GetWidgetStyle(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return FAppStyle::Get().GetWidgetStyle< T >(PropertyName, Specifier);
	}

	static float GetFloat(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return FAppStyle::Get().GetFloat(PropertyName, Specifier);
	}

	static UE::Slate::FDeprecateVector2DResult GetVector(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return FAppStyle::Get().GetVector(PropertyName, Specifier);
	}

	static const FLinearColor& GetColor(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return FAppStyle::Get().GetColor(PropertyName, Specifier);
	}

	static const FSlateColor GetSlateColor(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return FAppStyle::Get().GetSlateColor(PropertyName, Specifier);
	}

	static const FMargin& GetMargin(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return FAppStyle::Get().GetMargin(PropertyName, Specifier);
	}

	static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return FAppStyle::Get().GetBrush(PropertyName, Specifier);
	}

	static const TSharedPtr< FSlateDynamicImageBrush > GetDynamicImageBrush(FName BrushTemplate, FName TextureName, const ANSICHAR* Specifier = NULL)
	{
		return FAppStyle::Get().GetDynamicImageBrush(BrushTemplate, TextureName, Specifier);
	}

	static const TSharedPtr< FSlateDynamicImageBrush > GetDynamicImageBrush(FName BrushTemplate, const ANSICHAR* Specifier, class UTexture2D* TextureResource, FName TextureName)
	{
		return FAppStyle::Get().GetDynamicImageBrush(BrushTemplate, Specifier, TextureResource, TextureName);
	}

	static const TSharedPtr< FSlateDynamicImageBrush > GetDynamicImageBrush(FName BrushTemplate, class UTexture2D* TextureResource, FName TextureName)
	{
		return FAppStyle::Get().GetDynamicImageBrush(BrushTemplate, TextureResource, TextureName);
	}

	static const FSlateSound& GetSound(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return FAppStyle::Get().GetSound(PropertyName, Specifier);
	}

	static FSlateFontInfo GetFontStyle(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return FAppStyle::Get().GetFontStyle(PropertyName, Specifier);
	}

	static const FSlateBrush* GetDefaultBrush()
	{
		return FAppStyle::Get().GetDefaultBrush();
	}

	static const FSlateBrush* GetNoBrush()
	{
		return FStyleDefaults::GetNoBrush();
	}

	static const FSlateBrush* GetOptionalBrush(FName PropertyName, const ANSICHAR* Specifier = NULL, const FSlateBrush* const DefaultBrush = FStyleDefaults::GetNoBrush())
	{
		return FAppStyle::Get().GetOptionalBrush(PropertyName, Specifier, DefaultBrush);
	}

/**
 * Concatenates two FNames.e If A and B are "Path.To" and ".Something"
 * the result "Path.To.Something".
 *
 * @param A  First FName
 * @param B  Second name
 *
 * @return New FName that is A concatenated with B.
 */
	static FName Join(FName A, const ANSICHAR* B)
	{
		if (B == NULL)
		{
			return A;
		}
		else
		{
			return FName(*(A.ToString() + B));
		}
	}

private:

	static SLATECORE_API FName AppStyleName;

};
