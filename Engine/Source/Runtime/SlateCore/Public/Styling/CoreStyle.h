// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Styling/AppStyle.h"

struct FSlateDynamicImageBrush;

/**
 * Core slate style
 */
class FCoreStyle 
{
public:

	static SLATECORE_API TSharedRef<class ISlateStyle> Create( const FName& InStyleSetName = "CoreStyle" );

	/** 
	* @return the Application Style 
	*
	* NOTE: Until the Editor can be fully updated, calling FCoreStyle::Get() will
	* return the AppStyle instead of the style definied in this class.  
	*
	* Using the AppStyle is preferred in most cases as it allows the style to be changed 
	* and restyled more easily.
	*
	* In cases requiring explicit use of the CoreStyle where a Slate Widget should not take on
	* the appearance of the rest of the application, use FCoreStyle::GetCoreStyle().
	*
	*/
	static const ISlateStyle& Get( )
	{
		return FAppStyle::Get();
	}

	/** @return the singleton instance of the style created in . */
	static const ISlateStyle& GetCoreStyle()
	{
		return *(Instance.Get());
	}

	/** Get the default font for Slate */
	static SLATECORE_API TSharedRef<const FCompositeFont> GetDefaultFont();

	/** Get the icon font for Slate (works only in Editor, otherwise the DefaultFont is returned) */
	static SLATECORE_API TSharedRef<const FCompositeFont> GetIconFont();

	/** Get a font style using the default font for Slate */
	static SLATECORE_API FSlateFontInfo GetDefaultFontStyle(const FName InTypefaceFontName, const float InSize, const FFontOutlineSettings& InOutlineSettings = FFontOutlineSettings());

	/** Get a font style using the icon font for Slate (always using "Regular" typeface */
	static SLATECORE_API FSlateFontInfo GetRegularIconFontStyle(const float InSize, const FFontOutlineSettings& InOutlineSettings = FFontOutlineSettings());

	static SLATECORE_API void ResetToDefault( );

	/** Used to override the default selection colors */
	static SLATECORE_API void SetSelectorColor( const FLinearColor& NewColor );
	static SLATECORE_API void SetSelectionColor( const FLinearColor& NewColor );
	static SLATECORE_API void SetInactiveSelectionColor( const FLinearColor& NewColor );
	static SLATECORE_API void SetPressedSelectionColor( const FLinearColor& NewColor );
	static SLATECORE_API void SetFocusBrush(FSlateBrush* NewBrush);

	// todo: jdale - These are only here because of UTouchInterface::Activate and the fact that GetDynamicImageBrush is non-const
	static SLATECORE_API const TSharedPtr<FSlateDynamicImageBrush> GetDynamicImageBrush( FName BrushTemplate, FName TextureName, const ANSICHAR* Specifier = nullptr );
	static SLATECORE_API const TSharedPtr<FSlateDynamicImageBrush> GetDynamicImageBrush( FName BrushTemplate, const ANSICHAR* Specifier, class UTexture2D* TextureResource, FName TextureName );
	static SLATECORE_API const TSharedPtr<FSlateDynamicImageBrush> GetDynamicImageBrush( FName BrushTemplate, class UTexture2D* TextureResource, FName TextureName );

	static const int32 RegularTextSize = 9;
	static const int32 SmallTextSize = 8;

	UE_DEPRECATED(5.6, "IsStarshipStyle is deprecated, this function will always return true. Please remove any special case handling for the legacy slate style and calls to this function.")
	static constexpr bool IsStarshipStyle() { return true; }

	static bool IsInitialized() { return Instance.IsValid(); }

private:

	static SLATECORE_API void SetStyle( const TSharedRef< class ISlateStyle >& NewStyle );

private:

	/** Singleton instances of this style. */
	static SLATECORE_API TSharedPtr< class ISlateStyle > Instance;
};

namespace CoreStyleConstants
{
	// Note, these sizes are in Slate Units.
	// Slate Units do NOT have to map to pixels.
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon5x16;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon6x8;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon8x4;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon16x4;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon8x8;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon4x4;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon10x10;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon12x12;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon12x16;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon14x14;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon16x16;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon18x18;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon20x20;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon22x22;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon24x24;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon25x25;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon26x26;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon32x32;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon40x40;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon64x64;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon36x24;
	extern SLATECORE_API const UE::Slate::FDeprecateVector2DResult Icon128x128;

	// Common Margins
	extern SLATECORE_API const FMargin DefaultMargins;
	// Buttons already have a built in (4., 2.) padding - adding to that a little
	extern SLATECORE_API const FMargin ButtonMargins;

	extern SLATECORE_API const FMargin PressedButtonMargins;
	extern SLATECORE_API const FMargin ToggleButtonMargins;
	extern SLATECORE_API const FMargin ComboButtonMargin;
	extern SLATECORE_API const FMargin PressedComboButtonMargin;

	extern SLATECORE_API const float InputFocusRadius;
	extern SLATECORE_API const float InputFocusThickness;
}
