// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateTypes.h"

class FLazySingleton;

namespace UE::Slate::Private
{

/**
 * Single point of access for various default styles used in UMG for runtime and editor
 * with each style already having it's colors unlinked for reuse
 */
struct FDefaultStyleCache
{
	/**
	 * Internal style default holder struct, used so that one can get a bundle of defaults relevant for runtime or editor
	 */
	struct FStyles
	{
		const FButtonStyle& GetButtonStyle() const { return ButtonStyle; };

		const FCheckBoxStyle& GetCheckboxStyle() const { return CheckboxStyle; };

		const FSlateBrush& GetCircularThrobberBrushStyle() const { return CircularThrobberBrushStyle; };

		const FComboBoxStyle& GetComboBoxStyle() const { return ComboBoxStyle; };

		const FEditableTextStyle& GetEditableTextStyle() const { return EditableTextStyle; };

		const FEditableTextBoxStyle& GetEditableTextBoxStyle() const { return EditableTextBoxStyle; };

		const FExpandableAreaStyle& GetExpandableAreaStyle() const { return ExpandableAreaStyle; };
		const FSlateBrush& GetExpandableAreaBorderBrush() const { return ExpandableAreaBorderBrush; };

		const FTableViewStyle& GetListViewStyle() const { return ListViewStyle; };

		const FProgressBarStyle& GetProgressBarStyle() const { return ProgressBarStyle; };

		const FScrollBarStyle& GetScrollBarStyle() const { return ScrollBarStyle; };

		const FScrollBoxStyle& GetScrollBoxStyle() const { return ScrollBoxStyle; };

		const FSliderStyle& GetSliderStyle() const { return SliderStyle; };

		const FSpinBoxStyle& GetSpinBoxStyle() const { return SpinBoxStyle; };

		const FTableRowStyle& GetTableRowStyle() const { return TableRowStyle; };

		const FSlateBrush& GetThrobberBrush() const { return ThrobberBrush; };

		const FTextBlockStyle& GetTextBlockStyle() const { return TextBlockStyle; };

		const FTableViewStyle& GetTreeViewStyle() const { return TreeViewStyle; };

	private:
		friend FDefaultStyleCache;

		FButtonStyle ButtonStyle;

		FCheckBoxStyle CheckboxStyle;

		FSlateBrush CircularThrobberBrushStyle;

		FComboBoxStyle ComboBoxStyle;

		FEditableTextStyle EditableTextStyle;

		FEditableTextBoxStyle EditableTextBoxStyle;

		FExpandableAreaStyle ExpandableAreaStyle;
		FSlateBrush ExpandableAreaBorderBrush;

		FTableViewStyle ListViewStyle;

		FProgressBarStyle ProgressBarStyle;

		FScrollBarStyle ScrollBarStyle;

		FScrollBoxStyle ScrollBoxStyle;

		FSliderStyle SliderStyle;

		FSpinBoxStyle SpinBoxStyle;

		FTableRowStyle TableRowStyle;

		FSlateBrush ThrobberBrush;

		FTextBlockStyle TextBlockStyle;

		FTableViewStyle TreeViewStyle;

		TMap<UScriptStruct*, FSlateWidgetStyle*> TypeInstanceMap;
	};

	/** Gets singleton and returns runtime styles from singleton */
	static const FStyles& GetRuntime() { return Get().Runtime; }
#if WITH_EDITOR
	/** Gets singleton and returns editor styles from singleton */
	static const FStyles& GetEditor() { return Get().Editor; }
#endif

private:

	/** Singleton getter, however private used since GetRuntime & GetEditor is preferred for styles */
	static SLATECORE_API FDefaultStyleCache& Get();

	friend ::FLazySingleton;

	SLATECORE_API FDefaultStyleCache();

	FStyles Runtime;

#if WITH_EDITOR
	FStyles Editor;
#endif
};

} //namespace UE::Slate::Private
