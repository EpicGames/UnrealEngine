// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/StyleDefaults.h"

class Error;
class UTexture2D;
struct FSlateDynamicImageBrush;
struct FSlateWidgetStyle;

/**
 * 
 */
class ISlateStyle
{
public:
	/** Constructor. */
	ISlateStyle()
	{}

	/** Destructor. */
	virtual ~ISlateStyle() = default;

	/** @return The name used to identity this style set */
	virtual const FName& GetStyleSetName() const = 0;

	/**
	 * Populate an array of FSlateBrush with resources consumed by this style.
	 * @param OutResources - the array to populate.
	 */
	virtual void GetResources( TArray< const FSlateBrush* >& OutResources ) const = 0;
	virtual FString GetContentRootDir() const = 0;

	/**
	 * Gets the names of every style entry using a brush in this style
	 * Note: this function is expensive and is not designed to be used in performance critical situations
	 *
	 * @param BrushName The name of the brush to find style entries from
	 * @return Array of style names using the brush
	 */
	virtual TArray<FName> GetEntriesUsingBrush(const FName BrushName) const = 0;

	/**
	 * Locates a widget style of a specified type and name.
	 * Allows customization of whether a warning should be logged.
	 * @return The found widget style or the default value.
	 */
	template< typename WidgetStyleType >            
	const WidgetStyleType& GetWidgetStyle( FName PropertyName, const ANSICHAR* Specifier, const WidgetStyleType* DefaultValue, bool bWarnIfNotFound ) const 
	{
		const FSlateWidgetStyle* ExistingStyle = GetWidgetStyleInternal(WidgetStyleType::TypeName, Join( PropertyName, Specifier ), DefaultValue, bWarnIfNotFound);

		if ( ExistingStyle == nullptr )
		{
			return WidgetStyleType::GetDefault();
		}

		return *static_cast< const WidgetStyleType* >( ExistingStyle );
	}

	/**
	 * Locates a widget style of a specified type and name.
	 * @param DefaultValue The value to use if a widget style could not be found. If null and no style could be found, a warning will be logged.
	 * @return The found widget style or the default value.
	 */
	template< typename WidgetStyleType >
    const WidgetStyleType& GetWidgetStyle( FName PropertyName, const ANSICHAR* Specifier = nullptr, const WidgetStyleType* DefaultValue = nullptr ) const
    {
    	return GetWidgetStyle< WidgetStyleType >( PropertyName, Specifier, DefaultValue, DefaultValue == nullptr );
    }

	template< typename WidgetStyleType >            
	bool HasWidgetStyle( FName PropertyName, const ANSICHAR* Specifier = nullptr ) const
	{
		const bool bWarnIfNotFound = false;
		return GetWidgetStyleInternal( WidgetStyleType::TypeName, Join( PropertyName, Specifier ), nullptr, bWarnIfNotFound) != nullptr;
	}
	
	const FSlateWidgetStyle* GetWidgetStyle( const FName& WidgetStyleTypeName, FName PropertyName, const ANSICHAR* Specifier = nullptr, const FSlateWidgetStyle* DefaultValue = nullptr, bool bWarnIfNotFound = true ) const 
	{
		const FSlateWidgetStyle* ExistingStyle = GetWidgetStyleInternal(WidgetStyleTypeName, Join( PropertyName, Specifier ), DefaultValue, bWarnIfNotFound);

		if ( ExistingStyle == nullptr )
		{
			return DefaultValue;
		}

		return ExistingStyle ? ExistingStyle : DefaultValue;
	}

	/**
	 * @param PropertyName   Name of the property to get
	 * @param Specifier      An optional string to append to the property name
	 * @param DefaultValue	 The default value to return if the property cannot be found
	 *
	 * @return a float property.
	 */
	virtual float GetFloat(const FName PropertyName, const ANSICHAR* Specifier = nullptr, float DefaultValue = FStyleDefaults::GetFloat(), const ISlateStyle* RequestingStyle = nullptr) const = 0;

	/**
	 * @param PropertyName   Name of the property to get
	 * @param Specifier      An optional string to append to the property name
	 * @param DefaultValue	 The default value to return if the property cannot be found
	 *
	 * @return a FVector2D property.
	 */
	virtual UE::Slate::FDeprecateVector2DResult GetVector(const FName PropertyName, const ANSICHAR* Specifier = nullptr, UE::Slate::FDeprecateVector2DParameter DefaultValue = FStyleDefaults::GetVector2D(), const ISlateStyle* RequestingStyle = nullptr) const = 0;

	/**
	 * @param PropertyName   Name of the property to get
	 * @param Specifier      An optional string to append to the property name
	 * @param DefaultValue	 The default value to return if the property cannot be found
	 *
	 * @return a FLinearColor property.
	 */
	virtual const FLinearColor& GetColor(const FName PropertyName, const ANSICHAR* Specifier = nullptr, const FLinearColor& DefaultValue = FStyleDefaults::GetColor(), const ISlateStyle* RequestingStyle = nullptr) const = 0;

	/**
	 * @param PropertyName   Name of the property to get
	 * @param Specifier      An optional string to append to the property name
	 * @param DefaultValue	 The default value to return if the property cannot be found
	 *
	 * @return a FLinearColor property.
	 */
	virtual const FSlateColor GetSlateColor(const FName PropertyName, const ANSICHAR* Specifier = nullptr, const FSlateColor& DefaultValue = FStyleDefaults::GetSlateColor(), const ISlateStyle* RequestingStyle = nullptr) const = 0;

	/**
	 * @param PropertyName   Name of the property to get
	 * @param Specifier      An optional string to append to the property name
	 * @param DefaultValue	 The default value to return if the property cannot be found
	 *
	 * @return a FMargin property.
	 */
	virtual const FMargin& GetMargin(const FName PropertyName, const ANSICHAR* Specifier = nullptr, const FMargin& DefaultValue = FStyleDefaults::GetMargin(), const ISlateStyle* RequestingStyle = nullptr) const = 0;

	/**
	 * @param PropertyName   Name of the property to get
	 * @param Specifier      An optional string to append to the property name
	 *
	 * @return an FSlateBrush property.
	 */		
	virtual const FSlateBrush* GetBrush( const FName PropertyName, const ANSICHAR* Specifier = nullptr, const ISlateStyle* RequestingStyle = nullptr) const = 0;

	/**
	 * Just like GetBrush, but returns DefaultBrush instead of the
	 * "missing brush" image when the resource is not found.
	 * 
	 * @param PropertyName   Name of the property to get
	 * @param Specifier      An optional string to append to the property name
	 *
	 * @return an FSlateBrush property.
	 */		
	virtual const FSlateBrush* GetOptionalBrush( const FName PropertyName, const ANSICHAR* Specifier = nullptr, const FSlateBrush* const DefaultBrush = FStyleDefaults::GetNoBrush()) const = 0;

	virtual const TSharedPtr< FSlateDynamicImageBrush > GetDynamicImageBrush( const FName BrushTemplate, const FName TextureName, const ANSICHAR* Specifier = nullptr, const ISlateStyle* RequestingStyle = nullptr ) const = 0;

	virtual const TSharedPtr< FSlateDynamicImageBrush > GetDynamicImageBrush( const FName BrushTemplate, const ANSICHAR* Specifier, UTexture2D* TextureResource, const FName TextureName, const ISlateStyle* RequestingStyle = nullptr ) const = 0;

	virtual const TSharedPtr< FSlateDynamicImageBrush > GetDynamicImageBrush( const FName BrushTemplate, UTexture2D* TextureResource, const FName TextureName, const ISlateStyle* RequestingStyle = nullptr ) const = 0;

	/**
	 * Get default FSlateBrush.
	 * @return -	Default Slate brush value.
	 */
	virtual FSlateBrush* GetDefaultBrush() const = 0;

	/** Look up a sound property specified by PropertyName and optional Specifier. */
	virtual const FSlateSound& GetSound( const FName PropertyName, const ANSICHAR* Specifier = nullptr , const ISlateStyle* RequestingStyle = nullptr) const = 0;

	/**
	 * @param Propertyname   Name of the property to get
	 * @param InFontInfo     The value to set
	 *
	 * @return a InFontInfo property.
	 */		
	virtual FSlateFontInfo GetFontStyle( const FName PropertyName, const ANSICHAR* Specifier = nullptr ) const = 0;

	static FName Join( FName A, const ANSICHAR* B )
	{
		if( B == nullptr )
		{
			return A;
		}
		else
		{
			return FName( *( A.ToString() + B ) );
		}
	}

	virtual TSet<FName> GetStyleKeys() const = 0;

	virtual const TSet<FName>& GetWidgetStyleNames() const
	{
		static TSet<FName> Dummy;
		return Dummy;
	}
protected:

	/** 
	 * The severity of the message type 
	 * Ordered according to their severity
	 */
	enum class EStyleMessageSeverity : uint8
	{
		CriticalError UE_DEPRECATED(5.1, "CriticalError was removed because it can't trigger an assert at the callsite. Use 'checkf' instead.") = 0,
		Error = 1,
		PerformanceWarning = 2,
		Warning = 3,
		Info = 4,	// Should be last
	};


protected:

	friend class FSlateStyleSet;
	friend class ISlateStyle;

	/**
	 * 
	 */
	virtual const FSlateWidgetStyle* GetWidgetStyleInternal( const FName DesiredTypeName, const FName StyleName, const FSlateWidgetStyle* DefaultStyle, bool bWarnIfNotFound) const = 0;

	/**
	 * 
	 */
	virtual void Log( EStyleMessageSeverity Severity, const FText& Message ) const = 0;
	
	virtual void LogMissingResource(EStyleMessageSeverity Severity, const FText& Message, const FName& MissingResource) const = 0;

	virtual const TSharedPtr< FSlateDynamicImageBrush > MakeDynamicImageBrush( const FName BrushTemplate, UTexture2D* TextureResource, const FName TextureName ) const = 0;
};
