// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateTypes.h"

#include "SegmentedControlStyle.generated.h"

/**
 * Represents the appearance of an SSegmentedControl
 */
USTRUCT(BlueprintType)
struct FSegmentedControlStyle: public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FSegmentedControlStyle();

	virtual ~FSegmentedControlStyle() = default;

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FSegmentedControlStyle& GetDefault();

	/**
	 * The style to use for our Center Control
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FCheckBoxStyle ControlStyle;
	FSegmentedControlStyle& SetControlStyle( const FCheckBoxStyle& InStyle ){ ControlStyle = InStyle; return *this; }

	/**
	 * The style to use for our Left Control
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FCheckBoxStyle FirstControlStyle;
	FSegmentedControlStyle& SetFirstControlStyle( const FCheckBoxStyle& InStyle ){ FirstControlStyle = InStyle; return *this; }

	/**
	 * The style to use for our Left Control
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FCheckBoxStyle LastControlStyle;
	FSegmentedControlStyle& SetLastControlStyle( const FCheckBoxStyle& InStyle ){ LastControlStyle = InStyle; return *this; }

	/** Background of the segmented control */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BackgroundBrush;
	FSegmentedControlStyle& SetBackgroundBrush(const FSlateBrush& InBrush) { BackgroundBrush = InBrush; return *this; }

	/**
	 * Padding between each control
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin UniformPadding;
	FSegmentedControlStyle& SetUniformPadding(const FMargin& InMargin) { UniformPadding = InMargin; return *this; }

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		ControlStyle.UnlinkColors();
		FirstControlStyle.UnlinkColors();
		LastControlStyle.UnlinkColors();
		BackgroundBrush.UnlinkColors();
	}

};
