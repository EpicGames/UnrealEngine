// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateColor.h"
#include "Layout/Margin.h"
#include "Rendering/SlateResourceHandle.h"
#include "Types/SlateBox2.h"
#include "Types/SlateVector2.h"
#include "SlateBrush.generated.h"

/**
 * Enumerates ways in which an image can be drawn.
 */
UENUM(BlueprintType)
namespace ESlateBrushDrawType
{
	enum Type : int
	{
		/** Don't do anything */
		NoDrawType UMETA(DisplayName="None"),

		/** Draw a 3x3 box, where the sides and the middle stretch based on the Margin */
		Box,

		/** Draw a 3x3 border where the sides tile and the middle is empty */
		Border,

		/** Draw an image; margin is ignored */
		Image,

		/** Draw a solid rectangle with an outline and corner radius */
		RoundedBox
	};
}


/**
 * Enumerates tiling options for image drawing.
 */
UENUM(BlueprintType)
namespace ESlateBrushTileType
{
	enum Type : int
	{
		/** Just stretch */
		NoTile,

		/** Tile the image horizontally */
		Horizontal,

		/** Tile the image vertically */
		Vertical,

		/** Tile in both directions */
		Both
	};
}


/**
 * Possible options for mirroring the brush image
 */
UENUM()
namespace ESlateBrushMirrorType
{
	enum Type : int
	{
		/** Don't mirror anything, just draw the texture as it is. */
		NoMirror,

		/** Mirror the image horizontally. */
		Horizontal,

		/** Mirror the image vertically. */
		Vertical,

		/** Mirror in both directions. */
		Both
	};
}


/**
 * Enumerates brush image types.
 */
UENUM()
namespace ESlateBrushImageType
{
	enum Type : int
	{
		/** No image is loaded.  Color only brushes, transparent brushes etc. */
		NoImage,

		/** The image to be loaded is in full color. */
		FullColor,

		/** The image is a special texture in linear space (usually a rendering resource such as a lookup table). */
		Linear,

		/** The image is vector graphics and will be rendered and cached in full color using size/scale requested by slate */
		Vector,
	};
}


/**
 * Enumerates rounding options
 */
UENUM()
namespace ESlateBrushRoundingType
{
	enum Type : int
	{
		/** Use the specified Radius **/
		FixedRadius, 

		/** The rounding radius should be half the height such that it always looks perfectly round **/
		HalfHeightRadius,
	};
}


/**
 * Possible options for rounded box brush image
 */
USTRUCT(BlueprintType)
struct FSlateBrushOutlineSettings
{
	GENERATED_USTRUCT_BODY()

	FSlateBrushOutlineSettings()
		: CornerRadii(FVector4(0.0, 0.0, 0.0, 0.0))
		, Color(FLinearColor::Transparent)
		, Width(0.0)
		, RoundingType(ESlateBrushRoundingType::HalfHeightRadius)
		, bUseBrushTransparency(false)
	{}

	FSlateBrushOutlineSettings(float InUniformRadius)
		: CornerRadii(FVector4(InUniformRadius, InUniformRadius, InUniformRadius, InUniformRadius))
		, Color(FLinearColor::Transparent)
		, Width(0.0)
		, RoundingType(ESlateBrushRoundingType::FixedRadius)
		, bUseBrushTransparency(false)
	{}

	FSlateBrushOutlineSettings(FVector4 InRadius)
		: CornerRadii(InRadius)
		, Color(FLinearColor::Transparent)
		, Width(0.0)
		, RoundingType(ESlateBrushRoundingType::FixedRadius)
		, bUseBrushTransparency(false)
	{}

	FSlateBrushOutlineSettings(const FSlateColor& InColor, float InWidth)
		: CornerRadii(FVector4(0.0, 0.0, 0.0, 0.0))
		, Color(InColor)
		, Width(InWidth)
		, RoundingType(ESlateBrushRoundingType::HalfHeightRadius)
		, bUseBrushTransparency(false)
	{}

	FSlateBrushOutlineSettings(float InUniformRadius, const FSlateColor& InColor, float InWidth)
		: CornerRadii(FVector4(InUniformRadius, InUniformRadius, InUniformRadius, InUniformRadius))
		, Color(InColor)
		, Width(InWidth)
		, RoundingType(ESlateBrushRoundingType::FixedRadius)
		, bUseBrushTransparency(false)
	{}

	FSlateBrushOutlineSettings(FVector4 InRadius, const FSlateColor& InColor, float InWidth)
		: CornerRadii(InRadius)
		, Color(InColor)
		, Width(InWidth)
		, RoundingType(ESlateBrushRoundingType::FixedRadius)
		, bUseBrushTransparency(false)
	{}

	/**
	 * Compares these outline settings with another for equality.
	 *
	 * @param Other The other outline settings.
	 *
	 * @return true if settings are equal, false otherwise.
	 */
	bool operator==(const FSlateBrushOutlineSettings& Other) const
	{
		return CornerRadii == Other.CornerRadii
			&& Color == Other.Color
			&& Width == Other.Width
			&& RoundingType == Other.RoundingType
			&& bUseBrushTransparency == Other.bUseBrushTransparency;
	}

	/** Radius in Slate Units applied to the outline at each corner. X = Top Left, Y = Top Right, Z = Bottom Right, W = Bottom Left */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush)
	FVector4 CornerRadii;

	/** Tinting applied to the border outline. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush, meta=(DisplayName="Outline", sRGB="true"))
	FSlateColor Color;

	/** Line width in Slate Units applied to the border outline. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush)
	float Width;

	/** The Rounding Type **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush)
	TEnumAsByte<enum ESlateBrushRoundingType::Type > RoundingType;

	/** True if we should use the owning brush's transparency as our own **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush)
	bool bUseBrushTransparency;

};

namespace SlateBrushDefs
{
	inline constexpr float DefaultImageSize = 32.0f;
}

// Hacky base class to avoid 8 bytes of padding after the vtable
struct FSlateBrushFixLayout
{
	virtual ~FSlateBrushFixLayout() = default;
};

/**
* A brush which contains information about how to draw a Slate element
*/
USTRUCT(BlueprintType) //, meta = (HasNativeMake = ""))
struct FSlateBrush 
#if CPP 
: public FSlateBrushFixLayout
#endif
{
	GENERATED_USTRUCT_BODY()

	friend class FSlateShaderResourceManager;

public:
	/** Tinting applied to the image. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush, meta=( DisplayName="Tint", sRGB="true" ))
	FSlateColor TintColor;

	/** How to draw the image */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush)
	TEnumAsByte<enum ESlateBrushDrawType::Type > DrawAs;

	/** How to tile the image in Image mode */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush)
	TEnumAsByte<enum ESlateBrushTileType::Type> Tiling;

	/** How to mirror the image in Image mode.  This is normally only used for dynamic image brushes where the source texture
	    comes from a hardware device such as a web camera. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush)
	TEnumAsByte<enum ESlateBrushMirrorType::Type> Mirroring;

	/** The type of image */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Brush)
	TEnumAsByte<enum ESlateBrushImageType::Type> ImageType;

	/** Size of the resource in Slate Units */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush)
	FDeprecateSlateVector2D ImageSize;

	/** The margin to use in Box and Border modes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush, meta=( UVSpace="true" ))
	FMargin Margin;

#if WITH_EDITORONLY_DATA
	/** Tinting applied to the image. */
	UPROPERTY(NotReplicated)
	FLinearColor Tint_DEPRECATED;
#endif

public:

	/**
	 * Default constructor.
	 */
	SLATECORE_API FSlateBrush();

	virtual ~FSlateBrush()= default;

public:

	UE::Slate::FDeprecateVector2DResult GetImageSize() const { return UE::Slate::FDeprecateVector2DResult(ImageSize); }
	void SetImageSize(UE::Slate::FDeprecateVector2DParameter InImageSize) { ImageSize = InImageSize; }

	const FMargin& GetMargin() const { return Margin; }

	ESlateBrushTileType::Type GetTiling() const { return Tiling; }

	ESlateBrushMirrorType::Type GetMirroring() const { return Mirroring; }

	ESlateBrushImageType::Type GetImageType() const { return ImageType; }

	ESlateBrushDrawType::Type GetDrawType() const { return DrawAs; }

	/**
	 * Gets the name of the resource object, if any.
	 *
	 * @return Resource name, or NAME_None if the resource object is not set.
	 */
	const FName GetResourceName() const
	{
		return ( ( ResourceName != NAME_None ) || ( ResourceObject == nullptr ) )
			? ResourceName
			: ResourceObject->GetFName();
	}

	/**
	 * Gets the UObject that represents the brush resource, if any.
	 *
	 * The object may be a UMaterialInterface or a UTexture.
	 *
	 * @return The resource object, or nullptr if it is not set.
	 */
	class UObject* GetResourceObject( ) const
	{
		return ResourceObject;
	}

	/**
	 * Sets the UObject that represents the brush resource.
	 */
	SLATECORE_API void SetResourceObject(class UObject* InResourceObject);

	/**
	 * Gets the brush's tint color.
	 *
	 * @param InWidgetStyle The widget style to get the tint for.
	 * @return Tint color.
	 */
	FLinearColor GetTint( const FWidgetStyle& InWidgetStyle ) const
	{
		return TintColor.GetColor(InWidgetStyle);
	}

	/**
	 * Unlinks all colors in this brush.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		TintColor.Unlink();
	}

	/**
	 * Checks whether this brush has a UTexture object
	 *
	 * @return true if it has a UTexture object, false otherwise.
	 */
	bool HasUObject( ) const
	{
		return (ResourceObject != nullptr) || (bHasUObject_DEPRECATED);
	}

	/**
	 * Checks whether the brush resource is loaded dynamically.
	 *
	 * @return true if loaded dynamically, false otherwise.
	 */
	bool IsDynamicallyLoaded( ) const
	{
		return bIsDynamicallyLoaded;
	}

	/**
	 * Get brush UV region, should check if region is valid before using it
	 *
	 * @return UV region
	 */
	UE::Slate::FDeprecateBox2D GetUVRegion() const
	{
		return UVRegion;
	}

	/**
	 * Set brush UV region
	 *
	 * @param InUVRegion When valid - overrides UV region specified in resource proxy
	 */
	void SetUVRegion(const FBox2d& InUVRegion)
	{
		UVRegion = FBox2f(InUVRegion);
	}

	void SetUVRegion(const FBox2f& InUVRegion)
	{
		UVRegion = InUVRegion;
	}

	/**
	 * Compares this brush with another for equality.
	 *
	 * @param Other The other brush.
	 *
	 * @return true if the two brushes are equal, false otherwise.
	 */
	bool operator==( const FSlateBrush& Other ) const 
	{
		return ImageSize == Other.ImageSize
			&& DrawAs == Other.DrawAs
			&& Margin == Other.Margin
			&& TintColor == Other.TintColor
			&& Tiling == Other.Tiling
			&& Mirroring == Other.Mirroring
			&& ResourceObject == Other.ResourceObject
			&& ResourceName == Other.ResourceName
			&& bIsDynamicallyLoaded == Other.bIsDynamicallyLoaded
			&& UVRegion == Other.UVRegion
			&& (DrawAs != ESlateBrushDrawType::RoundedBox || OutlineSettings == Other.OutlineSettings); // Compare outline settings for equality only if we have a rounded box brush.
	}

	/**
	 * Compares this brush with another for inequality.
	 *
	 * @param Other The other brush.
	 *
	 * @return false if the two brushes are equal, true otherwise.
	 */
	bool operator!=( const FSlateBrush& Other ) const 
	{
		return !(*this == Other);
	}

	/** Report any references to UObjects to the reference collector. */
	void AddReferencedObjects(FReferenceCollector& Collector, UObject* ReferencingObject = nullptr)
	{
		Collector.AddReferencedObject(ResourceObject, ReferencingObject);
	}

	/**
	 * Gets the identifier for UObject based texture paths.
	 *
	 * @return Texture identifier string.
	 */
	static SLATECORE_API const FString UTextureIdentifier( );
	
	const FSlateResourceHandle& GetRenderingResource(UE::Slate::FDeprecateVector2DParameter LocalSize, float DrawScale) const
	{
		UpdateRenderingResource(LocalSize, DrawScale);

		return ResourceHandle;
	}

	SLATECORE_API const FSlateResourceHandle& GetRenderingResource() const;

	bool IsSet() const { return bIsSet; }

#if WITH_EDITOR
	void InvalidateResourceHandle()
	{
		ResourceHandle = FSlateResourceHandle();
	};
#endif

private:
	SLATECORE_API void UpdateRenderingResource(FVector2f LocalSize, float DrawScale) const;
	SLATECORE_API bool CanRenderResourceObject(UObject* InResourceObject) const;

private:

	/**
	 * The image to render for this brush, can be a UTexture or UMaterialInterface or an object implementing 
	 * the AtlasedTextureInterface. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush, meta=( AllowPrivateAccess="true", DisplayThumbnail="true", DisplayName="Image", AllowedClasses="/Script/Engine.Texture,/Script/Engine.MaterialInterface,/Script/Engine.SlateTextureAtlasInterface", DisallowedClasses = "/Script/MediaAssets.MediaTexture"))
	TObjectPtr<UObject> ResourceObject;

public:
	/** How to draw the outline.  Currently only used for RoundedBox type brushes. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Brush)
	FSlateBrushOutlineSettings OutlineSettings;

protected:
	/** 
	 *  Optional UV region for an image
	 *  When valid - overrides UV region specified in resource proxy
	 */
	UPROPERTY()
	FBox2f UVRegion;

	/** Whether or not the brush path is a path to a UObject */
	UPROPERTY()
	uint8 bIsDynamicallyLoaded:1;

	/** Whether or not the brush has a UTexture resource */
	UPROPERTY()
	uint8 bHasUObject_DEPRECATED:1;

	/** This is true for all constructed brushes except for optional brushes */
	uint8 bIsSet : 1;

public:

	/** Rendering resource for this brush */
	mutable FSlateResourceHandle ResourceHandle;

protected:
	/** The name of the rendering resource to use */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Brush)
	FName ResourceName;

	/** 
	 * This constructor is protected; use one of the deriving classes instead.
	 *
	 * @param InDrawType      How to draw the texture
	 * @param InResourceName  The name of the resource
	 * @param InMargin        Margin to use in border and box modes
	 * @param InTiling        Tile horizontally/vertically or both? (only in image mode)
	 * @param InImageType	  The type of image
	 * @param InTint		  Tint to apply to the element.
	 * @param InOutlineSettings Optional Outline Border Settings for RoundedBox mode
	 */
	 SLATECORE_API FORCENOINLINE FSlateBrush( ESlateBrushDrawType::Type InDrawType, const FName InResourceName, const FMargin& InMargin, ESlateBrushTileType::Type InTiling, ESlateBrushImageType::Type InImageType, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FLinearColor& InTint = FLinearColor::White, UObject* InObjectResource = nullptr, bool bInDynamicallyLoaded = false);

	 SLATECORE_API FORCENOINLINE FSlateBrush( ESlateBrushDrawType::Type InDrawType, const FName InResourceName, const FMargin& InMargin, ESlateBrushTileType::Type InTiling, ESlateBrushImageType::Type InImageType, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const TSharedRef< FLinearColor >& InTint, UObject* InObjectResource = nullptr, bool bInDynamicallyLoaded = false);

	 SLATECORE_API FORCENOINLINE FSlateBrush( ESlateBrushDrawType::Type InDrawType, const FName InResourceName, const FMargin& InMargin, ESlateBrushTileType::Type InTiling, ESlateBrushImageType::Type InImageType, const UE::Slate::FDeprecateVector2DParameter& InImageSize, const FSlateColor& InTint, UObject* InObjectResource = nullptr, bool bInDynamicallyLoaded = false);

};

/** Provides a means to hold onto the source of a slate brush. */
class ISlateBrushSource
{
public:
	virtual ~ISlateBrushSource() = default;
	virtual const FSlateBrush* GetSlateBrush() const = 0;
};
