// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Styling/WidgetStyle.h"
#include "SlateColor.generated.h"

/**
 * Enumerates types of color values that can be held by Slate color.
 *
 * Should we use the specified color? If not, then which color from the style should we use.
 */
UENUM(BlueprintType)
enum class ESlateColorStylingMode : uint8
{
	/** Color value is stored in this Slate color. */
	UseColor_Specified UMETA(DisplayName = "Specified Color"),

	/** Color value is stored in the linked color table. */
	UseColor_ColorTable UMETA(Hidden),

	/** Use the widget's foreground color. */
	UseColor_Foreground UMETA(DisplayName = "Foreground Color"),

	/** Use the widget's subdued color. */
	UseColor_Foreground_Subdued UMETA(Hidden),

	/** Use the foreground color defined in a widget specific style. */
	UseColor_UseStyle UMETA(Hidden),
};

enum class EStyleColor : uint8;

/**
 * A Slate color can be a directly specified value, or the color can be pulled from a WidgetStyle.
 */
USTRUCT(BlueprintType)
struct FSlateColor
{
	GENERATED_USTRUCT_BODY()

public:

	/**
	 * Default constructor.
	 *
	 * Uninitialized Slate colors are Fuchsia by default.
	 */
	FSlateColor()
		: SpecifiedColor(FLinearColor(1.0f, 0.0f, 1.0f))
		, ColorUseRule(ESlateColorStylingMode::UseColor_Specified)
	{ }

	/**
	 * Creates a new Slate color with the specified values.
	 *
	 * @param InColor The color value to assign.
	 */
	FSlateColor(const FLinearColor& InColor)
		: SpecifiedColor(InColor)
		, ColorUseRule(ESlateColorStylingMode::UseColor_Specified)
	{ }


	/**
	 * Creates a new Slate color with the specified values.
	 *
	 * @param InColor The color value to assign.
	 */
	FSlateColor(const FColor InColor)
		: SpecifiedColor(InColor.ReinterpretAsLinear())
		, ColorUseRule(ESlateColorStylingMode::UseColor_Specified)
	{ }


	/*
	 * Creates a new Slate color that is linked to the given values.
	 *
	 * @param InColor The color value to assign.
	 */
	FSlateColor(const TSharedRef< FLinearColor >& InColor)
		: SpecifiedColor(*InColor)
		, ColorUseRule(ESlateColorStylingMode::UseColor_Specified)
	{ }

	FSlateColor(EStyleColor InColorTableId)
		: SpecifiedColor()
		, ColorUseRule(ESlateColorStylingMode::UseColor_ColorTable)
		, ColorTableId(InColorTableId)
	{ }

public:

	/**
	 * Gets the color value represented by this Slate color.
	 *
	 * @param InWidgetStyle The widget style to use when this color represents a foreground or subdued color.
	 * @return The color value.
	 * @see GetSpecifiedColor
	 */
	const FLinearColor& GetColor( const FWidgetStyle& InWidgetStyle ) const
	{
		switch(ColorUseRule)
		{
		default:
		case ESlateColorStylingMode::UseColor_Foreground:
		case ESlateColorStylingMode::UseColor_UseStyle:
			return InWidgetStyle.GetForegroundColor();
			break;

		case ESlateColorStylingMode::UseColor_Specified:
			return SpecifiedColor;
			break;

		case ESlateColorStylingMode::UseColor_ColorTable:
			return GetColorFromTable();
			break;

		case ESlateColorStylingMode::UseColor_Foreground_Subdued:
			return InWidgetStyle.GetSubduedForegroundColor();
			break;
		};
	}

	/**
	 * Gets the specified color value.
	 *
	 * @return The specified color value.
	 * @see GetColor, IsColorSpecified
	 */
	FLinearColor GetSpecifiedColor( ) const
	{
		if (ColorUseRule == ESlateColorStylingMode::UseColor_ColorTable)
		{
			return GetColorFromTable();
		}

		return SpecifiedColor;
	}

	/**
	 * Checks whether the values for this color have been specified.
	 *
	 * @return true if specified, false otherwise.
	 * @see GetSpecifiedColor
	 */
	bool IsColorSpecified( ) const
	{
		return (ColorUseRule == ESlateColorStylingMode::UseColor_Specified) || (ColorUseRule == ESlateColorStylingMode::UseColor_ColorTable);
	}

	/**
	 * If the color rule is set to UseColor_Specified_Link, this will copy the linked color internally,
	 * unlink it and set the color rule to UseColor_Specified.
	 * Nothing happens if the color rule is not set to UseColor_Specified_Link.
	 */
	void Unlink()
	{
		if (ColorUseRule == ESlateColorStylingMode::UseColor_ColorTable)
		{
			SpecifiedColor = GetColorFromTable();
			ColorUseRule = ESlateColorStylingMode::UseColor_Specified;
		}
	}

	/**
	 * Compares this color with another for equality.
	 *
	 * @param Other The other color.
	 *
	 * @return true if the two colors are equal, false otherwise.
	 */
	bool operator==( const FSlateColor& Other ) const 
	{
		return SpecifiedColor == Other.SpecifiedColor
			&& ColorUseRule == Other.ColorUseRule
			&& (ColorUseRule != ESlateColorStylingMode::UseColor_ColorTable || ColorTableId == Other.ColorTableId);
	}

	/**
	 * Compares this color with another for inequality.
	 *
	 * @param Other The other color.
	 *
	 * @return false if the two colors are equal, true otherwise.
	 */
	bool operator!=( const FSlateColor& Other ) const 
	{
		return !(*this == Other);
	}

public:

	/** @returns an FSlateColor that is the widget's foreground. */
	static FSlateColor UseForeground( )
	{
		return FSlateColor( ESlateColorStylingMode::UseColor_Foreground );
	}

	/** @returns an FSlateColor that is the subdued version of the widget's foreground. */
	static FSlateColor UseSubduedForeground( )
	{
		return FSlateColor( ESlateColorStylingMode::UseColor_Foreground_Subdued );
	}

	/** @returns an FSlateColor that is the subdued version of the widget's foreground. */
	static FSlateColor UseStyle()
	{
		return FSlateColor(ESlateColorStylingMode::UseColor_UseStyle);
	}


	/** Used to upgrade an FColor or FLinearColor property to an FSlateColor property */
	SLATECORE_API bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

protected:
	
	// Private constructor to prevent construction of invalid FSlateColors
	FSlateColor(ESlateColorStylingMode InColorUseRule)
		: SpecifiedColor(FLinearColor(1.0f, 0.0f, 1.0f))
		, ColorUseRule(InColorUseRule)
		, ColorTableId()
	{ }

	SLATECORE_API const FLinearColor& GetColorFromTable() const;
protected:

	// The current specified color; only meaningful when ColorToUse == UseColor_Specified.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Color)
	FLinearColor SpecifiedColor;

	// The rule for which color to pick.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Color)
	ESlateColorStylingMode ColorUseRule;

private:
	// Id to a color in a color table to be used with ESlateColorStylingMode::UseColor_ColorTable
	EStyleColor ColorTableId;
};

template<>
struct TStructOpsTypeTraits<FSlateColor>
	: public TStructOpsTypeTraitsBase2<FSlateColor>
{
	enum 
	{
		WithStructuredSerializeFromMismatchedTag = true,
	};
};
