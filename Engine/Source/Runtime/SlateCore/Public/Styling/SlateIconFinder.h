// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateIcon.h"

struct FSlateBrush;

/** Class used for finding icons within a registered set of styles */
class FSlateIconFinder
{
public:

	/**
	 * Find the icon to use for the supplied struct
	 *
	 * @param InStruct			The struct to locate an icon for
	 * @param InDefaultName		The default icon name to use, if no specialized icon could be found
	 * @return An FSlateIcon structure defining the struct's icon
	 */
	static SLATECORE_API FSlateIcon FindIconForClass(const UStruct* InStruct, const FName& InDefaultName = FName());

	/**
	 * Find a custom icon to use for the supplied struct, according to the specified base style
	 *
	 * @param InStruct			The struct to locate an icon for
	 * @param InStyleBasePath	Style base path to use for the search (e.g. ClassIcon, or ClassThumbnail)
	 * @param InDefaultName		The default icon name to use, if no specialized icon could be found
	 * @return An FSlateIcon structure defining the struct's icon
	 */
	static SLATECORE_API FSlateIcon FindCustomIconForClass(const UStruct* InStruct, const TCHAR* InStyleBasePath, const FName& InDefaultName = FName());

	/**
	 * Find a slate brush to use for the supplied struct's icon
	 *
	 * @param InStruct			The struct to locate an icon for
	 * @param InDefaultName		The default icon name to use, if no specialized icon could be found
	 * @return A slate brush, or nullptr if no icon was found
	 */
	static const FSlateBrush* FindIconBrushForClass(const UStruct* InStruct, const FName& InDefaultName = FName())
	{
		return FindIconForClass(InStruct, InDefaultName).GetIcon();
	}
	
	/**
	 * Find a custom icon to use for the supplied struct, according to the specified base style
	 *
	 * @param InStruct			The struct to locate an icon for
	 * @param InStyleBasePath	Style base path to use for the search (e.g. ClassIcon, or ClassThumbnail)
	 * @param InDefaultName		The default icon name to use, if no specialized icon could be found
	 * @return A slate brush, or nullptr if no icon was found
	 */
	static const FSlateBrush* FindCustomIconBrushForClass(const UStruct* InStruct, const TCHAR* InStyleBasePath, const FName& InDefaultName = FName())
	{
		return FindCustomIconForClass(InStruct, InStyleBasePath, InDefaultName).GetIcon();
	}

	/**
	 * Find the first occurrence of a brush represented by the specified IconName in any of the registered style sets
	 * @param InIconName 		The fully qualified style name of the icon to find
	 * @return An FSlateIcon structure
	 */
	static SLATECORE_API FSlateIcon FindIcon(const FName& InIconName);
};
