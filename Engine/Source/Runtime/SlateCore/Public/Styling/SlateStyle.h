// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/Margin.h"
#include "Sound/SlateSound.h"
#include "Styling/StyleDefaults.h"
#include "Styling/ISlateStyle.h"
#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Trace/SlateMemoryTags.h"

class UTexture2D;
struct FSlateWidgetStyle;

/**
 * A slate style chunk that contains a collection of named properties that guide the appearance of Slate.
 * At the moment, basically FAppStyle.
 */
class FSlateStyleSet : public ISlateStyle
{
public:
	/**
	 * Construct a style chunk.
	 * @param InStyleSetName The name used to identity this style set
	 */
	SLATECORE_API FSlateStyleSet(const FName& InStyleSetName);
	SLATECORE_API FSlateStyleSet(FSlateStyleSet&&);
	SLATECORE_API FSlateStyleSet(const FSlateStyleSet&);

	/** Destructor. */
	SLATECORE_API virtual ~FSlateStyleSet();

	SLATECORE_API virtual const FName& GetStyleSetName() const override;
	SLATECORE_API virtual void GetResources(TArray< const FSlateBrush* >& OutResources) const override;
	SLATECORE_API virtual TArray<FName> GetEntriesUsingBrush(const FName BrushName) const override;
	SLATECORE_API virtual void SetContentRoot(const FString& InContentRootDir);
	SLATECORE_API virtual FString RootToContentDir(const ANSICHAR* RelativePath, const TCHAR* Extension);
	SLATECORE_API virtual FString RootToContentDir(const WIDECHAR* RelativePath, const TCHAR* Extension);
	SLATECORE_API virtual FString RootToContentDir(const FString& RelativePath, const TCHAR* Extension);
	SLATECORE_API virtual FString RootToContentDir(const ANSICHAR* RelativePath);
	SLATECORE_API virtual FString RootToContentDir(const WIDECHAR* RelativePath);
	SLATECORE_API virtual FString RootToContentDir(const FString& RelativePath);
	virtual FString GetContentRootDir() const { return ContentRootDir; }

	SLATECORE_API virtual void SetCoreContentRoot(const FString& InCoreContentRootDir);

	SLATECORE_API virtual FString RootToCoreContentDir(const ANSICHAR* RelativePath, const TCHAR* Extension);
	SLATECORE_API virtual FString RootToCoreContentDir(const WIDECHAR* RelativePath, const TCHAR* Extension);
	SLATECORE_API virtual FString RootToCoreContentDir(const FString& RelativePath, const TCHAR* Extension);
	SLATECORE_API virtual FString RootToCoreContentDir(const ANSICHAR* RelativePath);
	SLATECORE_API virtual FString RootToCoreContentDir(const WIDECHAR* RelativePath);
	SLATECORE_API virtual FString RootToCoreContentDir(const FString& RelativePath);

	SLATECORE_API virtual float GetFloat(const FName PropertyName, const ANSICHAR* Specifier = nullptr, float DefaultValue = FStyleDefaults::GetFloat(), const ISlateStyle* RequestingStyle = nullptr) const override;
	SLATECORE_API virtual UE::Slate::FDeprecateVector2DResult GetVector(const FName PropertyName, const ANSICHAR* Specifier = nullptr, UE::Slate::FDeprecateVector2DParameter DefaultValue = FStyleDefaults::GetVector2D(), const ISlateStyle* RequestingStyle = nullptr) const override;
	SLATECORE_API virtual const FLinearColor& GetColor(const FName PropertyName, const ANSICHAR* Specifier = nullptr, const FLinearColor& DefaultValue = FStyleDefaults::GetColor(), const ISlateStyle* RequestingStyle = nullptr) const override;
	SLATECORE_API virtual const FSlateColor GetSlateColor(const FName PropertyName, const ANSICHAR* Specifier = nullptr, const FSlateColor& DefaultValue = FStyleDefaults::GetSlateColor(), const ISlateStyle* RequestingStyle = nullptr) const override;
	SLATECORE_API virtual const FMargin& GetMargin(const FName PropertyName, const ANSICHAR* Specifier = nullptr, const FMargin& DefaultValue = FStyleDefaults::GetMargin(), const ISlateStyle* RequestingStyle = nullptr) const override;

	SLATECORE_API virtual const FSlateBrush* GetBrush(const FName PropertyName, const ANSICHAR* Specifier = nullptr, const ISlateStyle* RequestingStyle = nullptr) const override;
	SLATECORE_API virtual const FSlateBrush* GetOptionalBrush(const FName PropertyName, const ANSICHAR* Specifier = nullptr, const FSlateBrush* const DefaultBrush = FStyleDefaults::GetNoBrush()) const override;

	SLATECORE_API virtual const TSharedPtr<FSlateDynamicImageBrush> GetDynamicImageBrush(const FName BrushTemplate, const FName TextureName, const ANSICHAR* Specifier = nullptr, const ISlateStyle* RequestingStyle = nullptr) const override;
	SLATECORE_API virtual const TSharedPtr<FSlateDynamicImageBrush> GetDynamicImageBrush(const FName BrushTemplate, const ANSICHAR* Specifier, UTexture2D* TextureResource, const FName TextureName, const ISlateStyle* RequestingStyle = nullptr) const override;
	SLATECORE_API virtual const TSharedPtr<FSlateDynamicImageBrush> GetDynamicImageBrush(const FName BrushTemplate, UTexture2D* TextureResource, const FName TextureName, const ISlateStyle* RequestingStyle = nullptr) const override;

	SLATECORE_API virtual FSlateBrush* GetDefaultBrush() const override;

	SLATECORE_API virtual const FSlateSound& GetSound(const FName PropertyName, const ANSICHAR* Specifier = nullptr, const ISlateStyle* RequestingStyle = nullptr) const override;
	SLATECORE_API virtual FSlateFontInfo GetFontStyle(const FName PropertyName, const ANSICHAR* Specifier = nullptr) const override;

	/** Name a Parent Style to fallback on if the requested styles are specified in this style **/
	SLATECORE_API void SetParentStyleName(const FName& InParentStyleName);
public:

	template< typename DefinitionType >
	FORCENOINLINE void Set(const FName PropertyName, const DefinitionType& InStyleDefintion)
	{
		LLM_SCOPE_BYTAG(UI_Style);
		WidgetStyleValues.Add(PropertyName, MakeShared<DefinitionType>(InStyleDefintion));
		KnownWidgetStyles.Add(DefinitionType::TypeName);
	}

	/**
	 * Set float properties
	 * @param PropertyName - Name of the property to set.
	 * @param InFloat - The value to set.
	 */
	FORCENOINLINE void Set(const FName PropertyName, const float InFloat)
	{
		LLM_SCOPE_BYTAG(UI_Style);
		FloatValues.Add(PropertyName, InFloat);
	}

	/**
	 * Add a FVector2D property to this style's collection.
	 * @param PropertyName - Name of the property to set.
	 * @param InVector - The value to set.
	 */
	UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
	FORCENOINLINE void Set(const FName PropertyName, const FVector2D& InVector)
	{
		LLM_SCOPE_BYTAG(UI_Style);
		Vector2DValues.Add(PropertyName, UE::Slate::CastToVector2f(InVector));
	}
	FORCENOINLINE void Set(const FName PropertyName, const UE::Slate::FDeprecateVector2DResult& InVector)
	{
		LLM_SCOPE_BYTAG(UI_Style);
		Vector2DValues.Add(PropertyName, InVector);
	}
	FORCENOINLINE void Set(const FName PropertyName, const FVector2f& InVector)
	{
		LLM_SCOPE_BYTAG(UI_Style);
		Vector2DValues.Add(PropertyName, InVector);
	}

	/**
	 * Set FLinearColor property.
	 * @param PropertyName - Name of the property to set.
	 * @param InColor - The value to set.
	 */
	FORCENOINLINE void Set(const FName PropertyName, const FLinearColor& InColor)
	{
		LLM_SCOPE_BYTAG(UI_Style);
		ColorValues.Add(PropertyName, InColor);
	}

	FORCENOINLINE void Set(const FName PropertyName, const FColor& InColor)
	{
		LLM_SCOPE_BYTAG(UI_Style);
		ColorValues.Add(PropertyName, InColor);
	}

	/**
	 * Add a FSlateLinearColor property to this style's collection.
	 * @param PropertyName - Name of the property to add.
	 * @param InColor - The value to add.
	 */
	FORCENOINLINE void Set(const FName PropertyName, const FSlateColor& InColor)
	{
		LLM_SCOPE_BYTAG(UI_Style);
		SlateColorValues.Add(PropertyName, InColor);
	}

	/**
	 * Add a FMargin property to this style's collection.
	 * @param PropertyName - Name of the property to add.
	 * @param InMargin - The value to add.
	 */
	FORCENOINLINE void Set(const FName PropertyName, const FMargin& InMargin)
	{
		LLM_SCOPE_BYTAG(UI_Style);
		MarginValues.Add(PropertyName, InMargin);
	}

	/**
	 * Add a FSlateBrush property to this style's collection
	 * @param PropertyName - Name of the property to set.
	 * @param InBrush - The brush to set.
	 */
	template<typename BrushType>
	FORCENOINLINE void Set(const FName PropertyName, BrushType* InBrush)
	{
		// @TODO: This scope may not work, the memory was allocated from the caller not in this scope
		// Need some way to capure in parent scope?
		LLM_SCOPE_BYTAG(UI_Style);
		BrushResources.Add(PropertyName, InBrush);
	}
	/**
	 * Set FSlateSound properties
	 *
	 * @param PropertyName  Name of the property to set
	 * @param InSound		Sound to set
	 */
	FORCENOINLINE void Set(FName PropertyName, const FSlateSound& InSound)
	{
		LLM_SCOPE_BYTAG(UI_Style);
		Sounds.Add(PropertyName, InSound);
	}

	/**
	 * Set FSlateFontInfo properties
	 *
	 * @param PropertyName  Name of the property to set
	 * @param InFontStyle   The value to set
	 */
	FORCENOINLINE void Set(FName PropertyName, const FSlateFontInfo& InFontInfo)
	{
		LLM_SCOPE_BYTAG(UI_Style);
		FontInfoResources.Add(PropertyName, InFontInfo);
	}

protected:

	friend class FSlateStyleSet;
	friend class ISlateStyle;

	SLATECORE_API virtual const FSlateWidgetStyle* GetWidgetStyleInternal(const FName DesiredTypeName, const FName StyleName, const FSlateWidgetStyle* DefaultStyle, bool bWarnIfNotFound) const override;

	SLATECORE_API virtual void Log(ISlateStyle::EStyleMessageSeverity Severity, const FText& Message) const override;

	SLATECORE_API virtual void LogMissingResource(EStyleMessageSeverity Severity, const FText& Message, const FName& MissingResource) const override;

	SLATECORE_API virtual const TSharedPtr< FSlateDynamicImageBrush > MakeDynamicImageBrush(const FName BrushTemplate, UTexture2D* TextureResource, const FName TextureName) const override;

	SLATECORE_API virtual void LogUnusedBrushResources();

	SLATECORE_API virtual TSet<FName> GetStyleKeys() const override;

	SLATECORE_API virtual const TSet<FName>& GetWidgetStyleNames() const override;

protected:

	SLATECORE_API bool IsBrushFromFile(const FString& FilePath, const FSlateBrush* Brush);

	/** The name used to identity this style set */
	FName StyleSetName;

	/** This dir is Engine/Editor/Slate folder **/
	FString ContentRootDir;

	/** This dir is Engine/Slate folder to share the items **/
	FString CoreContentRootDir;

	TMap< FName, TSharedRef< struct FSlateWidgetStyle > > WidgetStyleValues;
	TSet<FName> KnownWidgetStyles;

	/** float property storage. */
	TMap< FName, float > FloatValues;

	/** FVector2D property storage. */
	TMap< FName, FVector2f > Vector2DValues;

	/** Color property storage. */
	TMap< FName, FLinearColor > ColorValues;

	/** FSlateColor property storage. */
	TMap< FName, FSlateColor > SlateColorValues;

	/** FMargin property storage. */
	TMap< FName, FMargin > MarginValues;

	/* FSlateBrush property storage */
	FSlateBrush* DefaultBrush;
	TMap< FName, FSlateBrush* > BrushResources;

	/** SlateSound property storage */
	TMap< FName, FSlateSound > Sounds;

	/** FSlateFontInfo property storage. */
	TMap< FName, FSlateFontInfo > FontInfoResources;

	/** A list of dynamic brushes */
	mutable TMap< FName, TWeakPtr< FSlateDynamicImageBrush > > DynamicBrushes;

	/** A set of resources that were requested, but not found. */
	mutable TSet< FName > MissingResources;

	FName ParentStyleName;
	SLATECORE_API const ISlateStyle* GetParentStyle() const;
};

