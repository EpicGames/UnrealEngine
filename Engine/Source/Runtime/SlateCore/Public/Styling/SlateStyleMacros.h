// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush( RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define BORDER_BRUSH( RelativePath, ... ) FSlateBorderBrush( RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )

#define IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush(RootToContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)
#define BOX_BRUSH_SVG( RelativePath, ... ) FSlateVectorBoxBrush(RootToContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)
#define BORDER_BRUSH_SVG( RelativePath, ... ) FSlateVectorBorderBrush(RootToContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)

#define CORE_IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( RootToCoreContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define CORE_BOX_BRUSH( RelativePath, ... ) FSlateBoxBrush( RootToCoreContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define CORE_BORDER_BRUSH( RelativePath, ... ) FSlateBorderBrush( RootToCoreContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )

#define CORE_IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush(RootToCoreContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)
#define CORE_BOX_BRUSH_SVG( RelativePath, ... ) FSlateVectorBoxBrush(RootToCoreContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)
#define CORE_BORDER_BRUSH_SVG( RelativePath, ... ) FSlateVectorBorderBrush(RootToCoreContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)

#define DEFAULT_FONT(...) FCoreStyle::GetDefaultFontStyle(__VA_ARGS__)
#define REGULAR_ICON_FONT(...) FCoreStyle::GetRegularIconFontStyle(__VA_ARGS__)