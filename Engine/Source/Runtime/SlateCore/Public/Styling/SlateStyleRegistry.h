// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ISlateStyle;
struct FSlateBrush;

/**
* A central repository that can be used to track and manage chunks of slate style data.
*/
class FSlateStyleRegistry
{
public:

	/**
	 * Add a slate style to the repository.
	 *
	 * @param ISlateStyle interface to a user-definable style to add.
	 */
	static SLATECORE_API void RegisterSlateStyle( const ISlateStyle& InSlateStyle );

	/**
	 * Remove a slate style from the repository.
	 *
	 * @param ISlateStyle Interface to the style to remove.
	 */
	static SLATECORE_API void UnRegisterSlateStyle( const ISlateStyle& InSlateStyle );

	/**
	 * Removes a slate style from the repository (by name).
	 *
	 * @param StyleSetName The name of the style set to remove.
	 */
	static SLATECORE_API void UnRegisterSlateStyle( const FName StyleSetName );

	/**
	 * Find a slate style in the repository.
	 *
	 * @param InSlateStyleName The name of the slate style to find.
	 * @return The slate style, or null if it couldn't be found
	 */
	static SLATECORE_API const ISlateStyle* FindSlateStyle( const FName& InSlateStyleName );

	/**
	 * Iterate all the slate style known to this registry
	 *
	 * @param Iter A predicate to call for each known style that returns true to continue iteration, and false to terminate iteration
	 * @return true where to loop completed, of false if it was terminated by the predicate
	 */
	static SLATECORE_API bool IterateAllStyles(const TFunctionRef<bool(const ISlateStyle&)>& Iter);

	/**
	 * Populate an array of slate brushes with all of the resources used by the registered styles.
	 *
	 * @param OutResources Array of slate brushes to populate.
	 */
	static SLATECORE_API void GetAllResources( TArray<const FSlateBrush*>& OutResources );

	/**
	 * Gets the names of every style entry using a brush
	 * Note: this function is expensive and is not designed to be used in performance critical situations
	 * 
	 * @param BrushName The name of the brush to find style entries from
	 * @return Array of style names using the brush
	 */
	static SLATECORE_API TArray<FName> GetSylesUsingBrush(const FName BrushName);

private:

	/** Repository is just a collection of shared style pointers. */
	static TMap<FName, const ISlateStyle*> SlateStyleRepository;
};
