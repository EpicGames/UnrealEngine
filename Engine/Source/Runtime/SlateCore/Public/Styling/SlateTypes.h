// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Styling/SlateColor.h"
#include "Fonts/CompositeFont.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Sound/SlateSound.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "Types/SlateVector2.h"

#include "SlateTypes.generated.h"

class SWidget;

/** Used to determine how we should handle mouse wheel input events when someone scrolls. */
UENUM()
enum class EConsumeMouseWheel : uint8
{
	/** Only consume the mouse wheel event when we actually scroll some amount. */
	WhenScrollingPossible,

	/** Always consume mouse wheel event even if we don't scroll at all. */
	Always,

	/** Never consume the mouse wheel */
	Never,
};

/** Used to determine which search method we should use when finding a suitable parent window */
UENUM(BlueprintType)
enum class ESlateParentWindowSearchMethod : uint8
{
	/** Favor using the active window (will fallback to the main window if the active window is unsuitable) */
	ActiveWindow,

	/** Favor using the main window */
	MainWindow,
};

/** Type of check box */
UENUM()
namespace ESlateCheckBoxType
{
	enum Type : int
	{
		/** Traditional check box with check button and label (or other content) */
		CheckBox,

		/** Toggle button.  You provide button content (such as an image), and the user can press to toggle it. */
		ToggleButton,
	};
}

/** Current state of the check box */
UENUM(BlueprintType)
enum class ECheckBoxState : uint8
{
	/** Unchecked */
	Unchecked,
	/** Checked */
	Checked,
	/** Neither checked nor unchecked */
	Undetermined
};

/**
 * The different methods that can be used to determine what happens to text when it is longer than its allowed length
 */
UENUM(BlueprintType)
enum class ETextOverflowPolicy : uint8
{
	/** Overflowing text will be clipped */
	Clip = 0,

	/** Overflowing text will be replaced with an ellipsis */
	Ellipsis,

	/** Overflowing text will be replaced with an ellipsis. A partially clipped line on the vertical axis will be totally clipped, and ellipsis displayed on previous line */
	MultilineEllipsis,

	/** Overflowing text will be replaced with an ellipsis starting from the center\n
	 * Current Limits:
	 * - Multiline is not supported
	 * - RichText is not fully supported
	 * - Highlight text is not supported
	 * - Arabic mix with western character is not supported
	 */
	MiddleEllipsis
};


/**
 * Represents the appearance of an SCheckBox
 */
USTRUCT(BlueprintType)
struct FCheckBoxStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FCheckBoxStyle();
	SLATECORE_API FCheckBoxStyle(const FCheckBoxStyle&);
	SLATECORE_API virtual ~FCheckBoxStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* > & OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FCheckBoxStyle& GetDefault();

	/** The visual type of the checkbox */	
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category=Appearance )
	TEnumAsByte<ESlateCheckBoxType::Type> CheckBoxType;
	FCheckBoxStyle& SetCheckBoxType( ESlateCheckBoxType::Type InCheckBoxType ){ CheckBoxType = InCheckBoxType; return *this; }

	/* CheckBox appearance when the CheckBox is unchecked (normal) */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category = Appearance )
	FSlateBrush UncheckedImage;
	FCheckBoxStyle& SetUncheckedImage( const FSlateBrush& InUncheckedImage ){ UncheckedImage = InUncheckedImage; return *this; }

	/* CheckBox appearance when the CheckBox is unchecked and hovered */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category = Appearance )
	FSlateBrush UncheckedHoveredImage;
	FCheckBoxStyle& SetUncheckedHoveredImage( const FSlateBrush& InUncheckedHoveredImage ){ UncheckedHoveredImage = InUncheckedHoveredImage; return *this; }

	/* CheckBox appearance when the CheckBox is unchecked and hovered */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category = Appearance )
	FSlateBrush UncheckedPressedImage;
	FCheckBoxStyle& SetUncheckedPressedImage( const FSlateBrush& InUncheckedPressedImage ){ UncheckedPressedImage = InUncheckedPressedImage; return *this; }

	/* CheckBox appearance when the CheckBox is checked */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category = Appearance )
	FSlateBrush CheckedImage;
	FCheckBoxStyle& SetCheckedImage( const FSlateBrush& InCheckedImage ){ CheckedImage = InCheckedImage; return *this; }

	/* CheckBox appearance when checked and hovered */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category = Appearance )
	FSlateBrush CheckedHoveredImage;
	FCheckBoxStyle& SetCheckedHoveredImage( const FSlateBrush& InCheckedHoveredImage ){ CheckedHoveredImage = InCheckedHoveredImage; return *this; }

	/* CheckBox appearance when checked and pressed */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category = Appearance )
	FSlateBrush CheckedPressedImage;
	FCheckBoxStyle& SetCheckedPressedImage( const FSlateBrush& InCheckedPressedImage ){ CheckedPressedImage = InCheckedPressedImage; return *this; }
	
	/* CheckBox appearance when the CheckBox is undetermined */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category = Appearance )
	FSlateBrush UndeterminedImage;
	FCheckBoxStyle& SetUndeterminedImage( const FSlateBrush& InUndeterminedImage ){ UndeterminedImage = InUndeterminedImage; return *this; }

	/* CheckBox appearance when CheckBox is undetermined and hovered */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category = Appearance )
	FSlateBrush UndeterminedHoveredImage;
	FCheckBoxStyle& SetUndeterminedHoveredImage( const FSlateBrush& InUndeterminedHoveredImage ){ UndeterminedHoveredImage = InUndeterminedHoveredImage; return *this; }

	/* CheckBox appearance when CheckBox is undetermined and pressed */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category = Appearance )
	FSlateBrush UndeterminedPressedImage;
	FCheckBoxStyle& SetUndeterminedPressedImage( const FSlateBrush& InUndeterminedPressedImage ){ UndeterminedPressedImage = InUndeterminedPressedImage; return *this; }

	/** Padding */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin Padding;
	FCheckBoxStyle& SetPadding( const FMargin& InPadding ){ Padding = InPadding; return *this; }

	/** Background appearance */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category = Appearance )
	FSlateBrush BackgroundImage;
	FCheckBoxStyle& SetBackgroundImage( const FSlateBrush& InBackgroundImage ){ BackgroundImage = InBackgroundImage; return *this; }

	/** Background appearance when hovered */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category = Appearance )
	FSlateBrush BackgroundHoveredImage;
	FCheckBoxStyle& SetBackgroundHoveredImage( const FSlateBrush& InBackgroundHoveredImage ){ BackgroundHoveredImage = InBackgroundHoveredImage; return *this; }

	/** Background appearance when pressed */
	UPROPERTY( EditAnywhere, BlueprintReadWrite, Category = Appearance )
	FSlateBrush BackgroundPressedImage;
	FCheckBoxStyle& SetBackgroundPressedImage( const FSlateBrush& InBackgroundPressedImage ){ BackgroundPressedImage = InBackgroundPressedImage; return *this; }

	/** The normal unchecked foreground color */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateColor ForegroundColor;
	FCheckBoxStyle& SetForegroundColor(const FSlateColor& InForegroundColor) { ForegroundColor = InForegroundColor; return *this; }

	/** Foreground Color when hovered */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Appearance)
	FSlateColor HoveredForeground;
	FCheckBoxStyle& SetHoveredForegroundColor(const FSlateColor& InHoveredForeground) { HoveredForeground = InHoveredForeground; return *this; }

	/** Foreground Color when pressed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Appearance)
	FSlateColor PressedForeground;
	FCheckBoxStyle& SetPressedForegroundColor(const FSlateColor& InPressedForeground) { PressedForeground = InPressedForeground; return *this; }

	/** Foreground Color when checked */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Appearance)
	FSlateColor CheckedForeground;
	FCheckBoxStyle& SetCheckedForegroundColor(const FSlateColor& InCheckedForeground) { CheckedForeground = InCheckedForeground; return *this; }

	/** Foreground Color when checked and pressed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Appearance)
	FSlateColor CheckedHoveredForeground;
	FCheckBoxStyle& SetCheckedHoveredForegroundColor(const FSlateColor& InCheckedHoveredForeground) { CheckedHoveredForeground = InCheckedHoveredForeground; return *this; }

	/** Foreground Color when checked and pressed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Appearance)
	FSlateColor CheckedPressedForeground;
	FCheckBoxStyle& SetCheckedPressedForegroundColor(const FSlateColor& InCheckedPressedForeground) { CheckedPressedForeground = InCheckedPressedForeground; return *this; }

	/** Foreground Color when the check state is indeterminate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = Appearance)
	FSlateColor UndeterminedForeground;
	FCheckBoxStyle& SetUndeterminedForegroundColor(const FSlateColor& InUndeterminedForeground) { UndeterminedForeground = InUndeterminedForeground; return *this; }

	/** BorderBackgroundColor refers to the actual color and opacity of the supplied border image on toggle buttons */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateColor BorderBackgroundColor;
	FCheckBoxStyle& SetBorderBackgroundColor(const FSlateColor& InBorderBackgroundColor) { BorderBackgroundColor = InBorderBackgroundColor; return *this; }

	/**
	 * The sound the check box should play when checked
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sound, meta=( DisplayName="Checked Sound" ))
	FSlateSound CheckedSlateSound;
	FCheckBoxStyle& SetCheckedSound( const FSlateSound& InCheckedSound ){ CheckedSlateSound = InCheckedSound; return *this; }

	/**
	 * The sound the check box should play when unchecked
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sound, meta=( DisplayName="Unchecked Sound" ))
	FSlateSound UncheckedSlateSound;
	FCheckBoxStyle& SetUncheckedSound( const FSlateSound& InUncheckedSound ){ UncheckedSlateSound = InUncheckedSound; return *this; }

	/**
	 * The sound the check box should play when initially hovered over
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sound, meta=( DisplayName="Hovered Sound" ))
	FSlateSound HoveredSlateSound;
	FCheckBoxStyle& SetHoveredSound( const FSlateSound& InHoveredSound ){ HoveredSlateSound = InHoveredSound; return *this; }

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName CheckedSound_DEPRECATED;
	UPROPERTY()
	FName UncheckedSound_DEPRECATED;
	UPROPERTY()
	FName HoveredSound_DEPRECATED;

	/**
	 * Used to upgrade the deprecated FName sound properties into the new-style FSlateSound properties
	 */	
	SLATECORE_API void PostSerialize(const FArchive& Ar);
#endif

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		UncheckedImage.UnlinkColors();
		UncheckedHoveredImage.UnlinkColors();
		UncheckedPressedImage.UnlinkColors();
		CheckedImage.UnlinkColors();
		CheckedHoveredImage.UnlinkColors();
		CheckedPressedImage.UnlinkColors();
		UndeterminedImage.UnlinkColors();
		UndeterminedHoveredImage.UnlinkColors();
		UndeterminedPressedImage.UnlinkColors();
		BackgroundImage.UnlinkColors();
		BackgroundHoveredImage.UnlinkColors();
		BackgroundPressedImage.UnlinkColors();

		ForegroundColor.Unlink();
		HoveredForeground.Unlink();
		PressedForeground.Unlink();
		CheckedForeground.Unlink();
		CheckedHoveredForeground.Unlink();
		CheckedPressedForeground.Unlink();
		UndeterminedForeground.Unlink();
		BorderBackgroundColor.Unlink();
	}
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FCheckBoxStyle> : public TStructOpsTypeTraitsBase2<FCheckBoxStyle>
{
	enum 
	{
		WithPostSerialize = true,
	};
};
#endif

/**
 * Text transformation policy that can be applied to the text before displaying it.
 */
UENUM(BlueprintType)
enum class ETextTransformPolicy : uint8
{
	/** No transform, just use the given text as-is */
	None = 0,

	/** Convert the text to lowercase for display */
	ToLower,

	/** Convert the text to uppercase for display */
	ToUpper,
};

/**
 * Represents the appearance of an STextBlock
 */
USTRUCT(BlueprintType)
struct FTextBlockStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FTextBlockStyle();
	SLATECORE_API FTextBlockStyle(const FTextBlockStyle&);
	SLATECORE_API virtual ~FTextBlockStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FTextBlockStyle& GetDefault();

	/** Font family and size to be used when displaying this text. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateFontInfo Font;
	FTextBlockStyle& SetFont(const FSlateFontInfo& InFont) { Font = InFont; return *this; }
	FTextBlockStyle& SetFont(TSharedPtr<const FCompositeFont> InCompositeFont, const float InSize, const FName& InTypefaceFontName = NAME_None) { Font = FSlateFontInfo(InCompositeFont, InSize, InTypefaceFontName); return *this; }
	FTextBlockStyle& SetFont(const UObject* InFontObject, const float InSize, const FName& InTypefaceFontName = NAME_None) { Font = FSlateFontInfo(InFontObject, InSize, InTypefaceFontName); return *this; }
	
	UE_DEPRECATED(5.6, "Use SetFont using FCompositeFont instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	FTextBlockStyle& SetFont(const FName& InFontName, float InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FCompositeFont instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	FTextBlockStyle& SetFont(const FString& InFontName, float InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FCompositeFont instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	FTextBlockStyle& SetFont(const WIDECHAR* InFontName, float InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FCompositeFont instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	FTextBlockStyle& SetFont(const ANSICHAR* InFontName, float InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FCompositeFont instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	FTextBlockStyle& SetFontName(const FName& InFontName);
	UE_DEPRECATED(5.6, "Use SetFont using FCompositeFont instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	FTextBlockStyle& SetFontName(const FString& InFontName);
	UE_DEPRECATED(5.6, "Use SetFont using FCompositeFont instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	FTextBlockStyle& SetFontName(const WIDECHAR* InFontName);
	UE_DEPRECATED(5.6, "Use SetFont using FCompositeFont instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	FTextBlockStyle& SetFontName(const ANSICHAR* InFontName);

	FTextBlockStyle& SetFontSize(float InSize) { Font.Size = InSize; return *this; }
	FTextBlockStyle& SetTypefaceFontName(const FName& InTypefaceFontName) { Font.TypefaceFontName = InTypefaceFontName; return *this; }
	FTextBlockStyle& SetFontMaterial(UObject* InMaterial) { Font.FontMaterial = InMaterial; return *this; }
	FTextBlockStyle& SetFontOutlineMaterial(UObject* InMaterial) { Font.OutlineSettings.OutlineMaterial = InMaterial; return *this; }

	/** The color and opacity of this text */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance, meta=(DisplayName="Color"))
	FSlateColor ColorAndOpacity;
	FTextBlockStyle& SetColorAndOpacity(const FSlateColor& InColorAndOpacity) { ColorAndOpacity = InColorAndOpacity; return *this; }

	/** How much should the shadow be offset? An offset of 0 implies no shadow. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FDeprecateSlateVector2D ShadowOffset;
	FTextBlockStyle& SetShadowOffset(const UE::Slate::FDeprecateVector2DParameter& InShadowOffset) { ShadowOffset = InShadowOffset; return *this; }

	/** The color and opacity of the shadow */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FLinearColor ShadowColorAndOpacity;
	FTextBlockStyle& SetShadowColorAndOpacity(const FLinearColor& InShadowColorAndOpacity) { ShadowColorAndOpacity = InShadowColorAndOpacity; return *this; }

	/** The background color of selected text */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateColor SelectedBackgroundColor;
	FTextBlockStyle& SetSelectedBackgroundColor(const FSlateColor& InSelectedBackgroundColor) { SelectedBackgroundColor = InSelectedBackgroundColor; return *this; }

	/** The color of highlighted text */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance, AdvancedDisplay)
	FSlateColor HighlightColor;
	FTextBlockStyle& SetHighlightColor(const FSlateColor& InHighlightColor) { HighlightColor = InHighlightColor; return *this; }

	/** The shape of highlighted text */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance, AdvancedDisplay)
	FSlateBrush HighlightShape;
	FTextBlockStyle& SetHighlightShape( const FSlateBrush& InHighlightShape ){ HighlightShape = InHighlightShape; return *this; }

	/** The brush used to draw an strike through the text (if any) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance, AdvancedDisplay)
	FSlateBrush StrikeBrush;
	FTextBlockStyle& SetStrikeBrush( const FSlateBrush& InStrikeBrush){ StrikeBrush = InStrikeBrush; return *this; }

	/** The brush used to draw an underline under the text (if any) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance, AdvancedDisplay)
	FSlateBrush UnderlineBrush;
	FTextBlockStyle& SetUnderlineBrush( const FSlateBrush& InUnderlineBrush ){ UnderlineBrush = InUnderlineBrush; return *this; }

	/** The Text Transform Policy (defaults to None) */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance, AdvancedDisplay)
	ETextTransformPolicy TransformPolicy;
	FTextBlockStyle& SetTransformPolicy( const ETextTransformPolicy& InTransformPolicy ){ TransformPolicy = InTransformPolicy; return *this; }

	/** Determines what happens to text that is clipped and doesn't fit within the clip rect of a text widget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance, AdvancedDisplay)
	ETextOverflowPolicy OverflowPolicy;
	FTextBlockStyle& SetOverflowPolicy(const ETextOverflowPolicy& InOverflowPolicy) { OverflowPolicy = InOverflowPolicy; return *this; }

	/**
	 * Checks to see whether this style is identical to another.
	 */
	bool IsIdenticalTo(const FTextBlockStyle& InOther) const
	{
		return Font.IsIdenticalTo(InOther.Font)
			&& ColorAndOpacity == InOther.ColorAndOpacity
			&& ShadowOffset == InOther.ShadowOffset
			&& ShadowColorAndOpacity == InOther.ShadowColorAndOpacity
			&& SelectedBackgroundColor == InOther.SelectedBackgroundColor
			&& HighlightColor == InOther.HighlightColor
			&& HighlightShape == InOther.HighlightShape
			&& StrikeBrush == InOther.StrikeBrush
			&& UnderlineBrush == InOther.UnderlineBrush
			&& TransformPolicy == InOther.TransformPolicy
			&& OverflowPolicy == InOther.OverflowPolicy;
	}


	/** Helper struct to compare two text styles without constructing a temporary text style */
	struct CompareParams
	{
		CompareParams(const FTextBlockStyle& InStyleBase
					, const FSlateFontInfo& InFont
					, const FSlateColor& InColorAndOpacity
					, const FVector2f InShadowOffset
					, const FLinearColor& InShadowColorAndOpacity
					, const FSlateColor InHighlightColor
					, const FSlateBrush* InHighlightShape
					, const FSlateBrush* InStrikeBrush)
			: StyleBase(InStyleBase)
			, Font(InFont)
			, ColorAndOpacity(InColorAndOpacity)
			, ShadowOffset(InShadowOffset)
			, ShadowColorAndOpacity(InShadowColorAndOpacity)
			, HighlightColor(InHighlightColor)
			, HighlightShape(InHighlightShape)
			, StrikeBrush(InStrikeBrush)
		{}

		const FTextBlockStyle& StyleBase;
		const FSlateFontInfo& Font;
		const FSlateColor& ColorAndOpacity;
		const FVector2f ShadowOffset;
		const FLinearColor& ShadowColorAndOpacity;
		const FSlateColor HighlightColor;
		const FSlateBrush* HighlightShape;
		const FSlateBrush* StrikeBrush;
	};

	/**
	 * Checks to see whether this style is identical to another.
	 */
	bool IsIdenticalTo(const FTextBlockStyle::CompareParams& InNewStyleParams) const
	{
		const FSlateBrush& NewStrikeBrush = InNewStyleParams.StrikeBrush ? *InNewStyleParams.StrikeBrush : InNewStyleParams.StyleBase.StrikeBrush;
		const FSlateBrush& NewHighlightShape = InNewStyleParams.HighlightShape ? *InNewStyleParams.HighlightShape : InNewStyleParams.StyleBase.HighlightShape;

		return Font.IsIdenticalTo(InNewStyleParams.Font)
			&& ColorAndOpacity == InNewStyleParams.ColorAndOpacity
			&& ShadowOffset == InNewStyleParams.ShadowOffset
			&& ShadowColorAndOpacity == InNewStyleParams.ShadowColorAndOpacity
			&& SelectedBackgroundColor == InNewStyleParams.StyleBase.SelectedBackgroundColor
			&& HighlightColor == InNewStyleParams.HighlightColor
			&& HighlightShape == NewHighlightShape
			&& StrikeBrush == NewStrikeBrush
			&& UnderlineBrush == InNewStyleParams.StyleBase.UnderlineBrush
			&& TransformPolicy == InNewStyleParams.StyleBase.TransformPolicy
			&& OverflowPolicy == InNewStyleParams.StyleBase.OverflowPolicy;
	}

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		ColorAndOpacity.Unlink();
		SelectedBackgroundColor.Unlink();
		HighlightShape.UnlinkColors();
		StrikeBrush.UnlinkColors();
		UnderlineBrush.UnlinkColors();
	}
};

/**
 * Represents the appearance of an SButton
 */
USTRUCT(BlueprintType)
struct FButtonStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FButtonStyle();
	SLATECORE_API FButtonStyle(const FButtonStyle&);

	SLATECORE_API virtual ~FButtonStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FButtonStyle& GetDefault();

	/** Button appearance when the button is not hovered or pressed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush Normal;
	FButtonStyle& SetNormal( const FSlateBrush& InNormal ){ Normal = InNormal; return *this; }

	/** Button appearance when hovered */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush Hovered;
	FButtonStyle& SetHovered( const FSlateBrush& InHovered){ Hovered = InHovered; return *this; }

	/** Button appearance when pressed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush Pressed;
	FButtonStyle& SetPressed( const FSlateBrush& InPressed ){ Pressed = InPressed; return *this; }

	/** Button appearance when disabled, by default this is set to an invalid resource when that is the case default disabled drawing is used. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush Disabled;
	FButtonStyle& SetDisabled( const FSlateBrush& InDisabled ){ Disabled = InDisabled; return *this; }

	/** Foreground Color when the button is not hovered or pressed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Appearance)
	FSlateColor NormalForeground;
	FButtonStyle& SetNormalForeground( const FSlateColor& InNormalForeground ){ NormalForeground = InNormalForeground; return *this; }

	/** Foreground Color when hovered */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Appearance)
	FSlateColor HoveredForeground;
	FButtonStyle& SetHoveredForeground( const FSlateColor& InHoveredForeground){ HoveredForeground = InHoveredForeground; return *this; }

	/** Foreground Color when pressed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Appearance)
	FSlateColor PressedForeground;
	FButtonStyle& SetPressedForeground( const FSlateColor& InPressedForeground ){ PressedForeground = InPressedForeground; return *this; }

	/** Foreground Color when disabled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category=Appearance)
	FSlateColor DisabledForeground;
	FButtonStyle& SetDisabledForeground( const FSlateColor& InDisabledForeground ){ DisabledForeground = InDisabledForeground; return *this; }


	/**
	 * Padding that accounts for the border in the button's background image.
	 * When this is applied, the content of the button should appear flush
	 * with the button's border. Use this padding when the button is not pressed.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin NormalPadding;
	FButtonStyle& SetNormalPadding( const FMargin& InNormalPadding){ NormalPadding = InNormalPadding; return *this; }

	/**
	 * Same as NormalPadding but used when the button is pressed. Allows for moving the content to match
	 * any "movement" in the button's border image.
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin PressedPadding;
	FButtonStyle& SetPressedPadding( const FMargin& InPressedPadding){ PressedPadding = InPressedPadding; return *this; }

	/**
	 * The sound the button should play when pressed
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sound, meta=( DisplayName="Pressed Sound" ))
	FSlateSound PressedSlateSound;
	FButtonStyle& SetPressedSound( const FSlateSound& InPressedSound ){ PressedSlateSound = InPressedSound; return *this; }

	/**
	 * The sound the button should play when clicked
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sound, meta=( DisplayName="Clicked Sound" ))
	FSlateSound ClickedSlateSound;
	FButtonStyle& SetClickedSound( const FSlateSound& InClickedSound ){ ClickedSlateSound = InClickedSound; return *this; }

	/**
	 * The sound the button should play when initially hovered over
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sound, meta=( DisplayName="Hovered Sound" ))
	FSlateSound HoveredSlateSound;
	FButtonStyle& SetHoveredSound( const FSlateSound& InHoveredSound ){ HoveredSlateSound = InHoveredSound; return *this; }

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName PressedSound_DEPRECATED;
	UPROPERTY()
	FName HoveredSound_DEPRECATED;

	/**
	 * Used to upgrade the deprecated FName sound properties into the new-style FSlateSound properties
	 */	
	SLATECORE_API void PostSerialize(const FArchive& Ar);
#endif

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		Normal.UnlinkColors();
		Hovered.UnlinkColors();
		Pressed.UnlinkColors();
		Disabled.UnlinkColors();
	}
};

template<>
struct TStructOpsTypeTraits<FButtonStyle> : public TStructOpsTypeTraitsBase2<FButtonStyle>
{
	enum 
	{
#if WITH_EDITORONLY_DATA
		WithPostSerialize = true,
#endif
		WithCopy = true,
	};
};

/**
 * Represents the appearance of an SComboButton
 */
USTRUCT(BlueprintType)
struct FComboButtonStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FComboButtonStyle();
	SLATECORE_API FComboButtonStyle(const FComboButtonStyle&);
	SLATECORE_API virtual ~FComboButtonStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FComboButtonStyle& GetDefault();

	/**
	 * The style to use for our SButton.
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FButtonStyle ButtonStyle;
	FComboButtonStyle& SetButtonStyle( const FButtonStyle& InButtonStyle ){ ButtonStyle = InButtonStyle; return *this; }

	/**
	 * Image to use for the down arrow.
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush DownArrowImage;
	FComboButtonStyle& SetDownArrowImage( const FSlateBrush& InDownArrowImage ){ DownArrowImage = InDownArrowImage; return *this; }

	/**
	  * How much should the shadow be offset for the down arrow? 
	  * An offset of 0 implies no shadow. 
	  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FDeprecateSlateVector2D ShadowOffset;
	FComboButtonStyle& SetShadowOffset(const UE::Slate::FDeprecateVector2DParameter& InShadowOffset) { ShadowOffset = InShadowOffset; return *this; }

	/** 
	  * The color and opacity of the shadow for the down arrow.
	  * Only active if ShadowOffset is not (0,0).
	  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FLinearColor ShadowColorAndOpacity;
	FComboButtonStyle& SetShadowColorAndOpacity(const FLinearColor& InShadowColorAndOpacity) { ShadowColorAndOpacity = InShadowColorAndOpacity; return *this; }

	/**
	 * Brush to use to add a "menu border" around the drop-down content.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush MenuBorderBrush;
	FComboButtonStyle& SetMenuBorderBrush( const FSlateBrush& InMenuBorderBrush ){ MenuBorderBrush = InMenuBorderBrush; return *this; }

	/**
	 * Padding to use to add a "menu border" around the drop-down content.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin MenuBorderPadding;
	FComboButtonStyle& SetMenuBorderPadding( const FMargin& InMenuBorderPadding ){ MenuBorderPadding = InMenuBorderPadding; return *this; }

	/*
	 * Button Content Padding 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin ContentPadding;
	FComboButtonStyle& SetContentPadding( const FMargin& InContentPadding ) { ContentPadding = InContentPadding; return *this; }

	/*
	 * Dropdown arrow padding (if a dropdown arrow exists)
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin DownArrowPadding;
	FComboButtonStyle& SetDownArrowPadding(const FMargin& InDownArrowPadding) { DownArrowPadding = InDownArrowPadding; return *this; }

	/*
	 * Dropdown arrow vertical alignment
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	TEnumAsByte<EVerticalAlignment> DownArrowAlign;
	FComboButtonStyle& SetDownArrowAlignment(const EVerticalAlignment& InVAlign) { DownArrowAlign = InVAlign; return *this; }

	/**
	* Unlinks all colors in this style.
	* @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		ButtonStyle.UnlinkColors();
		DownArrowImage.UnlinkColors();
		MenuBorderBrush.UnlinkColors();
	}
};


/**
 * Represents the appearance of an SComboBox
 */
USTRUCT(BlueprintType)
struct FComboBoxStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FComboBoxStyle();
	SLATECORE_API FComboBoxStyle(const FComboBoxStyle&);
	SLATECORE_API virtual ~FComboBoxStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FComboBoxStyle& GetDefault();

	/**
	 * The style to use for our SComboButton
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance, meta=( ShowOnlyInnerProperties ))
	FComboButtonStyle ComboButtonStyle;
	FComboBoxStyle& SetComboButtonStyle( const FComboButtonStyle& InComboButtonStyle ){ ComboButtonStyle = InComboButtonStyle; return *this; }

	/**
	 * The sound the button should play when pressed
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sound, meta=( DisplayName="Pressed Sound" ))
	FSlateSound PressedSlateSound;
	FComboBoxStyle& SetPressedSound( const FSlateSound& InPressedSound ){ PressedSlateSound = InPressedSound; return *this; }

	/**
	 * The Sound to play when the selection is changed
	 */	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Sound, meta=( DisplayName="Selection Change Sound" ))
	FSlateSound SelectionChangeSlateSound;
	FComboBoxStyle& SetSelectionChangeSound( const FSlateSound& InSelectionChangeSound ){ SelectionChangeSlateSound = InSelectionChangeSound; return *this; }

	/*
	 * Button Content Padding 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin ContentPadding;
	FComboBoxStyle& SetContentPadding( const FMargin& InContentPadding ) { ContentPadding = InContentPadding; return *this; }

	/*
	 * Menu Row Padding 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin MenuRowPadding;
	FComboBoxStyle& SetMenuRowPadding( const FMargin& InMenuRowPadding ) { MenuRowPadding = InMenuRowPadding; return *this; }



#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FName PressedSound_DEPRECATED;
	UPROPERTY()
	FName SelectionChangeSound_DEPRECATED;

	/**
	 * Used to upgrade the deprecated FName sound properties into the new-style FSlateSound properties
	 */	
	SLATECORE_API void PostSerialize(const FArchive& Ar);
#endif

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		ComboButtonStyle.UnlinkColors();
	}

};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FComboBoxStyle> : public TStructOpsTypeTraitsBase2<FComboBoxStyle>
{
	enum 
	{
		WithPostSerialize = true,
	};
};
#endif

/**
 * Represents the appearance of an SHyperlink
 */
USTRUCT(BlueprintType)
struct FHyperlinkStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FHyperlinkStyle();
	SLATECORE_API FHyperlinkStyle(const FHyperlinkStyle&);

	SLATECORE_API virtual ~FHyperlinkStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FHyperlinkStyle& GetDefault();

	/** Underline style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FButtonStyle UnderlineStyle;
	FHyperlinkStyle& SetUnderlineStyle( const FButtonStyle& InUnderlineStyle ){ UnderlineStyle = InUnderlineStyle; return *this; }

	/** Text style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FTextBlockStyle TextStyle;
	FHyperlinkStyle& SetTextStyle( const FTextBlockStyle& InTextStyle ){ TextStyle = InTextStyle; return *this; }

	/** Padding */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin Padding;
	FHyperlinkStyle& SetPadding( const FMargin& InPadding ){ Padding = InPadding; return *this; }
};

/**
 * Represents the appearance of an SEditableText
 */
USTRUCT(BlueprintType)
struct FEditableTextStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FEditableTextStyle();
	SLATECORE_API FEditableTextStyle(const FEditableTextStyle&);
	SLATECORE_API virtual ~FEditableTextStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; }

	static SLATECORE_API const FEditableTextStyle& GetDefault();

	/** Font family and size to be used when displaying this text. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateFontInfo Font;
	FEditableTextStyle& SetFont(const FSlateFontInfo& InFont) { Font = InFont; return *this; }

	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	FEditableTextStyle& SetFont(const FName& InFontName, float InSize);

	FEditableTextStyle& SetFontMaterial(UObject* InMaterial) { Font.FontMaterial = InMaterial; return *this; }
	FEditableTextStyle& SetFontOutlineMaterial(UObject* InMaterial) { Font.OutlineSettings.OutlineMaterial = InMaterial; return *this; }

	/** The color and opacity of this text */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateColor ColorAndOpacity;
	FEditableTextStyle& SetColorAndOpacity(const FSlateColor& InColorAndOpacity) { ColorAndOpacity = InColorAndOpacity; return *this; }

	/** Background image for the selected text */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BackgroundImageSelected;
	FEditableTextStyle& SetBackgroundImageSelected( const FSlateBrush& InBackgroundImageSelected ){ BackgroundImageSelected = InBackgroundImageSelected; return *this; }

	/** Background image for the selected text */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BackgroundImageComposing;
	FEditableTextStyle& SetBackgroundImageComposing( const FSlateBrush& InBackgroundImageComposing ){ BackgroundImageComposing = InBackgroundImageComposing; return *this; }	

	/** Image brush used for the caret */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush CaretImage;
	FEditableTextStyle& SetCaretImage( const FSlateBrush& InCaretImage ){ CaretImage = InCaretImage; return *this; }

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		ColorAndOpacity.Unlink();
		BackgroundImageSelected.UnlinkColors();
		BackgroundImageComposing.UnlinkColors();
		CaretImage.UnlinkColors();
	}
};


/**
 * Represents the appearance of an SScrollBar
 */
USTRUCT(BlueprintType)
struct FScrollBarStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FScrollBarStyle();
	SLATECORE_API FScrollBarStyle(const FScrollBarStyle&);
	SLATECORE_API virtual ~FScrollBarStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FScrollBarStyle& GetDefault();

	/** Background image to use when the scrollbar is oriented horizontally */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush HorizontalBackgroundImage;
	FScrollBarStyle& SetHorizontalBackgroundImage( const FSlateBrush& InHorizontalBackgroundImage ){ HorizontalBackgroundImage = InHorizontalBackgroundImage; return *this; }

	/** Background image to use when the scrollbar is oriented vertically */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush VerticalBackgroundImage;
	FScrollBarStyle& SetVerticalBackgroundImage( const FSlateBrush& InVerticalBackgroundImage ){ VerticalBackgroundImage = InVerticalBackgroundImage; return *this; }

	/** The image to use to represent the track above the thumb when the scrollbar is oriented vertically */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush VerticalTopSlotImage;
	FScrollBarStyle& SetVerticalTopSlotImage(const FSlateBrush& Value){ VerticalTopSlotImage = Value; return *this; }

	/** The image to use to represent the track above the thumb when the scrollbar is oriented horizontally */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush HorizontalTopSlotImage;
	FScrollBarStyle& SetHorizontalTopSlotImage(const FSlateBrush& Value){ HorizontalTopSlotImage = Value; return *this; }

	/** The image to use to represent the track below the thumb when the scrollbar is oriented vertically */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush VerticalBottomSlotImage;
	FScrollBarStyle& SetVerticalBottomSlotImage(const FSlateBrush& Value){ VerticalBottomSlotImage = Value; return *this; }

	/** The image to use to represent the track below the thumb when the scrollbar is oriented horizontally */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush HorizontalBottomSlotImage;
	FScrollBarStyle& SetHorizontalBottomSlotImage(const FSlateBrush& Value){ HorizontalBottomSlotImage = Value; return *this; }

	/** Image to use when the scrollbar thumb is in its normal state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush NormalThumbImage;
	FScrollBarStyle& SetNormalThumbImage( const FSlateBrush& InNormalThumbImage ){ NormalThumbImage = InNormalThumbImage; return *this; }

	/** Image to use when the scrollbar thumb is in its hovered state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush HoveredThumbImage;
	FScrollBarStyle& SetHoveredThumbImage( const FSlateBrush& InHoveredThumbImage ){ HoveredThumbImage = InHoveredThumbImage; return *this; }

	/** Image to use when the scrollbar thumb is in its dragged state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush DraggedThumbImage;
	FScrollBarStyle& SetDraggedThumbImage( const FSlateBrush& InDraggedThumbImage ){ DraggedThumbImage = InDraggedThumbImage; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	float Thickness;
	FScrollBarStyle& SetThickness(float InThickness) { Thickness = InThickness; return *this; }

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		HorizontalBackgroundImage.UnlinkColors();
		VerticalBackgroundImage.UnlinkColors();
		VerticalTopSlotImage.UnlinkColors();
		HorizontalTopSlotImage.UnlinkColors();
		VerticalBottomSlotImage.UnlinkColors();
		HorizontalBottomSlotImage.UnlinkColors();
		NormalThumbImage.UnlinkColors();
		HoveredThumbImage.UnlinkColors();
		DraggedThumbImage.UnlinkColors();
	}
};


/**
 * Represents the appearance of an SEditableTextBox
 */
USTRUCT(BlueprintType)
struct FEditableTextBoxStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FEditableTextBoxStyle();
	SLATECORE_API FEditableTextBoxStyle(const FEditableTextBoxStyle&);
	SLATECORE_API FEditableTextBoxStyle& operator=(const FEditableTextBoxStyle&);
	SLATECORE_API virtual ~FEditableTextBoxStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FEditableTextBoxStyle& GetDefault();

	/** Border background image when the box is not hovered or focused */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BackgroundImageNormal;
	FEditableTextBoxStyle& SetBackgroundImageNormal( const FSlateBrush& InBackgroundImageNormal ){ BackgroundImageNormal = InBackgroundImageNormal; return *this; }

	/** Border background image when the box is hovered */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BackgroundImageHovered;
	FEditableTextBoxStyle& SetBackgroundImageHovered( const FSlateBrush& InBackgroundImageHovered ){ BackgroundImageHovered = InBackgroundImageHovered; return *this; }

	/** Border background image when the box is focused */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BackgroundImageFocused;
	FEditableTextBoxStyle& SetBackgroundImageFocused( const FSlateBrush& InBackgroundImageFocused ){ BackgroundImageFocused = InBackgroundImageFocused; return *this; }

	/** Border background image when the box is read-only */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BackgroundImageReadOnly;
	FEditableTextBoxStyle& SetBackgroundImageReadOnly( const FSlateBrush& InBackgroundImageReadOnly ){ BackgroundImageReadOnly = InBackgroundImageReadOnly; return *this; }

	/** Padding */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin Padding;
	FEditableTextBoxStyle& SetPadding( const FMargin& InPadding ){ Padding = InPadding; return *this; }

#if WITH_EDITORONLY_DATA
	/** Font family and size to be used when displaying this text. */
	UE_DEPRECATED(5.1, "Font has been deprecated as it was duplicated information already available elsewhere. Please use TextStyle.Font instead.")
	UPROPERTY()
	FSlateFontInfo Font_DEPRECATED;
#endif
	FEditableTextBoxStyle& SetFont(const FSlateFontInfo& InFont) { TextStyle.Font = InFont; return *this; }

	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	FEditableTextBoxStyle& SetFont(const FName& InFontName, float InSize);

	/** The style of the text block, which dictates the font, color, and shadow options. Style overrides all other properties! */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FTextBlockStyle TextStyle;
	FEditableTextBoxStyle& SetTextStyle(const FTextBlockStyle& InTextStyle) { TextStyle = InTextStyle; return *this; }

	/** The foreground color of text. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateColor ForegroundColor;
	FEditableTextBoxStyle& SetForegroundColor(const FSlateColor& InForegroundColor) { ForegroundColor = InForegroundColor; return *this; }

	/** The background color applied to the active background image */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateColor BackgroundColor;
	FEditableTextBoxStyle& SetBackgroundColor(const FSlateColor& InBackgroundColor) { BackgroundColor = InBackgroundColor; return *this; }

	/** The read-only foreground color of text in read-only mode. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateColor ReadOnlyForegroundColor;
	FEditableTextBoxStyle& SetReadOnlyForegroundColor(const FSlateColor& InReadOnlyForegroundColor) {ReadOnlyForegroundColor = InReadOnlyForegroundColor; return *this; }

	/** The foreground color of text when the edit box has keyboard focus. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor FocusedForegroundColor;
	FEditableTextBoxStyle& SetFocusedForegroundColor(const FSlateColor& InFocusedForegroundColor) {FocusedForegroundColor = InFocusedForegroundColor; return *this; }

	/** Padding around the horizontal scrollbar */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin HScrollBarPadding;
	FEditableTextBoxStyle& SetHScrollBarPadding( const FMargin& InPadding ){ HScrollBarPadding = InPadding; return *this; }

	/** Padding around the vertical scrollbar */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin VScrollBarPadding;
	FEditableTextBoxStyle& SetVScrollBarPadding( const FMargin& InPadding ){ VScrollBarPadding = InPadding; return *this; }

	/** Style used for the scrollbars */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FScrollBarStyle ScrollBarStyle;
	FEditableTextBoxStyle& SetScrollBarStyle( const FScrollBarStyle& InScrollBarStyle ){ ScrollBarStyle = InScrollBarStyle; return *this; }

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		BackgroundImageNormal.UnlinkColors();
		BackgroundImageHovered.UnlinkColors();
		BackgroundImageFocused.UnlinkColors();
		BackgroundImageReadOnly.UnlinkColors();
		ForegroundColor.Unlink();
		BackgroundColor.Unlink();
		ReadOnlyForegroundColor.Unlink();
		FocusedForegroundColor.Unlink();
		ScrollBarStyle.UnlinkColors();
	}
};


/**
 * Represents the appearance of an SInlineEditableTextBlock
 */
USTRUCT(BlueprintType)
struct FInlineEditableTextBlockStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FInlineEditableTextBlockStyle();

	SLATECORE_API virtual ~FInlineEditableTextBlockStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FInlineEditableTextBlockStyle& GetDefault();

	/** The style of the editable text box, which dictates the font, color, and shadow options. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FEditableTextBoxStyle EditableTextBoxStyle;
	FInlineEditableTextBlockStyle& SetEditableTextBoxStyle( const FEditableTextBoxStyle& InEditableTextBoxStyle ){ EditableTextBoxStyle = InEditableTextBoxStyle; return *this; }

	/** The style of the text block, which dictates the font, color, and shadow options. Style overrides all other properties! */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FTextBlockStyle TextStyle;
	FInlineEditableTextBlockStyle& SetTextStyle( const FTextBlockStyle& InTextStyle ){ TextStyle = InTextStyle; return *this; }
};


/**
 * Represents the appearance of an SProgressBar
 */
USTRUCT(BlueprintType)
struct FProgressBarStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FProgressBarStyle();

	SLATECORE_API virtual ~FProgressBarStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FProgressBarStyle& GetDefault();

	/** Background image to use for the progress bar */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BackgroundImage;
	FProgressBarStyle& SetBackgroundImage( const FSlateBrush& InBackgroundImage ){ BackgroundImage = InBackgroundImage; return *this; }

	/** Foreground image to use for the progress bar */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush FillImage;
	FProgressBarStyle& SetFillImage( const FSlateBrush& InFillImage ){ FillImage = InFillImage; return *this; }

	/** Image to use for marquee mode */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush MarqueeImage;
	FProgressBarStyle& SetMarqueeImage( const FSlateBrush& InMarqueeImage ){ MarqueeImage = InMarqueeImage; return *this; }

	/** Enables a simple animation on the fill image to give the appearance that progress has not stalled. Disable this if you have a custom material which animates itself. 
	 * This requires a pattern in your material or texture to give the appearance of movement.  A solid color will show no movement. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	bool EnableFillAnimation;
	FProgressBarStyle& SetEnableFillAnimation(bool InEnableFillAnimation) { EnableFillAnimation = InEnableFillAnimation; return *this; }

	/**
	* Unlinks all colors in this style.
	* @see FSlateColor::Unlink
	*/
	void UnlinkColors()
	{
		BackgroundImage.UnlinkColors();
		FillImage.UnlinkColors();
		MarqueeImage.UnlinkColors();
	}
};


/**
 * Represents the appearance of an SExpandableArea
 */
USTRUCT()
struct FExpandableAreaStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FExpandableAreaStyle();

	SLATECORE_API virtual ~FExpandableAreaStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FExpandableAreaStyle& GetDefault();

	/** Image to use when the area is collapsed */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush CollapsedImage;
	FExpandableAreaStyle& SetCollapsedImage( const FSlateBrush& InCollapsedImage ){ CollapsedImage = InCollapsedImage; return *this; }

	/** Image to use when the area is expanded */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush ExpandedImage;
	FExpandableAreaStyle& SetExpandedImage( const FSlateBrush& InExpandedImage ){ ExpandedImage = InExpandedImage; return *this; }

	/** How long the rollout animation lasts */
	UPROPERTY(EditAnywhere, Category = Appearance)
	float RolloutAnimationSeconds;
	FExpandableAreaStyle& SetRolloutAnimationSeconds(float InLengthSeconds) { RolloutAnimationSeconds = InLengthSeconds; return *this; }

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		CollapsedImage.UnlinkColors();
		ExpandedImage.UnlinkColors();
	}
};


/**
 * Represents the appearance of an SSearchBox
 */
USTRUCT()
struct FSearchBoxStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FSearchBoxStyle();

	SLATECORE_API virtual ~FSearchBoxStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FSearchBoxStyle& GetDefault();

	/** Style to use for the text box part of the search box */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FEditableTextBoxStyle TextBoxStyle;
	SLATECORE_API FSearchBoxStyle& SetTextBoxStyle( const FEditableTextBoxStyle& InTextBoxStyle );

	/** Font to use for the text box part of the search box when a search term is entered*/
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateFontInfo ActiveFontInfo;
	FSearchBoxStyle& SetActiveFont( const FSlateFontInfo& InFontInfo ){ ActiveFontInfo = InFontInfo; return *this; }

	/** Image to use for the search "up" arrow */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush UpArrowImage;
	FSearchBoxStyle& SetUpArrowImage( const FSlateBrush& InUpArrowImage ){ UpArrowImage = InUpArrowImage; return *this; }

	/** Image to use for the search "down" arrow */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush DownArrowImage;
	FSearchBoxStyle& SetDownArrowImage( const FSlateBrush& InDownArrowImage ){ DownArrowImage = InDownArrowImage; return *this; }

	/** Image to use for the search "glass" */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush GlassImage;
	FSearchBoxStyle& SetGlassImage( const FSlateBrush& InGlassImage ){ GlassImage = InGlassImage; return *this; }

	/** Image to use for the search "clear" button */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush ClearImage;
	FSearchBoxStyle& SetClearImage( const FSlateBrush& InClearImage ){ ClearImage = InClearImage; return *this; }

	/** Padding to use around the images */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FMargin ImagePadding;
	FSearchBoxStyle& SetImagePadding(const FMargin& InImagePadding){ ImagePadding = InImagePadding; return *this; }

	/** Size of the images (auto size if not set) */
	UPROPERTY(EditAnywhere, Category = Appearance)
	TOptional<FVector2D> ImageSizeOverride;
	FSearchBoxStyle& SetImageSizeOverride(const TOptional<FVector2D>& InImageSizeOverride){ ImageSizeOverride = InImageSizeOverride; return *this; }

	/** If true, buttons appear to the left of the search text */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage="Use LeftAlignSearchResultButtons and LeftAlignGlassImageAndClearButton instead"))
	bool bLeftAlignButtons_DEPRECATED;
	SLATECORE_API FSearchBoxStyle& SetLeftAlignButtons(bool bInLeftAlignButtons);

	/** If true, search result buttons appear to the left of the search text */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool bLeftAlignSearchResultButtons;
	FSearchBoxStyle& SetLeftAlignSearchResultButtons(bool bInLeftAlignSearchResultButtons){ bLeftAlignSearchResultButtons = bInLeftAlignSearchResultButtons; return *this; }
	
	/** If true, glass image and clear button appear to the left of the search text */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool bLeftAlignGlassImageAndClearButton;
	FSearchBoxStyle& SetLeftAlignGlassImageAndClearButton(bool bInLeftAlignGlassImageAndClearButton){ bLeftAlignGlassImageAndClearButton = bInLeftAlignGlassImageAndClearButton; return *this; }
};


/**
 * Represents the appearance of an SSlider
 */
USTRUCT(BlueprintType)
struct FSliderStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FSliderStyle();
	SLATECORE_API FSliderStyle(const FSliderStyle&);
	SLATECORE_API virtual ~FSliderStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FSliderStyle& GetDefault();

	/** Image to use when the slider bar is in its normal state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush NormalBarImage;
	FSliderStyle& SetNormalBarImage(const FSlateBrush& InNormalBarImage){ NormalBarImage = InNormalBarImage; return *this; }

	/** Image to use when the slider bar is in its hovered state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush HoveredBarImage;
	FSliderStyle& SetHoveredBarImage(const FSlateBrush& InHoveredBarImage){ HoveredBarImage = InHoveredBarImage; return *this; }

	/** Image to use when the slider bar is in its disabled state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush DisabledBarImage;
	FSliderStyle& SetDisabledBarImage(const FSlateBrush& InDisabledBarImage){ DisabledBarImage = InDisabledBarImage; return *this; }

	/** Image to use when the slider thumb is in its normal state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush NormalThumbImage;
	FSliderStyle& SetNormalThumbImage( const FSlateBrush& InNormalThumbImage ){ NormalThumbImage = InNormalThumbImage; return *this; }

	/** Image to use when the slider thumb is in its hovered state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush HoveredThumbImage;
	FSliderStyle& SetHoveredThumbImage( const FSlateBrush& InHoveredThumbImage ){ HoveredThumbImage = InHoveredThumbImage; return *this; }

	/** Image to use when the slider thumb is in its disabled state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush DisabledThumbImage;
	FSliderStyle& SetDisabledThumbImage( const FSlateBrush& InDisabledThumbImage ){ DisabledThumbImage = InDisabledThumbImage; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	float BarThickness;
	FSliderStyle& SetBarThickness(float InBarThickness) { BarThickness = InBarThickness; return *this; }

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		NormalBarImage.UnlinkColors();
		HoveredBarImage.UnlinkColors();
		DisabledBarImage.UnlinkColors();
		NormalThumbImage.UnlinkColors();
		HoveredThumbImage.UnlinkColors();
		DisabledThumbImage.UnlinkColors();
	}
};


/**
 * Represents the appearance of an SVolumeControl
 */
USTRUCT()
struct FVolumeControlStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FVolumeControlStyle();

	SLATECORE_API virtual ~FVolumeControlStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FVolumeControlStyle& GetDefault();

	/** The style of the volume control slider */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSliderStyle SliderStyle;
	FVolumeControlStyle& SetSliderStyle( const FSliderStyle& InSliderStyle ){ SliderStyle = InSliderStyle; return *this; }

	/** Image to use when the volume is set to high */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush HighVolumeImage;
	FVolumeControlStyle& SetHighVolumeImage( const FSlateBrush& InHighVolumeImage ){ HighVolumeImage = InHighVolumeImage; return *this; }

	/** Image to use when the volume is set to mid-range */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush MidVolumeImage;
	FVolumeControlStyle& SetMidVolumeImage( const FSlateBrush& InMidVolumeImage ){ MidVolumeImage = InMidVolumeImage; return *this; }

	/** Image to use when the volume is set to low */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush LowVolumeImage;
	FVolumeControlStyle& SetLowVolumeImage( const FSlateBrush& InLowVolumeImage ){ LowVolumeImage = InLowVolumeImage; return *this; }

	/** Image to use when the volume is set to off */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush NoVolumeImage;
	FVolumeControlStyle& SetNoVolumeImage( const FSlateBrush& InNoVolumeImage ){ NoVolumeImage = InNoVolumeImage; return *this; }

	/** Image to use when the volume is muted */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush MutedImage;
	FVolumeControlStyle& SetMutedImage( const FSlateBrush& InMutedImage ){ MutedImage = InMutedImage; return *this; }
};

/**
 * Represents the appearance of an inline image used by rich text
 */
USTRUCT()
struct FInlineTextImageStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FInlineTextImageStyle();
	SLATECORE_API FInlineTextImageStyle(const FInlineTextImageStyle&);
	SLATECORE_API virtual ~FInlineTextImageStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FInlineTextImageStyle& GetDefault();

	/** Image to use when the slider thumb is in its normal state */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush Image;
	FInlineTextImageStyle& SetImage( const FSlateBrush& InImage ){ Image = InImage; return *this; }

	/** The offset from the bottom of the image height to the baseline. */
	UPROPERTY(EditAnywhere, Category=Appearance)
	int16 Baseline;
	FInlineTextImageStyle& SetBaseline( int16 InBaseline ){ Baseline = InBaseline; return *this; }
};

/**
 * Represents the appearance of an SSpinBox
 */
USTRUCT(BlueprintType)
struct FSpinBoxStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FSpinBoxStyle();
	SLATECORE_API FSpinBoxStyle(const FSpinBoxStyle&);
	SLATECORE_API virtual ~FSpinBoxStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FSpinBoxStyle& GetDefault();

	/** Brush used to draw the background of the spinbox */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BackgroundBrush;
	FSpinBoxStyle& SetBackgroundBrush( const FSlateBrush& InBackgroundBrush ){ BackgroundBrush = InBackgroundBrush; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush ActiveBackgroundBrush;
	FSpinBoxStyle& SetActiveBackgroundBrush(const FSlateBrush& InBackgroundBrush) { ActiveBackgroundBrush = InBackgroundBrush; return *this; }

	/** Brush used to draw the background of the spinbox when it's hovered over */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush HoveredBackgroundBrush;
	FSpinBoxStyle& SetHoveredBackgroundBrush( const FSlateBrush& InHoveredBackgroundBrush ){ HoveredBackgroundBrush = InHoveredBackgroundBrush; return *this; }

	/** Brush used to fill the spinbox when it's active */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush ActiveFillBrush;
	FSpinBoxStyle& SetActiveFillBrush( const FSlateBrush& InActiveFillBrush ){ ActiveFillBrush = InActiveFillBrush; return *this; }

	/** Brush used to fill the spinbox when it's hovered and not active */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush HoveredFillBrush;
	FSpinBoxStyle& SetHoveredFillBrush(const FSlateBrush& InHoveredBrush) { HoveredFillBrush = InHoveredBrush; return *this; }

	/** Brush used to fill the spinbox when it's inactive */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush InactiveFillBrush;
	FSpinBoxStyle& SetInactiveFillBrush( const FSlateBrush& InInactiveFillBrush ){ InactiveFillBrush = InInactiveFillBrush; return *this; }

	/** Image used to draw the spinbox arrows */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush ArrowsImage;
	FSpinBoxStyle& SetArrowsImage( const FSlateBrush& InArrowsImage ){ ArrowsImage = InArrowsImage; return *this; }

	/** Color used to draw the spinbox foreground elements */
	UPROPERTY()
	FSlateColor ForegroundColor;
	FSpinBoxStyle& SetForegroundColor( const FSlateColor& InForegroundColor ){ ForegroundColor = InForegroundColor; return *this; }

	/** Padding to add around the spinbox and its text */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin TextPadding;
	FSpinBoxStyle& SetTextPadding( const FMargin& InTextPadding ){ TextPadding = InTextPadding; return *this; }

	/** Padding between the background brush and the fill brush */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin InsetPadding;
	FSpinBoxStyle& SetInsetPadding(const FMargin& InInsetPadding) { InsetPadding = InInsetPadding; return *this; }

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		BackgroundBrush.UnlinkColors();
		HoveredBackgroundBrush.UnlinkColors();
		ActiveBackgroundBrush.UnlinkColors();
		ActiveFillBrush.UnlinkColors();
		HoveredFillBrush.UnlinkColors();
		InactiveFillBrush.UnlinkColors();
		ArrowsImage.UnlinkColors();
		ForegroundColor.Unlink();
	}
};


/**
 * Represents the appearance of an SSplitter
 */
USTRUCT(BlueprintType)
struct FSplitterStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FSplitterStyle();

	SLATECORE_API virtual ~FSplitterStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FSplitterStyle& GetDefault();

	/** Brush used to draw the handle in its normal state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush HandleNormalBrush;
	FSplitterStyle& SetHandleNormalBrush( const FSlateBrush& InHandleNormalBrush ){ HandleNormalBrush = InHandleNormalBrush; return *this; }

	/** Brush used to draw the handle in its highlight state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush HandleHighlightBrush;
	FSplitterStyle& SetHandleHighlightBrush( const FSlateBrush& InHandleHighlightBrush ){ HandleHighlightBrush = InHandleHighlightBrush; return *this; }
};

/**
 * Represents the appearance of an STableView
 */

USTRUCT(BlueprintType)
struct FTableViewStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FTableViewStyle();

	SLATECORE_API virtual ~FTableViewStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FTableViewStyle& GetDefault();

	/** Brush used when a selected row is active */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BackgroundBrush;
	FTableViewStyle& SetBackgroundBrush( const FSlateBrush& InBackgroundBrush ){ BackgroundBrush = InBackgroundBrush; return *this; }

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		BackgroundBrush.UnlinkColors();
	}
};


/**
 * Represents the appearance of an STableRow
 */
USTRUCT(BlueprintType)
struct FTableRowStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FTableRowStyle();

	SLATECORE_API virtual ~FTableRowStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FTableRowStyle& GetDefault();

	/** Brush used as a selector when a row is focused */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush SelectorFocusedBrush;
	FTableRowStyle& SetSelectorFocusedBrush( const FSlateBrush& InSelectorFocusedBrush ){ SelectorFocusedBrush = InSelectorFocusedBrush; return *this; }

	/** Brush used when a selected row is active and hovered */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush ActiveHoveredBrush;
	FTableRowStyle& SetActiveHoveredBrush( const FSlateBrush& InActiveHoveredBrush ){ ActiveHoveredBrush = InActiveHoveredBrush; return *this; }

	/** Brush used when a selected row is active */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush ActiveBrush;
	FTableRowStyle& SetActiveBrush( const FSlateBrush& InActiveBrush ){ ActiveBrush = InActiveBrush; return *this; }

	/** Brush used when a selected row is inactive and hovered */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush InactiveHoveredBrush;
	FTableRowStyle& SetInactiveHoveredBrush( const FSlateBrush& InInactiveHoveredBrush ){ InactiveHoveredBrush = InInactiveHoveredBrush; return *this; }

	/** Brush used when a selected row is inactive */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush InactiveBrush;
	FTableRowStyle& SetInactiveBrush( const FSlateBrush& InInactiveBrush ){ InactiveBrush = InInactiveBrush; return *this; }

	/** If using parent row brushes */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	bool bUseParentRowBrush;
	FTableRowStyle& SetUseParentRowBrush(bool InUseParentRowBrush) { bUseParentRowBrush = InUseParentRowBrush; return *this; }

	/** Brush used for the top parent row  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush ParentRowBackgroundBrush;
	FTableRowStyle& SetParentRowBackgroundBrush( const FSlateBrush& InParentRowBackgroundBrush ){ ParentRowBackgroundBrush = InParentRowBackgroundBrush; return *this; }

	/** Brush used for the top parent row and row is hovered */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush ParentRowBackgroundHoveredBrush;
	FTableRowStyle& SetParentRowBackgroundHoveredBrush( const FSlateBrush& InParentRowBackgroundHoveredBrush ){ ParentRowBackgroundHoveredBrush = InParentRowBackgroundHoveredBrush; return *this; }

	/** Brush used when an even row is hovered */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush EvenRowBackgroundHoveredBrush;
	FTableRowStyle& SetEvenRowBackgroundHoveredBrush( const FSlateBrush& InEvenRowBackgroundHoveredBrush ){ EvenRowBackgroundHoveredBrush = InEvenRowBackgroundHoveredBrush; return *this; }

	/** Brush used when an even row is in its normal state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush EvenRowBackgroundBrush;
	FTableRowStyle& SetEvenRowBackgroundBrush( const FSlateBrush& InEvenRowBackgroundBrush ){ EvenRowBackgroundBrush = InEvenRowBackgroundBrush; return *this; }

	/** Brush used when an odd row is hovered */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush OddRowBackgroundHoveredBrush;
	FTableRowStyle& SetOddRowBackgroundHoveredBrush( const FSlateBrush& InOddRowBackgroundHoveredBrush ){ OddRowBackgroundHoveredBrush = InOddRowBackgroundHoveredBrush; return *this; }

	/** Brush to used when an odd row is in its normal state */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush OddRowBackgroundBrush;
	FTableRowStyle& SetOddRowBackgroundBrush( const FSlateBrush& InOddRowBackgroundBrush ){ OddRowBackgroundBrush = InOddRowBackgroundBrush; return *this; }

	/** Text color used for all rows */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateColor TextColor;
	FTableRowStyle& SetTextColor( const FSlateColor& InTextColor ){ TextColor = InTextColor; return *this; }

	/** Text color used for the selected row */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateColor SelectedTextColor;
	FTableRowStyle& SetSelectedTextColor( const FSlateColor& InSelectedTextColor ){ SelectedTextColor = InSelectedTextColor; return *this; }

	/** Brush used to provide feedback that a user can drop above the hovered row. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush DropIndicator_Above;
	FTableRowStyle& SetDropIndicator_Above(const FSlateBrush& InValue){ DropIndicator_Above = InValue; return *this; }

	/** Brush used to provide feedback that a user can drop onto the hovered row. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush DropIndicator_Onto;
	FTableRowStyle& SetDropIndicator_Onto(const FSlateBrush& InValue){ DropIndicator_Onto = InValue; return *this; }

	/** Brush used to provide feedback that a user can drop below the hovered row. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush DropIndicator_Below;
	FTableRowStyle& SetDropIndicator_Below(const FSlateBrush& InValue){ DropIndicator_Below = InValue; return *this; }
	
	/** Brush used when a highlighted row is active */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush ActiveHighlightedBrush;
	FTableRowStyle& SetActiveHighlightedBrush( const FSlateBrush& InActiveHighlightedBrush ){ ActiveHighlightedBrush = InActiveHighlightedBrush; return *this; }
	
	/** Brush used when a highlighted row is inactive and hovered */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush InactiveHighlightedBrush;
	FTableRowStyle& SetInactiveHighlightedBrush( const FSlateBrush& InInactiveHighlightedBrush){ InactiveHighlightedBrush = InInactiveHighlightedBrush; return *this; }

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		SelectorFocusedBrush.UnlinkColors();
		ActiveHoveredBrush.UnlinkColors();
		ActiveBrush.UnlinkColors();
		InactiveHoveredBrush.UnlinkColors();
		InactiveBrush.UnlinkColors();
		ParentRowBackgroundBrush.UnlinkColors();
		ParentRowBackgroundHoveredBrush.UnlinkColors();
		EvenRowBackgroundHoveredBrush.UnlinkColors();
		EvenRowBackgroundBrush.UnlinkColors();
		OddRowBackgroundHoveredBrush.UnlinkColors();
		OddRowBackgroundBrush.UnlinkColors();
		TextColor.Unlink();
		SelectedTextColor.Unlink();
		DropIndicator_Above.UnlinkColors();
		DropIndicator_Onto.UnlinkColors();
		DropIndicator_Below.UnlinkColors();
		ActiveHighlightedBrush.UnlinkColors();
		InactiveHighlightedBrush.UnlinkColors();
	}
};


/**
 * Represents the appearance of an STableColumnHeader
 */
USTRUCT()
struct FTableColumnHeaderStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FTableColumnHeaderStyle();

	SLATECORE_API virtual ~FTableColumnHeaderStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FTableColumnHeaderStyle& GetDefault();

	/** Image used when a column is primarily sorted in ascending order */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush SortPrimaryAscendingImage;
	FTableColumnHeaderStyle& SetSortPrimaryAscendingImage(const FSlateBrush& InSortPrimaryAscendingImage){ SortPrimaryAscendingImage = InSortPrimaryAscendingImage; return *this; }

	/** Image used when a column is primarily sorted in descending order */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush SortPrimaryDescendingImage;
	FTableColumnHeaderStyle& SetSortPrimaryDescendingImage(const FSlateBrush& InSortPrimaryDescendingImage){ SortPrimaryDescendingImage = InSortPrimaryDescendingImage; return *this; }

	/** Image used when a column is secondarily sorted in ascending order */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SortSecondaryAscendingImage;
	FTableColumnHeaderStyle& SetSortSecondaryAscendingImage(const FSlateBrush& InSortSecondaryAscendingImage){ SortSecondaryAscendingImage = InSortSecondaryAscendingImage; return *this; }

	/** Image used when a column is secondarily sorted in descending order */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush SortSecondaryDescendingImage;
	FTableColumnHeaderStyle& SetSortSecondaryDescendingImage(const FSlateBrush& InSortSecondaryDescendingImage){ SortSecondaryDescendingImage = InSortSecondaryDescendingImage; return *this; }

	/** Brush used to draw the header in its normal state */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush NormalBrush;
	FTableColumnHeaderStyle& SetNormalBrush( const FSlateBrush& InNormalBrush ){ NormalBrush = InNormalBrush; return *this; }

	/** Brush used to draw the header in its hovered state */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush HoveredBrush;
	FTableColumnHeaderStyle& SetHoveredBrush( const FSlateBrush& InHoveredBrush ){ HoveredBrush = InHoveredBrush; return *this; }

	/** Image used for the menu drop-down button */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush MenuDropdownImage;
	FTableColumnHeaderStyle& SetMenuDropdownImage( const FSlateBrush& InMenuDropdownImage ){ MenuDropdownImage = InMenuDropdownImage; return *this; }

	/** Brush used to draw the menu drop-down border in its normal state */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush MenuDropdownNormalBorderBrush;
	FTableColumnHeaderStyle& SetMenuDropdownNormalBorderBrush( const FSlateBrush& InMenuDropdownNormalBorderBrush ){ MenuDropdownNormalBorderBrush = InMenuDropdownNormalBorderBrush; return *this; }

	/** Brush used to draw the menu drop-down border in its hovered state */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush MenuDropdownHoveredBorderBrush;
	FTableColumnHeaderStyle& SetMenuDropdownHoveredBorderBrush( const FSlateBrush& InMenuDropdownHoveredBorderBrush ){ MenuDropdownHoveredBorderBrush = InMenuDropdownHoveredBorderBrush; return *this; }
};


/**
 * Represents the appearance of an SHeaderRow
 */
USTRUCT()
struct FHeaderRowStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FHeaderRowStyle();

	SLATECORE_API virtual ~FHeaderRowStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FHeaderRowStyle& GetDefault();

	/** Style of the normal header row columns */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FTableColumnHeaderStyle ColumnStyle;
	FHeaderRowStyle& SetColumnStyle( const FTableColumnHeaderStyle& InColumnStyle ){ ColumnStyle = InColumnStyle; return *this; }

	/** Style of the last header row column */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FTableColumnHeaderStyle LastColumnStyle;
	FHeaderRowStyle& SetLastColumnStyle( const FTableColumnHeaderStyle& InLastColumnStyle ){ LastColumnStyle = InLastColumnStyle; return *this; }

	/** Style of the splitter used between the columns */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSplitterStyle ColumnSplitterStyle;
	FHeaderRowStyle& SetColumnSplitterStyle( const FSplitterStyle& InColumnSplitterStyle ){ ColumnSplitterStyle = InColumnSplitterStyle; return *this; }

	/** Size of the splitter used between the columns */
	UPROPERTY(EditAnywhere, Category=Appearance)
	float SplitterHandleSize;
	FHeaderRowStyle& SetSplitterHandleSize( const float  InSplitterHandleSize){ SplitterHandleSize = InSplitterHandleSize; return *this; }

	/** Brush used to draw the header row background */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush BackgroundBrush;
	FHeaderRowStyle& SetBackgroundBrush( const FSlateBrush& InBackgroundBrush ){ BackgroundBrush = InBackgroundBrush; return *this; }

	/** Color used to draw the header row foreground */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateColor ForegroundColor;
	FHeaderRowStyle& SetForegroundColor( const FSlateColor& InForegroundColor ){ ForegroundColor = InForegroundColor; return *this; }

	/** Brush used to draw the splitter between the header and the contents below it*/
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush HorizontalSeparatorBrush;
	FHeaderRowStyle& SetHorizontalSeparatorBrush(const FSlateBrush& InHorizontalSeparatorBrush) { HorizontalSeparatorBrush = InHorizontalSeparatorBrush; return *this; }

	UPROPERTY(EditAnywhere, Category=Appearance)
	float HorizontalSeparatorThickness;
	FHeaderRowStyle& SetHorizontalSeparatorThickness(const float InHorizontalSeparatorThickness) { HorizontalSeparatorThickness = InHorizontalSeparatorThickness; return *this; }


};


/**
 * Represents the appearance of an SDockTab
 */
USTRUCT()
struct FDockTabStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FDockTabStyle();

	SLATECORE_API virtual ~FDockTabStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FDockTabStyle& GetDefault();

	/** Style used for the close button */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FButtonStyle CloseButtonStyle;
	FDockTabStyle& SetCloseButtonStyle( const FButtonStyle& InCloseButtonStyle ){ CloseButtonStyle = InCloseButtonStyle; return *this; }

	/** Brush used when this tab is in its normal state */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush NormalBrush;
	FDockTabStyle& SetNormalBrush( const FSlateBrush& InNormalBrush ){ NormalBrush = InNormalBrush; return *this; }

	/** Brush used to overlay a given color onto this tab */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush ColorOverlayTabBrush;
	FDockTabStyle& SetColorOverlayTabBrush( const FSlateBrush& InColorOverlayBrush ){ ColorOverlayTabBrush = InColorOverlayBrush; return *this; }

	/** Brush used to overlay a given color onto this tab */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FSlateBrush ColorOverlayIconBrush;
	FDockTabStyle& SetColorOverlayIconBrush(const FSlateBrush& InColorOverlayBrush) { ColorOverlayIconBrush = InColorOverlayBrush; return *this; }

	/** Brush used when this tab is in the foreground */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush ForegroundBrush;
	FDockTabStyle& SetForegroundBrush( const FSlateBrush& InForegroundBrush ){ ForegroundBrush = InForegroundBrush; return *this; }

	/** Brush used when this tab is hovered over */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush HoveredBrush;
	FDockTabStyle& SetHoveredBrush( const FSlateBrush& InHoveredBrush ){ HoveredBrush = InHoveredBrush; return *this; }

	/** Brush used by the SDockingTabStack to draw the content associated with this tab; Documents, Apps, and Tool Panels have different backgrounds */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush ContentAreaBrush;
	FDockTabStyle& SetContentAreaBrush( const FSlateBrush& InContentAreaBrush ){ ContentAreaBrush = InContentAreaBrush; return *this; }

	/** Brush used by the SDockingTabStack to draw the content associated with this tab; Documents, Apps, and Tool Panels have different backgrounds */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateBrush TabWellBrush;
	FDockTabStyle& SetTabWellBrush( const FSlateBrush& InTabWellBrush ){ TabWellBrush = InTabWellBrush; return *this; }

	/** Tab Text Style */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FTextBlockStyle TabTextStyle;
	FDockTabStyle& SetTabTextStyle( const FTextBlockStyle& InTabTextStyle ){ TabTextStyle = InTabTextStyle; return *this; }

	/** Padding used around this tab */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FMargin TabPadding;
	FDockTabStyle& SetTabPadding( const FMargin& InTabPadding ){ TabPadding = InTabPadding; return *this; }

	/** Icon size for icons in this tab */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FDeprecateSlateVector2D IconSize;
	FDockTabStyle& SetIconSize(const UE::Slate::FDeprecateVector2DParameter& InIconSize) { IconSize = InIconSize; return *this; }

	/** The width that this tab will overlap with side-by-side tabs */
	UPROPERTY(EditAnywhere, Category=Appearance)
	float OverlapWidth;
	FDockTabStyle& SetOverlapWidth( const float InOverlapWidth ){ OverlapWidth = InOverlapWidth; return *this; }
		
	/** Color used when flashing this tab */
	UPROPERTY(EditAnywhere, Category=Appearance)
	FSlateColor FlashColor;
	FDockTabStyle& SetFlashColor( const FSlateColor& InFlashColor ){ FlashColor = InFlashColor; return *this; }

	/** Foreground Color when the tab is not hovered, pressed, active or in the foreground */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Appearance)
	FSlateColor NormalForegroundColor;
	FDockTabStyle& SetNormalForegroundColor( const FSlateColor& InNormalForegroundColor ){ NormalForegroundColor = InNormalForegroundColor; return *this; }

	/** Foreground Color when hovered */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Appearance)
	FSlateColor HoveredForegroundColor;
	FDockTabStyle& SetHoveredForegroundColor( const FSlateColor& InHoveredForegroundColor){ HoveredForegroundColor = InHoveredForegroundColor; return *this; }

	/** Foreground Color when Active */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Appearance)
	FSlateColor ActiveForegroundColor;
	FDockTabStyle& SetActiveForegroundColor( const FSlateColor& InActiveForegroundColor ){ ActiveForegroundColor = InActiveForegroundColor; return *this; }

	/** Foreground Color when this tab is the Foreground tab */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Appearance)
	FSlateColor ForegroundForegroundColor;
	FDockTabStyle& SetForegroundForegroundColor( const FSlateColor& InForegroundForegroundColor ){ ForegroundForegroundColor = InForegroundForegroundColor; return *this; }

	/** The padding applied to the border around the tab icon */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Appearance)
	float IconBorderPadding;
	FDockTabStyle& SetIconBorderPadding(const float InIconBorderPadding) { IconBorderPadding = InIconBorderPadding; return *this; }

};


/**
 * Represents the appearance of an SScrollBox
 */
USTRUCT(BlueprintType)
struct FScrollBoxStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FScrollBoxStyle();

	SLATECORE_API virtual ~FScrollBoxStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FScrollBoxStyle& GetDefault();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	float BarThickness;
	FScrollBoxStyle& SetBarThickness(float InBarThickness) { BarThickness = InBarThickness; return *this; }

	/** Brush used to draw the top shadow of a scrollbox */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush TopShadowBrush;
	FScrollBoxStyle& SetTopShadowBrush( const FSlateBrush& InTopShadowBrush ){ TopShadowBrush = InTopShadowBrush; return *this; }

	/** Brush used to draw the bottom shadow of a scrollbox */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BottomShadowBrush;
	FScrollBoxStyle& SetBottomShadowBrush( const FSlateBrush& InBottomShadowBrush ){ BottomShadowBrush = InBottomShadowBrush; return *this; }

	/** Brush used to draw the left shadow of a scrollbox */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush LeftShadowBrush;
	FScrollBoxStyle& SetLeftShadowBrush(const FSlateBrush& InLeftShadowBrush)
	{
		LeftShadowBrush = InLeftShadowBrush;
		return *this;
	}

	/** Brush used to draw the right shadow of a scrollbox */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush RightShadowBrush;
	FScrollBoxStyle& SetRightShadowBrush(const FSlateBrush& InRightShadowBrush)
	{
		RightShadowBrush = InRightShadowBrush;
		return *this;
	}

	/** Padding scroll panel that presents the scrolled content */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin HorizontalScrolledContentPadding = FMargin(0.0f, 0.0f, 1.0f, 0.0f);
	FScrollBoxStyle& SetHorizontalScrolledContentPadding(const FMargin& InPadding)
	{
		HorizontalScrolledContentPadding = InPadding;
		return *this;
	}

	/** Padding scroll panel that presents the scrolled content */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin VerticalScrolledContentPadding = FMargin(0.0f, 0.0f, 0.0f, 1.0f);
	FScrollBoxStyle& SetVerticalScrolledContentPadding(const FMargin& InPadding)
	{
		VerticalScrolledContentPadding = InPadding;
		return *this;
	}

	/**
	 * Unlinks all colors in this style.
	 * @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		TopShadowBrush.UnlinkColors();
		BottomShadowBrush.UnlinkColors();
		LeftShadowBrush.UnlinkColors();
		RightShadowBrush.UnlinkColors();
	}
};


/**
* Represents the appearance of an FScrollBorderStyle
*/
USTRUCT(BlueprintType)
struct FScrollBorderStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FScrollBorderStyle();

	SLATECORE_API virtual ~FScrollBorderStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FScrollBorderStyle& GetDefault();

	/** Brush used to draw the top shadow of a scrollborder */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush TopShadowBrush;
	FScrollBorderStyle& SetTopShadowBrush( const FSlateBrush& InTopShadowBrush ){ TopShadowBrush = InTopShadowBrush; return *this; }

	/** Brush used to draw the bottom shadow of a scrollborder */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush BottomShadowBrush;
	FScrollBorderStyle& SetBottomShadowBrush( const FSlateBrush& InBottomShadowBrush ){ BottomShadowBrush = InBottomShadowBrush; return *this; }
};


/**
 * Represents the appearance of an SWindow
 */
USTRUCT(BlueprintType)
struct FWindowStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	SLATECORE_API FWindowStyle();
	SLATECORE_API FWindowStyle(const FWindowStyle&);
	SLATECORE_API virtual ~FWindowStyle();

	SLATECORE_API virtual void GetResources( TArray< const FSlateBrush* >& OutBrushes ) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FWindowStyle& GetDefault();

	static SLATECORE_API const FWindowStyle& GetBorderless();

	/** Style used to draw the window minimize button */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FButtonStyle MinimizeButtonStyle;
	FWindowStyle& SetMinimizeButtonStyle( const FButtonStyle& InMinimizeButtonStyle ){ MinimizeButtonStyle = InMinimizeButtonStyle; return *this; }

	/** Style used to draw the window maximize button */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FButtonStyle MaximizeButtonStyle;
	FWindowStyle& SetMaximizeButtonStyle( const FButtonStyle& InMaximizeButtonStyle ){ MaximizeButtonStyle = InMaximizeButtonStyle; return *this; }

	/** Style used to draw the window restore button */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FButtonStyle RestoreButtonStyle;
	FWindowStyle& SetRestoreButtonStyle( const FButtonStyle& InRestoreButtonStyle ){ RestoreButtonStyle = InRestoreButtonStyle; return *this; }

	/** Style used to draw the enter fullscreen button */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
    FButtonStyle EnterFullscreenButtonStyle;
    FWindowStyle& SetEnterFullscreenButtonStyle( const FButtonStyle& InEnterFullscreenButtonStyle ){ EnterFullscreenButtonStyle = InEnterFullscreenButtonStyle; return *this; }

	/** Style used to draw the exit fullscreen button */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FButtonStyle ExitFullscreenButtonStyle;
	FWindowStyle& SetExitFullscreenButtonStyle( const FButtonStyle& InExitFullscreenButtonStyle ){ ExitFullscreenButtonStyle = InExitFullscreenButtonStyle; return *this; }
	
	/** Style used to draw the window close button */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FButtonStyle CloseButtonStyle;
	FWindowStyle& SetCloseButtonStyle( const FButtonStyle& InCloseButtonStyle ){ CloseButtonStyle = InCloseButtonStyle; return *this; }

	/** Style used to draw the window title text */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FTextBlockStyle TitleTextStyle;
	FWindowStyle& SetTitleTextStyle( const FTextBlockStyle& InTitleTextStyle ){ TitleTextStyle = InTitleTextStyle; return *this; }

	/** Brush used to draw the window title area when the window is active */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush ActiveTitleBrush;
	FWindowStyle& SetActiveTitleBrush( const FSlateBrush& InActiveTitleBrush ){ ActiveTitleBrush = InActiveTitleBrush; return *this; }

	/** Brush used to draw the window title area when the window is inactive */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush InactiveTitleBrush;
	FWindowStyle& SetInactiveTitleBrush( const FSlateBrush& InInactiveTitleBrush ){ InactiveTitleBrush = InInactiveTitleBrush; return *this; }

	/** Brush used to draw the window title area when the window is flashing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush FlashTitleBrush;
	FWindowStyle& SetFlashTitleBrush( const FSlateBrush& InFlashTitleBrush ){ FlashTitleBrush = InFlashTitleBrush; return *this; }

	/** Color used to draw the window background */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateColor BackgroundColor;
	FWindowStyle& SetBackgroundColor( const FSlateColor& InBackgroundColor ){ BackgroundColor = InBackgroundColor; return *this; }

	/** Brush used to draw the window outline */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush OutlineBrush;
	FWindowStyle& SetOutlineBrush( const FSlateBrush& InOutlineBrush ){ OutlineBrush = InOutlineBrush; return *this; }

	/** Color used to draw the window outline */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateColor OutlineColor;
	FWindowStyle& SetOutlineColor( const FSlateColor& InOutlineColor ){ OutlineColor = InOutlineColor; return *this; }

	/** Brush used to draw the window border */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BorderBrush;
	FWindowStyle& SetBorderBrush( const FSlateBrush& InBorderBrush ){ BorderBrush = InBorderBrush; return *this; }

	/** Color used to draw the window border */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor BorderColor;
	FWindowStyle& SetBorderColor(const FSlateColor& InBorderColor) { BorderColor = InBorderColor; return *this; }

	/** Brush used to draw the window background */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush BackgroundBrush;
	FWindowStyle& SetBackgroundBrush( const FSlateBrush& InBackgroundBrush ){ BackgroundBrush = InBackgroundBrush; return *this; }

	/** Brush used to draw the background of child windows */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush ChildBackgroundBrush;
	FWindowStyle& SetChildBackgroundBrush( const FSlateBrush& InChildBackgroundBrush ){ ChildBackgroundBrush = InChildBackgroundBrush; return *this; }

	/** Window corner rounding.  If this value is <= 0 no rounding will occur.   Used for regular, non-maximized windows only (not tool-tips or decorators.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	int32 WindowCornerRadius;
	FWindowStyle& SetCornerRadius(int32 InCornerRadius) { WindowCornerRadius = InCornerRadius; return *this; }

	/** Window corner rounding.  If this value is <= 0 no rounding will occur.   Used for regular, non-maximized windows only (not tool-tips or decorators.) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FMargin BorderPadding;
	FWindowStyle& SetBorderPadding(FMargin InBorderPadding) { BorderPadding = InBorderPadding; return *this; }

};


class FInvalidatableBrushAttribute
{
public:
	FInvalidatableBrushAttribute() { }
	FInvalidatableBrushAttribute(const TAttribute< const FSlateBrush* >& InImage)
		: Image(InImage)
	{
		// INFO don't call this in the ctor, users did not anticipate needing to have their accessors work.

		//const FSlateBrush* ImagePtr = Image.Get();
		//ImageCache = ImagePtr ? *ImagePtr : FSlateBrush();
	}

	bool IsBound() const { return Image.IsBound(); }

	const FSlateBrush* Get() const { return Image.Get(); }
	TAttribute< const FSlateBrush* > GetImage() const { return Image; }
	SLATECORE_API void SetImage(SWidget& ThisWidget, const TAttribute< const FSlateBrush* >& InImage);

private:
	/** The slate brush to draw for the image, or a bound delegate to a brush. */
	TAttribute< const FSlateBrush* > Image;

	/** The copy of the image data, some users reuse the same FSlateBrush pointer, so we need to check it against the last true data to see what changed. */
	FSlateBrush ImageCache;
};

/** A structure used to help the user identify deprecated FInvalidatableBrushAttribute that are now TSlateAttribute. */
class FDeprecatedInvalidatableBrushAttribute
{
public:
	FDeprecatedInvalidatableBrushAttribute() { }
	FDeprecatedInvalidatableBrushAttribute(const TAttribute< const FSlateBrush* >& InImage) {}

	bool IsBound() const { return false; }
	const FSlateBrush* Get() const { return nullptr; }
	TAttribute< const FSlateBrush* > GetImage() const { return {}; }
	void SetImage(SWidget& ThisWidget, const TAttribute< const FSlateBrush* >& InImage) {}
};

/**
 * Class similar to FInvalidatableBrushAttribute but to be used with TSlateAttribute.
 * Note that this class does not invalidate on value change unless used inside a TSlateAttribute
 */
class FSlateBrushCached
{
public:
	FSlateBrushCached() : Image(nullptr), ImageCache(FSlateBrush()) {}
	FSlateBrushCached(const FSlateBrush* InImage)
		: Image(InImage), ImageCache(InImage ? *InImage : FSlateBrush())
	{
	}

	const FSlateBrush* GetImage() const
	{
		return Image;
	}
	void SetImage(const FSlateBrush* InImage)
	{
		Image = InImage;
		ImageCache = InImage ? *InImage : FSlateBrush();
	}
	
	bool operator==(const FSlateBrushCached& Rhs) const
	{
		return Image == Rhs.Image;
	}
	bool operator==(const FSlateBrush* Rhs) const
	{
		if (Image == Rhs)
		{
			const FSlateBrush NewImageToCache = Image ? *Image : FSlateBrush();
			return ImageCache == NewImageToCache;
		}
		return false;
	}
private:
	/** The slate brush to draw for the image. */
	const FSlateBrush* Image;

	/** The copy of the image data, some users reuse the same FSlateBrush pointer, so we need to check it against the last true data to see what changed. */
	FSlateBrush ImageCache;
};

/** HACK: We need a UClass here or UHT will complain. */
UCLASS()
class USlateTypes : public UObject
{
public:
	GENERATED_BODY()

};
