// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "SlateWidgetStyle.generated.h"

struct FSlateBrush;

/**
 * Base structure for widget styles.
 */
USTRUCT()
struct FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

public:

	/** Default constructor. */
	SLATECORE_API FSlateWidgetStyle( );

	/** Virtual Destructor. */
	SLATECORE_API virtual ~FSlateWidgetStyle();

public:

	/**
	 * Gets the brush resources associated with this style.
	 *
	 * This method must be implemented by inherited structures.
	 *
	 * @param OutBrushes The brush resources.
	 */
	virtual void GetResources( TArray<const FSlateBrush*> & OutBrushes ) const { }

	/**
	 * Gets the name of this style.
	 *
	 * This method must be implemented by inherited structures.
	 *
	 * @return Widget style name.
	 */
	virtual const FName GetTypeName() const
	{
		return FName( NAME_None );
	} 
};
