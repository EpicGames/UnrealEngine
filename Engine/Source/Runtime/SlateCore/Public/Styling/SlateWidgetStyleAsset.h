// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "SlateWidgetStyleAsset.generated.h"

class Error;

/**
 * Just a wrapper for the struct with real data in it.
 */
UCLASS(MinimalApi, hidecategories=Object)
class USlateWidgetStyleAsset : public UObject
{
	GENERATED_BODY()
		  
public:  
	/**  */
	UPROPERTY(Category=Appearance, EditAnywhere, Instanced)
	TObjectPtr<USlateWidgetStyleContainerBase> CustomStyle;

	template< class WidgetStyleType >            
	const WidgetStyleType* GetStyle() const 
	{
		return static_cast< const WidgetStyleType* >( GetStyle( WidgetStyleType::TypeName ) );
	}

	template< class WidgetStyleType >            
	const WidgetStyleType* GetStyleChecked() const 
	{
		return static_cast< const WidgetStyleType* >( GetStyleChecked( WidgetStyleType::TypeName ) );
	}

	const FSlateWidgetStyle* GetStyle( const FName DesiredTypeName ) const 
	{
		if ( CustomStyle == nullptr )
		{
			return nullptr;
		}

		const FSlateWidgetStyle* Style = CustomStyle->GetStyle();

		if ( (Style == nullptr) || (Style->GetTypeName() != DesiredTypeName) )
		{
			return nullptr;
		}

		return Style;
	}

	const FSlateWidgetStyle* GetStyleChecked( const FName DesiredTypeName ) const 
	{
		if ( CustomStyle == nullptr )
		{
			UE_LOGF( LogSlateStyle, Error, "USlateWidgetStyleAsset::GetStyle : No custom style set for '%ls'.", *GetPathName() );
			return nullptr;
		}

		const FSlateWidgetStyle* Style = CustomStyle->GetStyle();

		if ( Style == nullptr )
		{
			UE_LOGF( LogSlateStyle, Error, "USlateWidgetStyleAsset::GetStyle : No style found in custom style set for '%ls'.", *GetPathName() );
			return nullptr;
		}

		if ( Style->GetTypeName() != DesiredTypeName )
		{
			UE_LOGF( LogSlateStyle, Error, "USlateWidgetStyleAsset::GetStyle : The custom style is not of the desired type. Desired: '%ls', Actual: '%ls'", *DesiredTypeName.ToString(), *Style->GetTypeName().ToString() );
			return nullptr;
		}

		return Style;
	}
};
