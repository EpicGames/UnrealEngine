// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Styling/SlateWidgetStyleContainerInterface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SlateWidgetStyleContainerBase.generated.h"

SLATECORE_API DECLARE_LOG_CATEGORY_EXTERN(LogSlateStyle, Log, All);

/**
 * Just a wrapper for the struct with real data in it.
 */
UCLASS(hidecategories=Object, MinimalAPI)
class USlateWidgetStyleContainerBase : public UObject, public ISlateWidgetStyleContainerInterface
{
	GENERATED_BODY()

public:

	SLATECORE_API virtual const struct FSlateWidgetStyle* const GetStyle() const override;
};
