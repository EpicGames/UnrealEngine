// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SlateWidgetStyleContainerInterface.generated.h"

class UObject;

UINTERFACE(MinimalAPI)
class USlateWidgetStyleContainerInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class ISlateWidgetStyleContainerInterface
{
	GENERATED_IINTERFACE_BODY()

public:

	virtual const struct FSlateWidgetStyle* const GetStyle() const = 0;
};
