// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Styling/AppStyle.h"

struct FSlateDynamicImageBrush;

/**
 * Core slate style
 */
class FStarshipCoreStyle
{
public:

	static SLATECORE_API TSharedRef<class ISlateStyle> Create();

	/** 
	* @return the Application Style 
	*
	* NOTE: Until the Editor can be fully updated, calling FStarshipCoreStyle::Get() will
	* return the AppStyle instead of the style defined in this class.  
	*
	* Using the AppStyle is preferred in most cases as it allows the style to be changed 
	* and restyled more easily.
	*
	* In cases requiring explicit use of the CoreStyle where a Slate Widget should not take on
	* the appearance of the rest of the application, use FStarshipCoreStyle::GetCoreStyle().
	*
	*/
	static const ISlateStyle& Get( )
	{
		return FAppStyle::Get();
	}

	/** @return the singleton instance of the style created in . */
	static const ISlateStyle& GetCoreStyle()
	{
		return *(Instance.Get());
	}

	/** Get the default font for Slate */
	static SLATECORE_API TSharedRef<const FCompositeFont> GetDefaultFont();

	/** Get a font style using the default for for Slate */
	static SLATECORE_API FSlateFontInfo GetDefaultFontStyle(const FName InTypefaceFontName, const float InSize, const FFontOutlineSettings& InOutlineSettings = FFontOutlineSettings());

	static SLATECORE_API void ResetToDefault( );

	/** Used to override the default selection colors */
	static SLATECORE_API void SetSelectorColor( const FLinearColor& NewColor );
	static SLATECORE_API void SetSelectionColor( const FLinearColor& NewColor );
	static SLATECORE_API void SetInactiveSelectionColor( const FLinearColor& NewColor );
	static SLATECORE_API void SetPressedSelectionColor( const FLinearColor& NewColor );
	static SLATECORE_API void SetFocusBrush(FSlateBrush* NewBrush);

	// todo: jdale - These are only here because of UTouchInterface::Activate and the fact that GetDynamicImageBrush is non-const
	static SLATECORE_API const TSharedPtr<FSlateDynamicImageBrush> GetDynamicImageBrush( FName BrushTemplate, FName TextureName, const ANSICHAR* Specifier = nullptr );
	static SLATECORE_API const TSharedPtr<FSlateDynamicImageBrush> GetDynamicImageBrush( FName BrushTemplate, const ANSICHAR* Specifier, class UTexture2D* TextureResource, FName TextureName );
	static SLATECORE_API const TSharedPtr<FSlateDynamicImageBrush> GetDynamicImageBrush( FName BrushTemplate, class UTexture2D* TextureResource, FName TextureName );

	static const int32 RegularTextSize = 10;
	static const int32 SmallTextSize = 8;

	static bool IsInitialized() { return Instance.IsValid(); }
private:
	class FStyle;

	static SLATECORE_API void SetStyle(const TSharedRef<class ISlateStyle>& NewStyle);
	
	static SLATECORE_API void SetupColors(TSharedRef<FStyle>& Style);
	static SLATECORE_API void SetupTextStyles(TSharedRef<FStyle>& Style);
	static SLATECORE_API void SetupButtonStyles(TSharedRef<FStyle>& Style);
	static SLATECORE_API void SetupComboButtonStyles(TSharedRef<FStyle>& Style);
	static SLATECORE_API void SetupCheckboxStyles(TSharedRef<FStyle>& Style);
	static SLATECORE_API void SetupDockingStyles(TSharedRef<FStyle>& Style);
	static SLATECORE_API void SetupColorPickerStyles(TSharedRef<FStyle>& Style);
	static SLATECORE_API void SetupTableViewStyles(TSharedRef<FStyle>& Style);
	static SLATECORE_API void SetupMultiboxStyles(TSharedRef<FStyle>& Style);

private:

	/** Singleton instances of this style. */
	static SLATECORE_API TSharedPtr< class ISlateStyle > Instance;

};

struct FStyleFonts
{

  public:
	static const FStyleFonts& Get()
	{
		if (Instance == nullptr)
		{
			Instance = MakeUnique<FStyleFonts>(FStyleFonts());
		}
		return *(Instance.Get());
	}

	const FSlateFontInfo Normal;
	const FSlateFontInfo NormalItalic;
	const FSlateFontInfo NormalBold;
	const FSlateFontInfo NormalBoldItalic;
	const FSlateFontInfo Small;
	const FSlateFontInfo SmallBold;
	const FSlateFontInfo Large;
	const FSlateFontInfo LargeBold;

	const FSlateFontInfo HeadingMedium;
	const FSlateFontInfo HeadingSmall;
	const FSlateFontInfo HeadingExtraSmall;

  private: 
  	SLATECORE_API FStyleFonts();
  	static SLATECORE_API TUniquePtr<struct FStyleFonts> Instance;
};

