// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/Guid.h"
#include "SlateColor.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "StyleColors.generated.h"

struct FSlateColor;

/** Themes are only allowed in the editor or standalone tools */
#define ALLOW_THEMES WITH_EDITOR || IS_PROGRAM

// HEX Colors from sRGB Color space
#define COLOR( HexValue ) FLinearColor::FromSRGBColor(FColor::FromHex(HexValue))

/**
 * Note: If you add another color here, you should update the Dark.json theme file in Engine\Content\Slate\Themes & FUMGColors in UMGCoreStyle.cpp for consistency
 */
UENUM()
enum class EStyleColor : uint8
{
	Black,
	Background,
	Title,
	WindowBorder,
	Foldout,
	Input,
	InputOutline,
	Recessed,
	Panel,
	Header,
	Dropdown,
	DropdownOutline,
	Hover,
	Hover2,
	White,
	White25,
	Highlight,
	Primary,
	PrimaryHover,
	PrimaryPress,
	Secondary, 
	Foreground,
	ForegroundHover,
	ForegroundInverted,
	ForegroundHeader,
	Select,
	SelectInactive,
	SelectParent,
	SelectHover,
	Notifications,
	AccentBlue,
	AccentPurple,
	AccentPink,
	AccentRed,
	AccentOrange,
	AccentYellow,
	AccentGreen,
	AccentBrown,
	AccentBlack,
	AccentGray,
	AccentWhite,
	AccentFolder,
	Warning,
	Error,
	Success,

	/** Only user colors should be below this line
	 * To use user colors:
	 * 1. Set an unused user enum value below as the color value for an FSlateColor. E.g. FSlateColor MyCustomColor(EStyleColors::User1)
	 * 2. Set the actual color. E.g USlateThemeManager::Get().SetDefaultColor(EStyleColor::User1, FLinearColor::White)
	 * 3. Give it a display name if you want it to be configurable by editor users. E.g.  UStyleColorTable::Get().SetColorDisplayName(EUserStyleColor::User1, "My Color Name")
	 */
	User1,
	User2,
	User3,
	User4,
	User5,
	User6,
	User7,
	User8,
	User9,
	User10,
	User11,
	User12,
	User13,
	User14,
	User15,
	User16,

	MAX
};

USTRUCT()
struct FStyleColorList
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Config, Category = Colors)
	FLinearColor StyleColors[(int32)EStyleColor::MAX] = {FLinearColor::Transparent};

	FText DisplayNames[(int32)EStyleColor::MAX];
};

/**
 * Represents a single theme
 */
USTRUCT()
struct FStyleTheme
{
	GENERATED_BODY()

	/** Unique Id for the theme */
	FGuid Id;
	/** Friendly, user customizable theme name */
	FText DisplayName;
	/** Filename where the theme is stored */
	FString Filename;
	/** The default colors for this theme. Used for resetting to default. Not the active colors*/
	TArray<FLinearColor> LoadedDefaultColors;

	bool operator==(const FStyleTheme& Other) const
	{
		return Id == Other.Id;
	}

	bool operator==(const FGuid& OtherId) const
	{
		return Id == OtherId;
	}
};

UCLASS(Config=EditorSettings, MinimalAPI)
class USlateThemeManager : public UObject 
{
	GENERATED_BODY()
public:
	static USlateThemeManager& Get()
	{
		return *GetMutableDefault<USlateThemeManager>();
	}

	const FLinearColor& GetColor(EStyleColor Color) const
	{
		return ActiveColors.StyleColors[static_cast<int32>(Color)];
	}

	const FGuid& GetCurrentThemeID() const
	{
		return CurrentThemeId; 
	}

	SLATECORE_API USlateThemeManager();

	/**
	 * Initializes default colors
	 */
	UE_DEPRECATED(5.8, "Use the properly spelled version InitializeDefaults")
	void InitalizeDefaults()
	{
		InitializeDefaults();
	}

	/**
	 * Initializes default colors
	 */
	SLATECORE_API void InitializeDefaults();

	/**
	 * Sets a default color to be used as a fallback if no theme is loaded
	 */
	void SetDefaultColor(EStyleColor InColorId, FLinearColor InColor)
	{
#if ALLOW_THEMES
		DefaultColors[static_cast<int32>(InColorId)] = InColor;
#else
		ActiveColors.StyleColors[static_cast<int32>(InColorId)] = InColor;
#endif
	}

#if ALLOW_THEMES

	/** Sets a custom display name for a style color */
	void SetColorDisplayName(EStyleColor InColorId, FText DisplayName)
	{
		ActiveColors.DisplayNames[static_cast<int32>(InColorId)] = DisplayName;
	}

	/** Gets a custom display name for a style color. This will be empty if no custom name was chosen */
	FText GetColorDisplayName(EStyleColor InColorId) const
	{
		return ActiveColors.DisplayNames[static_cast<int32>(InColorId)];
	}

	/**
	 * Load all known themes from engine, project, and user directories
	 */
	SLATECORE_API void LoadThemes();

	/**
	 * Saves the current theme
	 * 
	 * @param The filename to save the current theme as
	 */
	SLATECORE_API void SaveCurrentThemeAs(const FString& Filename);

	/**
	 * Exports the current theme to the given filename.
	 *
	 * The given filename will be replaced if it already exists.
	 * 
	 * @param The Filename to export the theme to
	 * @return true if the theme was successfully exported
	 */
	SLATECORE_API bool ExportCurrentThemeTo(const FString& Filename);

	/**
	 * Applies a theme as the active theme
	 */
	SLATECORE_API void ApplyTheme(FGuid ThemeId);

	/**
	 * Applies the default dark theme as the active theme
	*/
	SLATECORE_API void ApplyDefaultTheme();

	/**
	* Returns true if the active theme is an engine-specific theme
	*/
	SLATECORE_API bool IsEngineTheme() const;

	/**
	* Returns true if the active theme is a project-specific theme
	*/
	SLATECORE_API bool IsProjectTheme() const;

	/**
	 * Removes a theme. 
	 * Note: The active theme cannot be removed and there must always be an active theme.  Apply a new theme first before removing the current theme.
	 */
	SLATECORE_API void RemoveTheme(FGuid ThemeId);

	/**
	 * Duplicates the active theme
	 * 
	 * @return the id of the new theme
	 */
	SLATECORE_API FGuid DuplicateActiveTheme();

	/**
	 * Sets the display name for the current theme
	 */
	SLATECORE_API void SetCurrentThemeDisplayName(FText NewDisplayName);

	/**
	 * @return the current theme
	 */
	const FStyleTheme& GetCurrentTheme() const { return *Themes.FindByKey(CurrentThemeId); }

	/**
	 * @return All known themes
	 */
	const TArray<FStyleTheme>& GetThemes() const { return Themes; }

	/**
	 * Resets an active color to the default color for the current theme
	 */
	SLATECORE_API void ResetActiveColorToDefault(EStyleColor Color);

	/**
	 * Resets the colors to the default color for the current theme
	 */
	SLATECORE_API void ResetColorsToDefault();

	/**
	 * Validate that there is an active loaded theme
	 */
	SLATECORE_API void ValidateActiveTheme();

	/**
	 * @return the engine theme dir.  Engine themes are project agnostic
	 */
	SLATECORE_API FString GetEngineThemeDir() const;

	/**
	 * @return the project theme dir. Project themes can override engine themes
	 */
	SLATECORE_API FString GetProjectThemeDir() const;

	/**
	 * @return the user theme dir. Themes in this dir are per-user and override engine and project themes
	 */
	SLATECORE_API FString GetUserThemeDir() const;
		
	/**
	 * @return true if the ThemeID already exists in the theme dropdown. 
	 */
	SLATECORE_API bool DoesThemeExist(const FGuid& ThemeID) const;

	DECLARE_EVENT_OneParam(USlateThemeManager, FThemeChangedEvent, FGuid);
	/**
	 * Returns an event delegate that is executed when the themeID has changed.
	 *
	 * @return the event delegate.
	 */
	FThemeChangedEvent& OnThemeChanged() { return ThemeChangedEvent; }

private:

	FStyleTheme& GetMutableCurrentTheme() { return *Themes.FindByKey(CurrentThemeId); }
	SLATECORE_API void LoadThemesFromDirectory(const FString& Directory);
	SLATECORE_API bool ReadTheme(const FString& ThemeData, FStyleTheme& OutTheme);
	bool WriteCurrentTheme(FString& OutString);
	SLATECORE_API void EnsureValidCurrentTheme();
	SLATECORE_API void LoadThemeColors(FStyleTheme& Theme);

private:
	FStyleTheme DefaultDarkTheme;
	TArray<FStyleTheme> Themes;
	FLinearColor DefaultColors[(int32)EStyleColor::MAX];

	/** Broadcasts whenever the theme changes **/
	FThemeChangedEvent ThemeChangedEvent;

#endif // ALLOW_THEMES

	FLinearColor GetDefaultColor(EStyleColor InColorId)
	{
#if ALLOW_THEMES
		return DefaultColors[static_cast<int32>(InColorId)];
#else
		return ActiveColors.StyleColors[static_cast<int32>(InColorId)];
#endif
	}

	UPROPERTY(EditAnywhere, Config, Category=Colors)
	FGuid CurrentThemeId;

	UPROPERTY(EditAnywhere, transient, Category = Colors)
	FStyleColorList ActiveColors;
};

/**
 * Common/themeable colors used by all styles
 * Please avoid adding new generic colors to this list without discussion first
 */
struct FStyleColors
{
	static SLATECORE_API const FSlateColor Transparent;
	static SLATECORE_API const FSlateColor Black;
	static SLATECORE_API const FSlateColor Title;
	static SLATECORE_API const FSlateColor WindowBorder;
	static SLATECORE_API const FSlateColor Foldout;
	static SLATECORE_API const FSlateColor Input;
	static SLATECORE_API const FSlateColor InputOutline;
	static SLATECORE_API const FSlateColor Recessed;
	static SLATECORE_API const FSlateColor Background;
	static SLATECORE_API const FSlateColor Panel;
	static SLATECORE_API const FSlateColor Header;
	static SLATECORE_API const FSlateColor Dropdown;
	static SLATECORE_API const FSlateColor DropdownOutline;
	static SLATECORE_API const FSlateColor Hover;
	static SLATECORE_API const FSlateColor Hover2;
	static SLATECORE_API const FSlateColor White;
	static SLATECORE_API const FSlateColor White25;
	static SLATECORE_API const FSlateColor Highlight;

	static SLATECORE_API const FSlateColor Primary;
	static SLATECORE_API const FSlateColor PrimaryHover;
	static SLATECORE_API const FSlateColor PrimaryPress;
	static SLATECORE_API const FSlateColor Secondary;

	static SLATECORE_API const FSlateColor Foreground;
	static SLATECORE_API const FSlateColor ForegroundHover;
	static SLATECORE_API const FSlateColor ForegroundInverted;
	static SLATECORE_API const FSlateColor ForegroundHeader;

	static SLATECORE_API const FSlateColor Select;
	static SLATECORE_API const FSlateColor SelectInactive;
	static SLATECORE_API const FSlateColor SelectParent;
	static SLATECORE_API const FSlateColor SelectHover;

	static SLATECORE_API const FSlateColor Notifications;

	static SLATECORE_API const FSlateColor AccentBlue;
	static SLATECORE_API const FSlateColor AccentPurple;
	static SLATECORE_API const FSlateColor AccentPink;
	static SLATECORE_API const FSlateColor AccentRed;
	static SLATECORE_API const FSlateColor AccentOrange;
	static SLATECORE_API const FSlateColor AccentYellow;
	static SLATECORE_API const FSlateColor AccentGreen;
	static SLATECORE_API const FSlateColor AccentBrown;
	static SLATECORE_API const FSlateColor AccentBlack;
	static SLATECORE_API const FSlateColor AccentGray;
	static SLATECORE_API const FSlateColor AccentWhite;
	static SLATECORE_API const FSlateColor AccentFolder;

	static SLATECORE_API const FSlateColor Warning;
	static SLATECORE_API const FSlateColor Error;
	static SLATECORE_API const FSlateColor Success;
};
