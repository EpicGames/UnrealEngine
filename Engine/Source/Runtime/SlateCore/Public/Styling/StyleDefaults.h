// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Fonts/SlateFontInfo.h"
#include "Layout/Margin.h"
#include "Sound/SlateSound.h"
#include "Misc/Paths.h"
#include "Brushes/SlateNoResource.h"

class FStyleDefaults
{
public:

	/**
	 * Return the static default float value.
	 * @return - The static default float value.
	 */
	static float GetFloat()
	{
		return DefaultFloat;
	}
	
	/**
	 * Get default FVector2D value.
	 * @return - The default FVector2D.
	 */
	const static UE::Slate::FDeprecateVector2DResult GetVector2D()
	{
		return DefaultFVector2D;
	}
	
	/**
	 * Get default FLinearColor.
	 * @return - The default FLinearColor.
	 */
	static const FLinearColor& GetColor()
	{
		return DefaultColor;
	}

	/**
	 * Get default Slate Color.
	 * @return - The default Slate color.
	 */
	static const FSlateColor& GetSlateColor()
	{
		return DefaultSlateColor;
	}
	
	/**
	 * Get default FMargin.
	 * @return - The default FMargin value.
	 */
	static const FMargin& GetMargin()
	{
		return DefaultMargin;
	}

	/**
	 * @return - Returns no brush.
	 */
	static SLATECORE_API const FSlateBrush* GetNoBrush();

	/**
	 * Get default font.
	 * @return - The default font.
	 */
	static SLATECORE_API const FSlateFontInfo GetFontInfo(uint16 Size = 10);

	/**
	 * Return the static default sound value.
	 * @return - The static default sound value.
	 */
	static const FSlateSound& GetSound()
	{
		return DefaultSound;
	}

private:

	static SLATECORE_API float DefaultFloat;
	static SLATECORE_API FVector2f DefaultFVector2D;
	static SLATECORE_API FLinearColor DefaultColor;
	static SLATECORE_API FMargin DefaultMargin;
	static SLATECORE_API FSlateSound DefaultSound;
	static SLATECORE_API FSlateColor DefaultSlateColor;
};
