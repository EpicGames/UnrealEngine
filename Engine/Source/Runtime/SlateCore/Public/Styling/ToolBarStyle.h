// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateTypes.h"

#include "ToolBarStyle.generated.h"

USTRUCT(BlueprintType)
struct FWrapButtonStyle
{
	GENERATED_BODY()
	
	SLATECORE_API FWrapButtonStyle();
	SLATECORE_API FWrapButtonStyle(const FWrapButtonStyle&);
	
	SLATECORE_API void GetResources(TArray<const FSlateBrush*>& OutBrushes) const;

	/** The padding around the wrap button */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin Padding;
	FWrapButtonStyle& SetWrapButtonPadding(const FMargin& InWrapButtonPadding) { Padding = InWrapButtonPadding; return *this; }
	
	/** Where in the toolbar the wrap button should appear. e.g. 0 for the left side, -1 for the right side. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	int32 WrapButtonIndex;
	FWrapButtonStyle& SetWrapButtonIndex(int32 InWrapButtonIndex) { WrapButtonIndex = InWrapButtonIndex; return *this; }

	/** The brush used for the expand arrow when the toolbar runs out of room and needs to display toolbar items in a menu*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush ExpandBrush;
	FWrapButtonStyle& SetExpandBrush(const FSlateBrush& InExpandBrush) { ExpandBrush = InExpandBrush; return *this; }

	/** Whether the combo box includes a down arrow */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	bool bHasDownArrow = true;
	FWrapButtonStyle& SetHasDownArrow(bool bInHasDownArrow) { bHasDownArrow = bInHasDownArrow; return *this; }

	/** The styling of the combo button that opens the wrapping menu */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	TOptional<FComboButtonStyle> ComboButtonStyle;
	FWrapButtonStyle& SetComboButtonStyle(const FComboButtonStyle& InComboButtonStyle) { ComboButtonStyle = InComboButtonStyle; return *this; }
	
	/** Whether a separator should appear adjacent to the combo button */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	bool bIncludeSeparator = true;
	FWrapButtonStyle& SetIncludeSeparator(bool bInIncludeSeparator) { bIncludeSeparator = bInIncludeSeparator; return *this; }
	
	/** The appearance of the separator */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	TOptional<FSlateBrush> SeparatorBrush;
	FWrapButtonStyle& SetSeparatorBrush(const FSlateBrush& InSeparatorBrush) { SeparatorBrush = InSeparatorBrush; return *this; }

	/** How wide/tall the separator should be */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	TOptional<float> SeparatorThickness;
	FWrapButtonStyle& SetSeparatorThickness(float InSeparatorThickness) { SeparatorThickness = InSeparatorThickness; return *this; }
	
	/** Any padding around the separator */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	TOptional<FMargin> SeparatorPadding;
	FWrapButtonStyle& SetSeparatorPadding(const FMargin& InSeparatorPadding) { SeparatorPadding = InSeparatorPadding; return *this; }
};

/**
 * Represents the appearance of a toolbar 
 */
USTRUCT(BlueprintType)
struct FToolBarStyle : public FSlateWidgetStyle
{
	GENERATED_BODY()

	SLATECORE_API FToolBarStyle();
	SLATECORE_API FToolBarStyle(const FToolBarStyle&);
	SLATECORE_API virtual ~FToolBarStyle() override;

	SLATECORE_API virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;

	static SLATECORE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static SLATECORE_API const FToolBarStyle& GetDefault();

	/** The brush used for the background of the toolbar */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush BackgroundBrush;
	FToolBarStyle& SetBackground(const FSlateBrush& InBackground) { BackgroundBrush = InBackground; return *this; }

	/** The brush used for the expand arrow when the toolbar runs out of room and needs to display toolbar items in a menu*/
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use WrapButtonStyle.ExpandBrush instead"))
	FSlateBrush ExpandBrush_DEPRECATED;
	UE_DEPRECATED(5.6, "Use WrapButtonStyle.SetExpandBrush() instead")
	FToolBarStyle& SetExpandBrush(const FSlateBrush& InExpandBrush)
	{
		ExpandBrush_DEPRECATED = InExpandBrush; return *this;
	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush SeparatorBrush;
	FToolBarStyle& SetSeparatorBrush(const FSlateBrush& InSeparatorBrush) { SeparatorBrush = InSeparatorBrush; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FTextBlockStyle LabelStyle;
	FToolBarStyle& SetLabelStyle(const FTextBlockStyle& InLabelStyle) { LabelStyle = InLabelStyle; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FEditableTextBoxStyle EditableTextStyle;
	FToolBarStyle& SetEditableTextStyle(const FEditableTextBoxStyle& InEditableTextStyle) { EditableTextStyle = InEditableTextStyle; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FCheckBoxStyle ToggleButton;
	FToolBarStyle& SetToggleButtonStyle(const FCheckBoxStyle& InToggleButton) { ToggleButton = InToggleButton; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FComboButtonStyle ComboButtonStyle;
	FToolBarStyle& SetComboButtonStyle(const FComboButtonStyle& InComboButtonStyle) { ComboButtonStyle = InComboButtonStyle; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FButtonStyle SettingsButtonStyle;
	FToolBarStyle& SetSettingsButtonStyle(const FButtonStyle& InSettingsButton) { SettingsButtonStyle = InSettingsButton; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FComboButtonStyle SettingsComboButton;
	FToolBarStyle& SetSettingsComboButtonStyle(const FComboButtonStyle& InSettingsComboButton) { SettingsComboButton = InSettingsComboButton; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FCheckBoxStyle SettingsToggleButton;
	FToolBarStyle& SetSettingsToggleButtonStyle(const FCheckBoxStyle& InSettingsToggleButton) { SettingsToggleButton = InSettingsToggleButton; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FButtonStyle ButtonStyle;
	FToolBarStyle& SetButtonStyle(const FButtonStyle& InButtonStyle) { ButtonStyle = InButtonStyle; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin LabelPadding;
	FToolBarStyle& SetLabelPadding(const FMargin& InLabelPadding) { LabelPadding = InLabelPadding; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float UniformBlockWidth;
	FToolBarStyle& SetUniformBlockWidth(const float InUniformBlockWidth) { UniformBlockWidth = InUniformBlockWidth; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float UniformBlockHeight;
	FToolBarStyle& SetUniformBlockHeight(const float InUniformBlockHeight) { UniformBlockHeight = InUniformBlockHeight; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	int32 NumColumns;
	FToolBarStyle& SetNumColumns(const int32 InNumColumns) { NumColumns = InNumColumns; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin IconPadding;
	FToolBarStyle& SetIconPadding(const FMargin& InIconPadding) { IconPadding = InIconPadding; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin SeparatorPadding;
	FToolBarStyle& SetSeparatorPadding(const FMargin& InSeparatorPadding) { SeparatorPadding = InSeparatorPadding; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float SeparatorThickness;
	FToolBarStyle& SetSeparatorThickness(float InSeparatorThickness) { SeparatorThickness = InSeparatorThickness; return *this;	}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin ComboButtonPadding;
	FToolBarStyle& SetComboButtonPadding(const FMargin& InComboButtonPadding) { ComboButtonPadding = InComboButtonPadding; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin ButtonPadding;
	FToolBarStyle& SetButtonPadding(const FMargin& InButtonPadding) { ButtonPadding = InButtonPadding; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin CheckBoxPadding;
	FToolBarStyle& SetCheckBoxPadding(const FMargin& InCheckBoxPadding) { CheckBoxPadding = InCheckBoxPadding; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin BlockPadding;
	FToolBarStyle& SetBlockPadding(const FMargin& InBlockPadding) { BlockPadding = InBlockPadding; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin IndentedBlockPadding;
	FToolBarStyle& SetIndentedBlockPadding(const FMargin& InIndentedBlockPadding) { IndentedBlockPadding = InIndentedBlockPadding; return *this; }

	/** Hovered brush for an entire block */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush BlockHovered;
	FToolBarStyle& SetBlockHovered(const FSlateBrush& InBlockHovered) { BlockHovered = InBlockHovered; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin BackgroundPadding;
	FToolBarStyle& SetBackgroundPadding(const FMargin& InMargin) { BackgroundPadding = InMargin; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FWrapButtonStyle WrapButtonStyle;
	FToolBarStyle& SetWrapButtonStyle(const FWrapButtonStyle& InButtonStyle) { WrapButtonStyle = InButtonStyle; return *this; }

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use WrapButtonStyle.Padding instead"))
	FMargin WrapButtonPadding_DEPRECATED;
	UE_DEPRECATED(5.6, "Use WrapButtonStyle instead")
	FToolBarStyle& SetWrapButtonPadding(const FMargin& InWrapButtonPadding)
	{
		WrapButtonPadding_DEPRECATED = InWrapButtonPadding;
		WrapButtonStyle.SetWrapButtonPadding(InWrapButtonPadding);
		return *this;
	}

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use WrapButtonStyle.Padding instead"))
	int32 WrapButtonIndex_DEPRECATED;
	UE_DEPRECATED(5.6, "Use WrapButtonStyle instead")
	FToolBarStyle& SetWrapButtonIndex(int32 InWrapButtonIndex)
	{
		WrapButtonIndex_DEPRECATED = InWrapButtonIndex;
		WrapButtonStyle.SetWrapButtonIndex(InWrapButtonIndex);
		return *this;
	}

	/** Set to false if the wrap button should never be shown (even if entries are clipped) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	bool bAllowWrapButton = true;
	FToolBarStyle& SetAllowWrapButton(bool bInAllowWrapButton) { bAllowWrapButton = bInAllowWrapButton; return *this; }

	/** Set to false if the toolbar should not wrap (to the next line) by default, but can be overridden per section or entry */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	bool bAllowWrappingDefault = true;
	FToolBarStyle& SetAllowWrappingDefault(bool bInAllowWrapping) { bAllowWrappingDefault = bInAllowWrapping; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FDeprecateSlateVector2D IconSize;
	FToolBarStyle& SetIconSize(const UE::Slate::FDeprecateVector2DParameter& InIconSize) { IconSize = InIconSize; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	bool bShowLabels;
	FToolBarStyle& SetShowLabels(bool bInShowLabels) { bShowLabels = bInShowLabels; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	float ButtonContentMaxWidth = 64.0f;
	FToolBarStyle& SetButtonContentMaxWidth(float InButtonContentMaxWidth) { ButtonContentMaxWidth = InButtonContentMaxWidth; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float ButtonTextMinWidth = 0.0f;
	FToolBarStyle& SetButtonTextMinWidth(float InButtonTextMinWidth) { ButtonTextMinWidth = InButtonTextMinWidth; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float ButtonTextMaxWidth = FLT_MAX;
	FToolBarStyle& SetButtonTextMaxWidth(float InButtonTextMaxWidth) { ButtonTextMaxWidth = InButtonTextMaxWidth; return *this; }
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	float ButtonContentFillWidth = 1.0f;
	FToolBarStyle& SetButtonContentFillWidth(float InButtonContentFillWidth) { ButtonContentFillWidth = InButtonContentFillWidth; return *this; }

	/** Min width that label text block slot in combo buttons should have. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float ComboContentMinWidth;
	FToolBarStyle& SetComboLabelMinWidth(float InComboContentMinWidth) { ComboContentMinWidth = InComboContentMinWidth; return *this; }
	
	/** Max width that label text block slot in combo buttons should have. 0 means no max. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float ComboContentMaxWidth;
	FToolBarStyle& SetComboLabelMaxWidth(float InComboContentMaxWidth) { ComboContentMaxWidth = InComboContentMaxWidth; return *this; }
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	TEnumAsByte<EHorizontalAlignment> ComboContentHorizontalAlignment;
	FToolBarStyle& SetComboContentHorizontalAlignment(EHorizontalAlignment InAlignment) { ComboContentHorizontalAlignment = InAlignment; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin IconPaddingWithVisibleLabel;
	FToolBarStyle& SetIconPaddingWithVisibleLabel(const FMargin& InIconPaddingWithVisibleLabel) { IconPaddingWithVisibleLabel = InIconPaddingWithVisibleLabel; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin IconPaddingWithCollapsedLabel;
	FToolBarStyle& SetIconPaddingWithCollapsedLabel(const FMargin& InIconPaddingWithCollapsedLabel) { IconPaddingWithCollapsedLabel = InIconPaddingWithCollapsedLabel; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	TOptional<TEnumAsByte<EVerticalAlignment>> VerticalAlignmentOverride;
	FToolBarStyle& SetVerticalAlignment(const EVerticalAlignment& InVerticalAlignment) { VerticalAlignmentOverride = InVerticalAlignment; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float RaisedChildrenRightPadding = 0.0f;
	FToolBarStyle& SetRaisedChildrenRightPadding(float InRaisedChildrenRightPadding) { RaisedChildrenRightPadding = InRaisedChildrenRightPadding; return *this; }
};
