// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Styling/AppStyle.h"

/**
 * Core slate style
 */
class FUMGCoreStyle
{
public:

	static SLATECORE_API TSharedRef<class ISlateStyle> Create();

	static const ISlateStyle& Get()
	{
		return *(Instance.Get());
	}

	/** Get the default font for Slate */
	static SLATECORE_API TSharedRef<const FCompositeFont> GetDefaultFont();

	/** Get a font style using the default for for Slate */
	static SLATECORE_API FSlateFontInfo GetDefaultFontStyle(const FName InTypefaceFontName, const float InSize, const FFontOutlineSettings& InOutlineSettings = FFontOutlineSettings());

	static SLATECORE_API void ResetToDefault( );

	/** Used to override the default selection colors */
	static SLATECORE_API void SetSelectorColor( const FLinearColor& NewColor );
	static SLATECORE_API void SetSelectionColor( const FLinearColor& NewColor );
	static SLATECORE_API void SetInactiveSelectionColor( const FLinearColor& NewColor );
	static SLATECORE_API void SetPressedSelectionColor( const FLinearColor& NewColor );
	static SLATECORE_API void SetFocusBrush(FSlateBrush* NewBrush);

	static const int32 RegularTextSize = 10;
	static const int32 SmallTextSize = 8;

	static bool IsInitialized() { return Instance.IsValid(); }
private:

	static SLATECORE_API void SetStyle(const TSharedRef<class ISlateStyle>& NewStyle);

private:

	/** Singleton instances of this style. */
	static SLATECORE_API TSharedPtr< class ISlateStyle > Instance;
};
