// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Color.h"

struct FSlateColor;
template< typename ObjectType > class TAttribute;

/**
 * Contains info about those aspects of widget appearance that should be propagated hierarchically.
 */
class FWidgetStyle
{
public:
	
	/** Default constructor. */
	FWidgetStyle( )
		: ColorAndOpacityTint(FLinearColor::White)
		, ForegroundColor(FLinearColor::White)
		, SubduedForeground(FLinearColor::White*SubdueAmount)
	{ }

public:

	/**
	 * Blends the current tint color with the specified tint.
	 *
	 * @param InTint The color to blend with.
	 * @return This instance (for method chaining).
	 */
	FWidgetStyle& BlendColorAndOpacityTint( const FLinearColor& InTint )
	{
		ColorAndOpacityTint *= InTint;
		return *this;
	}

	/**
	 * Blends the current tint color with the specified alpha.
	 *
	 * @param InTint The color to blend with.
	 * @return This instance (for method chaining).
	 */
	FWidgetStyle& BlendOpacity( float InOpacity )
	{
		ColorAndOpacityTint.A *= InOpacity;
		return *this;
	}

	/**
	 * Sets the current foreground color from the given linear color.
	 *
	 * @param InForeground The foreground color value to set.
	 * @return This instance (for method chaining).
	 * @see GetForegroundColor
	 */
	FWidgetStyle& SetForegroundColor( const FLinearColor& InForeground )
	{
		ForegroundColor = InForeground;

		SubduedForeground = InForeground;
		SubduedForeground.A *= SubdueAmount;

		return *this;
	}

	/**
	 * Sets the current foreground color from the given Slate color attribute.
	 *
	 * If the attribute passed in is not set, the foreground will remain unchanged.
	 *
	 * @param InForeground The foreground color to set.
	 * @return This instance (for method chaining).
	 * @see GetForegroundColor
	 */
	SLATECORE_API FWidgetStyle& SetForegroundColor( const TAttribute<FSlateColor>& InForeground );

public:

	/**
	 * Gets the style's color opacity and tint.
	 *
	 * @return Current opacity and tint.
	 * @see GetForegroundColor, GetSubduedForegroundColor
	 */
	const FLinearColor& GetColorAndOpacityTint( ) const
	{
		return ColorAndOpacityTint;
	}

	/**
	 * Gets the style's foreground color.
	 *
	 * @return Current foreground color.
	 * @see GetColorAndOpacityTint, GetSubduedForegroundColor, SetForegroundColor
	 */
	const FLinearColor& GetForegroundColor( ) const
	{
		return ForegroundColor;
	}

	/**
	 * Gets the style's subdued color.
	 *
	 * @return Current subdued color.
	 * @see GetColorAndOpacityTint, GetForegroundColor
	 */
	const FLinearColor& GetSubduedForegroundColor() const
	{
		return SubduedForeground;
	}

private:

	FLinearColor ColorAndOpacityTint;
	FLinearColor ForegroundColor;
	FLinearColor SubduedForeground;

private:

	static SLATECORE_API const float SubdueAmount;
};
