// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FSlateBrush;

/**
 * Struct used to represent an icon in Slate
 */
struct FSlateIcon
{
	/**
	 * Default constructor (empty icon).
	 */
	SLATECORE_API FSlateIcon();

	/**
	 * Creates and initializes a new icon from a style set and style name
	 *
	 * @param InStyleSetName The name of the style set the icon can be found in.
	 * @param StyleName The name of the style for the icon
	 * @param InSmallStyleName The name of the style for the small icon
	 * @param InStatusOverlayStyleName The name of the style for a status icon to overlay on the base image
	 */
	SLATECORE_API FSlateIcon(const FName InStyleSetName, const FName InStyleName, const FName InSmallStyleName = NAME_None, const FName InStatusOverlayStyleName = NAME_None);

public:
	
	/**
	 * Compare 2 slate icons for equality
	 */
	friend bool operator==(const FSlateIcon& A, const FSlateIcon& B)
	{
		return A.IsSet() == B.IsSet()
			&& A.StyleSetName == B.StyleSetName
			&& A.StyleName == B.StyleName
			&& A.SmallStyleName == B.SmallStyleName
			&& A.StatusOverlayStyleName == B.StatusOverlayStyleName;
	}

	/**
	 * Compare 2 slate icons for inequality
	 */
	friend bool operator!=(const FSlateIcon& A, const FSlateIcon& B)
	{
		return !(A == B);
	}

public:

	/**
	 * Gets the resolved icon.
	 *
	 * @return Icon brush, or FStyleDefaults::GetNoBrush() if the icon wasn't found.
	 * @see GetSmallIcon
	 */
	SLATECORE_API const FSlateBrush* GetIcon( ) const;

	/**
	 * Optionally gets the resolved icon, returning nullptr if it's not defined
	 *
	 * @return Icon brush, or nullptr if the icon wasn't found.
	 */
	SLATECORE_API const FSlateBrush* GetOptionalIcon( ) const;

	/**
	 * Gets the resolved small icon.
	 *
	 * @return Icon brush, or FStyleDefaults::GetNoBrush() if the icon wasn't found.
	 * @see GetIcon
	 */
	SLATECORE_API const FSlateBrush* GetSmallIcon( ) const;

	/**
	 * Optionally gets the resolved small icon, returning nullptr if it's not defined
	 *
	 * @return Icon brush, or nullptr if the icon wasn't found.
	 */
	SLATECORE_API const FSlateBrush* GetOptionalSmallIcon( ) const;

	SLATECORE_API const FSlateBrush* GetOverlayIcon() const;

	/**
	 * Gets the name of the style for the icon.
	 *
	 * @return Style name.
	 * @see GetStyleName, GetStyleSet, GetStyleSetName
	 */
	const FName& GetSmallStyleName() const
	{
		return SmallStyleName;
	}

	/**
	 * Gets the name of the style for the icon.
	 *
	 * @return Style name.
	 * @see GetSmallStyleName, GetStyleSet, GetStyleSetName
	 */
	const FName& GetStyleName() const
	{
		return StyleName;
	}

	/**
	 * Gets the resolved style set.
	 *
	 * @return Style set, or nullptr if the style set wasn't found.
	 * @see GetSmallStyleName, GetStyleName, GetStyleSetName
	 */
	SLATECORE_API const class ISlateStyle* GetStyleSet() const;

	/**
	 * Gets the name of the style set the icon can be found in.
	 *
	 * @return Style name.
	 * @see GetSmallStyleName, GetStyleName, GetStyleSet
	 */
	const FName& GetStyleSetName() const
	{
		return StyleSetName;
	}

	/**
	 * Gets the name of the style for the status overlay icon.
	 *
	 * @return Style name (NAME_None when the icon has no overlay).
	 */
	FName GetOverlayStyleName() const
	{
		return StatusOverlayStyleName;
	}

	/**
	 * Checks whether the icon is set to something.
	 *
	 * @return true if the icon is set, false otherwise.
	 */
	const bool IsSet() const
	{
		return StyleSetName != NAME_None && StyleName != NAME_None;
	}

private:

	// The name of the style set the icon can be found in.
	FName StyleSetName;

	// The name of the style for the icon.
	FName StyleName;

	// The name of the style for the small icon.
	FName SmallStyleName;

	// Name of the style for the status overlay icon
	FName StatusOverlayStyleName;
};
