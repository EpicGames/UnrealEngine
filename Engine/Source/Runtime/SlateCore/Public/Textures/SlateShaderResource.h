// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "SlateGlobals.h"
#include "Rendering/SlateRendererTypes.h"
#include "Rendering/SlateResourceHandle.h"

class FSlateShaderResourceProxy;

DECLARE_MEMORY_STAT_EXTERN(TEXT("Texture Data Memory (GPU)"), STAT_SlateTextureGPUMemory, STATGROUP_SlateMemory, SLATECORE_API);

namespace ESlateShaderResource
{
	/**
	 * Enumerates Slate render resource types.
	 */
	enum Type
	{
		/** Texture resource. */
		NativeTexture,

		/** UTexture object resource */
		TextureObject,

		/** Material resource. */
		Material,

		/** Post Process. */
		PostProcess, 

		/** No Resource. */
		Invalid,
	};
}


/** 
 * Base class for all platform independent texture types
 */
class FSlateShaderResource
{
public:

	/**
	 * Gets the width of the resource.
	 *
	 * @return Resource width (in pixels).
	 */
	virtual uint32 GetWidth() const = 0;

	/**
	 * Gets the height of the resource.
	 *
	 * @return Resource height(in pixels).
	 */
	virtual uint32 GetHeight() const = 0;

	/**
	 * Gets the type of the resource.
	 *
	 * @return Resource type.
	 */
	virtual ESlateShaderResource::Type GetType() const = 0;

	/**
	 * Additional validation that can vary per resource type
	 */
	virtual bool IsResourceValid() const { return true; };

	/**
	 * Does this resource use slate post buffers? If no resources use a post buffer we won't populate it.
	 */
	virtual ESlatePostRT GetUsedSlatePostBuffers() const { return ESlatePostRT::None; }

	/**
	 * Does this resource require virtual texture feedback?
	*/
	virtual bool RequiresVirtualTextureFeedback() const { return false; }

#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
	virtual void CheckForStaleResources() const { }
	bool Debug_IsDestroyed() const { return DestroyState != 0x21; }

	/** Virtual destructor. */
	virtual ~FSlateShaderResource() { DestroyState = 0x84; }
#else
	inline void CheckForStaleResources() const { }
	bool Debug_IsDestroyed() const { return false; }

	/** Virtual destructor. */
	virtual ~FSlateShaderResource() = default;
#endif


#if SLATE_CHECK_UOBJECT_RENDER_RESOURCES
private:
	// Use an int, instead of a bool, to make sure the value is "exactly" what it's set in the destructor.
	uint8 DestroyState = 0x21;
#endif
};

/**
 * Data used to lookup a resource from a handle and to determine validity of the handle.  
 * This is shared between the handle and the resource
 *
 */
class FSlateSharedHandleData
{
public:
	FSlateSharedHandleData( FSlateShaderResourceProxy* InProxy = nullptr )
		: Proxy( InProxy )
	{}

public:
	/** Rendering proxy used to directly access the rendering resource quickly */
	FSlateShaderResourceProxy* Proxy;
};

/** 
 * A proxy resource.  
 *
 * May point to a full resource or point or to a texture resource in an atlas
 * Note: This class does not free any resources.  Resources should be owned and freed elsewhere
 */
class FSlateShaderResourceProxy
{
public:

	/** The start uv of the texture.  If atlased this is some subUV of the atlas, 0,0 otherwise */
	FVector2f StartUV;

	/** The size of the texture in UV space.  If atlas this some sub uv of the atlas.  1,1 otherwise */
	FVector2f SizeUV;

	/** The resource to be used for rendering */
	FSlateShaderResource* Resource;

	/** The size of the texture.  Regardless of atlasing this is the size of the actual texture */
	FIntPoint ActualSize;

	/** Shared data between resources and handles to this resource.  Is created the first time a handle is needed */
	TSharedPtr<FSlateSharedHandleData> HandleData;

	/** Default constructor. */
	FSlateShaderResourceProxy( )
		: StartUV(0.0f, 0.0f)
		, SizeUV(1.0f, 1.0f)
		, Resource(nullptr)
		, ActualSize(0, 0)
	{ }

	~FSlateShaderResourceProxy()
	{
		if( HandleData.IsValid() )
		{
			// Handles exist for this proxy.
			// Null out the proxy to invalidate all handles
			HandleData->Proxy = nullptr; 
		}
	}
};


/** 
 * Abstract base class for platform independent texture resource accessible by the shader.
 */
template <typename ResourceType>
class TSlateTexture
	: public FSlateShaderResource
{
public:

	/** Default constructor. */
	TSlateTexture( ) { }

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InShaderResource The resource to use.
	 */
	TSlateTexture( ResourceType& InShaderResource )
		: ShaderResource( InShaderResource )
	{ }

	virtual ~TSlateTexture() = default;

public:

	/**
	 * Gets the resource used by the shader.
	 *
	 * @return The resource.
	 */
	ResourceType& GetTypedResource()
	{
		return ShaderResource;
	}

public:

	// FSlateShaderResource interface
	virtual ESlateShaderResource::Type GetType() const override
	{
		return ESlateShaderResource::NativeTexture;
	}

protected:

	// Holds the resource.
	ResourceType ShaderResource;
};
