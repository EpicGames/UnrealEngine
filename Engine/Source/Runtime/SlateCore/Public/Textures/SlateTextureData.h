// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "SlateGlobals.h"

struct FImage;
struct FImageView;

DECLARE_MEMORY_STAT_EXTERN(TEXT("Texture Data Memory (CPU)"), STAT_SlateTextureDataMemory, STATGROUP_SlateMemory, SLATECORE_API);

/**
 * Holds texture data for upload to a rendering resource
 * Makes a copy of the bytes passed to it and holds ownership of the image data array.
 *
 * Note that "BytesPerPixel" is a variable but in practice this must be BGRA8-SRGB , and BytesPerPixel == 4
 */
struct FSlateTextureData
{
	FSlateTextureData( uint32 InWidth = 0, uint32 InHeight = 0, uint32 InBytesPerPixel = 0, const TArray<uint8>& InBytes = TArray<uint8>() )
		: Bytes(InBytes)
		, Width(InWidth)
		, Height(InHeight)
		, BytesPerPixel(InBytesPerPixel)
	{
		INC_MEMORY_STAT_BY( STAT_SlateTextureDataMemory, Bytes.GetAllocatedSize() );
	}

	// Move from TArray :
	FSlateTextureData(uint32 InWidth, uint32 InHeight, uint32 InBytesPerPixel, TArray<uint8>&& InBytes)
		: Bytes(InBytes)
		, Width(InWidth)
		, Height(InHeight)
		, BytesPerPixel(InBytesPerPixel)
	{
		INC_MEMORY_STAT_BY(STAT_SlateTextureDataMemory, Bytes.GetAllocatedSize());
	}

	/**
	 * Constructor to create texture data by copying from a pointer instead of an array
	 * @param InBuffer Pointer to Texture data (must contain InWidth*InHeight*InBytesPerPixel bytes).
	 * @param InWidth Width of the Texture.
	 * @param InHeight Height of the Texture.
	 * @param InBytesPerPixel Bytes per pixel of the Texture.
	 */
	FSlateTextureData( const uint8* InBuffer, uint32 InWidth, uint32 InHeight, uint32 InBytesPerPixel )
		: Width(InWidth)
		, Height(InHeight)
		, BytesPerPixel(InBytesPerPixel)
	{
		const uint32 BufferSize = Width*Height*BytesPerPixel;
		Bytes.SetNumUninitialized(BufferSize);
		if (InBuffer != nullptr)
		{
			FMemory::Memcpy(Bytes.GetData(), InBuffer, BufferSize);
		}
		INC_MEMORY_STAT_BY(STAT_SlateTextureDataMemory, Bytes.GetAllocatedSize());
	}

	FSlateTextureData( const FSlateTextureData &Other )
		: Bytes( Other.Bytes )
		, Width( Other.Width )
		, Height( Other.Height )
		, BytesPerPixel( Other.BytesPerPixel )
	{
		INC_MEMORY_STAT_BY( STAT_SlateTextureDataMemory, Bytes.GetAllocatedSize() );
	}
	
	FSlateTextureData( const FImageView &Other )
		: Bytes()
		, Width(0)
		, Height(0)
		, BytesPerPixel(0)
	{
		SetImage(Other);
	}
	
	FSlateTextureData( FImage && Other )
		: Bytes()
		, Width(0)
		, Height(0)
		, BytesPerPixel(0)
	{
		SetImage(MoveTemp(Other));
	}

	FSlateTextureData& operator=( const FSlateTextureData& Other )
	{
		if( this != &Other )
		{
			SetRawData( Other.Width, Other.Height, Other.BytesPerPixel, Other.Bytes );
		}
		return *this;
	}

	~FSlateTextureData()
	{
		DEC_MEMORY_STAT_BY( STAT_SlateTextureDataMemory, Bytes.GetAllocatedSize() );
	}

	void SetRawData( uint32 InWidth, uint32 InHeight, uint32 InBytesPerPixel, const TArray<uint8>& InBytes )
	{
		DEC_MEMORY_STAT_BY( STAT_SlateTextureDataMemory, Bytes.GetAllocatedSize() );

		Width = InWidth;
		Height = InHeight;
		BytesPerPixel = InBytesPerPixel;
		Bytes = InBytes;

		INC_MEMORY_STAT_BY( STAT_SlateTextureDataMemory, Bytes.GetAllocatedSize() );
	}
	
	// copy FImageView into the FSlateTextureData
	//	convert to BGRA8-SRGB if not already in that format
	SLATECORE_API void SetImage( const FImageView & Image );
		
	// move FImage into the FSlateTextureData
	//	convert to BGRA8-SRGB if not already in that format
	SLATECORE_API void SetImage( FImage && MoveFrom );

	void Empty()
	{
		DEC_MEMORY_STAT_BY( STAT_SlateTextureDataMemory, Bytes.GetAllocatedSize() );
		Bytes.Empty();
	}

	uint32 GetWidth() const
	{
		return Width;
	}
	
	uint32 GetHeight() const
	{
		return Height;
	}

	uint32 GetBytesPerPixel() const
	{
		return BytesPerPixel;
	}

	const TArray<uint8>& GetRawBytes() const
	{
		return Bytes;
	}

	/** Accesses the raw bytes of already sized texture data */
	uint8* GetRawBytesPtr( )
	{
		return Bytes.GetData();
	}

private:

	/** Raw uncompressed texture data (in practice always FColor, BGRA8-sRGB */
	TArray<uint8> Bytes;

	/** Width of the texture */
	uint32 Width;

	/** Height of the texture */
	uint32 Height;

	/** The number of bytes of each pixel (in practice must always be 4) */
	uint32 BytesPerPixel;
};


typedef TSharedPtr<FSlateTextureData, ESPMode::ThreadSafe> FSlateTextureDataPtr;
typedef TSharedRef<FSlateTextureData, ESPMode::ThreadSafe> FSlateTextureDataRef;
