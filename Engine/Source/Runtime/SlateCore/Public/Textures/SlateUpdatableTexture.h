// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRenderResource;
class FSlateShaderResource;
struct FSlateTextureData;

/**
 * An interface to deal with a slate texture that can be updated dynamically
 */
class FSlateUpdatableTexture
{
public:
	/** Virtual destructor */
	virtual ~FSlateUpdatableTexture() = default;

	/**
	 * Gets the interface to the underlying platform independent texture
	 *
	 * @return	FSlateShaderResource*	pointer to the shader resource
	 */
	virtual FSlateShaderResource* GetSlateResource() = 0;

	/**
	 * Gets the interface to the underlying render resource (may not always be used)
	 *
	 * @return	FRenderResource*	pointer to the render resource
	 */
	virtual FRenderResource* GetRenderResource() {return nullptr;}

	/**
	 * Deferred or Immediate cleanup of this data depending on what is required.
	 */
	virtual void Cleanup() = 0;

	/**
	 * Resize the texture.
	 *
	 * @param Width New texture width
	 * @param Height New texture height
	 */
	virtual void ResizeTexture( uint32 Width, uint32 Height ) = 0;

	/**
	 * Updates the texture contents via a byte array. 
	 * Note: This method is not thread safe so make sure you do not use the Bytes data on another after it is passed in
	 *
	 * @param Bytes Array of texture data
	 */
	virtual void UpdateTexture(const TArray<uint8>& Bytes) = 0;

	/**
	 * Updates the texture contents via a byte array making a copy first for thread safety
	 *
	 * @param Bytes Array of texture data
	 */
	virtual void UpdateTextureThreadSafe(const TArray<uint8>& Bytes) = 0;
	
	/**
	 * Update the texture from a raw byte buffer.
	 * Should only be used when integrating with third party APIs that provide a raw pointer to texture data.
	 * This method does a copy of the buffer for thread safety. 
	 * Will resize the texture if the passed in width and height is different from the current size.
	 * The passed in size must correspond to the size of the buffer and the data must be valid for the entire texture, even when
	 * passing in a Dirty rectangle, as the implementation may chose to copy a larger area than specified. 
	 * The RHI renderer currently ignores the Dirty argument completely.
	 *
	 * @param Width New texture width
	 * @param Height New texture height
	 * @param Buffer A void pointer to a byte buffer.
	 * @param Dirty An optional hint of the area to update. An empty rectangle means that the entire texture should be updated.
	 */
	virtual void UpdateTextureThreadSafeRaw(uint32 Width, uint32 Height, const void* Buffer, const FIntRect& Dirty = FIntRect()) = 0;

	/**
	* Update the texture from a provided FSlateTextureData buffer, also transferring ownership of the texture
	* @param TextureData A pointer to the provided FSlateTextureData.
	* 
	* NOTE: This function transfers ownership of the FSlateTextureData object. It will be deleted once the texture is used
	*/
	virtual void UpdateTextureThreadSafeWithTextureData(FSlateTextureData* TextureData) = 0;


	/**
	* Update the texture from a shared texture handle. The handle type/usage is render system dependent, D3D uses the D3D11 OpenSharedResource call
	*  On macOS we use an IOSurface.
	* @param TextureHandle A pointer to the opque texture handle provided by the OS.
	* @param Dirty An optional hint of the area to update. An empty rectangle means that the entire texture should be updated.
	*
	*/
	virtual void UpdateTextureThreadSafeWithKeyedTextureHandle(void* TextureHandle, int KeyLockVal, int KeyUnlockVal, const FIntRect& Dirty = FIntRect()) = 0;
};
