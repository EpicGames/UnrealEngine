// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/List.h"
#include "Layout/SlateRect.h"

#define WITH_ATLAS_DEBUGGING (WITH_EDITOR || IS_PROGRAM) && !UE_BUILD_SHIPPING

class FSlateShaderResource;

/**
 * Specifies the type of content of font atlas, based on which the texture format is determined.
 */
enum class ESlateFontAtlasContentType
{
	/** Alpha channel only (linear, formerly IsGrayscale = true) */
	Alpha,
	/** RGBA color data - sRGB color space */
	Color,
	/** Multi-channel signed distance field - linear color space */
	Msdf
};

/** 
 * Specifies how to handle texture atlas padding (when specified for the atlas). 
 * We only support one pixel of padding because we don't support mips or aniso filtering on atlas textures right now.
 */
enum ESlateTextureAtlasPaddingStyle
{
	/** Don't pad the atlas. */
	NoPadding,
	/** Dilate the texture by one pixel to pad the atlas. */
	DilateBorder,
	/** One pixel uniform padding border filled with zeros. */
	PadWithZero,
};

/** 
 * The type of thread that owns a texture atlas - this is the only thread that can safely update it 
 * If you need to update this enum, keep it in sync with the LexToString overload below.
 */
enum class ESlateTextureAtlasThreadId
{
	/** Owner thread is currently unknown */
	Unknown = -1,
	/** Atlas is owned by the game thread */
	Game = 0,
	/** Atlas is owned by the render thread */
	Render = 1,
};

SLATECORE_API const TCHAR* LexToString(ESlateTextureAtlasThreadId ThreadId);

/** 
 * Get the correct atlas thread ID based on the thread we're currently in 
 */
SLATECORE_API ESlateTextureAtlasThreadId GetCurrentSlateTextureAtlasThreadId();

/**
 * Convenience function to get the correct atlas thread ID based on the thread we're currently in as a string.
 */
SLATECORE_API const TCHAR* GetCurrentSlateTextureAtlasThreadIdAsString();

/**
 * Returns the byte size of a single pixel of a font atlas with the specified content type
 */
SLATECORE_API uint32 GetSlateFontAtlasContentBytesPerPixel(ESlateFontAtlasContentType InContentType);

/**
 * Structure holding information about where a texture is located in the atlas. Inherits a linked-list interface.
 *
 * When a slot is occupied by texture data, the remaining space in the slot (if big enough) is split off into two new (smaller) slots,
 * building a tree of texture rectangles which, instead of being stored as a tree, are flattened into two linked-lists:
 *	- AtlasEmptySlots:	A linked-list of empty slots ready for texture data - iterates in same order as a depth-first-search on a tree
 *	- AtlasUsedSlots:	An unordered linked-list of slots containing texture data
 */
struct FAtlasedTextureSlot : public TIntrusiveLinkedList<FAtlasedTextureSlot>
{
	/** The X position of the character in the texture */
	uint32 X;
	/** The Y position of the character in the texture */
	uint32 Y;
	/** The width of the character */
	uint32 Width;
	/** The height of the character */
	uint32 Height;
	/** Uniform Padding. can only be zero or one. See ESlateTextureAtlasPaddingStyle. */
	uint8 Padding;

	FAtlasedTextureSlot( uint32 InX, uint32 InY, uint32 InWidth, uint32 InHeight, uint8 InPadding )
		: TIntrusiveLinkedList<FAtlasedTextureSlot>()
		, X(InX)
		, Y(InY)
		, Width(InWidth)
		, Height(InHeight)
		, Padding(InPadding)
	{
	}
};

/**
 * Base class texture atlases in Slate
 */
class FSlateTextureAtlas
{
public:
	SLATECORE_API FSlateTextureAtlas( uint32 InWidth, uint32 InHeight, uint32 InBytesPerPixel, ESlateTextureAtlasPaddingStyle InPaddingStyle, bool bInUpdatesAfterInitialization );

	SLATECORE_API virtual ~FSlateTextureAtlas();

	/**
	 * Clears atlas cpu data.  It does not clear rendering data
	 */
	SLATECORE_API void EmptyAtlasData();

	/**
	 * Adds a texture to the atlas
	 *
	 * @param TextureWidth	Width of the texture
	 * @param TextureHeight	Height of the texture
	 * @param Data			Raw texture data
	 */
	SLATECORE_API const FAtlasedTextureSlot* AddTexture(uint32 TextureWidth, uint32 TextureHeight, const TArray<uint8>& Data);

	/** @return the width of the atlas */
	uint32 GetWidth() const { return AtlasWidth; }
	/** @return the height of the atlas */
	uint32 GetHeight() const { return AtlasHeight; }

	/** Marks the texture as dirty and needing its rendering resources updated */
	SLATECORE_API void MarkTextureDirty();
	
	/**
	 * Updates the texture used for rendering if needed
	 */
	virtual void ConditionalUpdateTexture() = 0;

	/**
	 * Releases rendering resources of this texture
	 */
	virtual void ReleaseResources() = 0;

	virtual FSlateShaderResource* GetAtlasTexture() const = 0;

#if WITH_ATLAS_DEBUGGING
	SLATECORE_API const FAtlasedTextureSlot* GetSlotAtPosition(FIntPoint InPosition) const;
#endif
protected:
	/**
	 * Finds the optimal slot for a texture in the atlas
	 * 
	 * @param Width The width of the texture we are adding
	 * @param Height The height of the texture we are adding
	 */
	SLATECORE_API const FAtlasedTextureSlot* FindSlotForTexture( uint32 InWidth, uint32 InHeight );

	/**
	 * Get the index to start looking for a free slot.
	 */
	static int32 GetFreeSlotSearchIndex(uint32 InWidth, uint32 InHeight);

	/**
	 * Adds a new slot to the free slot list.
	 */
	void AddFreeSlot(uint32 InX, uint32 InY, uint32 InWidth, uint32 InHeight);

	/**
	 * Creates enough space for a single texture the width and height of the atlas
	 */
	SLATECORE_API void InitAtlasData();

	struct FCopyRowData
	{
		/** Source data to copy */
		const uint8* SrcData;
		/** Place to copy data to */
		uint8* DestData;
		/** The row number to copy */
		uint32 SrcRow;
		/** The row number to copy to */
		uint32 DestRow;
		/** The width of a source row */
		uint32 RowWidth;
		/** The width of the source texture */
		uint32 SrcTextureWidth;
		/** The width of the dest texture */
		uint32 DestTextureWidth;
	};

	/**
	 * Copies a single row from a source texture to a dest texture,
	 * respecting the padding.
	 *
	 * @param CopyRowData	Information for how to copy a row
	 */
	SLATECORE_API void CopyRow( const FCopyRowData& CopyRowData );

	/**
	 * Zeros out a row in the dest texture (used with PaddingStyle == PadWithZero).
	 * respecting the padding.
	 *
	 * @param CopyRowData	Information for how to copy a row
	 */
	SLATECORE_API void ZeroRow( const FCopyRowData& CopyRowData );

	/** 
	 * Copies texture data into the atlas at a given slot
	 *
	 * @param SlotToCopyTo	The occupied slot in the atlas where texture data should be copied to
	 * @param Data			The data to copy into the atlas
	 */
	SLATECORE_API void CopyDataIntoSlot( const FAtlasedTextureSlot* SlotToCopyTo, const TArray<uint8>& Data );

private:
	/** Returns the amount of padding needed for the current padding style */
	inline uint8 GetPaddingAmount() const
	{
		return (PaddingStyle == ESlateTextureAtlasPaddingStyle::NoPadding) ? 0 : 1;
	}
protected:
	/** Actual texture data contained in the atlas */
	TArray<uint8> AtlasData;
	/** The list of atlas slots pointing to used texture data in the atlas */
	FAtlasedTextureSlot* AtlasUsedSlots;
	/** The list of atlas slots pointing to empty texture data in the atlas */
	TArray<FAtlasedTextureSlot*> AtlasEmptySlotsMap;
	/** Width of the atlas */
	uint32 AtlasWidth;
	/** Height of the atlas */
	uint32 AtlasHeight;
	/** Bytes per pixel in the atlas */
	uint32 BytesPerPixel;
	/** Padding style */
	ESlateTextureAtlasPaddingStyle PaddingStyle;

	/** True if this texture needs to have its rendering resources updated */
	bool bNeedsUpdate;
	/** True if this texture can update after initialziation and we should preserve the atlas slots and cpu memory */
	bool bUpdatesAfterInitialization;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** 
	 * The type of thread that owns this atlas - this is the only thread that can safely update it 
	 * NOTE: We don't use the thread ID here, as the render thread can be recreated if it gets suspended and resumed, giving it a new ID
	 */
	ESlateTextureAtlasThreadId AtlasOwnerThread;
#endif
};

struct FAtlasSlotInfo
{
	FAtlasSlotInfo()
		: AtlasSlotRect()
		, TextureName(NAME_None)
	{}

	bool operator!= (const FAtlasSlotInfo& Other) const
	{
		return AtlasSlotRect != Other.AtlasSlotRect & TextureName != Other.TextureName;
	}

	FSlateRect AtlasSlotRect;
	FName TextureName;
};


/** A factory capable of generating a texture atlas or shader resource for textures too big to be in an atlas */
class ISlateTextureAtlasFactory
{
public:
	virtual ~ISlateTextureAtlasFactory() = default;
	virtual TUniquePtr<FSlateTextureAtlas> CreateTextureAtlas(int32 AtlasSize, int32 AtlasStride, ESlateTextureAtlasPaddingStyle PaddingStyle, bool bUpdatesAfterInitialization) const = 0;

	virtual TUniquePtr<FSlateShaderResource> CreateNonAtlasedTexture(const uint32 InWidth, const uint32 InHeight, const TArray<uint8>& InRawData) const = 0;

	virtual void ReleaseTextureAtlases(const TArray<TUniquePtr<FSlateTextureAtlas>>& InTextureAtlases, const TArray<TUniquePtr<FSlateShaderResource>>& InNonAtlasedTextures, const bool bWaitForRelease) const = 0;
};

/** Parameters for flushable atlases that dictate when the atlas is allowed to flush after it becomes full */
struct FAtlasFlushParams
{
	int32 InitialMaxAtlasPagesBeforeFlushRequest = 1;
	int32 InitialMaxNonAtlasPagesBeforeFlushRequest = 1;
	int32 GrowAtlasFrameWindow = 1;
	int32 GrowNonAtlasFrameWindow = 1;
};

/** Base class for any atlas cache which has flushing logic to keep the number of in use pages small */
class FSlateFlushableAtlasCache
{
public:
	FSlateFlushableAtlasCache(const FAtlasFlushParams* InFlushParams);

	virtual ~FSlateFlushableAtlasCache() = default;

	/** 
	 * Called when this cache must be flushed 
	 * 
 	 * @param Reason A string explaining the reason the cache was flushed (generally for debugging or logging
	 */
	virtual void RequestFlushCache(const FString& Reason) = 0;

	/** Resets all counters to their initial state to start over flushing logic */
	void ResetFlushCounters();

	/** Increments counters that determine if a flush is needed.  If a flush is needed RequestFlushCache will be called from here */
	void UpdateFlushCounters(int32 NumGrayscale, int32 NumColor, int32 NumMsdf, int32 NumNonAtlased);

private:
	bool UpdateInternal(int32 CurrentNum, int32& MaxNum, int32 InitialMax, int32 FrameWindowNum);
private:
	/** Flush params that dictate when this atlas can flush.*/
	const FAtlasFlushParams* FlushParams;

	/** Number of grayscale atlas pages we can have before we request that the cache be flushed */
	int32 CurrentMaxGrayscaleAtlasPagesBeforeFlushRequest;

	/** Number of color atlas pages we can have before we request that the cache be flushed */
	int32 CurrentMaxColorAtlasPagesBeforeFlushRequest;

	/** Number of multi-channel signed distance field atlas pages we can have before we request that the cache be flushed */
	int32 CurrentMaxMsdfAtlasPagesBeforeFlushRequest;

	/** Number of non-atlased textures we can have before we request that the cache be flushed */
	int32 CurrentMaxNonAtlasedTexturesBeforeFlushRequest;

	/** The frame counter the last time the font cache was asked to be flushed */
	uint64 FrameCounterLastFlushRequest;
};

/**
 * Interface to allow the Slate atlas visualizer to query atlas page information for an atlas provider
 */
class ISlateAtlasProvider
{
public:
	/** Virtual destructor */
	virtual ~ISlateAtlasProvider() = default;

	/** Get the number of atlas pages this atlas provider has available when calling GetAtlasPageResource */
	virtual int32 GetNumAtlasPages() const = 0;

	/** Get the page resource for the given index (verify with GetNumAtlasPages) */ 
	virtual class FSlateShaderResource* GetAtlasPageResource(const int32 InIndex) const = 0;

	/** Does the page resources for the given index only contain alpha information? This affects how the atlas visualizer will sample them (verify with GetNumAtlasPages) */
	virtual bool IsAtlasPageResourceAlphaOnly(const int32 InIndex) const = 0;
	
#if WITH_ATLAS_DEBUGGING
	/** Finds a currently occupied slot at a position specified in atlas coordinates where 0,0 is the top left and the size of the atlas is bottom right */
	virtual FAtlasSlotInfo GetAtlasSlotInfoAtPosition(FIntPoint InPosition, int32 AtlasIndex) const = 0;
#endif
};


