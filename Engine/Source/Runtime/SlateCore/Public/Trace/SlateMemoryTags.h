// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/LowLevelMemTracker.h"
#include "HAL/LowLevelMemStats.h"

/**
 * This file contains LLM tags for subcategories of the UI LLM tag. 
 * 
 * For more into on each macro, please see: LowLevelMemTracker
 */

#if ENABLE_LOW_LEVEL_MEM_TRACKER

// Declare LLM tags, note that these can only be used by modules that include SlateCore
LLM_DECLARE_TAG_API(UI_Slate, SLATECORE_API);
LLM_DECLARE_TAG_API(UI_UMG, SLATECORE_API);
LLM_DECLARE_TAG_API(UI_Text, SLATECORE_API);
LLM_DECLARE_TAG_API(UI_Texture, SLATECORE_API);
LLM_DECLARE_TAG_API(UI_Style, SLATECORE_API);

// Declare stat groups per tag, definition and association with UI tag is done in .cpp
DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("UI_Slate"), STAT_UI_SlateLLM, STATGROUP_LLMFULL, SLATECORE_API);
DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("UI_UMG"), STAT_UI_UMGLLM, STATGROUP_LLMFULL, SLATECORE_API);
DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("UI_Text"), STAT_UI_TextLLM, STATGROUP_LLMFULL, SLATECORE_API);
DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("UI_Texture"), STAT_UI_TextureLLM, STATGROUP_LLMFULL, SLATECORE_API);
DECLARE_LLM_MEMORY_STAT_EXTERN(TEXT("UI_Style"), STAT_UI_StyleLLM, STATGROUP_LLMFULL, SLATECORE_API);

#endif // ENABLE_LOW_LEVEL_MEM_TRACKER