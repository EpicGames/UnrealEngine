// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Trace/Config.h"
#include "Trace/Trace.h"

#include "FastUpdate/WidgetUpdateFlags.h"
#include "Templates/UnrealTemplate.h"
#include "Widgets/InvalidateWidgetReason.h"

#if UE_TRACE_ENABLED && !IS_PROGRAM && !UE_BUILD_SHIPPING
#define UE_SLATE_TRACE_ENABLED 1
#else
#define UE_SLATE_TRACE_ENABLED 0
#endif


enum class ESlateTraceApplicationFlags : uint8
{
	None = 0,
	GlobalInvalidation = 1 << 0,
	FastWidgetPath = 1 << 1,
};
ENUM_CLASS_FLAGS(ESlateTraceApplicationFlags)


#if UE_SLATE_TRACE_ENABLED

class FSlateApplicationBase;
class FSlateTraceMetaData;
class SWidget;


UE_TRACE_CHANNEL_EXTERN(SlateChannel, SLATECORE_API);

class FSlateTrace : public FNoncopyable
{
public:
	struct FScopedWidgetPaintTrace
	{
		FScopedWidgetPaintTrace(const SWidget* InWidget);
		~FScopedWidgetPaintTrace();

		uint64 StartCycle;
		const SWidget* Widget;
		int32 StartPaintCount;
	};
	struct FScopedWidgetUpdateTrace
	{
		FScopedWidgetUpdateTrace(const SWidget* InWidget);
		~FScopedWidgetUpdateTrace();

		uint64 StartCycle;
		const SWidget* Widget;
		EWidgetUpdateFlags UpdateFlags;
	};

	SLATECORE_API static void ApplicationTickAndDrawWidgets(float DeltaTime);
	SLATECORE_API static void ApplicationRegisterTraceEvents(FSlateApplicationBase& SlateApplication);
	SLATECORE_API static void WidgetInvalidated(const SWidget* Widget, const SWidget* Investigator, EInvalidateWidgetReason Reason);
	SLATECORE_API static void RootInvalidated(const SWidget* Widget, const SWidget* Investigator);
	SLATECORE_API static void RootChildOrderInvalidated(const SWidget* Widget, const SWidget* Investigator);

	/** Called in widget ctor */
	static void AddWidget(const SWidget* Widget);

	/** Called as part of TSlateDecl widget creation (SNew, SAssignNew, SArgumentNew) */
	static void UpdateWidgetInfo(const SWidget* Widget);

	/** Called in widget dtor */
	static void RemoveWidget(const SWidget* Widget);

private:

	/** Called on all widget updates except for Add & Remove to ensure that trace WidgetInfo is populated */
	static void ConditionallyUpdateWidgetInfo(const SWidget* Widget);

	/** Called on all widget updates except for Add & Remove to ensure that trace WidgetInfo is populated */
	static void HandleOnTraceStarted(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);

	/** Used to log a particular widget update, tick timer, repaint, volatile prepass, etc. See: 'EWidgetUpdateFlags' for more info */
	static void OutputWidgetUpdate(const SWidget* Widget, uint64 StartCycle, uint64 EndCycle, EWidgetUpdateFlags UpdateFlags, uint32 AffectedCount);

private:
	/** Counter for number of traces started, used to conditionally re-send widget infos as needed */
	static uint8 TraceCounter;
};

#define UE_TRACE_SLATE_BOOKMARK(Format, ...) \
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(SlateChannel)) \
	{ \
		TRACE_BOOKMARK(Format, ##__VA_ARGS__); \
	}
#define UE_TRACE_SLATE_APPLICATION_TICK_AND_DRAW_WIDGETS(DeltaTime) \
	FSlateTrace::ApplicationTickAndDrawWidgets(DeltaTime);

#define UE_TRACE_SLATE_APPLICATION_REGISTER_TRACE_EVENTS(SlateApplication) \
	FSlateTrace::ApplicationRegisterTraceEvents(SlateApplication);
	
#define UE_TRACE_SLATE_WIDGET_ADDED(Widget) \
	FSlateTrace::AddWidget(Widget);
	
#define UE_TRACE_SLATE_WIDGET_DEBUG_INFO(Widget) \
	FSlateTrace::UpdateWidgetInfo(Widget);

#define UE_TRACE_SLATE_WIDGET_REMOVED(Widget) \
	FSlateTrace::RemoveWidget(Widget);

#define UE_TRACE_SCOPED_SLATE_WIDGET_PAINT(Widget) \
	FSlateTrace::FScopedWidgetPaintTrace _ScopedSlateWidgetPaintTrace(Widget);
	
#define UE_TRACE_SCOPED_SLATE_WIDGET_UPDATE(Widget) \
	FSlateTrace::FScopedWidgetUpdateTrace _ScopedSlateWidgetUpdateTrace(Widget);

#define UE_TRACE_SLATE_WIDGET_INVALIDATED(Widget, Investigator, InvalidateWidgetReason) \
	FSlateTrace::WidgetInvalidated(Widget, Investigator, InvalidateWidgetReason);

#define UE_TRACE_SLATE_ROOT_INVALIDATED(Widget, Investigator) \
	FSlateTrace::RootInvalidated(Widget, Investigator);

#define UE_TRACE_SLATE_ROOT_CHILDORDER_INVALIDATED(Widget, Investigator) \
	FSlateTrace::RootChildOrderInvalidated(Widget, Investigator);

#else //UE_SLATE_TRACE_ENABLED

#define UE_TRACE_SLATE_BOOKMARK(...)
#define UE_TRACE_SLATE_APPLICATION_TICK_AND_DRAW_WIDGETS(DeltaTime)
#define UE_TRACE_SLATE_APPLICATION_REGISTER_TRACE_EVENTS(SlateApplication)
#define UE_TRACE_SLATE_WIDGET_ADDED(Widget)
#define UE_TRACE_SLATE_WIDGET_DEBUG_INFO(Widget)
#define UE_TRACE_SLATE_WIDGET_REMOVED(Widget)
#define UE_TRACE_SCOPED_SLATE_WIDGET_PAINT(Widget)
#define UE_TRACE_SCOPED_SLATE_WIDGET_UPDATE(Widget)
#define UE_TRACE_SLATE_WIDGET_INVALIDATED(Widget, Investigator, InvalidateWidgetReason)
#define UE_TRACE_SLATE_ROOT_INVALIDATED(Widget, Investigator)
#define UE_TRACE_SLATE_ROOT_CHILDORDER_INVALIDATED(Widget, Investigator)

#endif //UE_SLATE_TRACE_ENABLED