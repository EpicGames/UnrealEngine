// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

/** */
namespace SlateAttributePrivate
{
	/**
	 * Attribute object
	 * InObjectType - Type of the value to store
	 * InInvalidationReasonPredicate - Predicate that returns the type of invalidation to do when the value changes (e.g layout or paint)
	 *								The invalidation can be overridden per widget. (This use memory allocation. See FSlateAttributeMetadata.)
	 * InComparePredicateType - Predicate to compare the cached value with the Getter.
	 * bInIsExternal - The attribute life is not controlled by the SWidget.
	 */
	template<typename ContainerType, typename InObjectType, typename InInvalidationReasonPredicate, typename InComparePredicateType, ESlateAttributeType InAttributeType>
	struct TSlateAttributeBase : public FSlateAttributeImpl
	{
	public:
		template<typename AttributeMemberType>
		friend struct TSlateMemberAttributeRef;

		using ObjectType = InObjectType;
		using FInvalidationReasonPredicate = InInvalidationReasonPredicate;
		using FGetter = typename TAttribute<ObjectType>::FGetter;
		using FComparePredicate = InComparePredicateType;

		static const ESlateAttributeType AttributeType = InAttributeType;
		static constexpr bool HasDefinedInvalidationReason = !std::is_same<InInvalidationReasonPredicate, FSlateAttributeNoInvalidationReason>::value;

		static_assert(std::is_same<ContainerType, SWidget>::value || std::is_same<ContainerType, ISlateAttributeContainer>::value, "The SlateAttribute container is not supported.");

		static EInvalidateWidgetReason GetInvalidationReason(const SWidget& Widget) { return FInvalidationReasonPredicate::GetInvalidationReason(Widget); }
		static EInvalidateWidgetReason GetInvalidationReason(const ISlateAttributeContainer& Container) { return FInvalidationReasonPredicate::GetInvalidationReason(Container.GetContainerWidget()); }
		static bool IdenticalTo(const SWidget& Widget, const ObjectType& Lhs, const ObjectType& Rhs) { return FComparePredicate::IdenticalTo(Widget, Lhs, Rhs); }
		static bool IdenticalTo(const ISlateAttributeContainer& Container, const ObjectType& Lhs, const ObjectType& Rhs) { return FComparePredicate::IdenticalTo(Container.GetContainerWidget(), Lhs, Rhs); }

	private:
		void VerifyOwningWidget(const SWidget& Widget) const
		{
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
			checkf(&Widget == Debug_OwningWidget || Debug_OwningWidget == nullptr,
				TEXT("The Owning Widget is not the same as used at construction. This will cause bad memory access."));
#endif
		}

		void VerifyOwningWidget(const ISlateAttributeContainer& Widget) const
		{
		}

		void VerifyNan() const
		{
#if UE_SLATE_WITH_ATTRIBUTE_NAN_DIAGNOSTIC
			if constexpr (std::is_same<float, InObjectType>::value)
			{
				ensureMsgf(FMath::IsFinite(Value), TEXT("Value contains a NaN. Initialize your float properly"));
			}
			else if constexpr (std::is_same<double, InObjectType>::value)
			{
				ensureMsgf(FMath::IsFinite(Value), TEXT("Value contains a NaN. Initialize your double properly"));
			}
			else if constexpr (std::is_same<FVector2f, InObjectType>::value)
			{
				ensureMsgf(!Value.ContainsNaN(), TEXT("Value contains a NaN. Initialize your FVector2f properly (see FVector2f::EForceInit)"));
			}
			else if constexpr (std::is_same<FVector2d, InObjectType>::value)
			{
				ensureMsgf(!Value.ContainsNaN(), TEXT("Value contains a NaN. Initialize your FVector2d properly (see FVector2d::EForceInit)"));
			}
			else if constexpr (std::is_same<FVector3f, InObjectType>::value)
			{
				ensureMsgf(!Value.ContainsNaN(), TEXT("Value contains a NaN. Initialize your FVector3f properly (see FVector3f::EForceInit)"));
			}
			else if constexpr (std::is_same<FVector3d, InObjectType>::value)
			{
				ensureMsgf(!Value.ContainsNaN(), TEXT("Value contains a NaN. Initialize your FVector3d properly (see FVector3d::EForceInit)"));
			}
			else if constexpr (std::is_same<FVector4f, InObjectType>::value)
			{
				ensureMsgf(!Value.ContainsNaN(), TEXT("Value contains a NaN. Initialize your FVector4f properly"));
			}
			else if constexpr (std::is_same<FVector4d, InObjectType>::value)
			{
				ensureMsgf(!Value.ContainsNaN(), TEXT("Value contains a NaN. Initialize your FVector4d properly"));
			}
			else if constexpr (std::is_same<FLinearColor, InObjectType>::value)
			{
				ensureMsgf(!FVector4(Value).ContainsNaN(), TEXT("Value contains a NaN. Initialize your FLinearColor properly (see FLinearColor::EForceInit)"));
			}
			else if constexpr (std::is_same<FRotator, InObjectType>::value)
			{
				ensureMsgf(!Value.ContainsNaN(), TEXT("Value contains a NaN. Initialize your FRotator properly (see FRotator::EForceInit)"));
			}
#endif
		}

	public:
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
		TSlateAttributeBase()
			: Value()
			, Debug_OwningWidget(nullptr)
		{
			VerifyNan();
		}
#else
		TSlateAttributeBase()
			: Value()
		{
			VerifyNan();
		}
#endif

		TSlateAttributeBase(const ObjectType& InValue)
			: Value(InValue)
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
			, Debug_OwningWidget(nullptr)
#endif
		{
			VerifyNan();
		}

		TSlateAttributeBase(ObjectType&& InValue)
			: Value(MoveTemp(InValue))
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
			, Debug_OwningWidget(nullptr)
#endif
		{
			VerifyNan();
		}

		TSlateAttributeBase(SWidget& Widget)
			: Value()
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
			, Debug_OwningWidget(&Widget)
#endif
		{
			VerifyNan();
		}

		TSlateAttributeBase(SWidget& Widget, const ObjectType& InValue)
			: Value(InValue)
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
			, Debug_OwningWidget(&Widget)
#endif
		{
			VerifyNan();
		}

		TSlateAttributeBase(SWidget& Widget, ObjectType&& InValue)
			: Value(MoveTemp(InValue))
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
			, Debug_OwningWidget(&Widget)
#endif
		{
			VerifyNan();
		}

		TSlateAttributeBase(SWidget& Widget, const FGetter& Getter, const ObjectType& InitialValue)
			: Value(InitialValue)
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
			, Debug_OwningWidget(&Widget)
#endif
		{
			VerifyNan();
			if (Getter.IsBound())
			{
				ConstructWrapper(Widget, Getter);
			}
		}

		TSlateAttributeBase(SWidget& Widget, const FGetter& Getter, ObjectType&& InitialValue)
			: Value(MoveTemp(InitialValue))
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
			, Debug_OwningWidget(&Widget)
#endif
		{
			VerifyNan();
			if (Getter.IsBound())
			{
				ConstructWrapper(Widget, Getter);
			}
		}

		TSlateAttributeBase(SWidget& Widget, FGetter&& Getter, const ObjectType& InitialValue)
			: Value(InitialValue)
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
			, Debug_OwningWidget(&Widget)
#endif
		{
			VerifyNan();
			if (Getter.IsBound())
			{
				ConstructWrapper(Widget, MoveTemp(Getter));
			}
		}

		TSlateAttributeBase(SWidget& Widget, FGetter&& Getter, ObjectType&& InitialValue)
			: Value(MoveTemp(InitialValue))
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
			, Debug_OwningWidget(&Widget)
#endif
		{
			VerifyNan();
			if (Getter.IsBound())
			{
				ConstructWrapper(Widget, MoveTemp(Getter));
			}
		}

		TSlateAttributeBase(SWidget& Widget, const TAttribute<ObjectType>& Attribute, const ObjectType& InitialValue)
			: Value((Attribute.IsSet() && !Attribute.IsBound()) ? Attribute.Get() : InitialValue)
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
			, Debug_OwningWidget(&Widget)
#endif
		{
			VerifyNan();
			if (Attribute.IsBound())
			{
				ConstructWrapper(Widget, Attribute.GetBinding());
			}
		}

		TSlateAttributeBase(SWidget& Widget, TAttribute<ObjectType>&& Attribute, ObjectType&& InitialValue)
			: Value((Attribute.IsSet() && !Attribute.IsBound()) ? MoveTemp(Attribute.Steal().template Get<ObjectType>()) : MoveTemp(InitialValue))
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
			, Debug_OwningWidget(&Widget)
#endif
		{
			VerifyNan();
			if (Attribute.IsBound())
			{
				ConstructWrapper(Widget, MoveTemp(Attribute.Steal().template Get<FGetter>()));
			}
		}

	public:
		/** @return the SlateAttribute cached value. If the SlateAttribute is bound, the value will be cached at the end of the every frame. */
		[[nodiscard]] const ObjectType& Get() const
		{
			return Value;
		}

		/** Update the cached value and invalidate the widget if needed. */
		void UpdateNow(ContainerType& Widget)
		{
			VerifyOwningWidget(Widget);
			ProtectedUpdateNow(Widget, InAttributeType);
		}

	public:
		/**
		 * Unbind the SlateAttribute and set its value. It may invalidate the Widget if the value is different.
		 * @return true if the value is considered different and an invalidation occurred.
		 */
		bool Set(ContainerType& Widget, const ObjectType& NewValue)
		{
			VerifyOwningWidget(Widget);
			ProtectedUnregisterAttribute(Widget, InAttributeType);

			const bool bIsIdentical = IdenticalTo(Widget, Value, NewValue);
			if (!bIsIdentical)
			{
				Value = NewValue;
				ProtectedInvalidateWidget(Widget, InAttributeType, GetInvalidationReason(Widget));
			}
			return !bIsIdentical;
		}

		/**
		 * Unbind the SlateAttribute and set its value. It may invalidate the Widget if the value is different.
		 * @return true if the value is considered different and an invalidation occurred.
		 */
		bool Set(ContainerType& Widget, ObjectType&& NewValue)
		{
			VerifyOwningWidget(Widget);
			ProtectedUnregisterAttribute(Widget, InAttributeType);

			const bool bIsIdentical = IdenticalTo(Widget, Value, NewValue);
			if (!bIsIdentical)
			{
				Value = MoveTemp(NewValue);
				ProtectedInvalidateWidget(Widget, InAttributeType, GetInvalidationReason(Widget));
			}
			return !bIsIdentical;
		}

	public:
		/**
		 * Bind the SlateAttribute to the Getter function.
		 * (If enabled) Update the value from the Getter. If the value is different, then invalidate the widget.
		 * The SlateAttribute will now be updated every frame from the Getter.
		 */
		void Bind(ContainerType& Widget, const FGetter& Getter)
		{
			VerifyOwningWidget(Widget);
			if (Getter.IsBound())
			{
				AssignBinding(Widget, Getter);
			}
			else
			{
				ProtectedUnregisterAttribute(Widget, InAttributeType);
			}
		}

		/**
		 * Bind the SlateAttribute to the Getter function.
		 * (If enabled) Update the value from the Getter. If the value is different, then invalidate the widget.
		 * The SlateAttribute will now be updated every frame from the Getter.
		 */
		void Bind(ContainerType& Widget, FGetter&& Getter)
		{
			VerifyOwningWidget(Widget);
			if (Getter.IsBound())
			{
				AssignBinding(Widget, MoveTemp(Getter));
			}
			else
			{
				ProtectedUnregisterAttribute(Widget, InAttributeType);
			}
		}

		/**
		 * Bind the SlateAttribute to the newly create getter function.
		 * (If enabled) Update the value from the getter. If the value is different, then invalidate the widget.
		 * The SlateAttribute will now be updated every frame from the Getter.
		 */
		template<typename WidgetType, typename U = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value && std::is_same<ContainerType, SWidget>::value>::type>
		void Bind(WidgetType& Widget, typename FGetter::template TConstMethodPtr<WidgetType> MethodPtr)
		{
			Bind(Widget, FGetter::CreateSP(&Widget, MethodPtr));
		}

		/**
		 * Bind the SlateAttribute to the Attribute Getter function (if it exist).
		 * (If enabled) Update the value from the getter. If the value is different, then invalidate the widget.
		 * The SlateAttribute will now be updated every frame from the Getter.
		 * Or
		 * Set the SlateAttribute's value if the Attribute is not bound but is set.
		 * This will Unbind any previously bound getter.
		 * If the value is different, then invalidate the widget.
		 * Or
		 * Unbind the SlateAttribute if the Attribute is not bound and not set.
		 * @see Set
		 * @return true if the getter was assigned or if the value is considered different and an invalidation occurred.
		 */
		bool Assign(ContainerType& Widget, const TAttribute<ObjectType>& OtherAttribute, ESlateAttributeBindAction Actions = UE_SLATE_WITH_ATTRIBUTE_DEFAULT_INITIALIZATION_ACTION)
		{
			VerifyOwningWidget(Widget);
			if (OtherAttribute.IsBound())
			{
				return AssignBinding(Widget, OtherAttribute.GetBinding(), Actions);
			}
			else if (OtherAttribute.IsSet())
			{
				return Set(Widget, OtherAttribute.Get());
			}
			else
			{
				ProtectedUnregisterAttribute(Widget, InAttributeType);
				return false;
			}
		}

		bool Assign(ContainerType& Widget, TAttribute<ObjectType>&& OtherAttribute, ESlateAttributeBindAction Actions = UE_SLATE_WITH_ATTRIBUTE_DEFAULT_INITIALIZATION_ACTION)
		{
			VerifyOwningWidget(Widget);
			if (OtherAttribute.IsBound())
			{
				return AssignBinding(Widget, MoveTemp(OtherAttribute.Steal().template Get<FGetter>()), Actions);
			}
			else if (OtherAttribute.IsSet())
			{
				return Set(Widget, MoveTemp(OtherAttribute.Steal().template Get<ObjectType>()));
			}
			else
			{
				ProtectedUnregisterAttribute(Widget, InAttributeType);
				return false;
			}
		}

		bool Assign(ContainerType& Widget, const TAttribute<ObjectType>& OtherAttribute, const ObjectType& DefaultValue, ESlateAttributeBindAction Actions = UE_SLATE_WITH_ATTRIBUTE_DEFAULT_INITIALIZATION_ACTION)
		{
			VerifyOwningWidget(Widget);
			if (OtherAttribute.IsBound())
			{
				return AssignBinding(Widget, OtherAttribute.GetBinding(), Actions);
			}
			else if (OtherAttribute.IsSet())
			{
				return Set(Widget, OtherAttribute.Get());
			}
			else
			{
				return Set(Widget, DefaultValue);
			}
		}

		bool Assign(ContainerType& Widget, TAttribute<ObjectType>&& OtherAttribute, const ObjectType& DefaultValue, ESlateAttributeBindAction Actions = UE_SLATE_WITH_ATTRIBUTE_DEFAULT_INITIALIZATION_ACTION)
		{
			VerifyOwningWidget(Widget);
			if (OtherAttribute.IsBound())
			{
				return AssignBinding(Widget, MoveTemp(OtherAttribute.Steal().template Get<FGetter>()), Actions);
			}
			else if (OtherAttribute.IsSet())
			{
				return Set(Widget, MoveTemp(OtherAttribute.Steal().template Get<ObjectType>()));
			}
			else
			{
				return Set(Widget, DefaultValue);
			}
		}

		bool Assign(ContainerType& Widget, const TAttribute<ObjectType>& OtherAttribute, ObjectType&& DefaultValue, ESlateAttributeBindAction Actions = UE_SLATE_WITH_ATTRIBUTE_DEFAULT_INITIALIZATION_ACTION)
		{
			VerifyOwningWidget(Widget);
			if (OtherAttribute.IsBound())
			{
				return AssignBinding(Widget, OtherAttribute.GetBinding(), Actions);
			}
			else if (OtherAttribute.IsSet())
			{
				return Set(Widget, OtherAttribute.Get());
			}
			else
			{
				return Set(Widget, MoveTemp(DefaultValue));
			}
		}

		bool Assign(ContainerType& Widget, TAttribute<ObjectType>&& OtherAttribute, ObjectType&& DefaultValue, ESlateAttributeBindAction Actions = UE_SLATE_WITH_ATTRIBUTE_DEFAULT_INITIALIZATION_ACTION)
		{
			VerifyOwningWidget(Widget);
			if (OtherAttribute.IsBound())
			{
				return AssignBinding(Widget, MoveTemp(OtherAttribute.Steal().template Get<FGetter>()), Actions);
			}
			else if (OtherAttribute.IsSet())
			{
				return Set(Widget, MoveTemp(OtherAttribute.Steal().template Get<ObjectType>()));
			}
			else
			{
				return Set(Widget, MoveTemp(DefaultValue));
			}
		}

		/**
		 * Remove the Getter function.
		 * The Slate Attribute will not be updated anymore and will keep its current cached value.
		 */
		void Unbind(ContainerType& Widget)
		{
			VerifyOwningWidget(Widget);
			ProtectedUnregisterAttribute(Widget, InAttributeType);
		}

	public:
		/** Build a Attribute from this SlateAttribute. */
		[[nodiscard]] TAttribute<ObjectType> ToAttribute(const ContainerType& Widget) const
		{
			if (ISlateAttributeGetter* Delegate = ProtectedFindGetter(Widget, InAttributeType))
			{
				return TAttribute<ObjectType>::Create(static_cast<FSlateAttributeGetterWrapper<TSlateAttributeBase>*>(Delegate)->GetDelegate());
			}
			return TAttribute<ObjectType>(Get());
		}

		/** @return True if the SlateAttribute is bound to a getter function. */
		[[nodiscard]] bool IsBound(const ContainerType& Widget) const
		{
			VerifyOwningWidget(Widget);
			return ProtectedIsBound(Widget, InAttributeType);
		}

		/** @return True if they have the same Getter or the same value. */
		[[nodiscard]] bool IdenticalTo(const ContainerType& Widget, const TSlateAttributeBase& Other) const
		{
			VerifyOwningWidget(Widget);
			FDelegateHandle ThisDelegateHandle = ProtectedFindGetterHandle(Widget, InAttributeType);
			FDelegateHandle OthersDelegateHandle = ProtectedFindGetterHandle(Widget, InAttributeType);
			if (ThisDelegateHandle == OthersDelegateHandle)
			{
				if (ThisDelegateHandle.IsValid())
				{
					return true;
				}
				return IdenticalTo(Widget, Get(), Other.Get());
			}
			return false;
		}

		/** @return True if they have the same Getter or, if the Attribute is set, the same value. */
		[[nodiscard]] bool IdenticalTo(const ContainerType& Widget, const TAttribute<ObjectType>& Other) const
		{
			VerifyOwningWidget(Widget);
			FDelegateHandle ThisDelegateHandle = ProtectedFindGetterHandle(Widget, InAttributeType);
			if (Other.IsBound())
			{
				return Other.GetBinding().GetHandle() == ThisDelegateHandle;
			}
			return !ThisDelegateHandle.IsValid() && Other.IsSet() && IdenticalTo(Widget, Get(), Other.Get());
		}

	private:
		void ConstructWrapper(ContainerType& Widget, const FGetter& Getter, ESlateAttributeBindAction Actions = UE_SLATE_WITH_ATTRIBUTE_DEFAULT_INITIALIZATION_ACTION)
		{
			TUniquePtr<ISlateAttributeGetter> Wrapper = MakeUniqueGetter(*this, Getter);
			ProtectedRegisterAttribute(Widget, InAttributeType, MoveTemp(Wrapper));
			if (EnumHasAllFlags(Actions, ESlateAttributeBindAction::Initialize))
			{
				ProtectedUpdateNow(Widget, InAttributeType);
			}
		}

		void ConstructWrapper(ContainerType& Widget, FGetter&& Getter, ESlateAttributeBindAction Actions = UE_SLATE_WITH_ATTRIBUTE_DEFAULT_INITIALIZATION_ACTION)
		{
			TUniquePtr<ISlateAttributeGetter> Wrapper = MakeUniqueGetter(*this, MoveTemp(Getter));
			ProtectedRegisterAttribute(Widget, InAttributeType, MoveTemp(Wrapper));
			if (EnumHasAllFlags(Actions, ESlateAttributeBindAction::Initialize))
			{
				ProtectedUpdateNow(Widget, InAttributeType);
			}
		}

		bool AssignBinding(ContainerType& Widget, const FGetter& Getter, ESlateAttributeBindAction Actions = UE_SLATE_WITH_ATTRIBUTE_DEFAULT_INITIALIZATION_ACTION)
		{
			const FDelegateHandle PreviousGetterHandle = ProtectedFindGetterHandle(Widget, InAttributeType);
			if (PreviousGetterHandle != Getter.GetHandle())
			{
				ConstructWrapper(Widget, Getter, Actions);
				return true;
			}
			return false;
		}

		bool AssignBinding(ContainerType& Widget, FGetter&& Getter, ESlateAttributeBindAction Actions = UE_SLATE_WITH_ATTRIBUTE_DEFAULT_INITIALIZATION_ACTION)
		{
			const FDelegateHandle PreviousGetterHandle = ProtectedFindGetterHandle(Widget, InAttributeType);
			if (PreviousGetterHandle != Getter.GetHandle())
			{
				ConstructWrapper(Widget, MoveTemp(Getter), Actions);
				return true;
			}
			return false;
		}

	private:
		template<typename SlateAttributeType>
		class FSlateAttributeGetterWrapper final : public ISlateAttributeGetter
		{
		public:
			using ObjectType = typename SlateAttributeType::ObjectType;
			using FGetter = typename TAttribute<ObjectType>::FGetter;
			using FComparePredicate = typename SlateAttributeType::FComparePredicate;

			FSlateAttributeGetterWrapper() = delete;
			FSlateAttributeGetterWrapper(const FSlateAttributeGetterWrapper&) = delete;
			FSlateAttributeGetterWrapper& operator= (const FSlateAttributeGetterWrapper&) = delete;
			virtual ~FSlateAttributeGetterWrapper() = default;

		public:
			FSlateAttributeGetterWrapper(SlateAttributeType& InOwningAttribute, const FGetter& InGetterDelegate)
				: Getter(InGetterDelegate)
				, Attribute(&InOwningAttribute)
			{
			}
			FSlateAttributeGetterWrapper(SlateAttributeType& InOwningAttribute, FGetter&& InGetterDelegate)
				: Getter(MoveTemp(InGetterDelegate))
				, Attribute(&InOwningAttribute)
			{
			}

			virtual FUpdateAttributeResult UpdateAttribute(const SWidget& Widget) override
			{
				if (Getter.IsBound())
				{
					ObjectType NewValue = Getter.Execute();

					const bool bIsIdentical = Attribute->IdenticalTo(Widget, Attribute->Value, NewValue);
					if (!bIsIdentical)
					{
						// Set the value on the widget
						Attribute->Value = MoveTemp(NewValue);
						return Attribute->GetInvalidationReason(Widget);
					}
				}
				return FUpdateAttributeResult();
			}

			virtual const FSlateAttributeBase& GetAttribute() const override
			{
				return *Attribute;
			}

			virtual void SetAttribute(FSlateAttributeBase& InBase) override
			{
				Attribute = &static_cast<SlateAttributeType&>(InBase);
			}

			virtual FDelegateHandle GetDelegateHandle() const override
			{
				return Getter.GetHandle();
			}

			const FGetter& GetDelegate() const
			{
				return Getter;
			}

		private:
			/** Getter function to fetch the new value of the SlateAttribute. */
			FGetter Getter;
			/** The SlateAttribute of the SWidget owning the value. */
			SlateAttributeType* Attribute;
		};

	private:
		static TUniquePtr<ISlateAttributeGetter> MakeUniqueGetter(TSlateAttributeBase& Attribute, const FGetter& Getter)
		{
			return MakeUnique<FSlateAttributeGetterWrapper<TSlateAttributeBase>>(Attribute, Getter);
		}

		static TUniquePtr<ISlateAttributeGetter> MakeUniqueGetter(TSlateAttributeBase& Attribute, FGetter&& Getter)
		{
			return MakeUnique<FSlateAttributeGetterWrapper<TSlateAttributeBase>>(Attribute, MoveTemp(Getter));
		}

	private:
		ObjectType Value;

	protected:
#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
		void Debug_SetOwningWidget(SWidget* InWidget)
		{
			Debug_OwningWidget = InWidget;
		}

		SWidget* Debug_OwningWidget;
#endif
	};


} // SlateAttributePrivate
