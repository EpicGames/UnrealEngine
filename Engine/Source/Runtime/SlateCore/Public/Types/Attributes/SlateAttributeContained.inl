// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>


class SWidget;


/** */
namespace SlateAttributePrivate
{
	/**
	 *
	 */
	template<typename InObjectType, typename InInvalidationReasonPredicate, typename InComparePredicate>
	struct TSlateContainedAttribute : public TSlateAttributeBase<ISlateAttributeContainer, InObjectType, InInvalidationReasonPredicate, InComparePredicate, ESlateAttributeType::Contained>
	{
	private:
		using Super = TSlateAttributeBase<ISlateAttributeContainer, InObjectType, InInvalidationReasonPredicate, InComparePredicate, ESlateAttributeType::Contained>;

		template<typename ContainerType, typename U = typename std::enable_if<std::is_base_of<ISlateAttributeContainer, ContainerType>::value>::type>
		static void VerifyAttributeAddress(const ContainerType& Container, const TSlateContainedAttribute& Self)
		{
			checkf((UPTRINT)&Self >= (UPTRINT)&Container && (UPTRINT)&Self < (UPTRINT)&Container + sizeof(ContainerType),
				TEXT("Use TAttribute or TSlateManagedAttribute instead. See SlateAttribute.h for more info."));
			static_assert(!Super::HasDefinedInvalidationReason, "Define the Invalidation Reason in SlateAttributeDescriptor.");
		}

	public:
		using FGetter = typename Super::FGetter;
		using ObjectType = typename Super::ObjectType;

	public:
		//~ You can only register Attribute that are defined in a ISlateAttributeContainerOwner.
		//~ Use the constructor with a ISlateAttributeContainerOwner pointer.
		//~ See documentation in SlateAttribute.h for more info.
		TSlateContainedAttribute() = delete;
		TSlateContainedAttribute(const TSlateContainedAttribute&) = delete;
		TSlateContainedAttribute(TSlateContainedAttribute&&) = delete;
		TSlateContainedAttribute& operator=(const TSlateContainedAttribute&) = delete;
		TSlateContainedAttribute& operator=(TSlateContainedAttribute&&) = delete;

		template<typename ContainerType, typename U = typename std::enable_if<std::is_base_of<ISlateAttributeContainer, ContainerType>::value>::type>
		explicit TSlateContainedAttribute(ContainerType& Container)
		{
			VerifyAttributeAddress(Container, *this);
		}

		template<typename ContainerType, typename U = typename std::enable_if<std::is_base_of<ISlateAttributeContainer, ContainerType>::value>::type>
		explicit TSlateContainedAttribute(ContainerType& Container, const ObjectType& InValue)
			: Super(InValue)
		{
			VerifyAttributeAddress(Container, *this);
		}

		template<typename ContainerType, typename U = typename std::enable_if<std::is_base_of<ISlateAttributeContainer, ContainerType>::value>::type>
		explicit TSlateContainedAttribute(ContainerType& Container, ObjectType&& InValue)
			: Super(MoveTemp(InValue))
		{
			VerifyAttributeAddress(Container, *this);
		}
	};
} // SlateAttributePrivate
