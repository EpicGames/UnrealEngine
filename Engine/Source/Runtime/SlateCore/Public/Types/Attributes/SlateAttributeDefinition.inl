// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Templates/EqualTo.h"
#include "Widgets/InvalidateWidgetReason.h"

#include <type_traits>


class SWidget;

/** */
namespace SlateAttributePrivate
{
	/** Predicate used to identify if the InvalidationWidgetReason is defined in the attribute descriptor. */
	struct FSlateAttributeNoInvalidationReason
	{
		static constexpr EInvalidateWidgetReason GetInvalidationReason(const SWidget&) { return EInvalidateWidgetReason::None; }
	};


	/** */
	enum class ESlateAttributeType : uint8
	{
		Member		= 0,	// Member of a SWidget (are not allowed to move).
		Managed		= 1,	// External to the SWidget, global variable or member that can moved.
		Contained		= 2,	// Inside a FSlot or other container that use dynamic memory (always attached to one and only one SWidget).
	};


	class ISlateAttributeGetter;
	template<typename ContainerType, typename ObjectType, typename InvalidationReasonPredicate, typename FComparePredicate, ESlateAttributeType AttributeType>
	struct TSlateAttributeBase;
	template<typename AttributeMemberType>
	struct TSlateMemberAttributeRef;


	/** Interface for structure that can be used to contain a SlateAttribute instead of a SWidget. */
	class ISlateAttributeContainer
	{
	public:
		virtual SWidget& GetContainerWidget() const = 0;
		virtual FName GetContainerName() const = 0;
		virtual uint32 GetContainerSortOrder() const = 0;

	protected:
		SLATECORE_API void RemoveContainerWidget(SWidget& Widget);
		SLATECORE_API void UpdateContainerSortOrder(SWidget& Widget);
	};


	/** */
	class ISlateAttributeGetter
	{
	public:
		struct FUpdateAttributeResult
		{
			FUpdateAttributeResult(EInvalidateWidgetReason InInvalidationReason)
				: InvalidationReason(InInvalidationReason)
				, bInvalidationRequested(true)
			{}
			FUpdateAttributeResult()
				: InvalidationReason(EInvalidateWidgetReason::None)
				, bInvalidationRequested(false)
			{}
			EInvalidateWidgetReason InvalidationReason;
			bool bInvalidationRequested;
		};


		virtual FUpdateAttributeResult UpdateAttribute(const SWidget& Widget) = 0;
		virtual const FSlateAttributeBase& GetAttribute() const = 0;
		virtual void SetAttribute(FSlateAttributeBase&) = 0;
		virtual FDelegateHandle GetDelegateHandle() const = 0;
		virtual ~ISlateAttributeGetter() = default;
	};


	/** */
	struct FSlateAttributeImpl : public FSlateAttributeBase
	{
	protected:
		SLATECORE_API void ProtectedUnregisterAttribute(SWidget& Widget, ESlateAttributeType AttributeType) const;
		SLATECORE_API void ProtectedRegisterAttribute(SWidget& Widget, ESlateAttributeType AttributeType, TUniquePtr<ISlateAttributeGetter>&& Wrapper);
		SLATECORE_API void ProtectedInvalidateWidget(SWidget& Widget, ESlateAttributeType AttributeType, EInvalidateWidgetReason InvalidationReason) const;
		SLATECORE_API bool ProtectedIsBound(const SWidget& Widget, ESlateAttributeType AttributeType) const;
		SLATECORE_API ISlateAttributeGetter* ProtectedFindGetter(const SWidget& Widget, ESlateAttributeType AttributeType) const;
		SLATECORE_API FDelegateHandle ProtectedFindGetterHandle(const SWidget& Widget, ESlateAttributeType AttributeType) const;
		SLATECORE_API void ProtectedUpdateNow(SWidget& Widget, ESlateAttributeType AttributeType);

		//~ For Member
		SLATECORE_API bool ProtectedIsWidgetInDestructionPath(SWidget* Widget) const;
		SLATECORE_API bool ProtectedIsImplemented(const SWidget& Widget) const;

		//~ For Manage
		SLATECORE_API void ProtectedMoveAttribute(SWidget& Widget, ESlateAttributeType AttributeType, const FSlateAttributeBase* Other);

		//~ For Contain
		SLATECORE_API void ProtectedUnregisterAttribute(ISlateAttributeContainer& Widget, ESlateAttributeType AttributeType) const;
		SLATECORE_API void ProtectedRegisterAttribute(ISlateAttributeContainer& Widget, ESlateAttributeType AttributeType, TUniquePtr<ISlateAttributeGetter>&& Wrapper);
		SLATECORE_API void ProtectedInvalidateWidget(ISlateAttributeContainer& Widget, ESlateAttributeType AttributeType, EInvalidateWidgetReason InvalidationReason) const;
		SLATECORE_API bool ProtectedIsBound(const ISlateAttributeContainer& Widget, ESlateAttributeType AttributeType) const;
		SLATECORE_API ISlateAttributeGetter* ProtectedFindGetter(const ISlateAttributeContainer& Widget, ESlateAttributeType AttributeType) const;
		SLATECORE_API FDelegateHandle ProtectedFindGetterHandle(const ISlateAttributeContainer& Widget, ESlateAttributeType AttributeType) const;
		SLATECORE_API void ProtectedUpdateNow(ISlateAttributeContainer& Widget, ESlateAttributeType AttributeType);
	};

} // SlateAttributePrivate
