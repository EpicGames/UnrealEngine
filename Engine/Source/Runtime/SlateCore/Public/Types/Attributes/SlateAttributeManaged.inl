// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** */
namespace SlateAttributePrivate
{
	/**
	 *
	 */
	template<typename InObjectType, typename InInvalidationReasonPredicate, typename InComparePredicate>
	struct TSlateManagedAttribute : protected TSlateAttributeBase<SWidget, InObjectType, InInvalidationReasonPredicate, InComparePredicate, ESlateAttributeType::Managed>
	{
	private:
		using Super = TSlateAttributeBase<SWidget, InObjectType, InInvalidationReasonPredicate, InComparePredicate, ESlateAttributeType::Managed>;

	public:
		using FGetter = typename Super::FGetter;
		using ObjectType = typename Super::ObjectType;

		static EInvalidateWidgetReason GetInvalidationReason(const SWidget& Widget) { return Super::GetInvalidationReason(Widget); }
		static bool IdenticalTo(const SWidget& Widget, const ObjectType& Lhs, const ObjectType& Rhs) { return Super::IdenticalTo(Widget, Lhs, Rhs); }

	public:
		TSlateManagedAttribute() = delete;
		TSlateManagedAttribute(const TSlateManagedAttribute&) = delete;
		TSlateManagedAttribute& operator=(const TSlateManagedAttribute&) = delete;

		~TSlateManagedAttribute()
		{
			Unbind();
		}

		TSlateManagedAttribute(TSlateManagedAttribute&& Other)
			: Super(MoveTemp(Other))
			, ManagedWidget(MoveTemp(Other.ManagedWidget))
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				FSlateAttributeImpl::ProtectedMoveAttribute(*Pin.Get(), ESlateAttributeType::Managed, &Other);
			}
		}

		TSlateManagedAttribute& operator=(TSlateManagedAttribute&& Other)
		{
			Super::operator=(MoveTemp(Other));
			ManagedWidget = MoveTemp(Other.ManagedWidget);

			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				FSlateAttributeImpl::ProtectedMoveAttribute(*Pin.Get(), ESlateAttributeType::Managed, &Other);
			}
			return *this;
		}

		explicit TSlateManagedAttribute(TSharedRef<SWidget> Widget)
			: Super(Widget.Get())
			, ManagedWidget(Widget)
		{ }

		explicit TSlateManagedAttribute(TSharedRef<SWidget> Widget, const ObjectType& InValue)
			: Super(Widget.Get(), InValue)
			, ManagedWidget(Widget)
		{
		}

		explicit TSlateManagedAttribute(TSharedRef<SWidget> Widget, ObjectType&& InValue)
			: Super(Widget.Get(), MoveTemp(InValue))
			, ManagedWidget(Widget)
		{
		}

		explicit TSlateManagedAttribute(TSharedRef<SWidget> Widget, const FGetter& Getter, const ObjectType& InitialValue)
			: Super(Widget.Get(), Getter, InitialValue)
			, ManagedWidget(Widget)
		{
		}

		explicit TSlateManagedAttribute(TSharedRef<SWidget> Widget, const FGetter& Getter, ObjectType&& InitialValue)
			: Super(Widget.Get(), Getter, MoveTemp(InitialValue))
			, ManagedWidget(Widget)
		{
		}

		explicit TSlateManagedAttribute(TSharedRef<SWidget> Widget, FGetter&& Getter, const ObjectType& InitialValue)
			: Super(Widget.Get(), MoveTemp(Getter), InitialValue)
			, ManagedWidget(Widget)
		{
		}

		explicit TSlateManagedAttribute(TSharedRef<SWidget> Widget, FGetter&& Getter, ObjectType&& InitialValue)
			: Super(Widget.Get(), MoveTemp(Getter), MoveTemp(InitialValue))
			, ManagedWidget(Widget)
		{
		}

		explicit TSlateManagedAttribute(TSharedRef<SWidget> Widget, const TAttribute<ObjectType>& Attribute, const ObjectType& InitialValue)
			: Super(Widget.Get(), Attribute, InitialValue)
			, ManagedWidget(Widget)
		{
		}

		explicit TSlateManagedAttribute(TSharedRef<SWidget> Widget, TAttribute<ObjectType>&& Attribute, ObjectType&& InitialValue)
			: Super(Widget.Get(), MoveTemp(Attribute), MoveTemp(InitialValue))
			, ManagedWidget(Widget)
		{
		}

	public:
		const ObjectType& Get() const
		{
			return Super::Get();
		}

		void UpdateNow()
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				Super::UpdateNow(*Pin.Get());
			}
		}

	public:
		void Set(const ObjectType& NewValue)
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				Super::Set(*Pin.Get(), NewValue);
			}
		}

		void Set(ObjectType&& NewValue)
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				Super::Set(*Pin.Get(), MoveTemp(NewValue));
			}
		}

		void Bind(const FGetter& Getter)
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				Super::Bind(*Pin.Get(), Getter);
			}
		}

		void Bind(FGetter&& Getter)
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				Super::Bind(*Pin.Get(), MoveTemp(Getter));
			}
		}

		void Assign(const TAttribute<ObjectType>& OtherAttribute)
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				Super::Assign(*Pin.Get(), OtherAttribute);
			}
		}

		void Assign(TAttribute<ObjectType>&& OtherAttribute)
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				Super::Assign(*Pin.Get(), MoveTemp(OtherAttribute));
			}
		}

		void Assign(const TAttribute<ObjectType>& OtherAttribute, const ObjectType& DefaultValue)
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				Super::Assign(*Pin.Get(), OtherAttribute, DefaultValue);
			}
		}

		void Assign(const TAttribute<ObjectType>& OtherAttribute, ObjectType&& DefaultValue)
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				Super::Assign(*Pin.Get(), OtherAttribute, MoveTemp(DefaultValue));
			}
		}

		void Assign(TAttribute<ObjectType>&& OtherAttribute, const ObjectType& DefaultValue)
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				Super::Assign(*Pin.Get(), MoveTemp(OtherAttribute), DefaultValue);
			}
		}

		void Assign(TAttribute<ObjectType>&& OtherAttribute, ObjectType&& DefaultValue)
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				Super::Assign(*Pin.Get(), MoveTemp(OtherAttribute), MoveTemp(DefaultValue));
			}
		}

		void Unbind()
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				Super::Unbind(*Pin.Get());
			}
		}

	public:
		bool IsBound() const
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				return Super::IsBound(*Pin.Get());
			}
			return false;
		}

		bool IdenticalTo(const TSlateManagedAttribute& Other) const
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				return Super::IdenticalTo(*Pin.Get(), Other);
			}
			return false;
		}

		bool IdenticalTo(const TAttribute<ObjectType>& Other) const
		{
			if (TSharedPtr<SWidget> Pin = ManagedWidget.Pin())
			{
				return Super::IdenticalTo(*Pin.Get(), Other);
			}
			return false;
		}

	private:
		TWeakPtr<SWidget> ManagedWidget;
	};

} // SlateAttributePrivate
