// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** */
namespace SlateAttributePrivate
{
	/**
	 *
	 */
	template<typename InObjectType, typename InInvalidationReasonPredicate, typename InComparePredicate>
	struct TSlateMemberAttribute : public TSlateAttributeBase<SWidget, InObjectType, InInvalidationReasonPredicate, InComparePredicate, ESlateAttributeType::Member>
	{
	private:
		using Super = TSlateAttributeBase<SWidget, InObjectType, InInvalidationReasonPredicate, InComparePredicate, ESlateAttributeType::Member>;

		template<typename WidgetType, typename U = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value>::type>
		static void VerifyAttributeAddress(const WidgetType& Widget, const TSlateMemberAttribute& Self)
		{
			checkf((UPTRINT)&Self >= (UPTRINT)&Widget && (UPTRINT)&Self < (UPTRINT)&Widget + sizeof(WidgetType),
				TEXT("Use TAttribute or TSlateManagedAttribute instead. See SlateAttribute.h for more info."));
			ensureAlwaysMsgf((Super::HasDefinedInvalidationReason || Self.ProtectedIsImplemented(Widget)),
				TEXT("The TSlateAttribute could not be found in the SlateAttributeDescriptor.\n")
				TEXT("Use the SLATE_DECLARE_WIDGET and add the attribute in PrivateRegisterAttributes,\n")
				TEXT("Or use TSlateAttribute with a valid Invalidation Reason instead."));
		}

	public:
		using FGetter = typename Super::FGetter;
		using ObjectType = typename Super::ObjectType;

	public:
		//~ You can only register Attribute that are defined in a SWidget (member of the class).
		//~ Use the constructor with Widget pointer.
		//~ See documentation in SlateAttribute.h for more info.
		TSlateMemberAttribute() = delete;
		TSlateMemberAttribute(const TSlateMemberAttribute&) = delete;
		TSlateMemberAttribute(TSlateMemberAttribute&&) = delete;
		TSlateMemberAttribute& operator=(const TSlateMemberAttribute&) = delete;
		TSlateMemberAttribute& operator=(TSlateMemberAttribute&&) = delete;
		void* operator new(size_t) = delete;
		void* operator new[](size_t) = delete;

#if UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
		~TSlateMemberAttribute()
		{
			/**
			 * The parent should now be destroyed and the shared pointer should be invalidate.
			 * If you hit the check, that means the TSlateAttribute is not a variable member of SWidget.
			 * It will introduced bad memory access.
			 * See documentation in SlateAttribute.h.
			*/
			checkf(Super::ProtectedIsWidgetInDestructionPath(Super::Debug_OwningWidget), TEXT("The Owning widget should be invalid."));
		}
#endif

		template<typename WidgetType, typename U = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value>::type>
		explicit TSlateMemberAttribute(WidgetType& Widget)
			: Super(Widget)
		{
			VerifyAttributeAddress(Widget, *this);
		}

		template<typename WidgetType, typename U = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value>::type>
		explicit TSlateMemberAttribute(WidgetType& Widget, const ObjectType& InValue)
			: Super(Widget, InValue)
		{
			VerifyAttributeAddress(Widget, *this);
		}

		template<typename WidgetType, typename U = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value>::type>
		explicit TSlateMemberAttribute(WidgetType& Widget, ObjectType&& InValue)
			: Super(Widget, MoveTemp(InValue))
		{
			VerifyAttributeAddress(Widget, *this);
		}
	};



	enum ESlateMemberAttributeRefNoCheckParam { SlateMemberAttributeRefNoCheckParam };

	/*
	 * A reference to a SlateAttribute that can be returned and saved for later.
	 */
	template<typename AttributeMemberType>
	struct TSlateMemberAttributeRef
	{
	public:
		using SlateAttributeType = AttributeMemberType;
		using ObjectType = typename AttributeMemberType::ObjectType;
		using AttributeType = TAttribute<ObjectType>;

	private:
		template<typename WidgetType>
		static void VerifyAttributeAddress(const WidgetType& InWidget, const AttributeMemberType& InAttribute)
		{
			checkf((UPTRINT)&InAttribute >= (UPTRINT)&InWidget && (UPTRINT)&InAttribute < (UPTRINT)&InWidget + sizeof(WidgetType),
				TEXT("The attribute is not a member of the widget."));
			InAttribute.VerifyOwningWidget(InWidget);
		}

	public:
		/** Constructor */
		template<typename WidgetType, typename V = typename std::enable_if<std::is_base_of<SWidget, WidgetType>::value>::type>
		explicit TSlateMemberAttributeRef(const TSharedRef<WidgetType>& InOwner, const AttributeMemberType& InAttribute)
			: Owner(InOwner)
			, Attribute(InAttribute)
		{
			VerifyAttributeAddress(InOwner.Get(), InAttribute);
		}

		/** Constructor */
		explicit TSlateMemberAttributeRef(ESlateMemberAttributeRefNoCheckParam, const TSharedRef<SWidget>& InOwner, const AttributeMemberType& InAttribute)
			: Owner(InOwner)
			, Attribute(InAttribute)
		{
		}

	public:
		/** @return if the reference is valid. A reference can be invalid if the SWidget is destroyed. */
		[[nodiscard]] bool IsValid() const
		{
			return Owner.IsValid();
		}

		/** @return the SlateAttribute cached value; undefined when IsValid() returns false. */
		[[nodiscard]] const ObjectType& Get() const
		{
			if (TSharedPtr<const SWidget> Pin = Owner.Pin())
			{
				return Attribute.Get();
			}
			checkf(false, TEXT("It is an error to call GetValue() on an unset TSlateMemberAttributeRef. Please either check IsValid() or use Get(DefaultValue) instead."));
			static ObjectType Tmp;
			return Tmp;
		}

		/** @return the SlateAttribute cached value or the DefaultValue if the reference is invalid. */
		[[nodiscard]] const ObjectType& Get(const ObjectType& DefaultValue) const
		{
			if (TSharedPtr<const SWidget> Pin = Owner.Pin())
			{
				return Attribute.Get();
			}
			return DefaultValue;
		}

		/** Update the cached value and invalidate the widget if needed. */
		void UpdateValue()
		{
			if (TSharedPtr<const SWidget> Pin = Owner.Pin())
			{
				const_cast<AttributeMemberType&>(Attribute).UpdateNow(const_cast<SWidget&>(*Pin.Get()));
			}
		}

		/**
		 * Assumes the reference is valid.
		 * Shorthand for the boilerplace code: MyAttribute.UpdateValueNow(); MyAttribute.Get();
		 */
		[[nodiscard]] const ObjectType& UpdateAndGet()
		{
			if (TSharedPtr<const SWidget> Pin = Owner.Pin())
			{
				const_cast<AttributeMemberType&>(Attribute)->UpdateNow(const_cast<SWidget&>(*Pin.Get()));
				return Attribute.Get();
			}
			checkf(false, TEXT("It is an error to call GetValue() on an unset TSlateMemberAttributeRef. Please either check IsValid() or use Get(DefaultValue) instead."));
		}

		/**
		 * Shorthand for the boilerplace code: MyAttribute.UpdateValueNow(); MyAttribute.Get(DefaultValue);
		 * @return the SlateAttribute cached value or the DefaultValue if the reference is invalid.
		 */
		[[nodiscard]] const ObjectType& UpdateAndGet(const ObjectType& DefaultValue)
		{
			if (TSharedPtr<const SWidget> Pin = Owner.Pin())
			{
				const_cast<AttributeMemberType&>(Attribute)->UpdateNow(const_cast<SWidget&>(*Pin.Get()));
				return Attribute.Get();
			}
			return DefaultValue;
		}

		/** Build a Attribute from this SlateAttribute. */
		[[nodiscard]] TAttribute<ObjectType> ToAttribute() const
		{
			if (TSharedPtr<const SWidget> Pin = Owner.Pin())
			{
				return Attribute.ToAttribute(*Pin.Get());
			}
			return TAttribute<ObjectType>();
		}

		/** @return True if the SlateAttribute is bound to a getter function. */
		[[nodiscard]] bool IsBound() const
		{
			if (TSharedPtr<const SWidget> Pin = Owner.Pin())
			{
				return Attribute.IsBound(*Pin.Get());
			}
			return false;
		}

		/** @return True if they have the same Getter or the same value. */
		[[nodiscard]] bool IdenticalTo(const TSlateMemberAttributeRef& Other) const
		{
			TSharedPtr<const SWidget> SelfPin = Owner.Pin();
			TSharedPtr<const SWidget> OtherPin = Other.Owner.Pin();
			if (SelfPin == OtherPin && SelfPin)
			{
				return Attribute.IdenticalTo(*SelfPin.Get(), Other.Attribute);
			}
			return SelfPin == OtherPin;
		}

		/** @return True if they have the same Getter or, if the Attribute is set, the same value. */
		[[nodiscard]] bool IdenticalTo(const TAttribute<ObjectType>& Other) const
		{
			if (TSharedPtr<const SWidget> Pin = Owner.Pin())
			{
				return Attribute.IdenticalTo(*Pin.Get(), Other);
			}
			return !Other.IsSet(); // if the other is not set, then both are invalid.
		}

	private:
		TWeakPtr<const SWidget> Owner;
		const AttributeMemberType& Attribute;
	 };

} // SlateAttributePrivate
