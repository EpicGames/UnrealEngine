// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Traits/MemberPtrOuter.h"
#include "Traits/RemoveMemberPtr.h"

/** */
namespace SlateAttributePrivate
{
	/**
	 * A Slate Attribute that can live in a class/struct living in a widget.
	 * ex: Widget -> FAttributeHolder -> TSlateNestedMemberAttribute
	 */
	template<typename InObjectType, typename InInvalidationReasonPredicate, typename InComparePredicate>
	struct TSlateNestedMemberAttribute : public TSlateMemberAttribute<InObjectType, InInvalidationReasonPredicate, InComparePredicate>
	{
		/**
		 * A Nested Attribute is just a TSlateMemberAttribute with an offset carefully computed.
		 * This function checks that the passed in Pointer to Members are what we expect for a nested attribute.
		 * See SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION
		 *
		 * @tparam PointerToAttributeOwnerType The pointer to member from the Widget to the class/struct owning the nested attribute
		 * @tparam PointerToAttributeType The pointer to member from the class/struct owning the nested attribute to the nested attribute
		 * @return True if valid, otherwise a static_assert would have been raised
		 */
		template<
			typename PointerToAttributeOwnerType,
			typename PointerToAttributeType
		>
		static constexpr bool IsNestedAttributeValid()
		{
			static_assert(std::is_member_pointer_v<PointerToAttributeOwnerType>, "The PointerToAttributeOwnerType needs to be a Pointer to Member.");
			static_assert(std::is_member_pointer_v<PointerToAttributeType>, "The PointerToAttributeType needs to be a Pointer to Member.");

			using WidgetType = TMemberPtrOuter_T<PointerToAttributeOwnerType>;
			using AttributeOwnerType = TRemoveMemberPtr_T<PointerToAttributeOwnerType>;
			using AttributeContainerType = TMemberPtrOuter_T<PointerToAttributeType>;
			using AttributeType = TRemoveMemberPtr_T<PointerToAttributeType>;

			static_assert(std::is_base_of<SWidget, WidgetType>::value, "The class owning the PointerToAttributeOwner needs to derive from SWidget.");
			static_assert(std::is_same<AttributeOwnerType, AttributeContainerType>::value, "The Attribute Owner needs to live inside the widget and needs to own the Attribute.");
			static_assert(std::is_base_of<TSlateNestedMemberAttribute, AttributeType>::value, "The Attribute needs to be a TSlateNestedMemberAttribute.");
			return true;
		}

		using TSlateMemberAttribute<InObjectType, InInvalidationReasonPredicate, InComparePredicate>::TSlateMemberAttribute;
	};

} // SlateAttributePrivate
