// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Base class for all slate metadata
 */
class ISlateMetaData
{
public:
	/** Check if this metadata operation can cast safely to the specified template type */
	template<class TType>
	bool IsOfType() const
	{
		return IsOfTypeImpl(TType::GetTypeId());
	}

	/** Virtual destructor. */
	virtual ~ISlateMetaData() = default;

protected:
	/**
	 * Checks whether this drag and drop operation can cast safely to the specified type.
	 */
	virtual bool IsOfTypeImpl(const FName& Type) const
	{
		return false;
	}
};

/**
 * All metadata-derived classes must include this macro.
 * Example Usage:
 *	class FMyMetaData : public ISlateMetaData
 *	{
 *	public:
 *		SLATE_METADATA_TYPE(FMyMetaData, ISlateMetaData)
 *		...
 *	};
 */
#define SLATE_METADATA_TYPE(TYPE, BASE) \
	static const FName& GetTypeId() { static FName Type(TEXT(#TYPE)); return Type; } \
	virtual bool IsOfTypeImpl(const FName& Type) const override { return GetTypeId() == Type || BASE::IsOfTypeImpl(Type); }

 /**
  * Simple tagging metadata
  */
class FTagMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FTagMetaData, ISlateMetaData)

		FTagMetaData(FName InTag)
		: Tag(InTag)
	{}

	/** Tag name for a widget */
	FName Tag;
};
