// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "Types/ISlateMetaData.h"

/**
 * When using the widget reflector, it may be necessary to make some widgets non-pickable, like the debug canvas.
 */
class FInvisibleToWidgetReflectorMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FInvisibleToWidgetReflectorMetaData, ISlateMetaData)

	FInvisibleToWidgetReflectorMetaData()
	{
	}
};
