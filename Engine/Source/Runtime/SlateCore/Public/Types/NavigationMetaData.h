// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/NavigationReply.h"
#include "Input/NavigationRouting.h"
#include "Input/NavigationMethod.h"
#include "StructUtils/InstancedStruct.h"
#include "Types/ISlateMetaData.h"
#include "Types/SlateEnums.h"

class SWidget;

/**
 *  Metadata to override the navigation behavior or regular SWidget
 */
class FNavigationMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FNavigationMetaData, ISlateMetaData)

	FNavigationMetaData()
	{
		for (SNavData& Rule : Rules)
		{
			Rule.BoundaryRule = EUINavigationRule::Escape;
			Rule.FocusDelegate = nullptr;
			Rule.FocusRecipient = nullptr;
		}
	}

	/**
	 * Get the boundary rule  for the provided navigation type
	 * 
	 * @param InNavigation the type of navigation to get the boundary rule for
	 * @return the boundary rule for the provided navigation type
	 */
	EUINavigationRule GetBoundaryRule(EUINavigation InNavigation) const
	{
		return Rules[(uint8)InNavigation].BoundaryRule;
	}

	/**
	 * Get the focus recipient for the provided navigation type
	 * 
	 * @param InNavigation the type of navigation to get the focus recipient for
	 * @return the focus recipient for the provided navigation type
	 */
	const TWeakPtr<SWidget>& GetFocusRecipient(EUINavigation InNavigation) const
	{
		return Rules[(uint8)InNavigation].FocusRecipient;
	}

	/**
	 * Get the focus recipient delegate for the provided navigation type
	 * 
	 * @param InNavigation the type of navigation to get the focus recipient delegate for
	 * @return the focus recipient delegate for the provided navigation type
	 */
	const FNavigationDelegate& GetFocusDelegate(EUINavigation InNavigation) const
	{
		return Rules[(uint8)InNavigation].FocusDelegate;
	}

	/**
	 * Set the navigation behavior for the provided navigation type to be explicit, using a constant widget
	 * 
	 * @param InNavigation the type of navigation to set explicit
	 * @param InFocusRecipient the widget used when the system requests the destination for the explicit navigation
	 */
	void SetNavigationExplicit(EUINavigation InNavigation, TSharedPtr<SWidget> InFocusRecipient)
	{
		Rules[(uint8)InNavigation].BoundaryRule = EUINavigationRule::Explicit;
		Rules[(uint8)InNavigation].FocusDelegate = nullptr;
		Rules[(uint8)InNavigation].FocusRecipient = InFocusRecipient;
	}

	/**
	 * Set the navigation behavior for the provided navigation type to be a custom delegate
	 * 
	 * @param InNavigation the type of navigation to set custom
	 * @param InCustomBoundaryRule
	 * @param InFocusRecipient the delegate used when the system requests the destination for the custom navigation
	 */
	void SetNavigationCustom(EUINavigation InNavigation, EUINavigationRule InCustomBoundaryRule, FNavigationDelegate InFocusDelegate)
	{ 
		ensure(InCustomBoundaryRule == EUINavigationRule::Custom || InCustomBoundaryRule == EUINavigationRule::CustomBoundary);
		Rules[(uint8)InNavigation].BoundaryRule = InCustomBoundaryRule;
		Rules[(uint8)InNavigation].FocusDelegate = InFocusDelegate;
		Rules[(uint8)InNavigation].FocusRecipient = nullptr;
	}

	/**
	* Set the navigation behavior for the provided navigation type to be wrap
	*
	* @param InNavigation the type of navigation to set wrapping 
	*/
	void SetNavigationWrap(EUINavigation InNavigation)
	{
		Rules[(uint8)InNavigation].BoundaryRule = EUINavigationRule::Wrap;
		Rules[(uint8)InNavigation].FocusDelegate = nullptr;
		Rules[(uint8)InNavigation].FocusRecipient = nullptr;
	}

	/**
	 * An event should return a FNavigationReply::Explicit() to let the system know to stop at the bounds of this widget.
	 */
	void SetNavigationStop(EUINavigation InNavigation)
	{
		Rules[(uint8)InNavigation].BoundaryRule = EUINavigationRule::Stop;
		Rules[(uint8)InNavigation].FocusDelegate = nullptr;
		Rules[(uint8)InNavigation].FocusRecipient = nullptr;
	}

	/**
	 * An event should return a FNavigationReply::Escape() to let the system know that a navigation can escape the bounds of this widget.
	 */
	void SetNavigationEscape(EUINavigation InNavigation)
	{
		Rules[(uint8)InNavigation].BoundaryRule = EUINavigationRule::Escape;
		Rules[(uint8)InNavigation].FocusDelegate = nullptr;
		Rules[(uint8)InNavigation].FocusRecipient = nullptr;
	}

	void SetNavigationMethodStruct(const TInstancedStruct<FNavigationMethod>& InNavigationMethodStruct) 
	{
		NavigationMethodStruct = InNavigationMethodStruct;
	}

	TInstancedStruct<FNavigationMethod>& GetMutableNavigationMethodStruct()
	{
		return NavigationMethodStruct;
	}

	const TInstancedStruct<FNavigationMethod>& GetNavigationMethodStruct() const
	{
		return NavigationMethodStruct;
	}

	void SetNavigationRoutingPolicy(EWidgetNavigationRoutingPolicy InNavigationRoutingPolicy) 
	{
		NavigationRoutingPolicy = InNavigationRoutingPolicy;
	}

	EWidgetNavigationRoutingPolicy GetNavigationRoutingPolicy() const
	{
		return NavigationRoutingPolicy;
	}

private:

	struct SNavData
	{
		EUINavigationRule BoundaryRule;
		TWeakPtr<SWidget> FocusRecipient;
		FNavigationDelegate FocusDelegate;
	};
	SNavData Rules[(uint8)EUINavigation::Num];

	EWidgetNavigationRoutingPolicy NavigationRoutingPolicy = EWidgetNavigationRoutingPolicy::Default;
	TInstancedStruct<FNavigationMethod> NavigationMethodStruct;
};

#ifndef UE_WITH_SLATE_SIMULATEDNAVIGATIONMETADATA
#define UE_WITH_SLATE_SIMULATEDNAVIGATIONMETADATA !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif

#if UE_WITH_SLATE_SIMULATEDNAVIGATIONMETADATA
/**
 * Navigation meta data to used when using the Navigation Event Simulator
 * The OnNavigation function is not call by default on the widget, unless specified with "IsOnNavigationConst".
 */
class FSimulatedNavigationMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FSimulatedNavigationMetaData, ISlateMetaData)

	FSimulatedNavigationMetaData() = default;

	FSimulatedNavigationMetaData(const FNavigationMetaData& InSimulatedNavigation)
	{
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(Rules); ++Index)
		{
			Rules[Index].BoundaryRule = InSimulatedNavigation.GetBoundaryRule((EUINavigation)Index);
			Rules[Index].FocusRecipient = InSimulatedNavigation.GetFocusRecipient((EUINavigation)Index);
		}
	}

	FSimulatedNavigationMetaData(EUINavigationRule InNavigationRule)
	{
		for(SNavData& Rule : Rules)
		{
			Rule.BoundaryRule = InNavigationRule;
		}
	}

	enum EOnNavigationIsConst { OnNavigationIsConst };
	FSimulatedNavigationMetaData(EOnNavigationIsConst)
	{
		for (SNavData& Rule : Rules)
		{
			Rule.BoundaryRule = EUINavigationRule::Escape;
		}
		bIsOnNavigationConst = true;
	}

public:
	/** Get the boundary rule for the provided navigation type. */
	EUINavigationRule GetBoundaryRule(EUINavigation InNavigation) const
	{
		return Rules[(uint8)InNavigation].BoundaryRule;
	}

	/** Get the focus recipient for the provided navigation type */
	const TWeakPtr<SWidget>& GetFocusRecipient(EUINavigation InNavigation) const
	{
		return Rules[(uint8)InNavigation].FocusRecipient;
	}

	/** Is the OnNavigation function const and should be called when simulating to determine the navigation rule. */
	bool IsOnNavigationConst() const { return bIsOnNavigationConst; }

private:
	struct SNavData
	{
		EUINavigationRule BoundaryRule;
		TWeakPtr<SWidget> FocusRecipient;
	};
	SNavData Rules[(uint8)EUINavigation::Num];
	bool bIsOnNavigationConst = false;
};
#endif //UE_WITH_SLATE_SIMULATEDNAVIGATIONMETADATA
