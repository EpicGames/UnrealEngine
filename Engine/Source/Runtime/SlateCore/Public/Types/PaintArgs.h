// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Layout/Geometry.h"
#include "SlateGlobals.h"

class FHittestGrid;
class FSlateRect;
class ICustomHitTestPath;
class SWidget;
class SWindow;


/**
 * SWidget::OnPaint and SWidget::Paint use FPaintArgs as their
 * sole parameter in order to ease the burden of passing
 * through multiple fields.
 */
class FPaintArgs
{
	friend class SInvalidationPanel;
	friend class SRetainerWidget;
public:
	SLATECORE_API FPaintArgs(const SWidget* PaintParent, FHittestGrid& InRootHittestGrid, FHittestGrid& InCurrentHitTestGrid, UE::Slate::FDeprecateVector2DParameter InWindowOffset, double InCurrentTime, float InDeltaTime);
	SLATECORE_API FPaintArgs(const SWidget* PaintParent, FHittestGrid& InRootHittestGrid, UE::Slate::FDeprecateVector2DParameter InWindowOffset, double InCurrentTime, float InDeltaTime);

	[[nodiscard]] inline FPaintArgs WithNewParent(const SWidget* PaintParent) const
	{
		FPaintArgs Args(*this);
		Args.PaintParentPtr = PaintParent;

		return Args;
	}

	[[nodiscard]] inline FPaintArgs WithNewHitTestGrid(FHittestGrid& NewHitTestGrid) const
	{
		FPaintArgs NewArgs(PaintParentPtr, RootGrid, NewHitTestGrid, WindowOffset, CurrentTime, DeltaTime);
		return NewArgs;
	}

	SLATECORE_API FPaintArgs InsertCustomHitTestPath(const SWidget* Widget, TSharedRef<ICustomHitTestPath> CustomHitTestPath) const;

	void SetInheritedHittestability(bool InInheritedHittestability)
	{
		bInheritedHittestability = InInheritedHittestability;
	}

	bool GetInheritedHittestability() const
	{
		return bInheritedHittestability;
	}

	FHittestGrid& GetHittestGrid() const
	{
		return CurrentGrid;
	}

	const SWidget* GetPaintParent() const
	{
		return PaintParentPtr; 
	}

	UE::Slate::FDeprecateVector2DResult GetWindowToDesktopTransform() const
	{
		return UE::Slate::FDeprecateVector2DResult(WindowOffset);
	}

	double GetCurrentTime() const
	{
		return CurrentTime;
	}

	float GetDeltaTime() const
	{
		return DeltaTime;
	}

	void SetDeferredPaint(bool InDeferredPaint)
	{
		bDeferredPainting = InDeferredPaint;
	}

	bool GetDeferredPaint() const
	{
		return bDeferredPainting;
	}
	
private:

	/** The root most grid.  Only the window should set this and only invalidation panels should modify it */
	FHittestGrid& RootGrid;

	/** The current hit test grid.  Its possible that there is more than one grid when there is nested invalidation panels.  This is what widgets should add to always */
	FHittestGrid& CurrentGrid;

	FVector2f WindowOffset;
	const SWidget* PaintParentPtr;

	double CurrentTime;
	float DeltaTime;
	uint8 bInheritedHittestability : 1;
	uint8 bDeferredPainting : 1;

};
