// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "Types/ISlateMetaData.h"

class SWidget;

/**
 * Reflection meta-data that can be used by the widget reflector to determine
 * additional information about slate widgets that are constructed by UObject
 * classes for UMG.
 */
class FReflectionMetaData : public ISlateMetaData
{
public:
	SLATE_METADATA_TYPE(FReflectionMetaData, ISlateMetaData)

	FReflectionMetaData(FName InName, UClass* InClass, UObject* InSourceObject, const UObject* InAsset)
		: Name(InName)
		, Class(InClass)
		, SourceObject(InSourceObject)
		, Asset(InAsset)
	{
	}

	/** The name of the widget in the hierarchy */
	FName Name;
	
	/** The class the constructed the slate widget. */
	TWeakObjectPtr<UClass> Class;

	/** The UObject wrapper that creates the widget, this is expected to be a UWidget. */
	TWeakObjectPtr<UObject> SourceObject;

	/** The asset that owns the widget and is responsible for its specific existence. */
	TWeakObjectPtr<const UObject> Asset;

public:

	static SLATECORE_API FString GetWidgetPath(const SWidget* InWidget, bool bShort = true, bool bNativePathOnly = false);
	static SLATECORE_API FString GetWidgetPath(const SWidget& InWidget, bool bShort = true, bool bNativePathOnly = false);

	static SLATECORE_API FString GetWidgetDebugInfo(const SWidget* InWidget);
	static SLATECORE_API FString GetWidgetDebugInfo(const SWidget& InWidget);

	static SLATECORE_API UObject* GetWidgetSourceObject(const SWidget* InWidget);
	static SLATECORE_API UObject* GetWidgetSourceObject(const SWidget& InWidget);

	static SLATECORE_API TSharedPtr<FReflectionMetaData> GetWidgetOrParentMetaData(const SWidget* InWidget);
};
