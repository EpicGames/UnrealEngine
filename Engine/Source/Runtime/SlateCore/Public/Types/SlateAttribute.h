// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/EqualTo.h"
#include "Widgets/InvalidateWidgetReason.h"

#include <type_traits>

#ifndef UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING
	#define UE_SLATE_WITH_MEMBER_ATTRIBUTE_DEBUGGING 0
#endif
#ifndef UE_SLATE_WITH_ATTRIBUTE_INITIALIZATION_ON_BIND
	#define UE_SLATE_WITH_ATTRIBUTE_INITIALIZATION_ON_BIND 0
#endif
#ifndef UE_SLATE_WITH_ATTRIBUTE_NAN_DIAGNOSTIC
	#define UE_SLATE_WITH_ATTRIBUTE_NAN_DIAGNOSTIC DO_ENSURE
#endif

/**
 * Use TSlateAttribute when it's a SWidget member.
 * Use TSlateManagedAttribute when it's a member inside an array or other moving structure, and the array is a SWidget member. They can only be moved (they can't be copied). THey consume more memory.
 * For everything else, use TAttribute.
 *
 *
 *
 * In Slate, TAttributes are optimized for developer efficiency.
 * They enable widgets to poll for data instead of requiring the user to manually set the state on widgets.
 * Attributes generally work well when performance is not a concern but break down when performance is critical (like a game UI).
 *
 * The invalidation system allows only widgets that have changed to perform expensive Slate layout.
 * Bound TAttributes are incompatible with invalidation because we do not know when the data changes.
 * Additionally TAttributes for common functionality such as visibility are called multiple times per frame and the delegate overhead for that alone is very high.
 * TAttributes have a high memory overhead and are not cache-friendly.
 *
 * TSlateAttribute makes the attribute system viable for invalidation and more performance-friendly while keeping the benefits of attributes intact.
 * TSlateAttributes are updated once per frame in the Prepass update phase. If the cached value of the TSlateAttribute changes, then it will invalidate the widget.
 * The member attributes are updated in the order the variables are defined in the SWidget definition (by default).
 * The managed attributes are updated in a random order (after TSlateAttribute).
 * The update order of member attributes can be defined/override by setting a Prerequisite (see bellow for an example).
 * The invalidation reason can be a predicate (see bellow for an example).
 * The invalidation reason can be override per SWidget. Use override with precaution since it can break the invalidation of widget's parent.
 * The widget attributes are updated only if the widget is visible/not collapsed.
 *
 *
 * TSlateAttribute is not copyable and can only live inside a SWidget.
 * For performance reasons, we do not save the extra information that would be needed to be "memory save" in all contexts.
 * If you create a TSlateAttribute that can be moved, you need to use TSlateManagedAttribute.
 * TSlateManagedAttribute is as fast as TSlateAttribute but use more memory and is less cache-friendly.
 * Note, if you use TAttribute to change the state of a SWidget, you need to override bool ComputeVolatility() const.
 * Note, ComputeVolatility() is not needed for TSlateAttribute and TSlateManagedAttribute
 *
 * TSlateAttribute request a SWidget pointer. The "this" pointer should ALWAYS be used.
 * (The SlateAttribute pointer is saved inside the SlateAttributeMetaData. The widget needs to be aware when the pointer changes.)
 *
 *
 *	.h
 *  // The new way of using attribute in your SWidget
 *	class SMyNewWidget : public SLeafWidget
 *	{
 *		SLATE_DECLARE_WIDGET(SMyNewWidget, SLeafWidget)	// It defined the static data needed by the SlateAttribute system.
 *														// The invalidation reason is defined in the static function "void PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)"
 *														// You need to implement the function PrivateRegisterAttributes.
 *														// If you can't implement the SLATE_DECLARE_WIDGET pattern, use TSlateAttribute and provide a reason instead.
 *
 *		SLATE_BEGIN_ARGS(SMyNewWidget)
 *			SLATE_ATTRIBUTE(FLinearColor, Color)
 *		SLATE_END_ARGS()
 *
 *		SMyNewWidget()
 *			: NewWay(*this, FLinearColor::White) // Always use "this" pointer. No exception. If you can't, use TAttribute instead.
 *		{}
 *
 *		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
 *		{
 *			++LayerId;
 *			FSlateDrawElement::MakeDebugQuad(
 *				OutDrawElements,
 *				LayerId,
 *				AllottedGeometry.ToPaintGeometry(),
 *				NewWay.Get()							// The value is already cached.
 *			);
 *			return LayerId;
 *		}
 *
 *		// bEnabled do not invalidate Layout. We need to override the default invalidation.
 *		virtual FVector2D ComputeDesiredSize(float) const override { return IsEnabled() ? FVector2D(10.f, 10.f) : FVector2D(20.f, 20.f); }
 *
 *		void Construct(const FArguments& InArgs)
 *		{
 *			// Assigned the attribute if needed.
 *			// If the TAttribute is bound, copy the TSlateAttribute to the TAttribute getter, update the value, test if the value have changed and, if so, invalidate the widget.
 *			// It the TAttribute is set, remove previous getter, set the TSlateAttribute value, test if the value changed and, if so, invalidate the widget.
 *			// Else, nothing (use the previous value).
 *			NewWays.Assign(*this, InArgs._Color); // Always use the "this" pointer.
 *		}
 *
 *		// If NewWay is bounded and the values changed from the previous frame, the widget will be invalidated.
 *		//In this case, it will only paint the widget (no layout required).
 *		TSlateAttribute<FLinearColor> NewWay;
 *
 *		// Note that you can put a predicate or put the invalidation as a template argument.
 *		//Either you define the attribute in PrivateRegisterAttributes or you define the invalidation at declaration
 *		//but you need to use one and one of the 2 methods. Compile and runtime check will check if you did it properly.
 *		//Setting the reason as a template argument is less error prone (ie. missing from the PrivateRegisterAttributes, bad copy paste, bad reason...)
 *		//Setting the reason in PrivateRegisterAttributes enable override and ease debugging later (ie. the attribute will be named)
 *		//TSlateAttribute<FLinearColor, EInvalidationReason::Paint> NewWayWithout_SLATE_DECLARE_WIDGET_;
 *	};
 *
 *  .cpp
 *	// This is optional. It is still a good practice to implement it since it will allow user to override the default behavior and control the update order.
 *	//The WidgetReflector use that information.
 *	//See NewWayWithout_SLATE_DECLARE_WIDGET_ for an example of how to use the system without the SLATE_DECLARE_WIDGET
 *
 *	SLATE_IMPLEMENT_WIDGET(SMyNewWidget)
 *	void SMyNewWidget::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
 *	{
 *		SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, NewWay, EInvalidationReason::Paint)
 *			.SetPrerequisite("bEnabled"); // SetPrerequisite is not needed here. This is just an example to show how you could do it if needed.
 *
 *		//bEnabled invalidate paint, we need it to invalidate the Layout.
 *		AttributeInitializer.OverrideInvalidationReason("bEnabled",
			FSlateAttributeDescriptor::FInvalidateWidgetReasonAttribute{EInvalidateWidgetReason::Layout | EInvalidateWidgetReason::Paint});
 *	}
 *
 *
 *	// We used to do this. Keep using this method when you are not able to provide the "this" pointer to TSlateAttribute.
 *	class SMyOldWidget : public SLeafWidget
 *	{
 *		SLATE_BEGIN_ARGS(SMyOldWidget)
 *			SLATE_ATTRIBUTE(FLinearColor, Color)
 *		SLATE_END_ARGS()
 *
 *		// Widget will be updated every frame if a TAttribute is bound. (Even if the value didn't change).
 *		virtual bool ComputeVolatility() const override { return SLeafWidget::ComputeVolatility() || OldWay.IsBound(); }
 *
 *		virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override
 *		{
 *			++LayerId;
 *			FSlateDrawElement::MakeDebugQuad(
 *				OutDrawElements,
 *				LayerId,
 *				AllottedGeometry.ToPaintGeometry(),
 *				OldWay.Get(FLinearColor::White)		// Fetch the value of OldWays and use White, if the value is not set.
 *			);
 *			return LayerId;
 *		}
 *
 *		virtual FVector2D ComputeDesiredSize(float) const override { return IsEnabled() ? FVector2D(10.f, 10.f) : FVector2D(20.f, 20.f); }
 *
 *		void Construct(const FArguments& InArgs)
 *		{
 *			NewWays = InArgs._Color;
 *		}
 *
 *	private:
 *		TAttribute<FLinearColor> OldWay;
 * };
 */


class SWidget;

enum class ESlateAttributeBindAction
{
	None = 0, // Do nothing when an attribute is bound.
	Initialize = 1, // Initialize the attribute value from the binding.
};
ENUM_CLASS_FLAGS(ESlateAttributeBindAction)
#if UE_SLATE_WITH_ATTRIBUTE_INITIALIZATION_ON_BIND
#	define UE_SLATE_WITH_ATTRIBUTE_DEFAULT_INITIALIZATION_ACTION ESlateAttributeBindAction::Initialize
#else
#	define UE_SLATE_WITH_ATTRIBUTE_DEFAULT_INITIALIZATION_ACTION ESlateAttributeBindAction::None
#endif

enum class ESlateAttributeInitialization
{
	None = 0, // Do not initialize Slate Attributes values
	AfterConstruct, // Initialize all Slate Attributes values after Construct
};

/** Base struct of all SlateAttribute type. */
struct FSlateAttributeBase
{
	/**
	 * Not all invalidation is supported by SlateAttribute.
	 * ChildOrder: The update of SlateAttribute is done in the SlatePrepass. We can't add or remove children in SlatePrepass.
	 * AttributeRegistration: In FastPath, the SlateAttribute are updated in a loop. The iterator can't be modified while we are looping.
	 */
	template<typename T>
	constexpr static bool IsInvalidateWidgetReasonSupported(T Reason)
	{
		return false;
	}
	constexpr static bool IsInvalidateWidgetReasonSupported(EInvalidateWidgetReason Reason)
	{
		return (Reason & (EInvalidateWidgetReason::ChildOrder | EInvalidateWidgetReason::AttributeRegistration)) == EInvalidateWidgetReason::None;
	}
};


/** Default predicate to compare of Object for SlateAttribute. */
template<typename ComparePredicate = TEqualTo<>>
struct TSlateAttributeComparePredicate
{
	template<typename ObjectType>
	static bool IdenticalTo(const SWidget&, ObjectType&& Lhs, ObjectType&& Rhs)
	{
		// If you have an error here, do you have a operator== const?
		return ComparePredicate{}(Forward<ObjectType>(Lhs), Forward<ObjectType>(Rhs));
	}
};


/** Default predicate to compare FText. */
struct TSlateAttributeFTextComparePredicate
{
	static bool IdenticalTo(const SWidget&, const FText& Lhs, const FText& Rhs)
	{
		return Lhs.IdenticalTo(Rhs, ETextIdenticalModeFlags::DeepCompare | ETextIdenticalModeFlags::LexicalCompareInvariants);
	}
};


/** Predicate that returns the InvalidationReason defined as argument type. */
template<EInvalidateWidgetReason InvalidationReason>
struct TSlateAttributeInvalidationReason
{
	static_assert(FSlateAttributeBase::IsInvalidateWidgetReasonSupported(InvalidationReason), "The invalidation is not supported by the SlateAttribute.");
	static constexpr EInvalidateWidgetReason GetInvalidationReason(const SWidget&) { return InvalidationReason; }
};


/** A structure used to help the user identify deprecated TAttribute that are now TSlateAttribute. */
template<typename ObjectType>
struct TSlateDeprecatedTAttribute
{
	TSlateDeprecatedTAttribute() = default;

	using FGetter = typename TAttribute<ObjectType>::FGetter;

	template<typename OtherType>
	TSlateDeprecatedTAttribute(const OtherType& InInitialValue)	{ }

	TSlateDeprecatedTAttribute(ObjectType&& InInitialValue)	{ }

	template<class SourceType>
	TSlateDeprecatedTAttribute(TSharedRef<SourceType> InUserObject, typename FGetter::template TConstMethodPtr< SourceType > InMethodPtr) {}

	template< class SourceType >
	TSlateDeprecatedTAttribute(SourceType* InUserObject, typename FGetter::template TConstMethodPtr< SourceType > InMethodPtr) { }

	bool IsSet() const { return false; }

	template<typename OtherType>
	void Set(const OtherType& InNewValue) {}

	const ObjectType& Get(const ObjectType& DefaultValue) const { return DefaultValue; }
	const ObjectType& Get() const { static ObjectType Temp; return Temp; }
	FGetter GetBinding() const { return false; }

	void Bind(const FGetter& InGetter) {}
	template<class SourceType>
	void Bind(SourceType* InUserObject, typename FGetter::template TConstMethodPtr< SourceType > InMethodPtr) {}
	bool IsBound() const { return false; }

	bool IdenticalTo(const TAttribute<ObjectType>& InOther) const { return false; }
};

// IWYU pragma: begin_exports
#include "Types/Attributes/SlateAttributeDefinition.inl"
#include "Types/Attributes/SlateAttributeBase.inl"

#include "Types/Attributes/SlateAttributeContained.inl"
#include "Types/Attributes/SlateAttributeManaged.inl"
#include "Types/Attributes/SlateAttributeMember.inl"
#include "Types/Attributes/SlateAttributeNestedMember.inl"
// IWYU pragma: end_exports
