// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "Widgets/InvalidateWidgetReason.h"
#include "Templates/Identity.h"
#include "Traits/FunctionParameterType.h"
#include "Traits/MemberPtrOuter.h"
#include "Traits/RemoveMemberPtr.h"

#include <type_traits>

class FSlateWidgetClassData;
class SWidget;

namespace SlateAttributePrivate
{
	enum class ESlateAttributeType : uint8;

	/**
	 * Find the offset of a member in its class based on a Pointer to Member variable
	 */
	template <typename InClassType, typename InMemberType>
	std::size_t OffsetOfPointerToMember(InMemberType InClassType::* Member)
	{
		const InClassType* Object = reinterpret_cast<const InClassType*>(alignof(InClassType));
		return reinterpret_cast<std::size_t>(&(Object->*Member))
			- reinterpret_cast<std::size_t>(Object);;
	}

	/**
	 * Find the offset of a base class from a derived class. Useful for multi inheritance
	 */
	template <typename InDerivedType, typename InBaseType>
	std::ptrdiff_t GetBaseClassOffset()
	{
		static_assert(std::is_base_of_v<InBaseType, InDerivedType>, "InDerivedType needs to derive from InBaseType");

		const InDerivedType* Object = reinterpret_cast<const InDerivedType*>(alignof(InDerivedType));
		return reinterpret_cast<std::ptrdiff_t>(static_cast<const InBaseType*>(Object))
			- reinterpret_cast<std::ptrdiff_t>(Object);
	}
}


/**
 * Describes the static information about a Widget's type SlateAttributes.
 **/
class FSlateAttributeDescriptor
{
public:
	/**
	 * A EInvalidationWidgetReason Attribute 
	 * It can be explicitly initialize or can be a callback static function or lambda that returns the EInvalidationReason.
	 * The signature of the function takes a const SWidget& as argument.
	 */
	struct FInvalidateWidgetReasonAttribute
	{
		friend FSlateAttributeDescriptor;

		using Arg1Type = const class SWidget&;
		// using "not checked" delegate to disable race detection. This delegate is used in a static (FSlateWidgetClassData) and will be destructed after the tls array used in the race detection code (which will result in a use after free).
		using FGetter = TDelegate<EInvalidateWidgetReason(Arg1Type), FNotThreadSafeNotCheckedDelegateUserPolicy>;


		FInvalidateWidgetReasonAttribute(const FInvalidateWidgetReasonAttribute&) = default;
		FInvalidateWidgetReasonAttribute(FInvalidateWidgetReasonAttribute&&) = default;
		FInvalidateWidgetReasonAttribute& operator=(const FInvalidateWidgetReasonAttribute&) = default;
		FInvalidateWidgetReasonAttribute& operator=(FInvalidateWidgetReasonAttribute&&) = default;

		/** Default constructor. */
		explicit FInvalidateWidgetReasonAttribute(EInvalidateWidgetReason InReason)
			: Reason(InReason)
			, Getter()
		{
		}

		template<typename... PayloadTypes>
		explicit FInvalidateWidgetReasonAttribute(TIdentity_T<typename FGetter::template TFuncPtr<PayloadTypes...>> InFuncPtr, PayloadTypes&&... InputPayload)
			: Reason(EInvalidateWidgetReason::None)
			, Getter(FGetter::CreateStatic(InFuncPtr, Forward<PayloadTypes>(InputPayload)...))
		{
		}

		template<typename LambdaType, typename... PayloadTypes>
		explicit FInvalidateWidgetReasonAttribute(LambdaType&& InCallable, PayloadTypes&&... InputPayload)
			: Reason(EInvalidateWidgetReason::None)
			, Getter(FGetter::CreateLambda(InCallable, Forward<PayloadTypes>(InputPayload)...))
		{
		}

		explicit FInvalidateWidgetReasonAttribute(FGetter InGetter)
			: Reason(EInvalidateWidgetReason::None)
			, Getter(MoveTemp(InGetter))
		{
		}

		bool IsBound() const
		{
			return Getter.IsBound();
		}

		EInvalidateWidgetReason Get(const SWidget& Widget) const
		{
			return IsBound() ? Getter.Execute(Widget) : Reason;
		}

	private:
		EInvalidateWidgetReason Reason;
		FGetter Getter;
	};

	/**
	 * Adjusted version of FInvalidateWidgetReasonAttribute for Nested Attributes.
	 * The only difference is the signature of the getter, as we need to pass the PointerToAttributeOwner for a nested attribute
	 */
	template<typename PointerToAttributeOwnerType>
	struct TInvalidateWidgetReasonNestedAttribute : FInvalidateWidgetReasonAttribute
	{
		static_assert(std::is_member_pointer_v<PointerToAttributeOwnerType>, "PointerToAttributeOwnerType needs to be a Pointer to Member."); \
		using WidgetType = TMemberPtrOuter_T<PointerToAttributeOwnerType>;
		using AttributeOwnerType = TRemoveMemberPtr_T<PointerToAttributeOwnerType>;

		// using "not checked" delegate to disable race detection. This delegate is used in a static (FSlateWidgetClassData) and will be destructed after the tls array used in the race detection code (which will result in a use after free).
		using FNestedGetter = TDelegate<EInvalidateWidgetReason(const SWidget&, const AttributeOwnerType&), FNotThreadSafeNotCheckedDelegateUserPolicy>;
		using FNestedGetterTyped = TDelegate<EInvalidateWidgetReason(const WidgetType&, const AttributeOwnerType&), FNotThreadSafeNotCheckedDelegateUserPolicy>;


		TInvalidateWidgetReasonNestedAttribute(const TInvalidateWidgetReasonNestedAttribute&) = default;
		TInvalidateWidgetReasonNestedAttribute(TInvalidateWidgetReasonNestedAttribute&&) = default;
		TInvalidateWidgetReasonNestedAttribute& operator=(const TInvalidateWidgetReasonNestedAttribute&) = default;
		TInvalidateWidgetReasonNestedAttribute& operator=(TInvalidateWidgetReasonNestedAttribute&&) = default;

		/** Default constructor. */
		using FInvalidateWidgetReasonAttribute::FInvalidateWidgetReasonAttribute;

		explicit TInvalidateWidgetReasonNestedAttribute(PointerToAttributeOwnerType InPointerToAttributeOwner, EInvalidateWidgetReason InReason)
			: FInvalidateWidgetReasonAttribute(InReason)
		{
		}

		template<typename... PayloadTypes>
		explicit TInvalidateWidgetReasonNestedAttribute(PointerToAttributeOwnerType InPointerToAttributeOwner, TIdentity_T<typename FNestedGetter::template TFuncPtr<PayloadTypes...>> InFuncPtr, PayloadTypes&&... InputPayload)
			: FInvalidateWidgetReasonAttribute(
				FGetter::CreateStatic(
					InvalidateWidgetReasonConverter,
					FNestedGetter::CreateStatic(InFuncPtr, Forward<PayloadTypes>(InputPayload)...),
					InPointerToAttributeOwner
				))
		{
		}

		template<typename... PayloadTypes, typename U = std::enable_if<!std::is_same_v<WidgetType, SWidget>>::type>
		explicit TInvalidateWidgetReasonNestedAttribute(PointerToAttributeOwnerType InPointerToAttributeOwner, TIdentity_T<typename FNestedGetterTyped::template TFuncPtr<PayloadTypes...>> InFuncPtr, PayloadTypes&&... InputPayload)
			: FInvalidateWidgetReasonAttribute(
				FGetter::CreateStatic(
					InvalidateWidgetReasonConverter,
					FNestedGetterTyped::CreateStatic(InFuncPtr, Forward<PayloadTypes>(InputPayload)...),
					InPointerToAttributeOwner
				))
		{
		}

		template<typename LambdaType, typename... PayloadTypes>
		explicit TInvalidateWidgetReasonNestedAttribute(PointerToAttributeOwnerType InPointerToAttributeOwner, LambdaType&& InCallable, PayloadTypes&&... InputPayload)
			: FInvalidateWidgetReasonAttribute(
				CreateNestedGetterLambda(InPointerToAttributeOwner, Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...)
			)
		{
		}

	private:
		template<typename LambdaType, typename... PayloadTypes>
		FGetter CreateNestedGetterLambda(PointerToAttributeOwnerType InPointerToAttributeOwner, LambdaType&& InCallable, PayloadTypes&&... InputPayload)
		{
			using LambdaOperatorType = decltype(&LambdaType::operator());
			static_assert(std::is_same_v<TFunctionParameterType_T<TRemoveMemberPtr_T<LambdaOperatorType>, 1>, const AttributeOwnerType&>, "The second argument of the lambda needs to be 'const AttributeOwnerType&'");

			if constexpr (std::is_same_v<TFunctionParameterType_T<TRemoveMemberPtr_T<LambdaOperatorType>, 0>, const SWidget&>)
			{
				return FGetter::CreateStatic(
					InvalidateWidgetReasonConverter,
					FNestedGetter::CreateLambda(Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...),
					InPointerToAttributeOwner
				);
			}
			else
			{
				static_assert(std::is_same_v<TFunctionParameterType_T<TRemoveMemberPtr_T<LambdaOperatorType>, 0>, const WidgetType&>, "The first argument of the lambda needs to be either 'const SWidget&' or 'const WidgetType&'");
				return FGetter::CreateStatic(
					InvalidateWidgetReasonTypedConverter,
					FNestedGetterTyped::CreateLambda(Forward<LambdaType>(InCallable), Forward<PayloadTypes>(InputPayload)...),
					InPointerToAttributeOwner
				);
			}
		}

		static EInvalidateWidgetReason InvalidateWidgetReasonConverter(const SWidget& Widget, FNestedGetter Getter, PointerToAttributeOwnerType InPointerToAttributeOwner)
		{
			return Getter.Execute(Widget, static_cast<const WidgetType&>(Widget).*InPointerToAttributeOwner);
		}
		static EInvalidateWidgetReason InvalidateWidgetReasonTypedConverter(const SWidget& Widget, FNestedGetterTyped Getter, PointerToAttributeOwnerType InPointerToAttributeOwner)
		{
			return Getter.Execute(static_cast<const WidgetType&>(Widget), static_cast<const WidgetType&>(Widget).*InPointerToAttributeOwner);
		}
	};

	// using "not checked" delegate to disable race detection. This delegate is used in a static (FSlateWidgetClassData) and will be destructed after the tls array used in the race detection code (which will result in a use after free).
	using FAttributeValueChangedDelegate = TDelegate<void(SWidget&), FNotThreadSafeNotCheckedDelegateUserPolicy>;

	/**
	 * A delegate similar to FAttributeValueChangedDelegate but for nested attributes.
	 * The main difference is that a reference to the object owning the nested attribute is also passed as an argument.
	 */
	template<typename PointerToAttributeOwnerType>
	using TNestedAttributeValueChangedDelegate = TDelegate<void(SWidget&, TRemoveMemberPtr_T<PointerToAttributeOwnerType>&), FNotThreadSafeNotCheckedDelegateUserPolicy>;
	/**
	 * A delegate similar to FAttributeValueChangedDelegate but for nested attributes.
	 * The main difference is that a reference to the object owning the nested attribute is also passed as an argument.
	 * This templated version already converts the owning widget to the right type.
	 */
	template<typename PointerToAttributeOwnerType>
	using TNestedAttributeValueChangedDelegateTyped = TDelegate<void(TMemberPtrOuter_T<PointerToAttributeOwnerType>&, TRemoveMemberPtr_T<PointerToAttributeOwnerType>&), FNotThreadSafeNotCheckedDelegateUserPolicy>;


	/** */
	enum class ECallbackOverrideType
	{
		/** Replace the callback that the base class defined. */
		ReplacePrevious,
		/** Execute the callback that the base class defined, then execute the new callback. */
		ExecuteAfterPrevious,
		/** Execute the new callback, then execute the callback that the base class defined. */
		ExecuteBeforePrevious,
	};

public:
	struct FAttribute;
	struct FContainer;
	struct FContainerInitializer;
	struct FInitializer;

	using OffsetType = uint32;

	/** The default sort order that define in which order attributes will be updated. */
	static constexpr uint32 DefaultSortOrder(OffsetType Offset) { return Offset * 100; }


	/** */
	struct FContainer
	{
		friend FInitializer;

	public:
		FContainer() = default;
		FContainer(FName InName, OffsetType InOffset)
			: Name(InName)
			, Offset(InOffset)
		{}

		bool IsValid() const
		{
			return !Name.IsNone();
		}

		FName GetName() const
		{
			return Name;
		}

		uint32 GetSortOrder() const
		{
			return SortOrder;
		}

	private:
		FName Name;
		OffsetType Offset = 0;
		uint32 SortOrder = 0;
	};


	/** */
	struct FAttribute
	{
		friend FSlateAttributeDescriptor;
		friend FInitializer;
		friend FContainerInitializer;

	public:
		FAttribute(FName Name, OffsetType Offset, FInvalidateWidgetReasonAttribute Reason);
		FAttribute(FName ContainerName, FName Name, OffsetType Offset, FInvalidateWidgetReasonAttribute Reason);

		FName GetName() const
		{
			return Name;
		}

		uint32 GetSortOrder() const
		{
			return SortOrder;
		}

		EInvalidateWidgetReason GetInvalidationReason(const SWidget& Widget) const
		{
			return InvalidationReason.Get(Widget);
		}

		SlateAttributePrivate::ESlateAttributeType GetAttributeType() const
		{
			return AttributeType;
		}

		bool DoesAffectVisibility() const
		{
			return bAffectVisibility;
		}

		void ExecuteOnValueChangedIfBound(SWidget& Widget) const
		{
			OnValueChanged.ExecuteIfBound(Widget);
		}

	private:
		FName Name;
		OffsetType Offset;
		FName Prerequisite;
		FName ContainerName; // if the container IsNone, then the container is the SWidget itself.
		FAttributeValueChangedDelegate OnValueChanged;
		FInvalidateWidgetReasonAttribute InvalidationReason;
		uint32 SortOrder;
		uint8 ContainerIndex;
		SlateAttributePrivate::ESlateAttributeType AttributeType;
		bool bAffectVisibility;
	};

	/** Internal class to initialize the SlateAttributeDescriptor::FContainer attributes (Add attributes or modify existing attributes). */
	struct FContainerInitializer
	{
	private:
		friend FSlateAttributeDescriptor;
		SLATECORE_API FContainerInitializer(FSlateAttributeDescriptor& InDescriptor, FName ContainerName);
		SLATECORE_API FContainerInitializer(FSlateAttributeDescriptor& InDescriptor, const FSlateAttributeDescriptor& ParentDescriptor, FName ContainerName);

	public:
		FContainerInitializer() = delete;
		FContainerInitializer(const FContainerInitializer&) = delete;
		FContainerInitializer& operator= (const FContainerInitializer&) = delete;

		struct FAttributeEntry
		{
			SLATECORE_API FAttributeEntry(FSlateAttributeDescriptor& Descriptor, FName ContainerName, int32 AttributeIndex);

			/**
			 * Update the attribute after the prerequisite.
			 * The order is guaranteed but other attributes may be updated in between.
			 * No order is guaranteed if the prerequisite or this property is updated manually.
			 */
			SLATECORE_API FAttributeEntry& UpdatePrerequisite(FName Prerequisite);

			/**
			 * Notified when the attribute value changed.
			 * It's preferable that you delay any action to the Tick or Paint function.
			 * You are not allowed to make changes that would affect the SWidget ChildOrder or its Visibility.
			 * It will not be called when the SWidget is in its construction phase.
			 * @see SWidget::IsConstructed
			 */
			SLATECORE_API FAttributeEntry& OnValueChanged(FAttributeValueChangedDelegate Callback);

		private:
			FSlateAttributeDescriptor& Descriptor;
			FName ContainerName;
			int32 AttributeIndex;
		};

		SLATECORE_API FAttributeEntry AddContainedAttribute(FName AttributeName, OffsetType Offset, const FInvalidateWidgetReasonAttribute& ReasonGetter);
		SLATECORE_API FAttributeEntry AddContainedAttribute(FName AttributeName, SIZE_T Offset, const FInvalidateWidgetReasonAttribute& ReasonGetter);
		SLATECORE_API FAttributeEntry AddContainedAttribute(FName AttributeName, OffsetType Offset, FInvalidateWidgetReasonAttribute&& ReasonGetter);
		SLATECORE_API FAttributeEntry AddContainedAttribute(FName AttributeName, SIZE_T Offset, FInvalidateWidgetReasonAttribute&& ReasonGetter);

	public:
		/** Change the InvalidationReason of an attribute defined in a base class. */
		SLATECORE_API void OverrideInvalidationReason(FName AttributeName, const FInvalidateWidgetReasonAttribute& Reason);
		/** Change the InvalidationReason of an attribute defined in a base class. */
		SLATECORE_API void OverrideInvalidationReason(FName AttributeName, FInvalidateWidgetReasonAttribute&& Reason);

		/** Change the FAttributeValueChangedDelegate of an attribute defined in a base class. */
		SLATECORE_API void OverrideOnValueChanged(FName AttributeName, ECallbackOverrideType OverrideType, FAttributeValueChangedDelegate Callback);

	private:
		FSlateAttributeDescriptor& Descriptor;
		FName ContainerName;
	};

	/** Internal class to initialize the SlateAttributeDescriptor (Add attributes or modify existing attributes). */
	struct FInitializer
	{
	private:
		friend FSlateWidgetClassData;
		SLATECORE_API FInitializer(FSlateAttributeDescriptor& InDescriptor);
		SLATECORE_API FInitializer(FSlateAttributeDescriptor& InDescriptor, const FSlateAttributeDescriptor& ParentDescriptor);
		FInitializer(const FInitializer&) = delete;
		FInitializer& operator=(const FInitializer&) = delete;

	public:
		SLATECORE_API ~FInitializer();

		struct FAttributeEntry
		{
			SLATECORE_API FAttributeEntry(FSlateAttributeDescriptor& Descriptor, int32 InAttributeIndex);

			/**
			 * Update the attribute after the prerequisite.
			 * The order is guaranteed but other attributes may be updated in between.
			 * No order is guaranteed if the prerequisite or this property is updated manually.
			 */
			SLATECORE_API FAttributeEntry& UpdatePrerequisite(FName Prerequisite);

			/**
			 * The attribute affect the visibility of the widget.
			 * We only update the attributes that can change the visibility of the widget when the widget is collapsed.
			 * Attributes that affect visibility must have the Visibility attribute as a Prerequisite or the Visibility attribute must have it as a Prerequisite.
			 */
			SLATECORE_API FAttributeEntry& AffectVisibility();

			/**
			 * Notified when the attribute value changed.
			 * It's preferable that you delay any action to the Tick or Paint function.
			 * You are not allowed to make changes that would affect the SWidget ChildOrder or its Visibility.
			 * It will not be called when the SWidget is in its construction phase.
			 * @see SWidget::IsConstructed
			 */
			SLATECORE_API FAttributeEntry& OnValueChanged(FAttributeValueChangedDelegate Callback);

		private:
			FSlateAttributeDescriptor& Descriptor;
			int32 AttributeIndex;
		};

		/**
		 * Adjusted version of FAttributeEntry for Nested Attributes.
		 * The only difference is the signature of OnValueChanged, as we need to pass the PointerToAttributeOwner for a nested attribute
		 */
		template<typename PointerToAttributeOwnerType>
		struct TNestedAttributeEntry
		{
			static_assert(std::is_member_pointer_v<PointerToAttributeOwnerType>, "PointerToAttributeOwnerType needs to be a Pointer to Member."); \
			using WidgetType = TMemberPtrOuter_T<PointerToAttributeOwnerType>;
			using AttributeOwnerType = TRemoveMemberPtr_T<PointerToAttributeOwnerType>;

			TNestedAttributeEntry(FAttributeEntry&& InEntry, PointerToAttributeOwnerType InPointerToAttributeOwner)
				: Entry(MoveTemp(InEntry))
				, PointerToAttributeOwner(InPointerToAttributeOwner)
			{
			}

			/**
			 * Update the attribute after the prerequisite.
			 * The order is guaranteed but other attributes may be updated in between.
			 * No order is guaranteed if the prerequisite or this property is updated manually.
			 */
			TNestedAttributeEntry& UpdatePrerequisite(FName Prerequisite)
			{
				Entry.UpdatePrerequisite(Prerequisite);
				return *this;
			}

			/**
			 * The attribute affect the visibility of the widget.
			 * We only update the attributes that can change the visibility of the widget when the widget is collapsed.
			 * Attributes that affect visibility must have the Visibility attribute as a Prerequisite or the Visibility attribute must have it as a Prerequisite.
			 */
			TNestedAttributeEntry& AffectVisibility()
			{
				Entry.AffectVisibility();
				return *this;
			}

			/**
			 * Notified when the attribute value changed.
			 * It's preferable that you delay any action to the Tick or Paint function.
			 * You are not allowed to make changes that would affect the SWidget ChildOrder or its Visibility.
			 * It will not be called when the SWidget is in its construction phase.
			 * @see SWidget::IsConstructed
			 */
			TNestedAttributeEntry& OnValueChanged(TNestedAttributeValueChangedDelegate<PointerToAttributeOwnerType> Callback)
			{
				if (Callback.IsBound())
				{
					Entry.OnValueChanged(FAttributeValueChangedDelegate::CreateStatic(OnValueChangedConverter, MoveTemp(Callback), PointerToAttributeOwner));
				}
				return *this;
			}
			TNestedAttributeEntry& OnValueChanged(TNestedAttributeValueChangedDelegateTyped<PointerToAttributeOwnerType> Callback)
			{
				if (Callback.IsBound())
				{
					Entry.OnValueChanged(FAttributeValueChangedDelegate::CreateStatic(OnValueChangedConverter, MoveTemp(Callback), PointerToAttributeOwner));
				}
				return *this;
			}

			template<typename... VarTypes>
			TNestedAttributeEntry& OnValueChanged_Static(TIdentity_T<typename TNestedAttributeValueChangedDelegate<PointerToAttributeOwnerType>::template TFuncPtr<VarTypes...>> InFunc, VarTypes... Vars)
			{
				return OnValueChanged(TNestedAttributeValueChangedDelegate<PointerToAttributeOwnerType>::CreateStatic(InFunc, Vars...));
			}
			template<typename... VarTypes, typename U = std::enable_if<!std::is_same_v<WidgetType, SWidget>>::type>
			TNestedAttributeEntry& OnValueChanged_Static(TIdentity_T<typename TNestedAttributeValueChangedDelegateTyped<PointerToAttributeOwnerType>::template TFuncPtr<VarTypes...>> InFunc, VarTypes... Vars)
			{
				return OnValueChanged(TNestedAttributeValueChangedDelegateTyped<PointerToAttributeOwnerType>::CreateStatic(InFunc, Vars...));
			}
			template<typename LambdaType>
			TNestedAttributeEntry& OnValueChanged_Lambda(LambdaType&& InFunctor)
			{
				using LambdaOperatorType = decltype(&LambdaType::operator());
				static_assert(std::is_same_v<TFunctionParameterType_T<TRemoveMemberPtr_T<LambdaOperatorType>, 1>, AttributeOwnerType&>, "The second argument of the lambda needs to be 'AttributeOwnerType&'");

				if constexpr (std::is_same_v<TFunctionParameterType_T<TRemoveMemberPtr_T<LambdaOperatorType>, 0>, SWidget&>)
				{
					return OnValueChanged(TNestedAttributeValueChangedDelegate<PointerToAttributeOwnerType>::CreateLambda(MoveTemp(InFunctor)));
				}
				else
				{
					static_assert(std::is_same_v<TFunctionParameterType_T<TRemoveMemberPtr_T<LambdaOperatorType>, 0>, WidgetType&>, "The first argument of the lambda needs to be either 'SWidget&' or 'WidgetType&'");
					return OnValueChanged(TNestedAttributeValueChangedDelegateTyped<PointerToAttributeOwnerType>::CreateLambda(MoveTemp(InFunctor)));
				}
			}

		private:
			static void OnValueChangedConverter(SWidget& Widget, TNestedAttributeValueChangedDelegate<PointerToAttributeOwnerType> InCallback, PointerToAttributeOwnerType InPointerToAttributeOwner)
			{
				InCallback.ExecuteIfBound(Widget, static_cast<WidgetType&>(Widget).*InPointerToAttributeOwner);
			}
			static void OnValueChangedConverter(SWidget& Widget, TNestedAttributeValueChangedDelegateTyped<PointerToAttributeOwnerType> InCallback, PointerToAttributeOwnerType InPointerToAttributeOwner)
			{
				InCallback.ExecuteIfBound(static_cast<WidgetType&>(Widget), static_cast<WidgetType&>(Widget).*InPointerToAttributeOwner);
			}

		private:
			FAttributeEntry Entry;
			PointerToAttributeOwnerType PointerToAttributeOwner;
		};

		SLATECORE_API FAttributeEntry AddMemberAttribute(FName AttributeName, OffsetType Offset, const FInvalidateWidgetReasonAttribute& ReasonGetter);
		SLATECORE_API FAttributeEntry AddMemberAttribute(FName AttributeName, OffsetType Offset, FInvalidateWidgetReasonAttribute&& ReasonGetter);

		template<typename PointerToAttributeOwnerType>
		TNestedAttributeEntry<PointerToAttributeOwnerType> AddNestedMemberAttribute(PointerToAttributeOwnerType InPointerToAttributeOwner, FName AttributeName, OffsetType Offset, const FInvalidateWidgetReasonAttribute& ReasonGetter)
		{
			return TNestedAttributeEntry<PointerToAttributeOwnerType>(AddMemberAttribute(AttributeName, Offset, ReasonGetter), InPointerToAttributeOwner);
		}
		template<typename PointerToAttributeOwnerType>
		TNestedAttributeEntry<PointerToAttributeOwnerType> AddNestedMemberAttribute(PointerToAttributeOwnerType InPointerToAttributeOwner, FName AttributeName, OffsetType Offset, FInvalidateWidgetReasonAttribute&& ReasonGetter)
		{
			return TNestedAttributeEntry<PointerToAttributeOwnerType>(AddMemberAttribute(AttributeName, Offset, MoveTemp(ReasonGetter)), InPointerToAttributeOwner);
		}

		SLATECORE_API FContainerInitializer AddContainer(FName ContainerName, OffsetType Offset);

	public:
		/** Change the InvalidationReason of an attribute defined in a base class. */
		SLATECORE_API void OverrideInvalidationReason(FName AttributeName, const FInvalidateWidgetReasonAttribute& Reason);
		/** Change the InvalidationReason of an attribute defined in a base class. */
		SLATECORE_API void OverrideInvalidationReason(FName AttributeName, FInvalidateWidgetReasonAttribute&& Reason);

		/** Change the FAttributeValueChangedDelegate of an attribute defined in a base class. */
		SLATECORE_API void OverrideOnValueChanged(FName AttributeName, ECallbackOverrideType OverrideType, FAttributeValueChangedDelegate Callback);

		/** Change the update type of an attribute defined in a base class. */
		SLATECORE_API void SetAffectVisibility(FName AttributeName, bool bAffectVisibility);

	private:
		FSlateAttributeDescriptor& Descriptor;
	};

	/** @returns the number of Attributes registered. */
	int32 GetAttributeNum() const
	{
		return Attributes.Num();
	}

	/** @returns the Attribute at the index previously found with IndexOfMemberAttribute */
	SLATECORE_API const FAttribute& GetAttributeAtIndex(int32 Index) const;

	/** @returns the Container with the corresponding name. */
	SLATECORE_API const FContainer* FindContainer(FName ContainerName) const;

	/** @returns the Attribute with the corresponding name. */
	SLATECORE_API const FAttribute* FindAttribute(FName AttributeName) const;

	/** @returns the Attribute of a SlateAttribute that have the corresponding memory offset. */
	SLATECORE_API const FAttribute* FindMemberAttribute(OffsetType AttributeOffset) const;

	/** @returns the Attribute of a SlateAttribute that have the corresponding memory offset. */
	SLATECORE_API const FAttribute* FindContainedAttribute(FName ContainerName, OffsetType AttributeOffset) const;

	/** @returns the index of the Container with the corresponding name. */
	SLATECORE_API int32 IndexOfContainer(FName AttributeName) const;

	/** @returns the index of a SlateAttribute that have the corresponding memory offset. */
	SLATECORE_API int32 IndexOfAttribute(FName AttributeName) const;

	/** @returns the index of a SlateAttribute that have the corresponding memory offset. */
	SLATECORE_API int32 IndexOfMemberAttribute(OffsetType AttributeOffset) const;

	/** @returns the index of a SlateAttribute that have the corresponding memory offset. */
	SLATECORE_API int32 IndexOfContainedAttribute(FName ContainerName, OffsetType AttributeOffset) const;

private:
	SLATECORE_API FAttribute* FindAttribute(FName AttributeName);

	SLATECORE_API FContainerInitializer AddContainer(FName AttributeName, OffsetType Offset);
	SLATECORE_API FInitializer::FAttributeEntry AddMemberAttribute(FName AttributeName, OffsetType Offset, FInvalidateWidgetReasonAttribute ReasonGetter);
	SLATECORE_API FContainerInitializer::FAttributeEntry AddContainedAttribute(FName ContainerName, FName AttributeName, OffsetType Offset, FInvalidateWidgetReasonAttribute ReasonGetter);
	SLATECORE_API void OverrideInvalidationReason(FName ContainerName, FName AttributeName, FInvalidateWidgetReasonAttribute ReasonGetter);
	SLATECORE_API void OverrideOnValueChanged(FName ContainerName, FName AttributeName, ECallbackOverrideType OverrideType, FAttributeValueChangedDelegate Callback);
	SLATECORE_API void SetPrerequisite(FName ContainerName, FAttribute& Attribute, FName Prerequisite);
	SLATECORE_API void SetAffectVisibility(FAttribute& Attribute, bool bUpdate);

private:
	TArray<FAttribute> Attributes;
	TArray<FContainer, TInlineAllocator<1>> Containers;
};

/**
 * Add a TSlateAttribute to the descriptor.
 * @param _Initializer The FSlateAttributeInitializer from the PrivateRegisterAttributes function.
 * @param _Property The TSlateAttribute property
 * @param _Reason The EInvalidationWidgetReason or a static function/lambda that takes a const SWidget& and that returns the invalidation reason.
 */
#define SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(_Initializer, _Name, _Property, _Reason) \
		static_assert(decltype(_Property)::AttributeType == SlateAttributePrivate::ESlateAttributeType::Member, "The SlateProperty is not a TSlateAttribute. Do not use SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION"); \
		static_assert(!decltype(_Property)::HasDefinedInvalidationReason, "When implementing the SLATE_DECLARE_WIDGET pattern, use TSlateAttribute without the invalidation reason."); \
		static_assert(!std::is_same<decltype(_Reason), EInvalidateWidgetReason>::value || FSlateAttributeBase::IsInvalidateWidgetReasonSupported(_Reason), "The invalidation is not supported by the SlateAttribute system."); \
		const std::ptrdiff_t UE_JOIN(__OffsetFromSWidget_, __LINE__) = STRUCT_OFFSET(PrivateThisType, _Property) - SlateAttributePrivate::GetBaseClassOffset<PrivateThisType, SWidget>(); \
		checkf(UE_JOIN(__OffsetFromSWidget_, __LINE__) >= std::numeric_limits<FSlateAttributeDescriptor::OffsetType>::lowest(), TEXT("The offset from SWidget for attribute " #_Property " is too small")); \
		checkf(UE_JOIN(__OffsetFromSWidget_, __LINE__) <= std::numeric_limits<FSlateAttributeDescriptor::OffsetType>::max(), TEXT("The offset from SWidget for attribute " #_Property " is too big")); \
		_Initializer.AddMemberAttribute(_Name, (FSlateAttributeDescriptor::OffsetType)UE_JOIN(__OffsetFromSWidget_, __LINE__), FSlateAttributeDescriptor::FInvalidateWidgetReasonAttribute{_Reason})

#define SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(_Initializer, _Property, _Reason) \
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(_Initializer, GET_MEMBER_NAME_CHECKED(PrivateThisType, _Property), _Property, _Reason)

/**
 * Add a TSlateAttribute nested inside a member of the widget to the descriptor.
 * @param _Initializer The FSlateAttributeInitializer from the PrivateRegisterAttributes function.
 * @param _PointerToAttributeOwner	The Pointer to Member of the widget property owning _Property.
 *									The widget's type is referred below to 'WidgetType' and the member type (owning the property) is referred to 'AttributeOwnerType'
 * @param _Property The TSlateAttribute property
 * @param _Reason The EInvalidationWidgetReason or a static function/lambda that takes (const SWidget&, const AttributeOwnerType&) or (const WidgetType&, const AttributeOwnerType&) as arguments, and that returns the invalidation reason.
 */
#define SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(_Initializer, _Name, _PointerToAttributeOwner, _Property, _Reason) \
		static_assert(std::is_member_pointer_v<decltype(_PointerToAttributeOwner)>, "The _PointerToAttributeOwner variable needs to be a Pointer to Member."); \
		{ \
			using __AttributeOwnerType = TRemoveMemberPtr_T<decltype(_PointerToAttributeOwner)>; \
			static_assert(decltype(__AttributeOwnerType::_Property)::template IsNestedAttributeValid<decltype(_PointerToAttributeOwner), decltype(&__AttributeOwnerType::_Property)>(), "_PointerToAttributeOwner needs to be defined inside a widget, and _Property needs to be defined inside _PointerToAttributeOwner"); \
			static_assert(decltype(__AttributeOwnerType::_Property)::AttributeType == SlateAttributePrivate::ESlateAttributeType::Member, "The SlateProperty is not a TSlateNestedAttribute. Use SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION"); \
			static_assert(!decltype(__AttributeOwnerType::_Property)::HasDefinedInvalidationReason, "When implementing the SLATE_DECLARE_WIDGET pattern, use TSlateAttribute without the invalidation reason."); \
			using __ReasonType = decltype(_Reason); \
			static_assert(!std::is_same_v<__ReasonType, EInvalidateWidgetReason> || FSlateAttributeBase::IsInvalidateWidgetReasonSupported(_Reason), "The invalidation is not supported by the SlateAttribute system."); \
		}\
		_Initializer.AddNestedMemberAttribute(_PointerToAttributeOwner, _Name, SlateAttributePrivate::OffsetOfPointerToMember(_PointerToAttributeOwner) + STRUCT_OFFSET(TRemoveMemberPtr_T<decltype(_PointerToAttributeOwner)>, _Property), FSlateAttributeDescriptor::TInvalidateWidgetReasonNestedAttribute{_PointerToAttributeOwner, _Reason})

#define SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(_Initializer, _PointerToAttributeOwner, _Property, _Reason) \
	SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(_Initializer, GET_MEMBER_NAME_CHECKED(TRemoveMemberPtr_T<decltype(_PointerToAttributeOwner)>, _Property), _PointerToAttributeOwner, _Property, _Reason)

#define SLATE_ADD_PANELCHILDREN_DEFINITION_WITH_NAME(_Initializer, _Name, _Container) \
		_Initializer.AddContainer(_Name, STRUCT_OFFSET(PrivateThisType, _Container))

#define SLATE_ADD_PANELCHILDREN_DEFINITION(_Initializer, _Container) \
		SLATE_ADD_PANELCHILDREN_DEFINITION_WITH_NAME(_Initializer, GET_MEMBER_NAME_CHECKED(PrivateThisType, _Container), _Container)

#define SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(_SlotType, _Initializer, _Name, _Property, _Reason) \
		static_assert(decltype(_Property)::AttributeType == SlateAttributePrivate::ESlateAttributeType::Contained, "The SlateProperty is not a TSlateAttribute. Do not use SLATE_ADD_CONTAINED_ATTRIBUTE_DEFINITION"); \
		static_assert(!decltype(_Property)::HasDefinedInvalidationReason, "When implementing the SLATE_DECLARE_WIDGET pattern, use TSlateSlotAttribute without the invalidation reason."); \
		static_assert(!std::is_same<decltype(_Reason), EInvalidateWidgetReason>::value || FSlateAttributeBase::IsInvalidateWidgetReasonSupported(_Reason), "The invalidation is not supported by the SlateAttribute system."); \
		/** We do not use STRUCT_OFFSET here. At runtime we use the ISlateAttributeContainer as the base pointer. FSlot uses multi-inheritance. */ \
		_Initializer.AddContainedAttribute(_Name, ((SIZE_T)(&(((_SlotType*)(0x1000))->_Property)) - (SIZE_T)(static_cast<SlateAttributePrivate::ISlateAttributeContainer*>((_SlotType*)(0x1000)))), FSlateAttributeDescriptor::FInvalidateWidgetReasonAttribute{_Reason})

#define SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION(_SlotType, _Initializer, _Property, _Reason) \
	SLATE_ADD_SLOT_ATTRIBUTE_DEFINITION_WITH_NAME(_SlotType, _Initializer, GET_MEMBER_NAME_CHECKED(_SlotType, _Property), _Property, _Reason)

using FSlateAttributeInitializer = FSlateAttributeDescriptor::FInitializer;
using FSlateWidgetSlotAttributeInitializer = FSlateAttributeDescriptor::FContainerInitializer;
