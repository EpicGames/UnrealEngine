// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Algo/BinarySearch.h"
#include "Types/ISlateMetaData.h"
#include "Types/SlateAttribute.h"
#include "Types/SlateAttributeDescriptor.h"
#include "Widgets/InvalidateWidgetReason.h"

#include <limits>

class SWidget;

/** */
class FSlateAttributeMetaData : public ISlateMetaData
{
	friend SlateAttributePrivate::FSlateAttributeImpl;
	friend SlateAttributePrivate::ISlateAttributeContainer;

public:
	SLATE_METADATA_TYPE(FSlateAttributeMetaData, ISlateMetaData);

	FSlateAttributeMetaData() = default;
	FSlateAttributeMetaData(const FSlateAttributeMetaData&) = delete;
	FSlateAttributeMetaData& operator=(const FSlateAttributeMetaData&) = delete;

public:
	/** @return the instance associated to the SWidget (if it exists). */
	static SLATECORE_API FSlateAttributeMetaData* FindMetaData(const SWidget& OwningWidget);

	enum class EInvalidationPermission : uint8
	{
		/** Invalidate the widget if it's needed and it's construction phase is completed. */
		AllowInvalidationIfConstructed,
		/** Invalidate the widget if it's needed. */
		AllowInvalidation,
		/** Cache the invalidation. On any future update, if it's needed, invalidate the widget. */
		DelayInvalidation,
		/** Never invalidate the widget. */
		DenyInvalidation,
		/** Never invalidate the widget and clear any delayed invalidation. */
		DenyAndClearDelayedInvalidation,
	};

	/**
	 * Update all the attributes.
	 * @param InvalidationStyle if we should invalidate the widget.
	 */
	static SLATECORE_API void UpdateAllAttributes(SWidget& OwningWidget, EInvalidationPermission InvalidationStyle);
	/**
	 * Update attributes that are responsible to change visibility of the widget.
	 * @param InvalidationStyle if we should invalidate the widget.
	 */
	static SLATECORE_API void UpdateOnlyVisibilityAttributes(SWidget& OwningWidget, EInvalidationPermission InvalidationStyle);
	/**
	 * Update attributes that are NOT responsible to change visibility of the widget.
	 * @param InvalidationStyle if we should invalidate the widget.
	 */
	static SLATECORE_API void UpdateExceptVisibilityAttributes(SWidget& OwningWidget, EInvalidationPermission InvalidationStyle);
	/**
	 * Execute UpdateOnlyVisibilityAttributes on every children of the widget.
	 * @param InvalidationStyle if we should invalidate the widget.
	 */
	static SLATECORE_API void UpdateChildrenOnlyVisibilityAttributes(SWidget& OwningWidget, EInvalidationPermission InvalidationStyle, bool bRecursive);
	/**
	 * Apply any delayed invalidation previously delayed with EInvalidationPermission::DelayInvalidation.
	 */
	static SLATECORE_API void ApplyDelayedInvalidation(SWidget& OwningWidget);

public:
	bool IsBound(const FSlateAttributeBase& Attribute) const
	{
		return IndexOfAttribute(Attribute) != INDEX_NONE;
	}

	int32 GetRegisteredAttributeCount() const
	{
		return Attributes.Num();
	}

	int32 GetRegisteredAffectVisibilityAttributeCount() const
	{
		return AffectVisibilityCounter;
	}

	/** Get the name of all the attributes, if available. */
	static SLATECORE_API TArray<FName> GetAttributeNames(const SWidget& OwningWidget);

	/**
	 * Helper function that can be used inside the debuggers watch window.
	 * E.g. "DebugSlateAttribute(this, 0)"
	 * E.g. "DebugSlateAttribute((SWidget*)(0x123456789), 0)"
	 */ 
	friend const TCHAR* DebugSlateAttribute(const SWidget* Widget, int32 Index);

private:
	using ISlateAttributeContainer = SlateAttributePrivate::ISlateAttributeContainer;
	using ESlateAttributeType = SlateAttributePrivate::ESlateAttributeType;
	using ISlateAttributeGetter = SlateAttributePrivate::ISlateAttributeGetter;
	static SLATECORE_API void RegisterAttribute(SWidget& OwningWidget, FSlateAttributeBase& Attribute, ESlateAttributeType AttributeType, TUniquePtr<ISlateAttributeGetter>&& Wrapper);
	static SLATECORE_API void RegisterAttribute(ISlateAttributeContainer& OwningContainer, FSlateAttributeBase& Attribute, ESlateAttributeType AttributeType, TUniquePtr<ISlateAttributeGetter>&& Wrapper);
	static SLATECORE_API bool UnregisterAttribute(SWidget& OwningWidget, const FSlateAttributeBase& Attribute);
	static SLATECORE_API bool UnregisterAttribute(ISlateAttributeContainer& OwningContainer, const FSlateAttributeBase& Attribute);
	static SLATECORE_API void InvalidateWidget(SWidget& OwningWidget, const FSlateAttributeBase& Attribute, ESlateAttributeType AttributeType, EInvalidateWidgetReason Reason);
	static SLATECORE_API void InvalidateWidget(ISlateAttributeContainer& OwningContainer, const FSlateAttributeBase& Attribute, ESlateAttributeType AttributeType, EInvalidateWidgetReason Reason);
	static SLATECORE_API void UpdateAttribute(SWidget& OwningWidget, FSlateAttributeBase& Attribute);
	static SLATECORE_API bool IsAttributeBound(const SWidget& OwningWidget, const FSlateAttributeBase& Attribute);
	static SLATECORE_API SlateAttributePrivate::ISlateAttributeGetter* GetAttributeGetter(const SWidget& OwningWidget, const FSlateAttributeBase& Attribute);
	static SLATECORE_API FDelegateHandle GetAttributeGetterHandle(const SWidget& OwningWidget, const FSlateAttributeBase& Attribute);
	static SLATECORE_API void MoveAttribute(const SWidget& OwningWidget, FSlateAttributeBase& NewAttribute, ESlateAttributeType AttributeType, const FSlateAttributeBase* PreviousAttribute);

	//~ For Container
	static SLATECORE_API void RemoveContainerWidget(SWidget& Widget, ISlateAttributeContainer& Container);
	static SLATECORE_API void UpdateContainerSortOrder(SWidget& Widget, ISlateAttributeContainer& Container);


private:
	SLATECORE_API void RegisterMemberAttributeImpl(SWidget& OwningWidget, FSlateAttributeBase& Attribute, TUniquePtr<ISlateAttributeGetter>&& Getter);
	SLATECORE_API void RegisterManagedAttributeImpl(SWidget& OwningWidget, FSlateAttributeBase& Attribute, TUniquePtr<ISlateAttributeGetter>&& Getter);
	SLATECORE_API void RegisterContainAttributeImpl(ISlateAttributeContainer& OwningContainer, SWidget& OwningWidget, FSlateAttributeBase& Attribute, TUniquePtr<ISlateAttributeGetter>&& Getter);
	SLATECORE_API bool UnregisterAttributeImpl(const FSlateAttributeBase& Attribute);
	SLATECORE_API void RemoveMetaDataIfNeeded(SWidget& OwningWidget, bool bRemoved) const;
	SLATECORE_API bool InvalidateWidgetAttribute(SWidget& OwningWidget, const FSlateAttributeBase& Attribute, EInvalidateWidgetReason Reason);
	SLATECORE_API void UpdateAttributesImpl(SWidget& OwningWidget, EInvalidationPermission InvaldiationStyle, int32 StartIndex, int32 IndexNum);

private:
	int32 IndexOfAttribute(const FSlateAttributeBase& Attribute) const
	{
		const FSlateAttributeBase* AttributePtr = &Attribute;
		return Attributes.IndexOfByPredicate([AttributePtr](const FGetterItem& Item) { return Item.Attribute == AttributePtr; });
	}

	struct FGetterItem
	{
		FGetterItem(const FGetterItem&) = delete;
		FGetterItem(FGetterItem&&) = default;
		FGetterItem& operator=(const FGetterItem&) = delete;
		FGetterItem(FSlateAttributeBase* InAttribute, uint32 InSortOrder, TUniquePtr<SlateAttributePrivate::ISlateAttributeGetter>&& InGetter)
			: Attribute(InAttribute)
			, Getter(MoveTemp(InGetter))
			, CachedAttributeDescriptor(nullptr)
			, SortOrder(InSortOrder)
			, AttributeContainerOffset(0)
		{ }
		FGetterItem(FSlateAttributeBase* InAttribute, uint32 InSortOrder, TUniquePtr<SlateAttributePrivate::ISlateAttributeGetter> && InGetter, const FSlateAttributeDescriptor::FAttribute* InAttributeDescriptor)
			: Attribute(InAttribute)
			, Getter(MoveTemp(InGetter))
			, CachedAttributeDescriptor(InAttributeDescriptor)
			, SortOrder(InSortOrder)
			, AttributeContainerOffset(0)
		{ }

		FSlateAttributeBase* Attribute;
		TUniquePtr<SlateAttributePrivate::ISlateAttributeGetter> Getter;
		const FSlateAttributeDescriptor::FAttribute* CachedAttributeDescriptor;
		uint32 SortOrder;
		int32 AttributeContainerOffset; // to fit inside a 32b structure, save the offset of the AttributeContainer;

		bool operator<(const FGetterItem& Other) const
		{
			// if in a container, the smaller container wins first, then the property SortOrder.
			if (AttributeContainerOffset && Other.AttributeContainerOffset)
			{
				const SlateAttributePrivate::ISlateAttributeContainer* Container = GetAttributeContainer();
				const SlateAttributePrivate::ISlateAttributeContainer* OtherContainer = Other.GetAttributeContainer();
				if (Container == OtherContainer)
				{
					return SortOrder < Other.SortOrder;
				}
				else
				{
					return Container->GetContainerSortOrder() < OtherContainer->GetContainerSortOrder();
				}
			}
			return SortOrder < Other.SortOrder;
		}

		/** If available, return the name of the attribute. */
		FName GetAttributeName(const SWidget& OwningWidget) const;

		/** @return a pointer to the attribute container if the SlateAttribute type is Contained. */
		const SlateAttributePrivate::ISlateAttributeContainer* GetAttributeContainer() const;
	};
	static_assert(sizeof(FGetterItem) <= 32, "The size of FGetterItem is bigger than expected.");

	TArray<FGetterItem, TInlineAllocator<4>> Attributes;
	//~ There is a possibility that the widget has a CachedInvalidationReason and a parent become collapsed.
	//~The invalidation will probably never get executed but
	//~1. The widget is collapsed indirectly, so we do not care if it's invalidated.
	//~2. The parent widget will clear this widget PersistentState.
	EInvalidateWidgetReason CachedInvalidationReason = EInvalidateWidgetReason::None;
	uint8 AffectVisibilityCounter = 0;
};

