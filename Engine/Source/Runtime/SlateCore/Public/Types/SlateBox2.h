// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/SlateVector2.h"
#include "Math/Box2D.h"

namespace UE::Slate
{
	
inline static FBox2f CastToBox2f(const FBox2d& InValue)
{
	return FBox2f(InValue);
}

/**
 * Structure for deprecating FBox2D to FBox2f
 */
struct FDeprecateBox2D
{
	FDeprecateBox2D() = default;
	FDeprecateBox2D(const FBox2f& InValue)
		: Data(InValue)
	{
	}

public:
	operator FBox2d() const
	{
		return FBox2d(Data);
	}

	operator FBox2f() const
	{
		return Data;
	}
	
public:
	bool operator==(const FBox2d& Other) const
	{
		return CastToBox2f(Other) == Data;
	}
	
	bool operator==(const FBox2f& Other) const
	{
		return Data == Other;
	}

	bool operator!=(const FBox2d& Other) const
	{
		return CastToBox2f(Other) != Data;
	}
	
	bool operator!=(const FBox2f& Other) const
	{
		return Data != Other;
	}

private:
	FBox2f Data;
};

} // namespace
