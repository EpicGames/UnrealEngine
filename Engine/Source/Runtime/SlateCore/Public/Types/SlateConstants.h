// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"

/** How much to scroll for each click of the mouse wheel (in Slate Screen Units). */
extern SLATECORE_API TAutoConsoleVariable<float> GlobalScrollAmount;
inline float GetGlobalScrollAmount() { return GlobalScrollAmount.GetValueOnAnyThread(); }

/** How much to multiply scroll amount when using gamepad. */
extern SLATECORE_API TAutoConsoleVariable<float> GlobalGamepadScrollMultiplier;
inline float GetGlobalGamepadScrollMultiplier() { return GlobalGamepadScrollMultiplier.GetValueOnAnyThread(); }

/**  */
extern SLATECORE_API float GSlateContrast;
