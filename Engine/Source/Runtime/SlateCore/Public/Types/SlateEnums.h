// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/EnumRange.h"
#include "SlateEnums.generated.h"


UENUM(BlueprintType)
namespace EButtonClickMethod
{
	/**
	 * Enumerates different methods that a button click can be triggered. Normally, DownAndUp is appropriate.
	 */
	enum Type : int
	{
		/**
		 * User must press the button, then release while over the button to trigger the click.
		 * This is the most common type of button.
		 */
		DownAndUp,

		/**
		 * Click will be triggered immediately on mouse down, and mouse will not be captured.
		 */
		MouseDown,

		/**
		 * Click will always be triggered when mouse button is released over the button,
		 * even if the button wasn't pressed down over it.
		 */
		MouseUp,

		/**
		 * Inside a list, buttons can only be clicked with precise tap.
		 * Moving the pointer will scroll the list, also allows drag-droppable buttons.
		 */
		 PreciseClick
	};
}


UENUM(BlueprintType)
namespace EButtonTouchMethod
{
	/** Ways in which touch interactions trigger a "Clicked" event. */
	enum Type : int
	{
		/** Most buttons behave this way. */
		DownAndUp,

		/**
		 * Click will be triggered immediately on touch down, and touch will not be captured.
		 */
		Down,

		/**
		 * Inside a list, buttons can only be clicked with precise tap.
		 * Moving the pointer will scroll the list.
		 */
		PreciseTap
	};
}

UENUM(BlueprintType)
namespace EButtonPressMethod
{
	/**
	* Enumerates different methods that a button can be triggered with keyboard/controller. Normally, DownAndUp is appropriate.
	*/
	enum Type : int
	{
		/**
		* User must press the button, then release while the button has focus to trigger the click.
		* This is the most common type of button.
		*/
		DownAndUp,

		/**
		* Click will be triggered immediately on button press.
		*/
		ButtonPress,

		/**
		* Click will always be triggered when a button release occurs on the focused button,
		* even if the button wasn't pressed while focused.
		*/
		ButtonRelease,
	};
}

/**
 * Navigation context for event
 */
UENUM(BlueprintType)
enum class EUINavigation : uint8
{
	/** Four cardinal directions*/
	Left,
	Right,
	Up,
	Down,

	/** Conceptual next and previous*/
	Next,
	Previous,

	/** Number of navigation types*/
	Num UMETA(Hidden),

	/** Denotes an invalid navigation, more important used to denote no specified navigation*/
	Invalid,
};

ENUM_RANGE_BY_COUNT(EUINavigation, EUINavigation::Num);

/**
 * Generic UI navigation action that are usually bound to a gamepad or other input device
 */
UENUM(BlueprintType)
enum class EUINavigationAction : uint8
{
	/** Accept, normally acts like a mouse click. Corresponds to Virtual_Gamepad_Accept key */
	Accept,

	/** Back/Cancel, normally acts like escape. Corresponds to Virtual_Gamepad_Back key */
	Back,

	/** Number of navigation actions */
	Num UMETA(Hidden),

	/** Invalid action */
	Invalid,
};

ENUM_RANGE_BY_COUNT(EUINavigationAction, EUINavigationAction::Num);

/**
 * Enumerates the source of the navigation
 */
UENUM()
enum class ENavigationSource : uint8
{
	/** Navigate from the focused widget */
	FocusedWidget,
	
	/** Navigate from the widget under the cursor */
	WidgetUnderCursor,
};

/**
 * Enumerates the genesis of the navigation, where generated the navigation
 */
UENUM()
enum class ENavigationGenesis : uint8
{
	/** Navigation caused by the Keyboard */
	Keyboard,
	
	/** Navigation caused by a Controller */
	Controller,

	/** Navigate caused by a user generated event (Users = WIdgets, Client Code, ...)*/
	User,
};

/**
 * Enumerates horizontal alignment options, i.e. for widget slots.
 */
UENUM(BlueprintType)
enum EHorizontalAlignment : int
{
	/** Fill the entire width. */
	HAlign_Fill UMETA(DisplayName="Fill"),

	/** Left-align. */
	HAlign_Left UMETA(DisplayName="Left"),

	/** Center-align. */
	HAlign_Center UMETA(DisplayName="Center"),

	/** Right-align. */
	HAlign_Right UMETA(DisplayName="Right"),	
};


/**
 * Enumerates vertical alignment options, i.e. for widget slots.
 */
UENUM(BlueprintType)
enum EVerticalAlignment : int
{
	/** Fill the entire height. */
	VAlign_Fill UMETA(DisplayName="Fill"),

	/** Top-align. */
	VAlign_Top UMETA(DisplayName="Top"),

	/** Center-align. */
	VAlign_Center UMETA(DisplayName="Center"),

	/** Bottom-align. */
	VAlign_Bottom UMETA(DisplayName="Bottom"),
};


/**
 * Enumerates possible placements for pop-up menus.
 */
UENUM(BlueprintType)
enum EMenuPlacement : int
{
	/** Place the menu immediately below the anchor */
	MenuPlacement_BelowAnchor UMETA(DisplayName="Below"),

	/** Place the menu immediately centered below the anchor */
	MenuPlacement_CenteredBelowAnchor UMETA(DisplayName = "Centered Below"),

	/** Place the menu immediately below the anchor aligned to the right of the content */
	MenuPlacement_BelowRightAnchor UMETA(DisplayName = "Below Right"),

	/** Place the menu immediately below the anchor and match is width to the anchor's content */
	MenuPlacement_ComboBox UMETA(DisplayName="Combo Box"),

	/** Place the menu immediately below the anchor and match is width to the anchor's content. If the width overflows, align with the right edge of the anchor. */
	MenuPlacement_ComboBoxRight UMETA(DisplayName="Combo Box Right"),

	/** Place the menu to the right of the anchor */
	MenuPlacement_MenuRight UMETA(DisplayName="Right"),

	/** Place the menu immediately above the anchor, no transition effect */
	MenuPlacement_AboveAnchor UMETA(DisplayName="Above"),

	/** Place the menu immediately centered above the anchor, no transition effect */
	MenuPlacement_CenteredAboveAnchor UMETA(DisplayName="Centered Above"),

	/** Place the menu immediately above the anchor aligned to the right of the content */
	MenuPlacement_AboveRightAnchor UMETA(DisplayName = "Above Right"),

	/** Place the menu to the left of the anchor */
	MenuPlacement_MenuLeft UMETA(DisplayName="Left"),

	/** Place the menu's center on top of the menu anchor's center point */
	MenuPlacement_Center UMETA(DisplayName = "Center"),

	/** Place the menu's vertical center on the left side at the menu anchor's vertical center on the right side */
	MenuPlacement_RightLeftCenter UMETA(DisplayName = "Right Center, Left Center"),
	
	/** Place the menu's bottom left corner directly on top of the menu anchor's bottom left corner */
	MenuPlacement_MatchBottomLeft UMETA(DisplayName = "Match Bottom Left Corner"),
};


/**
 * Enumerates widget orientations.
 */
UENUM(BlueprintType)
enum EOrientation : int
{
	/** Orient horizontally, i.e. left to right. */
	Orient_Horizontal UMETA(DisplayName="Horizontal"),

	/** Orient vertically, i.e. top to bottom. */
	Orient_Vertical UMETA(DisplayName="Vertical"),
};


/**
 * Enumerates scroll directions.
 */
UENUM(BlueprintType)
enum EScrollDirection : int
{
	/** Scroll down. */
	Scroll_Down UMETA(DisplayName="Down"),

	/** Scroll up. */
	Scroll_Up UMETA(DisplayName="Up"),
};


/**
 * Additional information about a text committal
 */
UENUM(BlueprintType)
namespace ETextCommit
{
	enum Type : int
	{
		/** Losing focus or similar event caused implicit commit */
		Default,
		/** User committed via the enter key */
		OnEnter,
		/** User committed via tabbing away or moving focus explicitly away */
		OnUserMovedFocus,
		/** Keyboard focus was explicitly cleared via the escape key or other similar action */
		OnCleared
	};
}


/**
 * Additional information about a selection event
 */
UENUM(BlueprintType)
namespace ESelectInfo
{
	enum Type : int
	{
		/** User selected via a key press */
		OnKeyPress,
		/** User selected by navigating to the item */
		OnNavigation,
		/** User selected by clicking on the item */
		OnMouseClick,
		/** Selection was directly set in code */
		Direct
	};
}


/**
* Return type for FWidgetActiveTimerDelegate.
* Don't expose to blueprints.
*/
enum class EActiveTimerReturnType : uint8
{
	/** If this value is returned, the widget's active timer will be unregistered automatically. No need to call UnRegisterActiveTimer. */
	Stop,
	/** If this value is returned, the widget will continue to have its timer delegate called on it. */
	Continue,
};
