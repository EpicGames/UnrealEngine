// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlateAttribute.h"
#include "Attributes/SlateAttributeDefinition.inl"
#include "Attributes/SlateAttributeNestedMember.inl"


/**
 * A base type that defines TSlateNestedAttributes.
 * These nested Attributes are similar to regular TSlateAttributes but can be defined outside a widget as long as
 * the type owning the TSlateNestedAttributes is inside a widget.
 *
 * Example implementation:
 *  @code
// ------ .h

struct SWidgetWithNestedAttributes;

struct FSimpleNestedAttributesOwner : public FSlateNestedAttributeContainer // Define the type owning our attributes
{
	// To register the Nested Attributes, we need a pointer to member from the widget to this struct
	static void RegisterNestedAttributes(FSlateAttributeInitializer& AttributeInitializer, FSimpleNestedAttributesOwner SWidgetWithNestedAttributes::* InPointerToOwner);

	FSimpleNestedAttributesOwner(SWidgetWithNestedAttributes& Owner);

	void SetNestedIntAttribute(TAttribute<int> Attr);
	int GetNestedIntAttr() const { return NestedIntAttribute.Get(); }
	void SetNestedFloatAttribute(TAttribute<float> Attr);
	float GetNestedFloatAttr() const { return NestedFloatAttribute.Get(); }

private:
	// Callbacks given to OnValueChanged. The first argument can either be SWidget& or the type of the widget (here SWidgetWithNestedAttributes&)
	static void OnNestedIntAttributeChanged(SWidget& OwningWidget, FSimpleNestedAttributesOwner& NestedAttributesOwner);
	static void OnNestedFloatAttributeChanged(SWidgetWithNestedAttributes& OwningWidget, FSimpleNestedAttributesOwner& NestedAttributesOwner);

	SWidgetWithNestedAttributes& Owner; // We need to keep track of the owning widget to call Assign on the attribute
	TSlateNestedAttribute<int> NestedIntAttribute; // <--- The nested attribute
	TSlateNestedAttribute<float> NestedFloatAttribute; // <--- The nested attribute
};

struct SWidgetWithNestedAttributes : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SWidgetWithNestedAttributes, SWidget)
public:
	SLATE_BEGIN_ARGS( SWidgetWithNestedAttributes )
		{}
	SLATE_END_ARGS()

	SWidgetWithNestedAttributes()
		: NestedAttributes(*this)
	{}

	void Construct( const FArguments& InArgs );

	void SetNestedIntAttribute(TAttribute<int> Attr) { NestedAttributes.SetNestedIntAttribute(MoveTemp(Attr)); }
	void SetNestedFloatAttribute(TAttribute<float> Attr) { NestedAttributes.SetNestedFloatAttribute(MoveTemp(Attr)); }
	int GetNestedIntAttribute() const { return NestedAttributes.GetNestedIntAttr(); }
	float GetNestedFloatAttribute() const { return NestedAttributes.GetNestedFloatAttr(); }
private:
	FSimpleNestedAttributesOwner NestedAttributes; // <-- The widget has a field to our custom struct that contains the attributes
};


// ------ .cpp

void FSimpleNestedAttributesOwner::RegisterNestedAttributes(FSlateAttributeInitializer& AttributeInitializer, FSimpleNestedAttributesOwner SWidgetWithNestedAttributes::* InPointerToOwner)
{
	// Here we define the nested attributes. We need to know which member of the widget owns this type

	SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToOwner, NestedIntAttribute, EInvalidateWidgetReason::Paint)
		.OnValueChanged_Static(OnNestedIntAttributeChanged);

	// The Invalidation reason can also be a lambda. The first argument can either be const SWidget& or the type of the widget (here const SWidgetWithNestedAttributes&)
	SLATE_ADD_NESTED_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, InPointerToOwner, NestedFloatAttribute,
		[](const SWidgetWithNestedAttributes& OwningWidget, const FSimpleNestedAttributesOwner& NestedAttributes)
	{
		UE_LOGF(LogTemp, Warning, "In Get Invalidation Reason: value changed to: %f   (direct access:  %f)", OwningWidget.GetNestedFloatAttribute(), NestedAttributes.GetNestedFloatAttr())
		return EInvalidateWidgetReason::Paint;
	})
		.OnValueChanged_Static(OnNestedFloatAttributeChanged);
}

FSimpleNestedAttributesOwner::FSimpleNestedAttributesOwner(SWidgetWithNestedAttributes& Owner)
	: Owner(Owner)
	, NestedIntAttribute(Owner)
	, NestedFloatAttribute(Owner)
{}

void FSimpleNestedAttributesOwner::SetNestedIntAttribute(TAttribute<int> Attr)
{
	NestedIntAttribute.Assign(Owner, MoveTemp(Attr));
}

void FSimpleNestedAttributesOwner::SetNestedFloatAttribute(TAttribute<float> Attr)
{
	NestedFloatAttribute.Assign(Owner, MoveTemp(Attr));
}

void FSimpleNestedAttributesOwner::OnNestedIntAttributeChanged(SWidget& OwningWidget, FSimpleNestedAttributesOwner& NestedAttributesOwner)
{
	// The OnValueChanged definition is slightly different from a regular TSlateAttribute, also providing with a reference to our container type

	SWidgetWithNestedAttributes& Widget = static_cast<SWidgetWithNestedAttributes&>(OwningWidget);
	UE_LOGF(LogTemp, Warning, "OnNestedIntAttributeChanged: value changed to: %d   (direct access:  %d)", Widget.GetNestedIntAttribute(), NestedAttributesOwner.GetNestedIntAttr())
}

void FSimpleNestedAttributesOwner::OnNestedFloatAttributeChanged(SWidgetWithNestedAttributes& OwningWidget, FSimpleNestedAttributesOwner& NestedAttributesOwner)
{
	UE_LOGF(LogTemp, Warning, "OnNestedFloatAttributeChanged: value changed to: %f   (direct access:  %f)", OwningWidget.GetNestedFloatAttribute(), NestedAttributesOwner.GetNestedFloatAttr())
}


SLATE_IMPLEMENT_WIDGET(SWidgetWithNestedAttributes)

void SWidgetWithNestedAttributes::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	// We need to register the nested attributes when registering the widget's attributes, passing the pointer to member
	FSimpleNestedAttributesOwner::RegisterNestedAttributes(AttributeInitializer, &SWidgetWithNestedAttributes::NestedAttributes);
}

void SWidgetWithNestedAttributes::Construct(const FArguments& InArgs)
{
	// as needed
}
 * @endcode
 */
class FSlateNestedAttributeContainer
{
protected:

	/**
	 * A SlateAttribute that is member variable of a member variable of a SWidget.
	 * @usage: TSlateNestedAttribute<int32> MyAttribute1; TSlateNestedAttribute<int32, EInvalidateWidgetReason::Paint> MyAttribute2; TSlateNestedAttribute<int32, EInvalidateWidgetReason::Paint, TSlateAttributeComparePredicate<>> MyAttribute3;
	 */
	template<typename InObjectType, EInvalidateWidgetReason InInvalidationReasonValue = EInvalidateWidgetReason::None, typename InComparePredicate = TSlateAttributeComparePredicate<>>
	struct TSlateNestedAttribute : public SlateAttributePrivate::TSlateNestedMemberAttribute<
			InObjectType,
			typename std::conditional<InInvalidationReasonValue == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InInvalidationReasonValue>>::type,
			InComparePredicate>
	{
		using ::SlateAttributePrivate::TSlateNestedMemberAttribute<
			InObjectType,
			typename std::conditional<InInvalidationReasonValue == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InInvalidationReasonValue>>::type,
			InComparePredicate>::TSlateNestedMemberAttribute;
	};

	//~ Override for FText that use the TSlateAttributeFTextComparePredicate to compare FText
	template<>
	struct TSlateNestedAttribute<FText, EInvalidateWidgetReason::None> : public ::SlateAttributePrivate::TSlateNestedMemberAttribute<FText, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeFTextComparePredicate>
	{
		using ::SlateAttributePrivate::TSlateNestedMemberAttribute<FText, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeFTextComparePredicate>::TSlateNestedMemberAttribute;
	};

	//~ Override for FText that use the TSlateAttributeFTextComparePredicate to compare FText
	template<EInvalidateWidgetReason InInvalidationReasonValue>
	struct TSlateNestedAttribute<FText, InInvalidationReasonValue> : public ::SlateAttributePrivate::TSlateNestedMemberAttribute<
		FText,
		typename std::conditional<InInvalidationReasonValue == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InInvalidationReasonValue>>::type,
		TSlateAttributeFTextComparePredicate>
	{
		using ::SlateAttributePrivate::TSlateNestedMemberAttribute<
			FText,
			typename std::conditional<InInvalidationReasonValue == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InInvalidationReasonValue>>::type,
			TSlateAttributeFTextComparePredicate>::TSlateNestedMemberAttribute;
	};
};

