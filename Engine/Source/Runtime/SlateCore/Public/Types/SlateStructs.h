// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"

/**
 * Structure for optional floating point sizes.
 */
struct FOptionalSize 
{
	/**
	 * Creates an unspecified size.
	 */
	FOptionalSize( )
		: Size(Unspecified)
	{ }

	/**
	 * Creates a size with the specified value.
	 *
	 * @param SpecifiedSize The size to set.
	 */
	FOptionalSize( const float SpecifiedSize )
		: Size(SpecifiedSize)
	{ }

	/**
	 * Creates a size with the TOptional value.
	 *
	 * @param OptionalSize The optional size to set.
	 */
	FOptionalSize( const TOptional<float>& OptionalSize )
		: Size(OptionalSize.Get(Unspecified))
	{ }

public:

	/**
	 * Checks whether the size is set.
	 *
	 * @return true if the size is set, false if it is unspecified.
	 * @see Get
	 */
	bool IsSet( ) const
	{
		return Size != Unspecified;
	}

	/**
	 * Gets the value of the size.
	 *
	 * Before calling this method, check with IsSet() whether the size is actually specified.
	 * Unspecified sizes a value of -1.0f will be returned.
	 *
	 * @see IsSet
	 */
	float Get( ) const
	{
		return Size;
	}

	/** Compare one optional size to another for equality */
	bool operator==(const FOptionalSize& Other) const
	{
		return (Size == Other.Size);
	}

private:

	// constant for unspecified sizes.
	SLATECORE_API static const float Unspecified;

	// Holds the size, if specified.
	float Size;
};


/**
 * Base structure for size parameters.
 *
 * Describes a way in which a parent widget allocates available space to its child widgets.
 *
 * When SizeRule is SizeRule_Auto, the required space is the widget's DesiredSize.
 * When SizeRule is SizeRule_Stretch, the required space is the available space distributed proportionately between peer Widgets.
 * When SizeRule is SizeRule_StretchContent, the required space is widget's content size adjusted proportionally to fit the available space.
 *
 * Available space is space remaining after all the peers' SizeRule_Auto requirements have been satisfied.
 * The available space is distributed proportionally between the peer widgets depending on the Value property.
 *
 * FSizeParam cannot be constructed directly - see FStretch, FStretchContent, FAuto, and FAspectRatio
 */
struct FSizeParam
{
	enum ESizeRule
	{
		SizeRule_Auto,
		SizeRule_Stretch,
		SizeRule_StretchContent,
	};
	
	/** The sizing rule to use. */
	ESizeRule SizeRule;

	/**
	 * The actual value this size parameter stores.
	 *
	 * This value can be driven by a delegate. It is only used for the Stretch and StretchContent modes.
	 */
	TAttribute<float> Value;

	/**
	 * The actual value this size parameter stores, used for shrinking.
	 * Treated as unused, if set to negative value.
	 *
	 * This value can be driven by a delegate. It is only used for the StretchContent mode.
	 */
	TAttribute<float> ShrinkValue;

protected:

	/**
	 * Hidden constructor.
	 *
	 * Use FAspectRatio, FAuto, FStretch to instantiate size parameters.
	 *
	 * @see FAspectRatio, FAuto, FStretch
	 */
	FSizeParam( ESizeRule InTypeOfSize, const TAttribute<float>& InValue, const TAttribute<float>& InShrinkValue )
		: SizeRule(InTypeOfSize)
		, Value(InValue)
		, ShrinkValue(InShrinkValue)
	{ }
};


/**
 * Structure for size parameters with SizeRule = SizeRule_Stretch.
 *
 * @see FStretchContent, FAspectRatio, FAuto
 */
struct FStretch
	: public FSizeParam
{
	FStretch( const TAttribute<float>& StretchAmount )
		: FSizeParam(SizeRule_Stretch, StretchAmount, StretchAmount)
	{ }

	FStretch( )
		: FSizeParam(SizeRule_Stretch, 1.0f, 1.0f)
	{ }
};

/**
 * Structure for size parameters with SizeRule = SizeRule_StretchContent.
 *
 * @see FStretch, FAspectRatio, FAuto
 */
struct FStretchContent
	: public FSizeParam
{
	FStretchContent( const TAttribute<float>& StretchAmount )
		: FSizeParam(SizeRule_StretchContent, StretchAmount, StretchAmount)
	{ }

	FStretchContent( const TAttribute<float>& GrowStretchAmount, const TAttribute<float>& ShrinkStretchAmount )
   		: FSizeParam(SizeRule_StretchContent, GrowStretchAmount, ShrinkStretchAmount)
   	{ }

	FStretchContent( )
		: FSizeParam(SizeRule_StretchContent, 1.0f, 1.0f)
	{ }
};

/**
 * Structure for size parameters with SizeRule = SizeRule_Auto.
 *
 * @see FAspectRatio, FStretch, FStretchContent
 */
struct FAuto
	: public FSizeParam
{
	FAuto()
		: FSizeParam(SizeRule_Auto, 0.0f, 0.0f)
	{ }
};
