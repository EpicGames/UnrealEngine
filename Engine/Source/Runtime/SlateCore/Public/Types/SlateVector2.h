// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Vector2D.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Optional.h"

#include "SlateVector2.generated.h"

/**
 * When disabled, all deprecation mechanisms will be disabled, and public slate APIs will only compile with single-precision
 */
#ifndef UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS
	#define UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS 1
#endif


/**
 * The subsequent deprecation macros can be enabled or disabled per-module
 * Tell the IncludeTool static analyzer to ignore that divergence
 */
#define UE_INCLUDETOOL_IGNORE_INCONSISTENT_STATE


/**
 * When a module enables UE_REPORT_SLATE_VECTOR_DEPRECATION through PrivateDefines.Add("UE_REPORT_SLATE_VECTOR_DEPRECATION=1"),
 * deprecation mechanisms will be enabled on that module for Slate double/single vector APIs
 */
#ifndef UE_REPORT_SLATE_VECTOR_DEPRECATION
	#define UE_REPORT_SLATE_VECTOR_DEPRECATION 0
#endif

static_assert(UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS || !UE_REPORT_SLATE_VECTOR_DEPRECATION, "UE_REPORT_SLATE_VECTOR_DEPRECATION should never be specified where UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS is disabled.");

#ifndef UE_REPORT_SLATE_VECTOR_DEPRECATION_VERSION
	#define UE_REPORT_SLATE_VECTOR_DEPRECATION_VERSION all
#endif


#if UE_REPORT_SLATE_VECTOR_DEPRECATION
	#define UE_SLATE_VECTOR_DEPRECATED(Text) UE_DEPRECATED(UE_REPORT_SLATE_VECTOR_DEPRECATION_VERSION, Text)
	#define UE_SLATE_VECTOR_DEPRECATED_DEFAULT() UE_DEPRECATED(UE_REPORT_SLATE_VECTOR_DEPRECATION_VERSION, "Use FVector2f or float directly; this Slate API no longer interfaces directly with double-precision scalars and vectors.")
#else
	#define UE_SLATE_VECTOR_DEPRECATED(Text)
	#define UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
#endif

#if UE_REPORT_SLATE_VECTOR_DEPRECATION && UE_WARNINGS_AS_ERRORS
	#define UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION final
#else
	#define UE_SLATE_DEPRECATED_VECTOR_VIRTUAL_FUNCTION
#endif


#undef UE_INCLUDETOOL_IGNORE_INCONSISTENT_STATE
#undef UE_REPORT_SLATE_VECTOR_DEPRECATION


struct FDeprecateSlateVector2D;

namespace UE::Slate
{

	/**
	 * CastToVector2f allows supported types to be explicitly coerced to an FVector2f or const FVector2f&
	 * This allows generic programming patterns that must operate on FVector2f given many input types
	 */
	const FVector2f& CastToVector2f(const FVector2f& InValue);
		  FVector2f  CastToVector2f(const FVector2d& InValue);


#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS

	struct FDeprecateVector2DParameter;
	struct FDeprecateOptionalVector2DParameter;

		  FVector2f  CastToVector2f(const FIntPoint& InValue);
	const FVector2f& CastToVector2f(const FDeprecateSlateVector2D& InValue);
	const FVector2f& CastToVector2f(const FDeprecateVector2DParameter& InValue);

	struct FDeprecateSlateVectorPtrVariant
	{
		SLATECORE_API explicit FDeprecateSlateVectorPtrVariant(FDeprecateSlateVector2D* InInstance);

		UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
		SLATECORE_API operator const FVector2D*() const &;

		SLATECORE_API operator const FVector2f*() const;

		SLATECORE_API operator FVector2f*();

		FDeprecateSlateVector2D* Instance;
		mutable FVector2D DoubleVector;
	};

#else // UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS

	using FDeprecateVector2DResult            = FVector2f;
	using FDeprecateVector2DParameter         = FVector2f;
	using FDeprecateOptionalVector2DParameter = TOptional<FVector2f>;

#endif

} // namespace UE::Slate


/**
 * Defines a return value or persistent member variable that was previously an FVector2D
 * but is in the process of being deprecated in favor of FVector2f. Once deprecated completely, usages will
 * be converted to using FVector2f directly. It is not (and will never be) constructible from double precision types.
 *
 * This type is implicitly convertible to FVector2D, and can be operated on as if it were an FVector2D,
 * but will emit deprecation warnings under these situations where UE_REPORT_SLATE_VECTOR_DEPRECATION is enabled.
 *
 * This type should not be used as a parameter, where FDeprecateVector2DParameter is preferred due to its
 * implicit conversion from many types, and separate deprecation mechanisms.
 *
 * NOTE: DO NOT USE THIS TYPE DIRECTLY IN CLIENT CODE - ALL USAGES SHOULD USE FVector2f or float
 */
USTRUCT(BlueprintInternalUseOnly, DisplayName="Vector2D", meta=(HiddenByDefault, ShortTooltip="Vector2D (single-precision)", ToolTip="Vector2D (single-precision)", ScriptDefaultMake, ScriptDefaultBreak, HasNativeMake="/Script/Engine.KismetMathLibrary.MakeVector2D", HasNativeBreak="/Script/Engine.KismetMathLibrary.BreakVector2D"))
struct FDeprecateSlateVector2D : public FVector2f
{
	GENERATED_BODY()

	FDeprecateSlateVector2D() = default;

	/** Construction from another vector */
	FDeprecateSlateVector2D(FVector2f InValue)
		: FVector2f(InValue)
	{
	}

	/** Explicit construction from single-precision component values */
	explicit FDeprecateSlateVector2D(float InX, float InY)
		: FVector2f(InX, InY)
	{
	}

	/** Copy-assignment from another vector */
	FDeprecateSlateVector2D& operator=(const FVector2f& InValue)
	{
		X = InValue.X;
		Y = InValue.Y;
		return *this;
	}

	UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
	FDeprecateSlateVector2D& operator=(const FVector2d& InValue)
	{
		X = UE_REAL_TO_FLOAT(InValue.X);
		Y = UE_REAL_TO_FLOAT(InValue.Y);
		return *this;
	}


	UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
	FDeprecateSlateVector2D& operator=(const FIntPoint& InValue)
	{
		X = UE_REAL_TO_FLOAT(InValue.X);
		Y = UE_REAL_TO_FLOAT(InValue.Y);
		return *this;
	}

	/**
	 * Deprecated conversion to a double-precision vector for interoperability with legacy code
	 */
	UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
	operator FVector2d() const
	{
		return FVector2d(X, Y);
	}

#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS

	const UE::Slate::FDeprecateSlateVectorPtrVariant operator&() const
	{
		return UE::Slate::FDeprecateSlateVectorPtrVariant(const_cast<FDeprecateSlateVector2D*>(this));
	}

	UE::Slate::FDeprecateSlateVectorPtrVariant operator&()
	{
		return UE::Slate::FDeprecateSlateVectorPtrVariant(this);
	}

#endif

	SLATECORE_API bool Serialize(FStructuredArchive::FSlot Slot);
	SLATECORE_API bool SerializeFromMismatchedTag(const struct FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

public:

	/*~ ---------------------------------------
	 * These functions hide the similarly named functions from FVector2f in order to provide FVector2d interop */

	/** Get this vector as a vector where each component has been rounded to the nearest int. */
	FDeprecateSlateVector2D RoundToVector() const
	{
		return FDeprecateSlateVector2D(FVector2f::RoundToVector());
	}

	bool Equals(const UE::Slate::FDeprecateVector2DParameter& V, float Tolerance=UE_KINDA_SMALL_NUMBER) const
	{
		return FVector2f::Equals(UE::Slate::CastToVector2f(V), Tolerance);
	}

	FDeprecateSlateVector2D GetSignVector() const
	{
		return FDeprecateSlateVector2D(FVector2f::GetSignVector());
	}

	FDeprecateSlateVector2D GetAbs() const
	{
		return FDeprecateSlateVector2D(FVector2f::GetAbs());
	}

	FDeprecateSlateVector2D ClampAxes(float MinAxisVal, float MaxAxisVal) const
	{
		return FVector2f::ClampAxes(MinAxisVal, MaxAxisVal);
	}

	FDeprecateSlateVector2D GetRotated(float AngleDeg) const
	{
		return FVector2f::GetRotated(AngleDeg);
	}

	FDeprecateSlateVector2D GetSafeNormal(float Tolerance=UE_SMALL_NUMBER) const
	{
		return FVector2f::GetSafeNormal(Tolerance);
	}

	SLATECORE_API bool ComponentwiseAllLessThan(const UE::Slate::FDeprecateVector2DParameter& Other) const;

	SLATECORE_API bool ComponentwiseAllGreaterThan(const UE::Slate::FDeprecateVector2DParameter& Other) const;

	SLATECORE_API bool ComponentwiseAllLessOrEqual(const UE::Slate::FDeprecateVector2DParameter& Other) const;

	SLATECORE_API bool ComponentwiseAllGreaterOrEqual(const UE::Slate::FDeprecateVector2DParameter& Other) const;

	UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
	FDeprecateSlateVector2D ClampAxes(double MinAxisVal, double MaxAxisVal) const
	{
		return ClampAxes(UE_REAL_TO_FLOAT(MinAxisVal), UE_REAL_TO_FLOAT(MaxAxisVal));
	}

	UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
	FDeprecateSlateVector2D GetRotated(double AngleDeg) const
	{
		return GetRotated(UE_REAL_TO_FLOAT(AngleDeg));
	}
	UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
	FDeprecateSlateVector2D GetSafeNormal(double Tolerance) const
	{
		return GetSafeNormal(UE_REAL_TO_FLOAT(Tolerance));
	}

	/*~ End interop
	 * --------------------------------------- */

public:

#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS

	/*~ Macros to assist in defining explicit binary mathematical operators.
	 These are generally all necessary in order to resolve otherwise ambiguous overloaded operators
	 if we were to define them in terms of FDeprecateVector2DParameter */
	#define UE_SLATE_BINARY_VECTOR_OPERATORS_A(Type, ...)\
		__VA_ARGS__\
		friend FDeprecateSlateVector2D operator+(const Type& A, const FDeprecateSlateVector2D& B)\
		{\
			return FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) + UE::Slate::CastToVector2f(B));\
		}\
		__VA_ARGS__\
		friend FDeprecateSlateVector2D operator-(const Type& A, const FDeprecateSlateVector2D& B)\
		{\
			return FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) - UE::Slate::CastToVector2f(B));\
		}\
		__VA_ARGS__\
		friend FDeprecateSlateVector2D operator*(const Type& A, const FDeprecateSlateVector2D& B)\
		{\
			return FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) * UE::Slate::CastToVector2f(B));\
		}\
		__VA_ARGS__\
		friend FDeprecateSlateVector2D operator/(const Type& A, const FDeprecateSlateVector2D& B)\
		{\
			return FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) / UE::Slate::CastToVector2f(B));\
		}\
		__VA_ARGS__\
		friend bool operator==(const Type& A, const FDeprecateSlateVector2D& B)\
		{\
			return UE::Slate::CastToVector2f(A) == UE::Slate::CastToVector2f(B);\
		}\
		__VA_ARGS__\
		friend bool operator!=(const Type& A, const FDeprecateSlateVector2D& B)\
		{\
			return UE::Slate::CastToVector2f(A) != UE::Slate::CastToVector2f(B);\
		}

	#define UE_SLATE_BINARY_ASSIGNMENT_VECTOR_OPERATORS_A(Type, ...)\
		__VA_ARGS__\
		friend Type& operator+=(Type& A, const FDeprecateSlateVector2D& B)\
		{\
			A = Type(FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) + UE::Slate::CastToVector2f(B)));\
			return A;\
		}\
		__VA_ARGS__\
		friend Type& operator-=(Type& A, const FDeprecateSlateVector2D& B)\
		{\
			A = Type(FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) - UE::Slate::CastToVector2f(B)));\
			return A;\
		}\
		__VA_ARGS__\
		friend Type& operator*=(Type& A, const FDeprecateSlateVector2D& B)\
		{\
			A = Type(FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) * UE::Slate::CastToVector2f(B)));\
			return A;\
		}\
		__VA_ARGS__\
		friend Type& operator/=(Type& A, const FDeprecateSlateVector2D& B)\
		{\
			A = Type(FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) / UE::Slate::CastToVector2f(B)));\
			return A;\
		}

	#define UE_SLATE_BINARY_VECTOR_OPERATORS_B(Type, ...)\
		__VA_ARGS__\
		friend FDeprecateSlateVector2D operator+(const FDeprecateSlateVector2D& A, const Type& B)\
		{\
			return FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) + UE::Slate::CastToVector2f(B));\
		}\
		__VA_ARGS__\
		friend FDeprecateSlateVector2D operator-(const FDeprecateSlateVector2D& A, const Type& B)\
		{\
			return FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) - UE::Slate::CastToVector2f(B));\
		}\
		__VA_ARGS__\
		friend FDeprecateSlateVector2D operator*(const FDeprecateSlateVector2D& A, const Type& B)\
		{\
			return FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) * UE::Slate::CastToVector2f(B));\
		}\
		__VA_ARGS__\
		friend FDeprecateSlateVector2D operator/(const FDeprecateSlateVector2D& A, const Type& B)\
		{\
			return FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) / UE::Slate::CastToVector2f(B));\
		}\
		__VA_ARGS__\
		friend bool operator==(const FDeprecateSlateVector2D& A, const Type& B)\
		{\
			return UE::Slate::CastToVector2f(A) == UE::Slate::CastToVector2f(B);\
		}\
		__VA_ARGS__\
		friend bool operator!=(const FDeprecateSlateVector2D& A, const Type& B)\
		{\
			return UE::Slate::CastToVector2f(A) != UE::Slate::CastToVector2f(B);\
		}

	#define UE_SLATE_BINARY_ASSIGNMENT_VECTOR_OPERATORS_B(Type, ...)\
		__VA_ARGS__\
		friend FDeprecateSlateVector2D& operator+=(FDeprecateSlateVector2D& A, const Type& B)\
		{\
			static_cast<FVector2f&>(A) += UE::Slate::CastToVector2f(B);\
			return A;\
		}\
		__VA_ARGS__\
		friend FDeprecateSlateVector2D& operator-=(FDeprecateSlateVector2D& A, const Type& B)\
		{\
			static_cast<FVector2f&>(A) -= UE::Slate::CastToVector2f(B);\
			return A;\
		}\
		__VA_ARGS__\
		friend FDeprecateSlateVector2D& operator*=(FDeprecateSlateVector2D& A, const Type& B)\
		{\
			static_cast<FVector2f&>(A) *= UE::Slate::CastToVector2f(B);\
			return A;\
		}\
		__VA_ARGS__\
		friend FDeprecateSlateVector2D& operator/=(FDeprecateSlateVector2D& A, const Type& B)\
		{\
			static_cast<FVector2f&>(A) /= UE::Slate::CastToVector2f(B);\
			return A;\
		}

	/*~ Begin binary operator overloads */
	UE_SLATE_BINARY_VECTOR_OPERATORS_A(FIntPoint, UE_SLATE_VECTOR_DEPRECATED_DEFAULT())
	UE_SLATE_BINARY_VECTOR_OPERATORS_B(FIntPoint, UE_SLATE_VECTOR_DEPRECATED_DEFAULT())

	UE_SLATE_BINARY_VECTOR_OPERATORS_A(FVector2d, UE_SLATE_VECTOR_DEPRECATED_DEFAULT())
	UE_SLATE_BINARY_VECTOR_OPERATORS_B(FVector2d, UE_SLATE_VECTOR_DEPRECATED_DEFAULT())
	UE_SLATE_BINARY_ASSIGNMENT_VECTOR_OPERATORS_A(FVector2d, UE_SLATE_VECTOR_DEPRECATED_DEFAULT())
	UE_SLATE_BINARY_ASSIGNMENT_VECTOR_OPERATORS_B(FVector2d, UE_SLATE_VECTOR_DEPRECATED_DEFAULT())

	UE_SLATE_BINARY_VECTOR_OPERATORS_A(FVector2f, )
	UE_SLATE_BINARY_VECTOR_OPERATORS_B(FVector2f, )
	UE_SLATE_BINARY_ASSIGNMENT_VECTOR_OPERATORS_A(FVector2f, )
	UE_SLATE_BINARY_ASSIGNMENT_VECTOR_OPERATORS_B(FVector2f, )

	UE_SLATE_BINARY_VECTOR_OPERATORS_A(FDeprecateSlateVector2D, )
	UE_SLATE_BINARY_ASSIGNMENT_VECTOR_OPERATORS_A(FDeprecateSlateVector2D, )

	friend FDeprecateSlateVector2D operator*(float A, const FDeprecateSlateVector2D& B)
	{
		return FDeprecateSlateVector2D(A * UE::Slate::CastToVector2f(B));
	}
	friend FDeprecateSlateVector2D operator+(const FDeprecateSlateVector2D& A, float B)
	{
		return FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) + B);
	}
	friend FDeprecateSlateVector2D operator-(const FDeprecateSlateVector2D& A, float B)
	{
		return FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) - B);
	}
	friend FDeprecateSlateVector2D operator*(const FDeprecateSlateVector2D& A, float B)
	{
		return FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) * B);
	}
	friend FDeprecateSlateVector2D operator/(const FDeprecateSlateVector2D& A, float B)
	{
		return FDeprecateSlateVector2D(UE::Slate::CastToVector2f(A) / B);
	}

	/*~ End binary operator overloads */

	/*~ Unary operator overloads */
	friend FDeprecateSlateVector2D operator-(const FDeprecateSlateVector2D& In)
	{
		return FDeprecateSlateVector2D(-UE::Slate::CastToVector2f(In));
	}
	/*~ Unary operator overloads */

#undef UE_SLATE_BINARY_VECTOR_OPERATORS_A
#undef UE_SLATE_BINARY_VECTOR_OPERATORS_B
#undef UE_SLATE_BINARY_ASSIGNMENT_VECTOR_OPERATORS_A
#undef UE_SLATE_BINARY_ASSIGNMENT_VECTOR_OPERATORS_B

#endif // UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS
};

template<>
struct TStructOpsTypeTraits<FDeprecateSlateVector2D> : public TStructOpsTypeTraitsBase2<FDeprecateSlateVector2D>
{
	enum
	{
		WithStructuredSerializer = true,
		WithStructuredSerializeFromMismatchedTag = true,
	};
};


#if UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS

namespace UE::Slate
{

/**
 * Defines a return value or persistent member variable that was previously an FVector2D
 * but is in the process of being deprecated in favor of FVector2f. Once deprecated completely, usages will
 * be converted to using FVector2f directly. It is not (and will never be) constructible from double precision types.
 *
 * This type is implicitly convertible to FVector2D, and can be operated on as if it were an FVector2D,
 * but will emit deprecation warnings under these situations where UE_REPORT_SLATE_VECTOR_DEPRECATION is enabled.
 *
 * This type should not be used as a parameter, where FDeprecateVector2DParameter is preferred due to its
 * implicit conversion from many types, and separate deprecation mechanisms.
 *
 * NOTE: DO NOT USE THIS TYPE DIRECTLY IN CLIENT CODE - ALL USAGES SHOULD USE FVector2f or float
 */
using FDeprecateVector2DResult = FDeprecateSlateVector2D;

/**
 * Defines a Slate vector used as a function parameter that was previously an FVector2D but is in the process
 * of being deprecated in favor of FVector2f. Once deprecated completely, usages will be converted to using FVector2f directly.
 *
 * This type is implicitly constructible from both single and double precision vectors and scalars but will emit
 * deprecation warnings for double-precision types where UE_REPORT_SLATE_VECTOR_DEPRECATION is enabled. It is not (and will never be)
 * convertible to double precision types.
 *
 * This type should only be used as a parameter (not a return type), where FDeprecateVector2DResult is preferred so it can
 * interop with legacy code that still requires conversion to FVector2d.
 *
 * NOTE: DO NOT USE THIS TYPE DIRECTLY IN CLIENT CODE - ALL USAGES SHOULD USE FVector2f or float
 */
struct FDeprecateVector2DParameter : FVector2f
{
	FDeprecateVector2DParameter(float InX, float InY)
		: FVector2f(InX, InY)
	{}

	FDeprecateVector2DParameter(const FVector2f& InValue)
		: FVector2f(InValue)
	{}

	FDeprecateVector2DParameter(const FDeprecateVector2DResult& InValue)
		: FVector2f(InValue)
	{}

	UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
	FDeprecateVector2DParameter(double InX, double InY)
		: FVector2f(UE_REAL_TO_FLOAT(InX), UE_REAL_TO_FLOAT(InY))
	{}

	UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
	FDeprecateVector2DParameter(const FVector2d& InValue)
		: FVector2f(UE_REAL_TO_FLOAT(InValue.X), UE_REAL_TO_FLOAT(InValue.Y))
	{}

	UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
	FDeprecateVector2DParameter(const FIntPoint& InValue)
		: FVector2f(FloatCastChecked<float, double>(InValue.X, 1.0), FloatCastChecked<float, double>(InValue.Y, 1.0))
	{}
};

/**
 * Defines an optional Slate vector parameter in a similar vein to FDeprecateVector2DParameter
 * NOTE: DO NOT USE THIS TYPE DIRECTLY IN CLIENT CODE - ALL USAGES SHOULD USE FVector2f or float
 */
struct FDeprecateOptionalVector2DParameter : TOptional<FVector2f>
{
	FDeprecateOptionalVector2DParameter()
	{}

	FDeprecateOptionalVector2DParameter(const FVector2f& In)
		: TOptional<FVector2f>(In)
	{}

	FDeprecateOptionalVector2DParameter(const TOptional<FVector2f>& In)
		: TOptional<FVector2f>(In)
	{}

	UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
	FDeprecateOptionalVector2DParameter(const FVector2d& In)
		: TOptional<FVector2f>(CastToVector2f(In))
	{
	}

	UE_SLATE_VECTOR_DEPRECATED_DEFAULT()
	FDeprecateOptionalVector2DParameter(const TOptional<FVector2d>& In)
	{
		if (In.IsSet())
		{
			this->Emplace(CastToVector2f(In.GetValue()));
		}
	}
};

inline FVector2f CastToVector2f(const FIntPoint& InValue)
{
	return InValue;
}

inline const FVector2f& CastToVector2f(const FDeprecateSlateVector2D& InValue)
{
	return InValue;
}

inline const FVector2f& CastToVector2f(const FDeprecateVector2DParameter& InValue)
{
	return InValue;
}

} // namespace UE::Slate

inline UE::Slate::FDeprecateVector2DResult TransformPoint(float Transform, const UE::Slate::FDeprecateVector2DResult& Point)
{
	return UE::Slate::FDeprecateVector2DResult(Transform * FVector2f(Point));
}

inline UE::Slate::FDeprecateVector2DResult TransformVector(float Transform, const UE::Slate::FDeprecateVector2DResult& Vector)
{
	return UE::Slate::FDeprecateVector2DResult(Transform * FVector2f(Vector));
}

template <typename TransformType>
inline UE::Slate::FDeprecateVector2DResult TransformPoint(const TransformType& Transform, const UE::Slate::FDeprecateVector2DResult& Point)
{
	return UE::Slate::FDeprecateVector2DResult(Transform.TransformPoint(Point));
}

template <typename TransformType>
inline UE::Slate::FDeprecateVector2DResult TransformVector(const TransformType& Transform, const UE::Slate::FDeprecateVector2DResult& Point)
{
	return UE::Slate::FDeprecateVector2DResult(Transform.TransformVector(Point));
}

#endif // UE_ENABLE_SLATE_VECTOR_DEPRECATION_MECHANISMS

namespace UE::Slate
{

	inline const FVector2f& CastToVector2f(const FVector2f& InValue)
	{
		return InValue;
	}
	inline FVector2f CastToVector2f(const FVector2d& InValue)
	{
		const float X = UE_REAL_TO_FLOAT(InValue.X);
		const float Y = UE_REAL_TO_FLOAT(InValue.Y);
	#if 0
		ensureAlways(FMath::IsNearlyEqual((double)X, InValue.X));
		ensureAlways(FMath::IsNearlyEqual((double)Y, InValue.Y));
	#endif
		return FVector2f(X, Y);
	}

} // namespace UE::Slate
