// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Types/ISlateMetaData.h"

class FName;
class SWidget;

/**
 * MetaData used to add and remove widgets to the Slate Widget Tracker for the specified tags.
 */
class FTrackedMetaData : public ISlateMetaData
{
public:

	SLATE_METADATA_TYPE(FTrackedMetaData, ISlateMetaData)

	SLATECORE_API FTrackedMetaData(const SWidget* InTrackedWidget, TArray<FName>&& InTags);
	SLATECORE_API FTrackedMetaData(const SWidget* InTrackedWidget, FName InTags);
	SLATECORE_API ~FTrackedMetaData();

	FTrackedMetaData(const FTrackedMetaData&) = delete;
	FTrackedMetaData& operator=(const FTrackedMetaData&) = delete;

private:

	const SWidget* TrackedWidget = nullptr;
	TArray<FName> Tags;

};
