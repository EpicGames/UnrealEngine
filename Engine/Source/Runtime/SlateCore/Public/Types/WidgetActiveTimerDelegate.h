// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class EActiveTimerReturnType : uint8;

/** Delegate used for registering a widget for tick services. */
DECLARE_DELEGATE_RetVal_TwoParams(EActiveTimerReturnType, FWidgetActiveTimerDelegate, double /*InCurrentTime*/, float /*InDeltaTime*/);
