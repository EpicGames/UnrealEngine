// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "Input/Events.h"
#include "Input/Reply.h"
#include "Layout/Geometry.h"

/** Delegate type for handling mouse events */
DECLARE_DELEGATE_RetVal_TwoParams(
	FReply, FPointerEventHandler,
	/** The geometry of the widget*/
	const FGeometry&,
	/** The Mouse Event that we are processing */
	const FPointerEvent&)

DECLARE_DELEGATE_TwoParams(
	FNoReplyPointerEventHandler,
	/** The geometry of the widget*/
	const FGeometry&,
	/** The Mouse Event that we are processing */
	const FPointerEvent&)

DECLARE_DELEGATE_OneParam(
	FSimpleNoReplyPointerEventHandler,
	/** The Mouse Event that we are processing */
	const FPointerEvent&)
