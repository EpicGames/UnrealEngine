// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY

#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "Misc/Variant.h"

class FSlateAccessibleWidget;
class SWidget;
class SWindow;

// Used to radically increase the responsiveness of accessibility navigation for debugging pruposes. The implementation is too slow to be feasible for shipping builds. 
#define ACCESSIBILITY_DEBUG_RESPONSIVENESS 0 && !UE_BUILD_SHIPPING

/**
 * Message handling system for Slate Accessibility API, dealing with both receiving events and pushing them back to the platform layer.
 * The message handler is also responsible for processing the Slate widget tree and queuing/processing accessibility requests for widget data from the platform.
 */
class FSlateAccessibleMessageHandler : public FGenericAccessibleMessageHandler
{
public:
	SLATECORE_API FSlateAccessibleMessageHandler();

	// FGenericAccessibleMessageHandler
	SLATECORE_API virtual void OnActivate() override;
	SLATECORE_API virtual void OnDeactivate() override;
	SLATECORE_API virtual TSharedPtr<IAccessibleWidget> GetAccessibleWindow(const TSharedRef<FGenericWindow>& InWindow) const override;
	SLATECORE_API virtual AccessibleWidgetId GetAccessibleWindowId(const TSharedRef<FGenericWindow>& InWindow) const override;
	SLATECORE_API virtual TSharedPtr<IAccessibleWidget> GetAccessibleWidgetFromId(AccessibleWidgetId Id) const override;
	SLATECORE_API virtual void RunInThread(const TFunction<void()>& InFunction, bool bWaitForCompletion = true, ENamedThreads::Type InThread = ENamedThreads::GameThread) override;
	SLATECORE_API virtual void MakeAccessibleAnnouncement(const FString& AnnouncementString) override;
	//~

	/**
	 * Callback for SWidget destructor. Removes the corresponding accessible widget for the Slate widget.
	 *
	 * @param Widget The widget that is being deleted.
	 */
	SLATECORE_API void OnWidgetRemoved(SWidget* Widget);

	/**
	 * The arguments that make up an accessible event raised by an SWidget.
	 * It is up to the user to create an instance of this struct when they want to raise an accessible event from an SWidget
	 * to be passed to an accessible event handler.
	 * @see OnWidgetEventRaised
	 */
	struct FSlateWidgetAccessibleEventArgs
	{
		FSlateWidgetAccessibleEventArgs(TSharedRef<SWidget> InWidget, EAccessibleEvent InEvent, FVariant InOldValue = FVariant(), FVariant InNewValue = FVariant(), FAccessibleUserIndex InSlateUserIndex = 0)
			: Widget(InWidget)
			, Event(InEvent)
			, OldValue(InOldValue)
			, NewValue(InNewValue)
			, SlateUserIndex(InSlateUserIndex)
		{}
			
		/** The widget that's raising the accessible event */
		TSharedRef<SWidget> Widget;
		/** The type of event being raised. */
		EAccessibleEvent Event;
		/** The value of the property being changed before the change took place. */
		FVariant OldValue;
		/** The value of the property being changed after the change took place. Alternatively, can contain any miscellaneous data for the accessible event. */
		FVariant NewValue;
		/** The index of the Slate user that feedback for the accessible event should be directed towards. */
		FAccessibleUserIndex SlateUserIndex;
	};
	/**
	 * Callback for a Slate widget indicating that a property change occurred. This may also be used by certain events
	 * such as Notification which don't have an 'OldValue'. Only NewValue should be set for those types of events.
	 *
	 * @param Args The arguments for the accessible event being raised.
	 * @see FSlateWidgetAccessibleEventArgs
	 */
	SLATECORE_API void OnWidgetEventRaised(const FSlateWidgetAccessibleEventArgs& Args);

	/**
	 * Refresh the accessible widget tree next available tick. This should be called any time the Slate tree changes.
	 */
	void MarkDirty() { bDirty = true; }

	/**
	* Processes all the queued tasks in the AccessibleTaskQueue
	* Should only be called from FSlateApplication::TickPlatform() 
	*/
	SLATECORE_API void ProcessAccessibleTasks();

	/**
	 * Process any pending Slate widgets and update the accessible widget tree.
	 */
	SLATECORE_API void Tick();


private:
	SLATECORE_API void HandleAccessibleWidgetFocusChangeEvent(const TSharedRef<IAccessibleWidget>& FocusWidget, bool bIsWidgetGainingFocus, FAccessibleUserIndex UserIndex);
	/**
	*  A helper class that wraps an accessibility task and the event to be triggered when the task finishes executing. 
	*/
	class FSlateAccessibleTask
	{
	public:
		/** If completion event is nullptr, The task is treated as asynchronous */
		FSlateAccessibleTask(const TFunction<void()>& InTask, FEvent* InCompletionEvent = nullptr)
			: Task(InTask)
			, CompletionEvent(InCompletionEvent)
		{
			check(Task);
		}

		/** Executes the task and triggers the completion event if available */
		void DoTask()
		{
			// We should always have a valid task 
			check(Task);
			Task();
			if (CompletionEvent)
			{
				CompletionEvent->Trigger();
			}
		}
	private:
		/** A lambda function that requests accessibility data from Slate in the Game Thread */
		TFunction<void()> Task;

		/** The event to be triggered when the task has completed execution. Can be null if the task should be executed asynchronously */
		FEvent* CompletionEvent;
	};

	/** Queues an FSlateAccessibleTask to be processed in the Game Thread */
	SLATECORE_API void EnqueueAccessibleTask(const FSlateAccessibleTask& InAccessibleTask);

	struct FWidgetAndParent {
		FWidgetAndParent(TWeakPtr<SWidget> InWidget, TSharedRef<FSlateAccessibleWidget> InParent)
			: Widget(InWidget), Parent(InParent)
		{
		}

		TWeakPtr<SWidget> Widget;
		TSharedRef<FSlateAccessibleWidget> Parent;
	};

	/** A list of widgets waiting to be processed in order to keep the accessible widget tree up to date */
	TArray<FWidgetAndParent> ToProcess;

	/** 
	* A queue of FSlateAccessibleTasks to be processed in the Game Thread 
	* This queue is only meant to store tasks, NOT process them . 
	* We opt for a TArray as a TQueue has slower performance in this use case 
	* @see ProcessAccessibleTasks() 
	*/
	TArray<FSlateAccessibleTask> AccessibleTaskStorageQueue;
	/**
	* The queue used to process all queued accessible tasks 
	* The contents of AccessibleTaskStorageQueue is  swapped inot this queue as an optimization to avoid holding on to the lock 
	* @see ProcessAccessibleTasks() 
	*/
	TArray<FSlateAccessibleTask> AccessibleTaskProcessingQueue;

	/** Critical section used for synchronization of the AccessibleTaskQueue */
	FCriticalSection QueueCriticalSection;

#if ACCESSIBILITY_DEBUG_RESPONSIVENESS
	FEvent* EnqueueEvent;
#endif
	/** If true, Tick() will begin the update process to the accessible widget tree. Use MarkDirty() to set. */
	bool bDirty;
};

#endif
