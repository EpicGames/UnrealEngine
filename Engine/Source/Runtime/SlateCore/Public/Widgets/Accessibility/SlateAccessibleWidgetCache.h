// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY

#include "CoreMinimal.h"
#include "Widgets/Accessibility/SlateCoreAccessibleWidgets.h"

class SWidget;

/**
 * Singleton used to retrieve accessible widgets for a given Slate widget. Accessible widgets will persist
 * so longer as the OS is sending accessibility events, after which the platform layer should call ClearAll().
 */
class FSlateAccessibleWidgetCache
{
public:
	/** Empty the cache and release all cached accessible widgets. This should generally only be done with accessibility is turned off. */
	static SLATECORE_API void ClearAll();

	/**
	 * Callback for when an SWidget is deleted. FSlateAccessibleMessageHandler should be the only thing to call this.
	 * The widget will be removed from the cache, and it should have already detached from its parent.
	 *
	 * @param Widget the Slate widget that is being deleted
	 */
	static SLATECORE_API TSharedPtr<FSlateAccessibleWidget> RemoveWidget(SWidget* Widget);

	/**
	 * Get a cached accessible widget for a given Slate widget. This may return nullptr in the case where the
	 * Slate widget is not a valid accessible widget, which is either because the widget's behavior is not
	 * accessible or it has a parent that cannot have accessible children.
	 *
	 * @param Widget The Slate widget to get the accessible widget for
	 * @return The accessible widget for the given Slate widget, or nullptr if the widget is not accessible.
	 */
	static SLATECORE_API TSharedPtr<FSlateAccessibleWidget> GetAccessibleWidgetChecked(const TSharedPtr<SWidget>& Widget);
	/**
	 * Get a cached accessible widget for a given Slate widget. The caller is responsible for making sure
	 * that the widget is fully accessible in the current Slate widget tree.
	 * 
	 * @param Widget The Slate widget to get the accessible widget for
	 * @return The accessible widget for the given Slate widget
	 */
	static SLATECORE_API TSharedRef<FSlateAccessibleWidget> GetAccessibleWidget(const TSharedRef<SWidget>& Widget);

	/**
	 * Get a cached accessible widget for an identifier that matches the accessible widget's GetId() return value.
	 * This will NOT create the widget if it does not exist yet, since Ids can only be generated by the widget itself.
	 * InvalidAccessibleWidgetId will always return null.
	 *
	 * @param Id The Id of a widget that corresponds to that widget's GetId() function
	 * @return The accessible widget for the given Id, or nullptr if there is no widget with that Id.
	 */
	static SLATECORE_API TSharedPtr<FSlateAccessibleWidget> GetAccessibleWidgetFromId(AccessibleWidgetId Id);

	static SLATECORE_API TMap<SWidget*, TSharedRef<FSlateAccessibleWidget>>::TConstIterator GetAllWidgets();

#if !UE_BUILD_SHIPPING
	static SLATECORE_API void DumpAccessibilityStats();
#endif

private:
	/**
	 * Hold on to a shared ref for each accessible widget until the SWidget is deleted. A raw pointer is used
	 * because widgets are removed from the cache via the SWidget destructor.
	 */
	static TMap<SWidget*, TSharedRef<FSlateAccessibleWidget>> AccessibleWidgetMap;
	/** Map each accessible widget to its ID to provide a second way to do lookups. */
	static TMap<AccessibleWidgetId, TSharedRef<FSlateAccessibleWidget>> AccessibleIdMap;
};

#endif
