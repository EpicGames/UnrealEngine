// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_ACCESSIBILITY

#include "CoreMinimal.h"
#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"

class SWidget;
class SWindow;

/**
 * The base implementation of IAccessibleWidget for all Slate widgets. Any new accessible widgets should
 * inherit directly from FSlateAccessibleWidget, and optionally inherit from other IAccessible interfaces to
 * provide more functionality.
 */
class FSlateAccessibleWidget : public IAccessibleWidget
{
	friend class FSlateAccessibleMessageHandler;
public:
	SLATECORE_API FSlateAccessibleWidget(TWeakPtr<SWidget> InWidget, EAccessibleWidgetType InWidgetType = EAccessibleWidgetType::Unknown);
	SLATECORE_API virtual ~FSlateAccessibleWidget();

	// IAccessibleWidget
	SLATECORE_API virtual AccessibleWidgetId GetId() const override final;
	SLATECORE_API virtual bool IsValid() const override final;
	SLATECORE_API virtual TSharedPtr<IAccessibleWidget> GetWindow() const override final;
	SLATECORE_API virtual FBox2D GetBounds() const override final;
	SLATECORE_API virtual TSharedPtr<IAccessibleWidget> GetParent() override final;
	SLATECORE_API virtual TSharedPtr<IAccessibleWidget> GetNextSibling() override final;
	SLATECORE_API virtual TSharedPtr<IAccessibleWidget> GetPreviousSibling() override final;
	SLATECORE_API virtual TSharedPtr<IAccessibleWidget> GetNextWidgetInHierarchy() override final;
	SLATECORE_API virtual TSharedPtr<IAccessibleWidget> GetPreviousWidgetInHierarchy() override final;
	SLATECORE_API virtual TSharedPtr<IAccessibleWidget> GetChildAt(int32 Index) override final;
	SLATECORE_API virtual int32 GetNumberOfChildren() override final;
	SLATECORE_API virtual FString GetClassName() const override final;
	SLATECORE_API virtual bool IsEnabled() const override final;
	SLATECORE_API virtual bool IsHidden() const override final;
	SLATECORE_API virtual bool SupportsFocus() const override final;
	SLATECORE_API virtual bool SupportsAccessibleFocus() const override final;
	SLATECORE_API virtual bool CanCurrentlyAcceptAccessibleFocus() const override final;
	SLATECORE_API virtual bool HasUserFocus(const FAccessibleUserIndex UserIndex) const override final;
	SLATECORE_API virtual bool SetUserFocus(const FAccessibleUserIndex UserIndex) override final;

	virtual EAccessibleWidgetType GetWidgetType() const override { return WidgetType; }
	SLATECORE_API virtual FString GetWidgetName() const override;
	SLATECORE_API virtual FString GetHelpText() const override;
	// ~

	/**
	 * Detach this widget from its current parent and attach it to a new parent. This will emit notifications back to the accessible message handler.
	 *
	 * @param NewParent The widget to assign as the new parent widget.
	 */
	SLATECORE_API void UpdateParent(TSharedPtr<IAccessibleWidget> NewParent);

protected:
	/** The underlying Slate widget backing this accessible widget. */
	TWeakPtr<SWidget> Widget;
	/** What type of widget the platform's accessibility API should treat this as. */
	EAccessibleWidgetType WidgetType;
	/** The accessible parent to this widget. This should usually be valid on widgets in the hierarchy, except for SWindows. */
	TWeakPtr<FSlateAccessibleWidget> Parent;
	/** All accessible widgets whose parent is this widget. */
	TArray<TWeakPtr<FSlateAccessibleWidget>> Children;
	/** A temporary array filled when processing the widget tree, which will eventually be moved to the Children array. */
	TArray<TWeakPtr<FSlateAccessibleWidget>> ChildrenBuffer;
	/** The index of this widget in its parent's list of children. */
	int32 SiblingIndex;
	/** An application-unique identifier for GetId(). */
	AccessibleWidgetId Id;

private:
	/**
	 * Find the Slate window containing this widget's underlying Slate widget.
	 *
	 * @return The parent SWindow for the Slate widget referenced by this accessible widget.
	 */
	SLATECORE_API TSharedPtr<SWindow> GetSlateWindow() const;
};

// SWindow
class FSlateAccessibleWindow
	: public FSlateAccessibleWidget
	, public IAccessibleWindow
{
public:
	FSlateAccessibleWindow(TWeakPtr<SWidget> InWidget) : FSlateAccessibleWidget(InWidget, EAccessibleWidgetType::Window) {}
	virtual ~FSlateAccessibleWindow() = default;

	// IAccessibleWidget
	virtual IAccessibleWindow* AsWindow() override { return this; }
	virtual FString GetWidgetName() const override;
	// ~

	// IAccessibleWindow
	virtual TSharedPtr<FGenericWindow> GetNativeWindow() const override;
	virtual TSharedPtr<IAccessibleWidget> GetChildAtPosition(int32 X, int32 Y) override;
	virtual TSharedPtr<IAccessibleWidget> GetUserFocusedWidget(const FAccessibleUserIndex UserIndex) const override;
	virtual void Close() override;
	virtual bool SupportsDisplayState(EWindowDisplayState State) const override;
	virtual EWindowDisplayState GetDisplayState() const override;
	virtual void SetDisplayState(EWindowDisplayState State) override;
	virtual bool IsModal() const override;
	// ~
};
// ~

// SImage
class FSlateAccessibleImage
	: public FSlateAccessibleWidget
{
public:
	FSlateAccessibleImage(TWeakPtr<SWidget> InWidget) : FSlateAccessibleWidget(InWidget, EAccessibleWidgetType::Image) {}
	virtual ~FSlateAccessibleImage() = default;

	// IAccessibleWidget
	SLATECORE_API virtual FString GetHelpText() const override;
	// ~
};
// ~

#endif
