// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/Accessibility/GenericAccessibleInterfaces.h"
#include "Misc/Attribute.h"

struct FAccessibleWidgetData
{
	FAccessibleWidgetData(EAccessibleBehavior InBehavior = EAccessibleBehavior::NotAccessible, EAccessibleBehavior InSummaryBehavior = EAccessibleBehavior::Auto, bool bInCanChildrenBeAccessible = true)
		: bCanChildrenBeAccessible(bInCanChildrenBeAccessible)
		, AccessibleBehavior(InBehavior)
		, AccessibleSummaryBehavior(InSummaryBehavior)
	{
	}
	FAccessibleWidgetData(const TAttribute<FText>& InAccessibleText, const TAttribute<FText>& InAccessibleSummaryText = TAttribute<FText>(), bool bInCanChildrenBeAccessible = true)
		: bCanChildrenBeAccessible(bInCanChildrenBeAccessible)
		, AccessibleBehavior(InAccessibleSummaryText.IsSet() ? EAccessibleBehavior::Custom : EAccessibleBehavior::NotAccessible)
		, AccessibleSummaryBehavior(InAccessibleSummaryText.IsSet() ? EAccessibleBehavior::Custom : EAccessibleBehavior::Auto)
		, AccessibleText(InAccessibleText)
		, AccessibleSummaryText(InAccessibleSummaryText)
	{
	}

	uint8 bCanChildrenBeAccessible : 1;
	EAccessibleBehavior AccessibleBehavior;
	EAccessibleBehavior AccessibleSummaryBehavior;
	TAttribute<FText> AccessibleText;
	TAttribute<FText> AccessibleSummaryText;
};
