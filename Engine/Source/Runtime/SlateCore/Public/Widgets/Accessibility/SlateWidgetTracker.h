// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class SWidget;
class ISlateMetaData;

enum class ETrackedSlateWidgetOperations : uint8
{
	AddedTrackedWidget,
	RemovedTrackedWidget
};

DECLARE_EVENT_ThreeParams(FSlateWidgetTracker, FTrackedWidgetsChangedEvent, const SWidget*, const FName&, ETrackedSlateWidgetOperations)

/**
 * The Slate Widget Tracker tracks widgets by tags and notifies listeners when widgets of a certain tag are added or removed.
 * Widgets can be added to the tracker using the FTrackedMetaData.
 */
class FSlateWidgetTracker
{

public:

	/** Get the Singleton instance of the Slate Widget Tracker. */
	static SLATECORE_API FSlateWidgetTracker& Get();

	/** Return true if Slate.EnableSlateWidgetTracker was set to true via config files or C++. */
	SLATECORE_API bool IsEnabled() const;

	/** Adds a tracked Widget for the specified tags. */
	SLATECORE_API void AddTrackedWidget(const SWidget* WidgetToTrack, const TArray<FName>& Tags);
	/** Adds a tracked Widget for the specified tag. */
	SLATECORE_API void AddTrackedWidget(const SWidget* WidgetToTrack, FName Tag);

	/** Removes a tracked Widget for all tags. */
	SLATECORE_API void RemoveTrackedWidget(const SWidget* WidgetToStopTracking);
	
	/** Returns an event that gets called when widgets tagged with the passed tag are added or removed from the tracker. */
	SLATECORE_API FTrackedWidgetsChangedEvent& OnTrackedWidgetsChanged(const FName& Tag);

	/** Calls Predicate for each of the tracked widgets of the specified tag. */
	template<class Predicate>
	void ForEachTrackedWidget(FName Tag, Predicate Pred)
	{
		if (TArray<const SWidget*>* Widgets = TrackedWidgets.Find(Tag))
		{
			for (const SWidget* Widget : *Widgets)
			{
				Pred(Widget);
			}
		}
	}

private:

	FSlateWidgetTracker() = default;
	~FSlateWidgetTracker() = default;

	TMap<FName, TArray<const SWidget*>> TrackedWidgets;
	TMap<FName, FTrackedWidgetsChangedEvent> TrackedWidgetsChangedEvents;

};
