// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Layout/Clipping.h"
#include "Widgets/WidgetPixelSnapping.h"
#include "Layout/FlowDirection.h"
#include "Rendering/SlateRenderTransform.h"
#include "GenericPlatform/ICursor.h"
#include "Types/ISlateMetaData.h"
#include "Trace/SlateMemoryTags.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Accessibility/SlateWidgetAccessibleTypes.h"
#include "Widgets/SWidget.h"
#include "Templates/Identity.h"

class IToolTip;
class SUserWidget;
class SWidget;
template<typename WidgetType> struct TSlateBaseNamedArgs;

/**
 * Slate widgets are constructed through SNew and SAssignNew.
 * e.g.
 *
 *     TSharedRef<SButton> MyButton = SNew(SButton);
 *        or
 *     TSharedPtr<SButton> MyButton;
 *     SAssignNew( MyButton, SButton );
 *
 * Using SNew and SAssignNew ensures that widgets are populated
 */

#define SNew( WidgetType, ... ) \
	MakeTDecl<WidgetType>( #WidgetType, __FILE__, __LINE__, RequiredArgs::MakeRequiredArgs(__VA_ARGS__) ) <<= TYPENAME_OUTSIDE_TEMPLATE WidgetType::FArguments()


#define SAssignNew( ExposeAs, WidgetType, ... ) \
	MakeTDecl<WidgetType>( #WidgetType, __FILE__, __LINE__, RequiredArgs::MakeRequiredArgs(__VA_ARGS__) ) . Expose( ExposeAs ) <<= TYPENAME_OUTSIDE_TEMPLATE WidgetType::FArguments()

#define SArgumentNew( InArgs, WidgetType, ... ) \
	MakeTDecl<WidgetType>( #WidgetType, __FILE__, __LINE__, RequiredArgs::MakeRequiredArgs(__VA_ARGS__) ) <<= InArgs


/**
 * Widget authors can use SLATE_BEGIN_ARGS and SLATE_END_ARS to add support
 * for widget construction via SNew and SAssignNew.
 * e.g.
 * 
 *    SLATE_BEGIN_ARGS( SMyWidget )
 *         , _PreferredWidth( 150.0f )
 *         , _ForegroundColor( FLinearColor::White )
 *         {}
 *
 *         SLATE_ATTRIBUTE(float, PreferredWidth)
 *         SLATE_ATTRIBUTE(FSlateColor, ForegroundColor)
 *    SLATE_END_ARGS()
 */

#define SLATE_BEGIN_ARGS( InWidgetType ) \
	public: \
	struct FArguments : public TSlateBaseNamedArgs<InWidgetType> \
	{ \
		typedef FArguments WidgetArgsType; \
		typedef InWidgetType WidgetType; \
		FORCENOINLINE FArguments()

/**
 * Just like SLATE_BEGIN_ARGS but requires the user to implement the New() method in the .CPP.
 * Allows for widget implementation details to be entirely reside in the .CPP file.
 * e.g.
 *   SMyWidget.h:
 *   ----------------------------------------------------------------
 *    SLATE_USER_ARGS( SMyWidget )
 *    {}
 *    SLATE_END_ARGS()
 *    
 *    virtual void DoStuff() = nullptr;
 *
 *   SMyWidget.cpp:
 *   ----------------------------------------------------------------
 *   namespace Implementation{
 *   class SMyWidget : public SMyWidget{
 *     void Construct( const FArguments& InArgs )
 *     {
 *         SUserWidget::Construct( SUSerWidget::FArguments()
 *         [
 *             SNew(STextBlock) .Text( "Implementation Details!" )
 *         ] );
 *     }
 *
 *     virtual void DoStuff() override {}
 *     
 *     // Truly private. All handlers can be inlined (less boilerplate).
 *     // All private members are truly private.
 *     private:
 *     FReply OnButtonClicked
 *     {
 *     }
 *     TSharedPtr<SButton> MyButton;
 *   }
 *   }
 */
#define SLATE_USER_ARGS( WidgetType ) \
	public: \
	static TSharedRef<WidgetType> New(); \
	struct FArguments; \
	struct FArguments : public TSlateBaseNamedArgs<WidgetType> \
	{ \
		typedef FArguments WidgetArgsType; \
		FORCENOINLINE FArguments()

#define SLATE_END_ARGS() \
	};


#define SLATE_PRIVATE_ATTRIBUTE_VARIABLE( AttrType, AttrName ) \
		TAttribute< AttrType > _##AttrName

#define SLATE_PRIVATE_ATTRIBUTE_FUNCTION( AttrType, AttrName ) \
		WidgetArgsType& AttrName( TAttribute< AttrType > InAttribute ) \
		{ \
			_##AttrName = MoveTemp(InAttribute); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		/* Bind attribute with delegate to a global function
		 * NOTE: We use a template here to avoid 'typename' issues when hosting attributes inside templated classes */ \
		template< typename... VarTypes > \
		WidgetArgsType& AttrName##_Static( TIdentity_T< typename TAttribute< AttrType >::FGetter::template TFuncPtr<VarTypes...> > InFunc, VarTypes... Vars )	\
		{ \
			_##AttrName.BindStatic( InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		/* Bind attribute with delegate to a lambda
		 * technically this works for any functor types, but lambdas are the primary use case */ \
		WidgetArgsType& AttrName##_Lambda(TFunction< AttrType(void) >&& InFunctor) \
		{ \
			_##AttrName.BindLambda(InFunctor); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		/* Bind attribute with delegate to a raw C++ class method */ \
		template< class UserClass, typename... VarTypes >	\
		WidgetArgsType& AttrName##_Raw( UserClass* InUserObject, typename TAttribute< AttrType >::FGetter::template TConstMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{ \
			_##AttrName.BindRaw( InUserObject, InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		/* Bind attribute with delegate to a shared pointer-based class method.  Slate mostly uses shared pointers so we use an overload for this type of binding. */ \
		template< class UserClass, typename... VarTypes >	\
		WidgetArgsType& AttrName( TSharedRef< UserClass > InUserObjectRef, typename TAttribute< AttrType >::FGetter::template TConstMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{ \
			_##AttrName.Bind( InUserObjectRef, InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		/* Bind attribute with delegate to a shared pointer-based class method.  Slate mostly uses shared pointers so we use an overload for this type of binding. */ \
		template< class UserClass, typename... VarTypes >	\
		WidgetArgsType& AttrName( UserClass* InUserObject, typename TAttribute< AttrType >::FGetter::template TConstMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{ \
			_##AttrName.Bind( InUserObject, InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		/* Bind attribute with delegate to a UObject-based class method */ \
		template< class UserClass, typename... VarTypes >	\
		WidgetArgsType& AttrName##_UObject( UserClass* InUserObject, typename TAttribute< AttrType >::FGetter::template TConstMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{ \
			_##AttrName.BindUObject( InUserObject, InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \

#define SLATE_PRIVATE_ARGUMENT_VARIABLE( ArgType, ArgName ) \
		ArgType _##ArgName


#define SLATE_PRIVATE_ARGUMENT_FUNCTION( ArgType, ArgName ) \
		WidgetArgsType& ArgName( ArgType InArg ) \
		{ \
			_##ArgName = InArg; \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}

/**
 * Use this macro to add a attribute to the declaration of your widget.
 * An attribute can be a value or a function.
 */
#define SLATE_ATTRIBUTE( AttrType, AttrName ) \
		SLATE_PRIVATE_ATTRIBUTE_VARIABLE( AttrType, AttrName ); \
		SLATE_PRIVATE_ATTRIBUTE_FUNCTION( AttrType, AttrName )

/**
 * Use when an argument used to be declared SLATE_ATTRIBUTE and it should no longer be used
 */
#define SLATE_ATTRIBUTE_DEPRECATED( AttrType, AttrName, DeprecationVersion, DeprecationMessage) \
		SLATE_PRIVATE_ATTRIBUTE_VARIABLE( AttrType, AttrName ); \
		UE_DEPRECATED(DeprecationVersion, DeprecationMessage) \
		SLATE_PRIVATE_ATTRIBUTE_FUNCTION( AttrType, AttrName )

/**
 * Use this macro to declare a slate argument.
 * Arguments differ from attributes in that they can only be values
 */
#define SLATE_ARGUMENT( ArgType, ArgName ) \
		SLATE_PRIVATE_ARGUMENT_VARIABLE( ArgType, ArgName ); \
		SLATE_PRIVATE_ARGUMENT_FUNCTION ( ArgType, ArgName )

/**
 * Use when an argument used to be declared SLATE_ARGUMENT and it should no longer be used
 */
#define SLATE_ARGUMENT_DEPRECATED( ArgType, ArgName, DeprecationVersion, DeprecationMessage) \
	SLATE_PRIVATE_ARGUMENT_VARIABLE( ArgType, ArgName ); \
	UE_DEPRECATED(DeprecationVersion, DeprecationMessage) \
	SLATE_PRIVATE_ARGUMENT_FUNCTION ( ArgType, ArgName )

/**
 * Like SLATE_ARGUMENT, but support a default value. e.g.
 * 
 * SLATE_ARGUMENT_DEFAULT(float, WheelScrollMultiplier) { 1.0f };
 */
#define SLATE_ARGUMENT_DEFAULT( ArgType, ArgName ) \
	SLATE_PRIVATE_ARGUMENT_FUNCTION(ArgType, ArgName) \
	SLATE_PRIVATE_ARGUMENT_VARIABLE(ArgType, ArgName)

#define SLATE_PRIVATE_STYLE_ARGUMENT_VARIABLE( ArgType, ArgName ) \
		const ArgType* _##ArgName; \

#define SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_ARGTYPE_PTR( ArgType, ArgName ) \
		WidgetArgsType& ArgName( const ArgType* InArg ) \
		{ \
			_##ArgName = InArg; \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}
#define SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_WIDGETSTYLE_PTR( ArgType, ArgName ) \
		WidgetArgsType& ArgName( const class USlateWidgetStyleAsset* const InSlateStyleAsset ) \
		{ \
			_##ArgName = InSlateStyleAsset->GetStyleChecked< ArgType >(); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}
#define SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_WIDGETSTYLE_WEAKPTR( ArgType, ArgName ) \
		WidgetArgsType& ArgName( const TWeakObjectPtr< const class USlateWidgetStyleAsset >& InSlateStyleAsset ) \
		{ \
			_##ArgName = InSlateStyleAsset.Get()->GetStyleChecked< ArgType >(); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}
#define SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_SLATESTYLE_PTR( ArgType, ArgName ) \
		WidgetArgsType& ArgName( const class ISlateStyle* InSlateStyle, const FName& StyleName, const ANSICHAR* Specifier = nullptr ) \
		{ \
			check( InSlateStyle != nullptr ); \
			_##ArgName = &InSlateStyle->GetWidgetStyle< ArgType >( StyleName, Specifier ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}
#define SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_SLATESTYLE_REF( ArgType, ArgName ) \
		WidgetArgsType& ArgName( const class ISlateStyle& InSlateStyle, const FName& StyleName, const ANSICHAR* Specifier = nullptr ) \
		{ \
			_##ArgName = &InSlateStyle.GetWidgetStyle< ArgType >( StyleName, Specifier ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}
#define SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_SLATESTYLE_WEAKPTR( ArgType, ArgName ) \
		WidgetArgsType& ArgName( const TWeakObjectPtr< const class ISlateStyle >& InSlateStyle, const FName& StyleName, const ANSICHAR* Specifier = nullptr ) \
		{ \
			_##ArgName = &InSlateStyle.Get()->GetWidgetStyle< ArgType >( StyleName, Specifier ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}
#define SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_SLATESTYLE_SHAREDPTR( ArgType, ArgName ) \
		WidgetArgsType& ArgName( const TSharedPtr< const class ISlateStyle >& InSlateStyle, const FName& StyleName, const ANSICHAR* Specifier = nullptr ) \
		{ \
			_##ArgName = &InSlateStyle->GetWidgetStyle< ArgType >( StyleName, Specifier ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} 

 /**
  * Use this macro to declare a slate style argument.
  * Arguments differ from attributes in that they can only be values
  */
#define SLATE_STYLE_ARGUMENT( ArgType, ArgName ) \
	SLATE_PRIVATE_STYLE_ARGUMENT_VARIABLE( ArgType, ArgName ); \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_ARGTYPE_PTR( ArgType, ArgName ) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_WIDGETSTYLE_PTR( ArgType, ArgName ) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_WIDGETSTYLE_WEAKPTR( ArgType, ArgName ) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_SLATESTYLE_PTR( ArgType, ArgName ) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_SLATESTYLE_REF( ArgType, ArgName ) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_SLATESTYLE_WEAKPTR( ArgType, ArgName ) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_SLATESTYLE_SHAREDPTR( ArgType, ArgName )

 /**
  * Use when a style argument used to be declared SLATE_STYLE_ARGUMENT and it should no longer be used
  */
#define SLATE_STYLE_ARGUMENT_DEPRECATED( ArgType, ArgName, DeprecationVersion, DeprecationMessage) \
	SLATE_PRIVATE_STYLE_ARGUMENT_VARIABLE( ArgType, ArgName ); \
	PRAGMA_DISABLE_DEPRECATION_WARNINGS\
	UE_DEPRECATED(DeprecationVersion, DeprecationMessage) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_ARGTYPE_PTR( ArgType, ArgName ) \
	UE_DEPRECATED(DeprecationVersion, DeprecationMessage) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_WIDGETSTYLE_PTR( ArgType, ArgName ) \
	UE_DEPRECATED(DeprecationVersion, DeprecationMessage) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_WIDGETSTYLE_WEAKPTR( ArgType, ArgName ) \
	UE_DEPRECATED(DeprecationVersion, DeprecationMessage) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_SLATESTYLE_PTR( ArgType, ArgName ) \
	UE_DEPRECATED(DeprecationVersion, DeprecationMessage) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_SLATESTYLE_REF( ArgType, ArgName ) \
	UE_DEPRECATED(DeprecationVersion, DeprecationMessage) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_SLATESTYLE_WEAKPTR( ArgType, ArgName ) \
	UE_DEPRECATED(DeprecationVersion, DeprecationMessage) \
	SLATE_PRIVATE_STYLE_ARGUMENT_FUNCTION_SLATESTYLE_SHAREDPTR( ArgType, ArgName )\
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

/**
 * Use this macro between SLATE_BEGIN_ARGS and SLATE_END_ARGS
 * in order to add support for slots.
 */
#define SLATE_SUPPORTS_SLOT( SlotType ) \
		UE_DEPRECATED(5.0, "SLATE_SUPPORTS_SLOT is deprecated. Use SLATE_SLOT_ARGUMENT") \
		TArray< SlotType* > Slots; \
		WidgetArgsType& operator + (SlotType& SlotToAdd) \
		{ \
			Slots.Add( &SlotToAdd ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}


 /**
 * Use this macro between SLATE_BEGIN_ARGS and SLATE_END_ARGS
 * in order to add support for slots that have slate args.
 */
#define SLATE_SUPPORTS_SLOT_WITH_ARGS( SlotType ) \
	TArray< SlotType* > Slots; \
	WidgetArgsType& operator + (const typename SlotType::FArguments& ArgumentsForNewSlot) \
		{ \
			Slots.Add( new SlotType( ArgumentsForNewSlot ) ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}


#define SLATE_PRIVATE_SLOT_BEGIN_ARGS( SlotType, SlotParentType ) \
	{ \
		using WidgetArgsType = SlotType::FSlotArguments; \
		using SlotParentType::FSlotArguments::FSlotArguments;

/**
 * Use this macro between SLATE_BEGIN_ARGS and SLATE_END_ARGS
 * in order to add support for slots with the construct pattern.
 */
#define SLATE_SLOT_ARGUMENT( SlotType, SlotName ) \
		TArray<typename SlotType::FSlotArguments> _##SlotName; \
		WidgetArgsType& operator + (typename SlotType::FSlotArguments& SlotToAdd) \
		{ \
			_##SlotName.Add( MoveTemp(SlotToAdd) ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		WidgetArgsType& operator + (typename SlotType::FSlotArguments&& SlotToAdd) \
		{ \
			_##SlotName.Add( MoveTemp(SlotToAdd) ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}

#define SLATE_SLOT_BEGIN_ARGS( SlotType, SlotParentType ) \
	public: \
	struct FSlotArguments : public SlotParentType::FSlotArguments \
	SLATE_PRIVATE_SLOT_BEGIN_ARGS( SlotType, SlotParentType )

#define SLATE_SLOT_BEGIN_ARGS_OneMixin( SlotType, SlotParentType, Mixin1 ) \
	public: \
	struct FSlotArguments : public SlotParentType::FSlotArguments, public Mixin1::FSlotArgumentsMixin \
	SLATE_PRIVATE_SLOT_BEGIN_ARGS( SlotType, SlotParentType )

#define SLATE_SLOT_BEGIN_ARGS_TwoMixins( SlotType, SlotParentType, Mixin1, Mixin2 ) \
	public: \
	struct FSlotArguments : public SlotParentType::FSlotArguments, public Mixin1::FSlotArgumentsMixin, public Mixin2::FSlotArgumentsMixin \
	SLATE_PRIVATE_SLOT_BEGIN_ARGS( SlotType, SlotParentType )

#define SLATE_SLOT_BEGIN_ARGS_ThreeMixins( SlotType, SlotParentType, Mixin1, Mixin2, Mixin3 ) \
	public: \
	struct FSlotArguments : public SlotParentType::FSlotArguments, public Mixin1::FSlotArgumentsMixin, public Mixin2::FSlotArgumentsMixin, public Mixin3::FSlotArgumentsMixin \
	SLATE_PRIVATE_SLOT_BEGIN_ARGS( SlotType, SlotParentType )

#define SLATE_SLOT_BEGIN_ARGS_FourMixins( SlotType, SlotParentType, Mixin1, Mixin2, Mixin3, Mixin4 ) \
	public: \
	struct FSlotArguments : public SlotParentType::FSlotArguments, public Mixin1::FSlotArgumentsMixin, public Mixin2::FSlotArgumentsMixin, public Mixin3::FSlotArgumentsMixin, public Mixin4::FSlotArgumentsMixin \
	SLATE_PRIVATE_SLOT_BEGIN_ARGS( SlotType, SlotParentType )

#define SLATE_SLOT_END_ARGS() \
	};


/** A widget reference that is always a valid pointer; defaults to SNullWidget */
struct TAlwaysValidWidget
{
	TAlwaysValidWidget()
	: Widget(SNullWidget::NullWidget)
	{
	}

	TSharedRef<SWidget> Widget;
};


/**
 * We want to be able to do:
 * SNew( ContainerWidget )
 * .SomeContentArea()
 * [
 *   // Child widgets go here
 * ]
 *
 * NamedSlotProperty is a helper that will be returned by SomeContentArea().
 */
template<class DeclarationType>
struct NamedSlotProperty
{
	NamedSlotProperty( DeclarationType& InOwnerDeclaration, TAlwaysValidWidget& ContentToSet )
		: OwnerDeclaration( InOwnerDeclaration )
		, SlotContent(ContentToSet)
	{}

	DeclarationType & operator[]( const TSharedRef<SWidget>& InChild )
	{
		SlotContent.Widget = InChild;
		return OwnerDeclaration;
	}

	DeclarationType & OwnerDeclaration;
	TAlwaysValidWidget & SlotContent;
};


/**
 * Use this macro to add support for named slot properties such as Content and Header. See NamedSlotProperty for more details.
 *
 * NOTE: If you're using this within a widget class that is templated, then you might have to specify a full name for the declaration.
 *       For example: SLATE_NAMED_SLOT( typename SSuperWidget<T>::Declaration, Content )
 */
#define SLATE_NAMED_SLOT( DeclarationType, SlotName ) \
		NamedSlotProperty< DeclarationType > SlotName() \
		{ \
			return NamedSlotProperty< DeclarationType >( *this, _##SlotName ); \
		} \
		TAlwaysValidWidget _##SlotName; \

#define SLATE_DEFAULT_SLOT( DeclarationType, SlotName ) \
		SLATE_NAMED_SLOT(DeclarationType, SlotName) ; \
		DeclarationType & operator[]( const TSharedRef<SWidget>& InChild ) \
		{ \
			_##SlotName.Widget = InChild; \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}


/**
 * Use this macro to add event handler support to the declarative syntax of your widget.
 * It is expected that the widget has a delegate called of type 'EventDelegateType' that is
 * named 'EventName'.
 */	
#define SLATE_EVENT( DelegateName, EventName ) \
		WidgetArgsType& EventName( const DelegateName& InDelegate ) \
		{ \
			_##EventName = InDelegate; \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		WidgetArgsType& EventName( DelegateName&& InDelegate ) \
		{ \
			_##EventName = MoveTemp(InDelegate); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		/* Set event delegate to a global function */ \
		/* NOTE: We use a template here to avoid 'typename' issues when hosting attributes inside templated classes */ \
		template< typename StaticFuncPtr, typename... VarTypes > \
		WidgetArgsType& EventName##_Static( StaticFuncPtr InFunc, VarTypes... Vars )	\
		{ \
			_##EventName.BindStatic( InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		/* Set event delegate to a lambda
		 * technically this works for any functor types, but lambdas are the primary use case */ \
		template<typename FunctorType, typename... VarTypes> \
		WidgetArgsType& EventName##_Lambda(FunctorType&& InFunctor, VarTypes... Vars) \
		{ \
			_##EventName.BindLambda( Forward<FunctorType>(InFunctor), Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		/* Set event delegate to a raw C++ class method */ \
		template< class UserClass, typename... VarTypes >	\
		WidgetArgsType& EventName##_Raw( UserClass* InUserObject, typename DelegateName::template TMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{ \
			_##EventName.BindRaw( InUserObject, InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		template< class UserClass, typename... VarTypes >	\
		WidgetArgsType& EventName##_Raw( UserClass* InUserObject, typename DelegateName::template TConstMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{ \
			_##EventName.BindRaw( InUserObject, InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		/* Set event delegate to a shared pointer-based class method.  Slate mostly uses shared pointers so we use an overload for this type of binding. */ \
		template< class UserClass, typename... VarTypes >	\
		WidgetArgsType& EventName( TSharedRef< UserClass > InUserObjectRef, typename DelegateName::template TMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{ \
			_##EventName.BindSP( InUserObjectRef, InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		template< class UserClass, typename... VarTypes >	\
		WidgetArgsType& EventName( TSharedRef< UserClass > InUserObjectRef, typename DelegateName::template TConstMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{ \
			_##EventName.BindSP( InUserObjectRef, InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		/* Set event delegate to a shared pointer-based class method.  Slate mostly uses shared pointers so we use an overload for this type of binding. */ \
		template< class UserClass, typename... VarTypes >	\
		WidgetArgsType& EventName( UserClass* InUserObject, typename DelegateName::template TMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{ \
			_##EventName.BindSP( InUserObject, InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		template< class UserClass, typename... VarTypes >	\
		WidgetArgsType& EventName( UserClass* InUserObject, typename DelegateName::template TConstMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{ \
			_##EventName.BindSP( InUserObject, InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		/* Set event delegate to a UObject-based class method */ \
		template< class UserClass, typename... VarTypes >	\
		WidgetArgsType& EventName##_UObject( UserClass* InUserObject, typename DelegateName::template TMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{ \
			_##EventName.BindUObject( InUserObject, InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		template< class UserClass, typename... VarTypes >	\
		WidgetArgsType& EventName##_UObject( UserClass* InUserObject, typename DelegateName::template TConstMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{ \
			_##EventName.BindUObject( InUserObject, InFunc, Vars... ); \
			return static_cast<WidgetArgsType*>(this)->Me(); \
		} \
		\
		DelegateName _##EventName; \

#define SLATE_EVENT_DEPRECATED( DeprecationVersion, DeprecationMessage, DelegateName, EventName, UpgradedEventName, UpgradeFuncName)	\
		UE_DEPRECATED(DeprecationVersion, DeprecationMessage)	\
		WidgetArgsType& EventName( const DelegateName& InDeprecatedDelegate ) \
		{	\
			if (InDeprecatedDelegate.IsBound())	\
			{	\
				_##UpgradedEventName = UpgradeFuncName(InDeprecatedDelegate);	\
			}	\
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}	\
		\
		template <typename StaticFuncPtr, typename... VarTypes>	\
		UE_DEPRECATED(DeprecationVersion, DeprecationMessage)	\
		WidgetArgsType& EventName##_Static( StaticFuncPtr InFunc, VarTypes... Vars )	\
		{	\
			if (InFunc)	\
			{	\
				DelegateName DeprecatedDelegate = DelegateName::CreateStatic(InFunc, Vars...);	\
				_##UpgradedEventName = UpgradeFuncName(DeprecatedDelegate);	\
			}	\
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}	\
		\
		template <typename FunctorType, typename... VarTypes>	\
		UE_DEPRECATED(DeprecationVersion, DeprecationMessage)	\
		WidgetArgsType& EventName##_Lambda(FunctorType&& InFunctor, VarTypes... Vars) \
		{	\
			DelegateName DeprecatedDelegate = DelegateName::CreateLambda(Forward<FunctorType>(InFunctor), Vars...);	\
			_##UpgradedEventName = UpgradeFuncName(DeprecatedDelegate);	\
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}	\
		\
		template <typename UserClass, typename... VarTypes>	\
		UE_DEPRECATED(DeprecationVersion, DeprecationMessage)	\
		WidgetArgsType& EventName##_Raw( UserClass* InUserObject, typename DelegateName::template TMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{	\
			DelegateName DeprecatedDelegate = DelegateName::CreateRaw(InUserObject, InFunc, Vars...);	\
			_##UpgradedEventName = UpgradeFuncName(DeprecatedDelegate);	\
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}	\
		\
		template <typename UserClass, typename... VarTypes>	\
		UE_DEPRECATED(DeprecationVersion, DeprecationMessage)	\
		WidgetArgsType& EventName##_Raw( UserClass* InUserObject, typename DelegateName::template TConstMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{	\
			DelegateName DeprecatedDelegate = DelegateName::CreateRaw(InUserObject, InFunc, Vars...);	\
			_##UpgradedEventName = UpgradeFuncName(DeprecatedDelegate);	\
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}	\
		\
		template <typename UserClass, typename... VarTypes>	\
		UE_DEPRECATED(DeprecationVersion, DeprecationMessage)	\
		WidgetArgsType& EventName( TSharedRef< UserClass > InUserObjectRef, typename DelegateName::template TMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{	\
			DelegateName DeprecatedDelegate = DelegateName::CreateSP(InUserObjectRef, InFunc, Vars...);	\
			_##UpgradedEventName = UpgradeFuncName(DeprecatedDelegate);	\
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}	\
		\
		template <typename UserClass, typename... VarTypes>	\
		UE_DEPRECATED(DeprecationVersion, DeprecationMessage)	\
		WidgetArgsType& EventName( TSharedRef< UserClass > InUserObjectRef, typename DelegateName::template TConstMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{	\
			DelegateName DeprecatedDelegate = DelegateName::CreateSP(InUserObjectRef, InFunc, Vars...);	\
			_##UpgradedEventName = UpgradeFuncName(DeprecatedDelegate);	\
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}	\
		\
		template <typename UserClass, typename... VarTypes>	\
		UE_DEPRECATED(DeprecationVersion, DeprecationMessage)	\
		WidgetArgsType& EventName( UserClass* InUserObject, typename DelegateName::template TMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{	\
			DelegateName DeprecatedDelegate = DelegateName::CreateSP(InUserObject, InFunc, Vars...);	\
			_##UpgradedEventName = UpgradeFuncName(DeprecatedDelegate);	\
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}	\
		\
		template <typename UserClass, typename... VarTypes>	\
		UE_DEPRECATED(DeprecationVersion, DeprecationMessage)	\
		WidgetArgsType& EventName( UserClass* InUserObject, typename DelegateName::template TConstMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{	\
			DelegateName DeprecatedDelegate = DelegateName::CreateSP(InUserObject, InFunc, Vars...);	\
			_##UpgradedEventName = UpgradeFuncName(DeprecatedDelegate);	\
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}	\
		\
		template <typename UserClass, typename... VarTypes>	\
		UE_DEPRECATED(DeprecationVersion, DeprecationMessage)	\
		WidgetArgsType& EventName##_UObject( UserClass* InUserObject, typename DelegateName::template TMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{	\
			DelegateName DeprecatedDelegate = DelegateName::CreateUObject(InUserObject, InFunc, Vars...);	\
			_##UpgradedEventName = UpgradeFuncName(DeprecatedDelegate);	\
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}	\
		\
		template <typename UserClass, typename... VarTypes>	\
		UE_DEPRECATED(DeprecationVersion, DeprecationMessage)	\
		WidgetArgsType& EventName##_UObject( UserClass* InUserObject, typename DelegateName::template TConstMethodPtr< UserClass, VarTypes... > InFunc, VarTypes... Vars )	\
		{	\
			DelegateName DeprecatedDelegate = DelegateName::CreateUObject(InUserObject, InFunc, Vars...);	\
			_##UpgradedEventName = UpgradeFuncName(DeprecatedDelegate);	\
			return static_cast<WidgetArgsType*>(this)->Me(); \
		}	\



/** Base class for named arguments. Provides settings necessary for all widgets. */
struct FSlateBaseNamedArgs
{
	FSlateBaseNamedArgs() = default;

	SLATE_PRIVATE_ATTRIBUTE_VARIABLE(FText, ToolTipText);
	SLATE_PRIVATE_ATTRIBUTE_VARIABLE(TSharedPtr<IToolTip>, ToolTip);
	SLATE_PRIVATE_ATTRIBUTE_VARIABLE(TOptional<EMouseCursor::Type>, Cursor);
	SLATE_PRIVATE_ATTRIBUTE_VARIABLE(EVisibility, Visibility) = EVisibility::Visible;
	SLATE_PRIVATE_ATTRIBUTE_VARIABLE(bool, IsEnabled) = true;
	SLATE_PRIVATE_ARGUMENT_VARIABLE(bool, ForceVolatile) = false;
	/** If true, bound Slate Attributes will be updated once per frame. */
	SLATE_PRIVATE_ARGUMENT_VARIABLE(bool, EnabledAttributesUpdate) = true;
	SLATE_PRIVATE_ARGUMENT_VARIABLE(ESlateAttributeInitialization, UpdateAttributes) = ESlateAttributeInitialization::None;
	SLATE_PRIVATE_ARGUMENT_VARIABLE(EWidgetClipping, Clipping) = EWidgetClipping::Inherit;
	SLATE_PRIVATE_ARGUMENT_VARIABLE(EWidgetPixelSnapping, PixelSnappingMethod) = EWidgetPixelSnapping::Inherit;
	SLATE_PRIVATE_ARGUMENT_VARIABLE(EFlowDirectionPreference, FlowDirectionPreference) = EFlowDirectionPreference::Inherit;
	SLATE_PRIVATE_ARGUMENT_VARIABLE(float, RenderOpacity) = 1.f;
	SLATE_PRIVATE_ATTRIBUTE_VARIABLE(TOptional<FSlateRenderTransform>, RenderTransform);
	SLATE_PRIVATE_ATTRIBUTE_VARIABLE(FVector2D, RenderTransformPivot) = FVector2D::ZeroVector;
	SLATE_PRIVATE_ARGUMENT_VARIABLE(FName, Tag);
	SLATE_PRIVATE_ARGUMENT_VARIABLE(TOptional<FAccessibleWidgetData>, AccessibleParams);
	SLATE_PRIVATE_ATTRIBUTE_VARIABLE(FText, AccessibleText);
	TArray<TSharedRef<ISlateMetaData>> MetaData;
};


/** Base class for named arguments. Provides settings necessary for all widgets. */
template<typename InWidgetType>
struct TSlateBaseNamedArgs : public FSlateBaseNamedArgs
{
	typedef InWidgetType WidgetType;
	typedef typename WidgetType::FArguments WidgetArgsType;

	SLATE_PRIVATE_ATTRIBUTE_FUNCTION(FText, ToolTipText)
	SLATE_PRIVATE_ATTRIBUTE_FUNCTION(TSharedPtr<IToolTip>, ToolTip)
	SLATE_PRIVATE_ATTRIBUTE_FUNCTION(TOptional<EMouseCursor::Type>, Cursor)
	SLATE_PRIVATE_ATTRIBUTE_FUNCTION(bool, IsEnabled)
	SLATE_PRIVATE_ATTRIBUTE_FUNCTION(EVisibility, Visibility)
	SLATE_PRIVATE_ARGUMENT_FUNCTION(bool, ForceVolatile)
	SLATE_PRIVATE_ARGUMENT_FUNCTION(ESlateAttributeInitialization, UpdateAttributes)
	SLATE_PRIVATE_ARGUMENT_FUNCTION(EWidgetClipping, Clipping)
	SLATE_PRIVATE_ARGUMENT_FUNCTION(EWidgetPixelSnapping, PixelSnappingMethod)
	SLATE_PRIVATE_ARGUMENT_FUNCTION(EFlowDirectionPreference, FlowDirectionPreference)
	SLATE_PRIVATE_ARGUMENT_FUNCTION(float, RenderOpacity)
	SLATE_PRIVATE_ATTRIBUTE_FUNCTION(TOptional<FSlateRenderTransform>, RenderTransform)
	SLATE_PRIVATE_ATTRIBUTE_FUNCTION(FVector2D, RenderTransformPivot)
	SLATE_PRIVATE_ARGUMENT_FUNCTION(FName, Tag)
	SLATE_PRIVATE_ARGUMENT_FUNCTION(TOptional<FAccessibleWidgetData>, AccessibleParams)
	SLATE_PRIVATE_ATTRIBUTE_FUNCTION(FText, AccessibleText)

	/** Used by the named argument pattern as a safe way to 'return *this' for call-chaining purposes. */
	WidgetArgsType& Me()
	{
		return *(static_cast<WidgetArgsType*>(this));
	}

	/** Add metadata to this widget. */
	WidgetArgsType& AddMetaData(TSharedRef<ISlateMetaData> InMetaData)
	{
		MetaData.Add(InMetaData);
		return Me();
	}

	/** Add metadata to this widget - convenience method - 1 argument */
	template<typename MetaDataType, typename Arg0Type>
	WidgetArgsType& AddMetaData(Arg0Type InArg0)
	{
		MetaData.Add(MakeShared<MetaDataType>(InArg0));
		return Me();
	}

	/** Add metadata to this widget - convenience method - 2 arguments */
	template<typename MetaDataType, typename Arg0Type, typename Arg1Type>
	WidgetArgsType& AddMetaData(Arg0Type InArg0, Arg1Type InArg1)
	{
		MetaData.Add(MakeShared<MetaDataType>(InArg0, InArg1));
		return Me();
	}
};

namespace RequiredArgs
{
	struct T0RequiredArgs
	{
		T0RequiredArgs()
		{
		}

		template<class WidgetType>
		void CallConstruct(WidgetType* OnWidget, const typename WidgetType::FArguments& WithNamedArgs) const
		{
			// YOUR WIDGET MUST IMPLEMENT void Construct(const FArguments& InArgs)
			OnWidget->Construct(WithNamedArgs);
		}
	};

	template<typename Arg0Type>
	struct T1RequiredArgs
	{
		T1RequiredArgs(Arg0Type&& InArg0)
			: Arg0(InArg0)
		{
		}

		template<class WidgetType>
		void CallConstruct(WidgetType* OnWidget, const typename WidgetType::FArguments& WithNamedArgs) const
		{
			// YOUR WIDGET MUST IMPLEMENT void Construct(const FArguments& InArgs)
			OnWidget->Construct(WithNamedArgs, Forward<Arg0Type>(Arg0));
		}

		Arg0Type& Arg0;
	};

	template<typename Arg0Type, typename Arg1Type>
	struct T2RequiredArgs
	{
		T2RequiredArgs(Arg0Type&& InArg0, Arg1Type&& InArg1)
			: Arg0(InArg0)
			, Arg1(InArg1)
		{
		}

		template<class WidgetType>
		void CallConstruct(WidgetType* OnWidget, const typename WidgetType::FArguments& WithNamedArgs) const
		{
			// YOUR WIDGET MUST IMPLEMENT Construct(const FArguments& InArgs)
			OnWidget->Construct(WithNamedArgs, Forward<Arg0Type>(Arg0), Forward<Arg1Type>(Arg1));
		}

		Arg0Type& Arg0;
		Arg1Type& Arg1;
	};

	template<typename Arg0Type, typename Arg1Type, typename Arg2Type>
	struct T3RequiredArgs
	{
		T3RequiredArgs(Arg0Type&& InArg0, Arg1Type&& InArg1, Arg2Type&& InArg2)
			: Arg0(InArg0)
			, Arg1(InArg1)
			, Arg2(InArg2)
		{
		}

		template<class WidgetType>
		void CallConstruct(WidgetType* OnWidget, const typename WidgetType::FArguments& WithNamedArgs) const
		{
			// YOUR WIDGET MUST IMPLEMENT Construct(const FArguments& InArgs)
			OnWidget->Construct(WithNamedArgs, Forward<Arg0Type>(Arg0), Forward<Arg1Type>(Arg1), Forward<Arg2Type>(Arg2));
		}

		Arg0Type& Arg0;
		Arg1Type& Arg1;
		Arg2Type& Arg2;
	};

	template<typename Arg0Type, typename Arg1Type, typename Arg2Type, typename Arg3Type>
	struct T4RequiredArgs
	{
		T4RequiredArgs(Arg0Type&& InArg0, Arg1Type&& InArg1, Arg2Type&& InArg2, Arg3Type&& InArg3)
			: Arg0(InArg0)
			, Arg1(InArg1)
			, Arg2(InArg2)
			, Arg3(InArg3)
		{
		}

		template<class WidgetType>
		void CallConstruct(WidgetType* OnWidget, const typename WidgetType::FArguments& WithNamedArgs) const
		{
			// YOUR WIDGET MUST IMPLEMENT Construct(const FArguments& InArgs)
			OnWidget->Construct(WithNamedArgs, Forward<Arg0Type>(Arg0), Forward<Arg1Type>(Arg1), Forward<Arg2Type>(Arg2), Forward<Arg3Type>(Arg3));
		}

		Arg0Type& Arg0;
		Arg1Type& Arg1;
		Arg2Type& Arg2;
		Arg3Type& Arg3;
	};

	template<typename Arg0Type, typename Arg1Type, typename Arg2Type, typename Arg3Type, typename Arg4Type>
	struct T5RequiredArgs
	{
		T5RequiredArgs(Arg0Type&& InArg0, Arg1Type&& InArg1, Arg2Type&& InArg2, Arg3Type&& InArg3, Arg4Type&& InArg4)
			: Arg0(InArg0)
			, Arg1(InArg1)
			, Arg2(InArg2)
			, Arg3(InArg3)
			, Arg4(InArg4)
		{
		}

		template<class WidgetType>
		void CallConstruct(WidgetType* OnWidget, const typename WidgetType::FArguments& WithNamedArgs) const
		{
			// YOUR WIDGET MUST IMPLEMENT Construct(const FArguments& InArgs)
			OnWidget->Construct(WithNamedArgs, Forward<Arg0Type>(Arg0), Forward<Arg1Type>(Arg1), Forward<Arg2Type>(Arg2), Forward<Arg3Type>(Arg3), Forward<Arg4Type>(Arg4));
		}

		Arg0Type& Arg0;
		Arg1Type& Arg1;
		Arg2Type& Arg2;
		Arg3Type& Arg3;
		Arg4Type& Arg4;
	};

	FORCEINLINE T0RequiredArgs MakeRequiredArgs()
	{
		return T0RequiredArgs();
	}

	template<typename Arg0Type>
	T1RequiredArgs<Arg0Type&&> MakeRequiredArgs(Arg0Type&& InArg0)
	{
		return T1RequiredArgs<Arg0Type&&>(Forward<Arg0Type>(InArg0));
	}

	template<typename Arg0Type, typename Arg1Type>
	T2RequiredArgs<Arg0Type&&, Arg1Type&&> MakeRequiredArgs(Arg0Type&& InArg0, Arg1Type&& InArg1)
	{
		return T2RequiredArgs<Arg0Type&&, Arg1Type&&>(Forward<Arg0Type>(InArg0), Forward<Arg1Type>(InArg1));
	}

	template<typename Arg0Type, typename Arg1Type, typename Arg2Type>
	T3RequiredArgs<Arg0Type&&, Arg1Type&&, Arg2Type&&> MakeRequiredArgs(Arg0Type&& InArg0, Arg1Type&& InArg1, Arg2Type&& InArg2)
	{
		return T3RequiredArgs<Arg0Type&&, Arg1Type&&, Arg2Type&&>(Forward<Arg0Type>(InArg0), Forward<Arg1Type>(InArg1), Forward<Arg2Type>(InArg2));
	}

	template<typename Arg0Type, typename Arg1Type, typename Arg2Type, typename Arg3Type>
	T4RequiredArgs<Arg0Type&&, Arg1Type&&, Arg2Type&&, Arg3Type&&> MakeRequiredArgs(Arg0Type&& InArg0, Arg1Type&& InArg1, Arg2Type&& InArg2, Arg3Type&& InArg3)
	{
		return T4RequiredArgs<Arg0Type&&, Arg1Type&&, Arg2Type&&, Arg3Type&&>(Forward<Arg0Type>(InArg0), Forward<Arg1Type>(InArg1), Forward<Arg2Type>(InArg2), Forward<Arg3Type>(InArg3));
	}

	template<typename Arg0Type, typename Arg1Type, typename Arg2Type, typename Arg3Type, typename Arg4Type>
	T5RequiredArgs<Arg0Type&&, Arg1Type&&, Arg2Type&&, Arg3Type&&, Arg4Type&&> MakeRequiredArgs(Arg0Type&& InArg0, Arg1Type&& InArg1, Arg2Type&& InArg2, Arg3Type&& InArg3, Arg4Type&& InArg4)
	{
		return T5RequiredArgs<Arg0Type&&, Arg1Type&&, Arg2Type&&, Arg3Type&&, Arg4Type&&>(Forward<Arg0Type>(InArg0), Forward<Arg1Type>(InArg1), Forward<Arg2Type>(InArg2), Forward<Arg3Type>(InArg3), Forward<Arg4Type>(InArg4));
	}
}


/**
 * Utility class used during widget instantiation.
 * Performs widget allocation and construction.
 * Ensures that debug info is set correctly.
 * Returns TSharedRef to widget.
 *
 * @see SNew
 * @see SAssignNew
 */
template<class WidgetType, typename RequiredArgsPayloadType>
struct TSlateDecl
{
	TSlateDecl( const ANSICHAR* InType, const ANSICHAR* InFile, int32 OnLine, RequiredArgsPayloadType&& InRequiredArgs )
		: _RequiredArgs(InRequiredArgs)
	{
		if constexpr (std::is_base_of_v<SUserWidget, WidgetType>)
		{
			/**
			 * SUserWidgets are allocated in the corresponding CPP file, so that
			 * the implementer can return an implementation that differs from the
			 * public interface. @see SUserWidgetExample
			 */
			_Widget = WidgetType::New();
		}
		else
		{
			/** Normal widgets are allocated directly by the TSlateDecl. */
			_Widget = MakeShared<WidgetType>();
		}

		_Widget->SetDebugInfo(InType, InFile, OnLine, sizeof(WidgetType));

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
		_Widget->SetClassSupportsInvalidation(TWidgetTypeTraits<WidgetType>::SupportsInvalidation());
#endif
	}

	/**
	 * Initialize OutVarToInit with the widget that is being constructed.
	 * @see SAssignNew
	 */
	template<class ExposeAsWidgetType>
	TSlateDecl&& Expose( TSharedPtr<ExposeAsWidgetType>& OutVarToInit ) &&
	{
		// Can't move out _Widget here because operator<<= needs it
		OutVarToInit = _Widget;
		return MoveTemp(*this);
	}

	/**
	 * Initialize OutVarToInit with the widget that is being constructed.
	 * @see SAssignNew
	 */
	template<class ExposeAsWidgetType>
	TSlateDecl&& Expose( TSharedRef<ExposeAsWidgetType>& OutVarToInit ) &&
	{
		// Can't move out _Widget here because operator<<= needs it
		OutVarToInit = _Widget.ToSharedRef();
		return MoveTemp(*this);
	}

	/**
	 * Initialize a WEAK OutVarToInit with the widget that is being constructed.
	 * @see SAssignNew
	 */
	template<class ExposeAsWidgetType>
	TSlateDecl&& Expose( TWeakPtr<ExposeAsWidgetType>& OutVarToInit ) &&
	{
		// Can't move out _Widget here because operator<<= needs it
		OutVarToInit = _Widget.ToWeakPtr();
		return MoveTemp(*this);
	}

	/**
	 * Complete widget construction from InArgs.
	 *
	 * @param InArgs  NamedArguments from which to construct the widget.
	 *
	 * @return A reference to the widget that we constructed.
	 */
	TSharedRef<WidgetType> operator<<=( const typename WidgetType::FArguments& InArgs ) &&
	{
		_Widget->SWidgetConstruct(InArgs);
		_RequiredArgs.CallConstruct(_Widget.Get(), InArgs);
		if (InArgs._UpdateAttributes == ESlateAttributeInitialization::AfterConstruct)
		{
			_Widget->UpdateAllAttributes();
		}
		_Widget->CacheVolatility();
		_Widget->bIsDeclarativeSyntaxConstructionCompleted = true;

		return MoveTemp(_Widget).ToSharedRef();
	}

	TSharedPtr<WidgetType> _Widget;
	RequiredArgsPayloadType& _RequiredArgs;
};


template<typename WidgetType, typename RequiredArgsPayloadType>
TSlateDecl<WidgetType, RequiredArgsPayloadType> MakeTDecl( const ANSICHAR* InType, const ANSICHAR* InFile, int32 OnLine, RequiredArgsPayloadType&& InRequiredArgs )
{
	LLM_SCOPE_BYTAG(UI_Slate);
	return TSlateDecl<WidgetType, RequiredArgsPayloadType>(InType, InFile, OnLine, Forward<RequiredArgsPayloadType>(InRequiredArgs));
}
