// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

/**
 * Interface for tool tips.
 */
class IToolTip
{
public:

	/**
	 * Gets the widget that this tool tip represents.
	 *
	 * @return The tool tip widget.
	 */
	virtual TSharedRef<class SWidget> AsWidget( ) = 0;

	/**
	 * Gets the tool tip's content widget.
	 *
	 * @return The content widget.
	 */
	virtual TSharedRef<SWidget> GetContentWidget( ) = 0;

	/**
	 * Sets the tool tip's content widget.
	 *
	 * @param InContentWidget The new content widget to set.
	 */
	virtual void SetContentWidget( const TSharedRef<SWidget>& InContentWidget ) = 0;

	/**
	* Resets the tooltip to its default without a content widget.
	*/
	virtual void ResetContentWidget () { }

	/**
	 * Checks whether this tool tip has no content to display right now.
	 *
	 * @return true if the tool tip has no content to display, false otherwise.
	 */
	virtual bool IsEmpty( ) const = 0;

	/**
	 * Checks whether this tool tip can be made interactive by the user (by holding Ctrl).
	 *
	 * @return true if it is an interactive tool tip, false otherwise.
	 */
	virtual bool IsInteractive( ) const = 0;

	/**
	 * Called when the tooltip widget is about to be requested for opening.
	 */
	virtual void OnOpening( ) = 0;

	/**
	 * Called when the tooltip widget is closed and the tooltip is no longer needed.
	 */
	virtual void OnClosed( ) = 0;

	/**
	 * Allows an interactive tooltip to adjust and/or change its window location on open,
	 * as interactive tooltips don't update the tooltip window to follow the cursor position.
	 * 
	 * @param InOutDesiredLocation	On input, contains the previous cursor position (in screen coordinates) plus the standard
	 *								tooltip offset (SlateDefs::TooltipOffsetFromMouse). This position can be modified and/or
	 *								reset to either adjust the offset or otherwise set to a new position as the output value.
	 */
	virtual void OnSetInteractiveWindowLocation(FVector2D& InOutDesiredLocation) const {}

public:

	/** Virtual destructor. */
	virtual ~IToolTip( ) = default;
};
