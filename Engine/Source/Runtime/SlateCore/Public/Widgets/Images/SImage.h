// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/SlateColor.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SLeafWidget.h"
#include "Styling/SlateTypes.h"

class FPaintArgs;
class FSlateWindowElementList;

template <>
struct TWidgetTypeTraits<class SImage>
{
	static constexpr bool SupportsInvalidation() { return true; }
};


/**
 * Implements a widget that displays an image with a desired width and height.
 */
class SImage
	: public SLeafWidget
{
	SLATE_DECLARE_WIDGET_API(SImage, SLeafWidget, SLATECORE_API)

public:
	SLATE_BEGIN_ARGS( SImage )
		: _Image( FCoreStyle::Get().GetDefaultBrush() )
		, _ColorAndOpacity( FLinearColor::White )
		, _FlipForRightToLeftFlowDirection( false )
		{ }

		/** Image resource */
		SLATE_ATTRIBUTE(const FSlateBrush*, Image)

		/** Color and opacity */
		SLATE_ATTRIBUTE(FSlateColor, ColorAndOpacity)

		/** When specified, ignore the brushes size and report the DesiredSizeOverride as the desired image size. */
		SLATE_ATTRIBUTE(TOptional<FVector2D>, DesiredSizeOverride)

		/** Flips the image if the localization's flow direction is RightToLeft */
		SLATE_ARGUMENT( bool, FlipForRightToLeftFlowDirection )

		/** Invoked when the mouse is pressed in the widget. */
		SLATE_EVENT(FPointerEventHandler, OnMouseButtonDown)
	SLATE_END_ARGS()

	/** Constructor */
	SLATECORE_API SImage();
	SLATECORE_API virtual ~SImage();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATECORE_API void Construct( const FArguments& InArgs );

public:

	/** Set the ColorAndOpacity attribute */
	SLATECORE_API void SetColorAndOpacity(TAttribute<FSlateColor> InColorAndOpacity);

	/** See the ColorAndOpacity attribute */
	SLATECORE_API void SetColorAndOpacity( FLinearColor InColorAndOpacity );

	/** Set the Image attribute */
	SLATECORE_API void SetImage(TAttribute<const FSlateBrush*> InImage);

	/** Invalidate the Image */
	SLATECORE_API void InvalidateImage();

	/** Set SizeOverride attribute */
	SLATECORE_API void SetDesiredSizeOverride(TAttribute<TOptional<FVector2D>> InDesiredSizeOverride);

	/** Set FlipForRightToLeftFlowDirection */
	SLATECORE_API void FlipForRightToLeftFlowDirection(bool InbFlipForRightToLeftFlowDirection);

public:

	// SWidget overrides
	SLATECORE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
#if WITH_ACCESSIBILITY
	SLATECORE_API virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override;
#endif

protected:
	// Begin SWidget overrides.
	SLATECORE_API virtual FVector2D ComputeDesiredSize(float) const override;
	// End SWidget overrides.

	/** @return an attribute reference of Image */
	TSlateAttributeRef<const FSlateBrush*> GetImageAttribute() const { return TSlateAttributeRef<FSlateBrush const*>(SharedThis(this), ImageAttribute); }

	/** @return an attribute reference of ColorAndOpacity */
	TSlateAttributeRef<FSlateColor> GetColorAndOpacityAttribute() const { return TSlateAttributeRef<FSlateColor>(SharedThis(this), ColorAndOpacityAttribute); }

	/** @return an attribute reference of DesiredSizeOverride */
	TSlateAttributeRef<TOptional<FVector2D>> GetDesiredSizeOverrideAttribute() const { return TSlateAttributeRef<TOptional<FVector2D>>(SharedThis(this), DesiredSizeOverrideAttribute); }

private:
	/** The slate brush to draw for the ImageAttribute that we can invalidate. */
	TSlateAttribute<const FSlateBrush*> ImageAttribute;

	/** Color and opacity scale for this ImageAttribute */
	TSlateAttribute<FSlateColor> ColorAndOpacityAttribute;

	/** When specified, ignore the content's desired size and report the.HeightOverride as the Box's desired height. */
	TSlateAttribute<TOptional<FVector2D>> DesiredSizeOverrideAttribute;

protected:
	/** Flips the image if the localization's flow direction is RightToLeft */
	bool bFlipForRightToLeftFlowDirection;
};
