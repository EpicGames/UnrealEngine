// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Images/SImage.h"

struct FSlateIcon;

template <>
struct TWidgetTypeTraits<class SLayeredImage>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** A slate widget to draw an arbitrary number of images over top of each other */
class SLayeredImage : public SImage
{
public:
	typedef TTuple<TAttribute<const FSlateBrush*>, TAttribute<FSlateColor> > ImageLayer;

	/** Constructor that adds InLayers on top of the base image  */
	SLATECORE_API void Construct(const FArguments& InArgs, const TArray<ImageLayer>& InLayers);
	
	/** Constructor that adds InLayers on top of the base image  */
	SLATECORE_API void Construct(const FArguments& InArgs, TArray<ImageLayer>&& InLayers);

	/** Constructor that adds a single layer on top of the base image */
	SLATECORE_API void Construct(const FArguments& InArgs, TAttribute<const FSlateBrush*> Brush, TAttribute<FSlateColor> Color);

	/** Constructor that adds NumLayers blank layers on top of the image for later use */
	SLATECORE_API void Construct(const FArguments& InArgs, int32 NumLayers = 0);

	/** Constructor that sets the base image and any overlay layers defined in the Slate icon */
	SLATECORE_API void Construct(const FArguments& InArgs, const FSlateIcon& InIcon);

	//~ Begin SWidget Interface
	SLATECORE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	//~ End SWidget Interface

	/** Adds a new layer to the top of the Image given a brush and color. Use White as the default color. */
	SLATECORE_API void AddLayer(TAttribute<const FSlateBrush*> Brush);

	/** Adds a new layer to the top of the Image given a brush and color */
	SLATECORE_API void AddLayer(TAttribute<const FSlateBrush*> Brush, TAttribute<FSlateColor> Color);

	/** Sets the base image and any overlay layers defined in the Slate icon. Note: removes all layers */
	SLATECORE_API void SetFromSlateIcon(const FSlateIcon& InIcon);

	/** Removes all layers */
	SLATECORE_API void RemoveAllLayers();

	/** Returns the total number of layers, including the base image. */
	SLATECORE_API int32 GetNumLayers() const;

	/** Determines whether a Layer exists at the given index */
	SLATECORE_API bool IsValidIndex(int32 Index) const;

	/** Sets the brush for a given layer, if it exists */
	SLATECORE_API void SetLayerBrush(int32 Index, TAttribute<const FSlateBrush*> Brush);

	/** Sets the color for a given layer, if it exists. */
	SLATECORE_API void SetLayerColor(int32 Index, TAttribute<FSlateColor> Color);

private:
	typedef TSlateManagedAttribute<const FSlateBrush*, EInvalidateWidgetReason::Paint> BrushAttributeType;
	typedef TSlateManagedAttribute<FSlateColor, EInvalidateWidgetReason::Paint> ColorAttributeType;
	typedef TTuple<BrushAttributeType, ColorAttributeType> InnerImageLayerType;
		
	/** An array to hold the additional draw layers */
	TArray<InnerImageLayerType,TInlineAllocator<2>> Layers;
};
