// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"

/**
 * The different types of invalidation that are possible for a widget.
 */
enum class EInvalidateWidgetReason : uint8
{
	None = 0,

	/**
	 * Use Layout invalidation if your widget needs to change desired size.  This is an expensive invalidation so do not use if all you need to do is redraw a widget
	 */
	Layout = 1 << 0,

	/**
	 * Use when the painting of widget has been altered, but nothing affecting sizing.
	 */
	Paint = 1 << 1,

	/**
	 * Use if just the volatility of the widget has been adjusted.
	 */
	Volatility = 1 << 2,

	/**
	 * A child was added or removed.   (this implies prepass and layout)
	 */
	ChildOrder = 1 << 3,

	/**
	 * A Widgets render transform changed
	 */
	RenderTransform = 1 << 4,

	/**
	 * Changing visibility (this implies layout)
	 */
	Visibility = 1 << 5,

	/**
	 * Attributes got bound or unbound (it's used by the SlateAttributeMetaData)
	 */
	AttributeRegistration = 1 << 6,

	/**
	 * Re-cache desired size of all of this widget's children recursively (this implies layout)
	 */
	Prepass = 1 << 7,

	/**
	 * Use Paint invalidation if you're changing a normal property involving painting or sizing.
	 * Additionally if the property that was changed affects Volatility in anyway, it's important
	 * that you invalidate volatility so that it can be recalculated and cached.
	 */
	PaintAndVolatility = Paint | Volatility,
	/**
	 * Use Layout invalidation if you're changing a normal property involving painting or sizing.
	 * Additionally if the property that was changed affects Volatility in anyway, it's important
	 * that you invalidate volatility so that it can be recalculated and cached.
	 */
	LayoutAndVolatility = Layout | Volatility,
};

ENUM_CLASS_FLAGS(EInvalidateWidgetReason)

SLATECORE_API FString LexToString(EInvalidateWidgetReason Reason);
SLATECORE_API bool LexTryParseString(EInvalidateWidgetReason& OutMode, const TCHAR* InBuffer);
SLATECORE_API void LexFromString(EInvalidateWidgetReason& OutMode, const TCHAR* InBuffer);

// This typedefed because EInvalidateWidget will be deprecated soon
typedef EInvalidateWidgetReason EInvalidateWidget;
