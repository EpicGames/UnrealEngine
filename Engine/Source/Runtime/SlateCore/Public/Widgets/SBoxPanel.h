// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Types/SlateStructs.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SlotBase.h"
#include "Widgets/SWidget.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"
#include "Layout/ArrangedChildren.h"
#include "Input/DragAndDrop.h"
#include "Rendering/DrawElements.h"
#include "Styling/SlateBrush.h"

class FArrangedChildren;

template <>
struct TWidgetTypeTraits<class SBoxPanel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};


/**
 * A BoxPanel contains one child and describes how that child should be arranged on the screen.
 */
class SBoxPanel : public SPanel
{
	SLATE_DECLARE_WIDGET_API(SBoxPanel, SPanel, SLATECORE_API)

protected:
	/**
	 * A BoxPanel contains one BoxPanel child and describes how that
	 * child should be arranged on the screen.
	 */
	template<typename SlotType>
	class TSlot : public TBasicLayoutWidgetSlot<SlotType>, public TResizingWidgetSlotMixin<SlotType>
	{
	public:
		SLATE_SLOT_BEGIN_ARGS_OneMixin(TSlot, TBasicLayoutWidgetSlot<SlotType>, TResizingWidgetSlotMixin<SlotType>)
		SLATE_SLOT_END_ARGS()

	protected:
		/** Default values for a slot. */
		TSlot()
			: TBasicLayoutWidgetSlot<SlotType>(HAlign_Fill, VAlign_Fill)
			, TResizingWidgetSlotMixin<SlotType>()
		{
		}

	public:
		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			TBasicLayoutWidgetSlot<SlotType>::Construct(SlotOwner, MoveTemp(InArgs));
			TResizingWidgetSlotMixin<SlotType>::ConstructMixin(SlotOwner, MoveTemp(InArgs));
		}

		static void RegisterAttributes(FSlateWidgetSlotAttributeInitializer& AttributeInitializer)
		{
			TBasicLayoutWidgetSlot<SlotType>::RegisterAttributes(AttributeInitializer);
			TResizingWidgetSlotMixin<SlotType>::RegisterAttributesMixin(AttributeInitializer);
		}
	};

public:
	class FSlot : public TSlot<FSlot>
	{
	};

protected:
	template<typename SlotType>
	struct FScopedWidgetSlotArguments : public SlotType::FSlotArguments
	{
	public:
		FScopedWidgetSlotArguments(TUniquePtr<SlotType> InSlot, TPanelChildren<FSlot>& InChildren, int32 InIndex)
			: SlotType::FSlotArguments(MoveTemp(InSlot))
			, Children(InChildren)
			, Index(InIndex)
		{
		}

		FScopedWidgetSlotArguments() = delete;
		FScopedWidgetSlotArguments(const FScopedWidgetSlotArguments&) = delete;
		FScopedWidgetSlotArguments& operator=(const FScopedWidgetSlotArguments&) = delete;
		FScopedWidgetSlotArguments(FScopedWidgetSlotArguments&&) = default;
		FScopedWidgetSlotArguments& operator=(FScopedWidgetSlotArguments&&) = default;

		virtual ~FScopedWidgetSlotArguments()
		{
			if (this->GetSlot())	// Is nullptr when the FScopedWidgetSlotArguments is moved-constructed.
			{
				SBoxPanel::FSlot::FSlotArguments* SelfAsBaseSlot = static_cast<SBoxPanel::FSlot::FSlotArguments*>(static_cast<FSlotBase::FSlotArguments*>(this));
				if (Index == INDEX_NONE)
				{
					Children.AddSlot(MoveTemp(*SelfAsBaseSlot));
				}
				else
				{
					Children.InsertSlot(MoveTemp(*SelfAsBaseSlot), Index);
				}
			}
		}

	private:
		TPanelChildren<FSlot>& Children;
		int32 Index;
	};

public:
	/** Removes a slot from this box panel which contains the specified SWidget
	 *
	 * @param SlotWidget The widget to match when searching through the slots
	 * @returns The index in the children array where the slot was removed and -1 if no slot was found matching the widget
	 */
	SLATECORE_API int32 RemoveSlot( const TSharedRef<SWidget>& SlotWidget );

	/** Removes all children from the box. */
	SLATECORE_API void ClearChildren();

	/** @return the number of slots. */
	int32 NumSlots() const { return Children.Num(); }

	/** @return if it's a valid index slot index. */
	bool IsValidSlotIndex(int32 Index) const { return Children.IsValidIndex(Index); }

	//~ Begin SWidget overrides.
public:
	SLATECORE_API virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	SLATECORE_API virtual FChildren* GetChildren() override;

protected:
	SLATECORE_API virtual FVector2D ComputeDesiredSize(float) const override;
	//~ End SWidget overrides.

protected:
	SLATECORE_API SBoxPanel();
	/** A Box Panel's orientation. */
	SLATECORE_API SBoxPanel(EOrientation InOrientation);
	SLATECORE_API ~SBoxPanel();

	/** Set the orientation of the Box. It will do a full invalidation of the widget. */
	SLATECORE_API void SetOrientation(EOrientation InOrientation);
	EOrientation GetOrientation() const
	{
		return Orientation;
	}

protected:
	/** The Box Panel's children. */
	TPanelChildren<FSlot> Children;

	/** The Box Panel's orientation; determined at construct time. */
	EOrientation Orientation;
};


template <>
struct TWidgetTypeTraits<class SHorizontalBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** A Horizontal Box Panel. See SBoxPanel for more info. */
class SHorizontalBox : public SBoxPanel
{
	SLATE_DECLARE_WIDGET_API(SHorizontalBox, SBoxPanel, SLATECORE_API)
public:
	class FSlot : public SBoxPanel::TSlot<FSlot>
	{
	public:
		SLATE_SLOT_BEGIN_ARGS(FSlot, SBoxPanel::TSlot<FSlot>)
			/**
			 * The widget's DesiredSize will be used as the space required.
			 */
			FSlotArguments& AutoWidth()
			{
				_SizeParam = FAuto();
				return Me();
			}
		
			/**
			 * The available space will be distributed proportionately to each slots stretch coefficient.
			 * A slot with coefficient of 2 will get assigned twice as much available space as slot with coefficient 1. 
			 * @param InStretchCoefficient Stretch coefficient for this slot.  
			 */
			FSlotArguments& FillWidth(TAttribute<float> InStretchCoefficient)
			{
				_SizeParam = FStretch(MoveTemp(InStretchCoefficient));
				return Me();
			}
		
			/**
			 * The widget's content size is adjusted proportionally to fit the available space.
			 * The slots size starts at DesiredSize, and a slot with coefficient of 2 will get adjusted twice as much as slot with coefficient 1 to fit the available space.
			 * @param InStretchCoefficient Stretch coefficient for this slot.
			 * @param InShrinkStretchCoefficient If specified, this stretch coefficient is used when the slots is shrinking below desired size. Otherwise InStretchCoefficient is used for both shrink and grow.  
			 */
			FSlotArguments& FillContentWidth(TAttribute<float> InStretchCoefficient, TAttribute<float> InShrinkStretchCoefficient = TAttribute<float>())
			{
				_SizeParam = FStretchContent(MoveTemp(InStretchCoefficient), MoveTemp(InShrinkStretchCoefficient));
				return Me();
			}

			/** Set the min size in SlateUnit this slot can be. */
			FSlotArguments& MinWidth(TAttribute<float> InMinWidth)
			{
				_MinSize = MoveTemp(InMinWidth);
				return Me();
			}

			/** Set the max size in SlateUnit this slot can be. */
			FSlotArguments& MaxWidth(TAttribute<float> InMaxWidth)
			{
				_MaxSize = MoveTemp(InMaxWidth);
				return Me();
			}
		SLATE_SLOT_END_ARGS()

		/**
		 * The widget's DesiredSize will be used as the space required.
		 */
		void SetAutoWidth()
		{
			SetSizeToAuto();
		}

		/**
		 * The available space will be distributed proportionately to each slots stretch coefficient.
		 * A slot with coefficient of 2 will get assigned twice as much available space as slot with coefficient 1. 
		 * @param InStretchCoefficient Stretch coefficient for this slot.  
		 */
		void SetFillWidth(TAttribute<float> InStretchCoefficient)
		{
			SetSizeToStretch(MoveTemp(InStretchCoefficient));
		}

		/**
		 * The widget's content size is adjusted proportionally to fit the available space.
		 * The slots size starts at DesiredSize, and a slot with coefficient of 2 will get adjusted twice as much as slot with coefficient 1 to fit the available space.
		 * @param InStretchCoefficient Stretch coefficient for this slot.
		 * @param InShrinkStretchCoefficient If specified, this stretch coefficient is used when the slots is shrinking below desired size. Otherwise InStretchCoefficient is used for both shrink and grow.  
		 */
		void SetFillContentWidth(TAttribute<float> InStretchCoefficient, TAttribute<float> InShrinkStretchCoefficient = TAttribute<float>())
		{
			SetSizeToStretchContent(MoveTemp(InStretchCoefficient), MoveTemp(InShrinkStretchCoefficient));
		}

		/** Set the min size in SlateUnit this slot can be. */
		void SetMinWidth(TAttribute<float> InMinWidth)
		{
			SetMinSize(MoveTemp(InMinWidth));
		}

		/** Set the max size in SlateUnit this slot can be. */
		void SetMaxWidth(TAttribute<float> InMaxWidth)
		{
			SetMaxSize(MoveTemp(InMaxWidth));
		}

		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			SBoxPanel::TSlot<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
		}
	};

	static FSlot::FSlotArguments Slot()
	{
		return FSlot::FSlotArguments(MakeUnique<FSlot>());
	}

	SLATE_BEGIN_ARGS( SHorizontalBox )
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}
		SLATE_SLOT_ARGUMENT(SHorizontalBox::FSlot, Slots)
	SLATE_END_ARGS()

	using FScopedWidgetSlotArguments = SBoxPanel::FScopedWidgetSlotArguments<SHorizontalBox::FSlot>;
	FScopedWidgetSlotArguments AddSlot()
	{
		return InsertSlot(INDEX_NONE);
	}

	FScopedWidgetSlotArguments InsertSlot(int32 Index = INDEX_NONE)
	{
		return FScopedWidgetSlotArguments(MakeUnique<FSlot>(), this->Children, Index);
	}

	SLATECORE_API FSlot& GetSlot(int32 SlotIndex);
	SLATECORE_API const FSlot& GetSlot(int32 SlotIndex) const;

	FORCENOINLINE SHorizontalBox()
		: SBoxPanel( Orient_Horizontal )
	{
		SetCanTick(false);
		bCanSupportFocus = false;
	}

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATECORE_API void Construct( const FArguments& InArgs );
};


template <>
struct TWidgetTypeTraits<class SVerticalBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** A Vertical Box Panel. See SBoxPanel for more info. */
class SVerticalBox : public SBoxPanel
{
	SLATE_DECLARE_WIDGET_API(SVerticalBox, SBoxPanel, SLATECORE_API)
public:
	class FSlot : public SBoxPanel::TSlot<FSlot>
	{
	public:
		SLATE_SLOT_BEGIN_ARGS(FSlot, SBoxPanel::TSlot<FSlot>)
		
			/**
			 * The widget's DesiredSize will be used as the space required.
			 */
			FSlotArguments& AutoHeight()
			{
				_SizeParam = FAuto();
				return Me();
			}
		
			/**
			 * The available space will be distributed proportionately to each slots stretch coefficient.
			 * A slot with coefficient of 2 will get assigned twice as much available space as slot with coefficient 1. 
			 * @param InStretchCoefficient Stretch coefficient for this slot.  
			 */
			FSlotArguments& FillHeight(TAttribute<float> InStretchCoefficient)
			{
				_SizeParam = FStretch(MoveTemp(InStretchCoefficient));
				return Me();
			}
		
			/**
			 * The widget's content size is adjusted proportionally to fit the available space.
			 * The slots size starts at DesiredSize, and a slot with coefficient of 2 will get adjusted twice as much as slot with coefficient 1 to fit the available space.
			 * @param InStretchCoefficient Stretch coefficient for this slot.
			 * @param InShrinkStretchCoefficient If specified, this stretch coefficient is used when the slots is shrinking below desired size. Otherwise InStretchCoefficient is used for both shrink and grow.  
			 */
			FSlotArguments& FillContentHeight(TAttribute<float> InStretchCoefficient, TAttribute<float> InShrinkStretchCoefficient = TAttribute<float>())
			{
				_SizeParam = FStretchContent(MoveTemp(InStretchCoefficient), MoveTemp(InShrinkStretchCoefficient));
				return Me();
			}

			/** Set the min size in SlateUnit this slot can be. */
			FSlotArguments& MinHeight(TAttribute<float> InMinHeight)
			{
				_MinSize = MoveTemp(InMinHeight);
				return Me();
			}
			
			/** Set the max size in SlateUnit this slot can be. */
			FSlotArguments& MaxHeight(TAttribute<float> InMaxHeight)
			{
				_MaxSize = MoveTemp(InMaxHeight);
				return Me();
			}
		SLATE_SLOT_END_ARGS()

		/**
		 * The widget's DesiredSize will be used as the space required.
		 */
		void SetAutoHeight()
		{
			SetSizeToAuto();
		}

		/**
		 * The available space will be distributed proportionately to each slots stretch coefficient.
		 * A slot with coefficient of 2 will get assigned twice as much available space as slot with coefficient 1. 
		 * @param InStretchCoefficient Stretch coefficient for this slot.  
		 */
		void SetFillHeight(TAttribute<float> InStretchCoefficient)
		{
			SetSizeToStretch(MoveTemp(InStretchCoefficient));
		}

		/**
		 * The widget's content size is adjusted proportionally to fit the available space.
		 * The slots size starts at DesiredSize, and a slot with coefficient of 2 will get adjusted twice as much as slot with coefficient 1 to fit the available space.
		 * @param InStretchCoefficient Stretch coefficient for this slot.
		 * @param InShrinkStretchCoefficient If specified, this stretch coefficient is used when the slots is shrinking below desired size. Otherwise InStretchCoefficient is used for both shrink and grow.  
		 */
		void SetFillContentHeight(TAttribute<float> InStretchCoefficient, TAttribute<float> InShrinkStretchCoefficient = TAttribute<float>())
		{
			SetSizeToStretchContent(MoveTemp(InStretchCoefficient), MoveTemp(InShrinkStretchCoefficient));
		}

		/** Set the min size in SlateUnit this slot can be. */
		void SetMinHeight(TAttribute<float> InMinHeight)
		{
			SetMinSize(MoveTemp(InMinHeight));
		}

		/** Set the max size in SlateUnit this slot can be. */
		void SetMaxHeight(TAttribute<float> InMaxHeight)
		{
			SetMaxSize(MoveTemp(InMaxHeight));
		}

		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			SBoxPanel::TSlot<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
		}
	};

	static FSlot::FSlotArguments Slot()
	{
		return FSlot::FSlotArguments(MakeUnique<FSlot>());
	}


	SLATE_BEGIN_ARGS( SVerticalBox )
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}
		SLATE_SLOT_ARGUMENT(SVerticalBox::FSlot, Slots)
	SLATE_END_ARGS()

	using FScopedWidgetSlotArguments = SBoxPanel::FScopedWidgetSlotArguments<SVerticalBox::FSlot>;
	FScopedWidgetSlotArguments AddSlot()
	{
		return InsertSlot(INDEX_NONE);
	}

	FScopedWidgetSlotArguments InsertSlot(int32 Index = INDEX_NONE)
	{
		return FScopedWidgetSlotArguments(MakeUnique<FSlot>(), this->Children, Index);
	}

	SLATECORE_API FSlot& GetSlot(int32 SlotIndex);
	SLATECORE_API const FSlot& GetSlot(int32 SlotIndex) const;

	FORCENOINLINE SVerticalBox()
		: SBoxPanel( Orient_Vertical )
	{
		SetCanTick(false);
		bCanSupportFocus = false;
	}

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATECORE_API void Construct( const FArguments& InArgs );
};


template <>
struct TWidgetTypeTraits<class SStackBox>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** A Stack Box Panel that stack vertically or horizontally. See SBoxPanel for more info. */
class SStackBox : public SBoxPanel
{
	SLATE_DECLARE_WIDGET_API(SStackBox, SBoxPanel, SLATECORE_API)
public:
	class FSlot : public SBoxPanel::TSlot<FSlot>
	{
	public:
		SLATE_SLOT_BEGIN_ARGS(FSlot, SBoxPanel::TSlot<FSlot>)
			/**
			 * The widget's DesiredSize will be used as the space required.
			 */
			FSlotArguments& AutoSize()
			{
				_SizeParam = FAuto();
				return Me();
			}

			/**
			 * The available space will be distributed proportionately to each slots stretch coefficient.
			 * A slot with coefficient of 2 will get assigned twice as much available space as slot with coefficient 1. 
			 * @param InStretchCoefficient Stretch coefficient for this slot.  
			 */
			FSlotArguments& FillSize(TAttribute<float> InStretchCoefficient)
			{
				_SizeParam = FStretch(MoveTemp(InStretchCoefficient));
				return Me();
			}

			/**
			 * The widget's content size is adjusted proportionally to fit the available space.
			 * The slots size starts at DesiredSize, and a slot with coefficient of 2 will get adjusted twice as much as slot with coefficient 1 to fit the available space.
			 * @param InStretchCoefficient Stretch coefficient for this slot.
			 * @param InShrinkStretchCoefficient If specified, this stretch coefficient is used when the slots is shrinking below desired size. Otherwise InStretchCoefficient is used for both shrink and grow.  
			 */
			FSlotArguments& FillContentSize(TAttribute<float> InStretchCoefficient, TAttribute<float> InShrinkStretchCoefficient = TAttribute<float>())
			{
				_SizeParam = FStretchContent(MoveTemp(InStretchCoefficient), MoveTemp(InShrinkStretchCoefficient));
				return Me();
			}

			/** Set the min size in SlateUnit this slot can be. */
			FSlotArguments& MinSize(TAttribute<float> InMinHeight)
			{
				_MinSize = MoveTemp(InMinHeight);
				return Me();
			}

			/** Set the max size in SlateUnit this slot can be. */
			FSlotArguments& MaxSize(TAttribute<float> InMaxHeight)
			{
				_MaxSize = MoveTemp(InMaxHeight);
				return Me();
			}
		SLATE_SLOT_END_ARGS()

		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			SBoxPanel::TSlot<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
		}
	};

	static FSlot::FSlotArguments Slot()
	{
		return FSlot::FSlotArguments(MakeUnique<FSlot>());
	}


	SLATE_BEGIN_ARGS(SStackBox)
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}
		SLATE_ARGUMENT(EOrientation, Orientation)
		SLATE_SLOT_ARGUMENT(SStackBox::FSlot, Slots)
	SLATE_END_ARGS()

	using FScopedWidgetSlotArguments = SBoxPanel::FScopedWidgetSlotArguments<SStackBox::FSlot>;
	FScopedWidgetSlotArguments AddSlot()
	{
		return InsertSlot(INDEX_NONE);
	}

	FScopedWidgetSlotArguments InsertSlot(int32 Index = INDEX_NONE)
	{
		return FScopedWidgetSlotArguments(MakeUnique<FSlot>(), this->Children, Index);
	}

	SLATECORE_API FSlot& GetSlot(int32 SlotIndex);
	SLATECORE_API const FSlot& GetSlot(int32 SlotIndex) const;

	FORCENOINLINE SStackBox()
	{
		SetCanTick(false);
		bCanSupportFocus = false;
	}

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATECORE_API void Construct(const FArguments& InArgs);

	using SBoxPanel::SetOrientation;
	using SBoxPanel::GetOrientation;
};

class FDragAndDropVerticalBoxOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDragAndDropVerticalBoxOp, FDragDropOperation)

	/** Data this item represent */
	int32 SlotIndexBeingDragged;
	SVerticalBox::FSlot* SlotBeingDragged;

	virtual ~FDragAndDropVerticalBoxOp() = default;
};

/** A Vertical Box Panel. See SBoxPanel for more info. */
class SDragAndDropVerticalBox : public SVerticalBox
{
public:
	/**
	* Where we are going to drop relative to the target item.
	*/
	enum class EItemDropZone
	{
		AboveItem,
		BelowItem
	};

	DECLARE_DELEGATE_RetVal_FourParams(FReply, FOnDragAndDropVerticalBoxDragDetected, const FGeometry&, const FPointerEvent&, int32, SVerticalBox::FSlot*)
	DECLARE_DELEGATE_OneParam(FOnDragAndDropVerticalBoxDragEnter, FDragDropEvent const&);
	DECLARE_DELEGATE_OneParam(FOnDragAndDropVerticalBoxDragLeave, FDragDropEvent const&);
	DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnDragAndDropVerticalBoxDrop, FDragDropEvent const&);

	UE_DEPRECATED(5.6, "FOnCanAcceptDrop is deprecated. Please use FOnCanAcceptDropAdvanced which adds the drop slot index parameter.")
	DECLARE_DELEGATE_RetVal_ThreeParams(TOptional<EItemDropZone>, FOnCanAcceptDrop, const FDragDropEvent&, SDragAndDropVerticalBox::EItemDropZone, SVerticalBox::FSlot*);
	/** Delegate signature for querying whether this FDragDropEvent will be handled by the drop target of type ItemType. */
	DECLARE_DELEGATE_RetVal_FourParams(TOptional<EItemDropZone>, FOnCanAcceptDropAdvanced, const FDragDropEvent&, SDragAndDropVerticalBox::EItemDropZone, int32, SVerticalBox::FSlot*);
	/** Delegate signature for handling the drop of FDragDropEvent onto target of type ItemType */
	DECLARE_DELEGATE_RetVal_FourParams(FReply, FOnAcceptDrop, const FDragDropEvent&, SDragAndDropVerticalBox::EItemDropZone, int32, SVerticalBox::FSlot*);

	SLATE_BEGIN_ARGS(SDragAndDropVerticalBox)
		{}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FOnCanAcceptDropAdvanced ConvertCanAcceptDropFn(const FOnCanAcceptDrop& LegacyDelegate)
		{
			return FOnCanAcceptDropAdvanced::CreateLambda([LegacyDelegate](const FDragDropEvent& Event, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot)
				{
					if (LegacyDelegate.IsBound())
					{
						return LegacyDelegate.Execute(Event, DropZone, Slot);
					}

					return TOptional<EItemDropZone>();
				});
		}

		SLATE_EVENT_DEPRECATED(5.6, "Use FOnCanAcceptDropAdvanced instead", FOnCanAcceptDrop, OnCanAcceptDrop, OnCanAcceptDropAdvanced, ConvertCanAcceptDropFn)
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		/**
		* Handle this event to determine whether a drag and drop operation can be executed on top of the target row widget.
		* Most commonly, this is used for previewing re-ordering and re-organization operations in lists or trees.
		* e.g. A user is dragging one item into a different spot in the list or tree.
		*      This delegate will be called to figure out if we should give visual feedback on whether an item will
		*      successfully drop into the list.
		*/
		SLATE_EVENT(FOnCanAcceptDropAdvanced, OnCanAcceptDropAdvanced)

		/**
		* Perform a drop operation onto the target row widget
		* Most commonly used for executing a re-ordering and re-organization operations in lists or trees.
		* e.g. A user was dragging one item into a different spot in the list; they just dropped it.
		*      This is our chance to handle the drop by reordering items and calling for a list refresh.
		*/
		SLATE_EVENT(FOnAcceptDrop, OnAcceptDrop)

		// Low level DragAndDrop
		SLATE_EVENT(FOnDragAndDropVerticalBoxDragDetected, OnDragDetected)
		SLATE_EVENT(FOnDragAndDropVerticalBoxDragEnter, OnDragEnter)
		SLATE_EVENT(FOnDragAndDropVerticalBoxDragLeave, OnDragLeave)
		SLATE_EVENT(FOnDragAndDropVerticalBoxDrop, OnDrop)
	SLATE_END_ARGS()

	SLATECORE_API void Construct(const FArguments& InArgs);

	/** Set the Drop indicators */
	SDragAndDropVerticalBox& SetDropIndicator_Above(const FSlateBrush& InValue) { DropIndicator_Above = InValue; return *this; }
	SDragAndDropVerticalBox& SetDropIndicator_Below(const FSlateBrush& InValue) { DropIndicator_Below = InValue; return *this; }

	/** Drag detection and handling */
	SLATECORE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
		
	SLATECORE_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	SLATECORE_API virtual void OnDragEnter(FGeometry const& MyGeometry, FDragDropEvent const& DragDropEvent) override;
	SLATECORE_API virtual void OnDragLeave(FDragDropEvent const& DragDropEvent) override;
	SLATECORE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	SLATECORE_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	SLATECORE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:
	/** @see SDragAndDropVerticalBox's OnCanAcceptDropAdvanced event */
	FOnCanAcceptDropAdvanced OnCanAcceptDropAdvanced;

	/** @see SDragAndDropVerticalBox's OnAcceptDrop event */
	FOnAcceptDrop OnAcceptDrop;

	/** Are we currently dragging/dropping over this item? */
	TOptional<EItemDropZone> ItemDropZone;

	/** Delegate triggered when a user starts to drag a slot item */
	FOnDragAndDropVerticalBoxDragDetected OnDragDetected_Handler;

	/** Delegate triggered when a user's drag enters the bounds of this slot item */
	FOnDragAndDropVerticalBoxDragEnter OnDragEnter_Handler;

	/** Delegate triggered when a user's drag leaves the bounds of this slot item */
	FOnDragAndDropVerticalBoxDragLeave OnDragLeave_Handler;

	/** Delegate triggered when a user's drag is dropped in the bounds of this slot item */
	FOnDragAndDropVerticalBoxDrop OnDrop_Handler;

	/** Brush used to provide feedback that a user can drop below the hovered row. */
	FSlateBrush DropIndicator_Above;
	FSlateBrush DropIndicator_Below;

	/** This is required for the paint to access which item we're hovering */
	FVector2f CurrentDragOperationScreenSpaceLocation;
	int32 CurrentDragOverSlotIndex;

	/** @return the zone (above, below) based on where the user is hovering over */
	SLATECORE_API EItemDropZone ZoneFromPointerPosition(FVector2f LocalPointerPos, const FGeometry& CurrentGeometry, const FGeometry& StartGeometry) const;
};
