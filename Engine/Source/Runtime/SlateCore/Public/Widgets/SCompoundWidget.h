// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Types/SlateAttribute.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

/**
 * A CompoundWidget is the base from which most non-primitive widgets should be built.
 * CompoundWidgets have a protected member named ChildSlot.
 */
class SCompoundWidget : public SWidget
{
	SLATE_DECLARE_WIDGET_API(SCompoundWidget, SWidget, SLATECORE_API)

public:

	/**
	 * Returns the size scaling factor for this widget.
	 *
	 * @return Size scaling factor.
	 */
	FVector2D GetContentScale() const
	{
		return ContentScaleAttribute.Get();
	}

	/**
	 * Sets the content scale for this widget.
	 *
	 * @param InContentScale Content scaling factor.
	 */
	void SetContentScale( TAttribute<FVector2D> InContentScale )
	{
		ContentScaleAttribute.Assign(*this, MoveTemp(InContentScale));
	}

	/**
	 * Gets the widget's color.
	 */
	FLinearColor GetColorAndOpacity() const
	{
		return ColorAndOpacityAttribute.Get();
	}

	/**
	 * Sets the widget's color.
	 *
	 * @param InColorAndOpacity The ColorAndOpacity to be applied to this widget and all its contents.
	 */
	void SetColorAndOpacity( TAttribute<FLinearColor> InColorAndOpacity )
	{
		ColorAndOpacityAttribute.Assign(*this, MoveTemp(InColorAndOpacity));
	}

	/**
	 * Sets the widget's foreground color.
	 *
	 * @param InForegroundColor The color to set.
	 */
	void SetForegroundColor( TAttribute<FSlateColor> InForegroundColor )
	{
		ForegroundColorAttribute.Assign(*this, MoveTemp(InForegroundColor));
	}

public:

	// SWidgetOverrides
	SLATECORE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	SLATECORE_API virtual FChildren* GetChildren() override;
	SLATECORE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATECORE_API virtual FSlateColor GetForegroundColor() const override;

public:
	SLATECORE_API virtual void SetVisibility( TAttribute<EVisibility> InVisibility ) override final;

protected:
	// Begin SWidget overrides.
	SLATECORE_API virtual FVector2D ComputeDesiredSize(float) const override;
	// End SWidget overrides.

	/** @return an attribute reference of ContentScale */
	TSlateAttributeRef<FVector2D> GetContentScaleAttribute() const { return TSlateAttributeRef<FVector2D>(SharedThis(this), ContentScaleAttribute); }

	/** @return an attribute reference of ColorAndOpacity */
	TSlateAttributeRef<FLinearColor> GetColorAndOpacityAttribute() const { return TSlateAttributeRef<FLinearColor>{SharedThis(this), ColorAndOpacityAttribute}; }

	/** @return an attribute reference of ForegroundColor */
	TSlateAttributeRef<FSlateColor> GetForegroundColorAttribute() const { return TSlateAttributeRef<FSlateColor>{SharedThis(this), ForegroundColorAttribute}; }

protected:

	/** Disallow public construction */
	SLATECORE_API SCompoundWidget();
	SLATECORE_API virtual ~SCompoundWidget();

	struct FCompoundWidgetOneChildSlot : ::TSingleWidgetChildrenWithBasicLayoutSlot<EInvalidateWidgetReason::None>
	{
		friend SCompoundWidget;
		using ::TSingleWidgetChildrenWithBasicLayoutSlot<EInvalidateWidgetReason::None>::TSingleWidgetChildrenWithBasicLayoutSlot;
	};

	/** The slot that contains this widget's descendants.*/
	FCompoundWidgetOneChildSlot ChildSlot;

private:
	/** The layout scale to apply to this widget's contents; useful for animation. */
	TSlateAttribute<FVector2D> ContentScaleAttribute;

	/** The color and opacity to apply to this widget and all its descendants. */
	TSlateAttribute<FLinearColor> ColorAndOpacityAttribute;

	/** Optional foreground color that will be inherited by all of this widget's contents */
	TSlateAttribute<FSlateColor> ForegroundColorAttribute;
};
