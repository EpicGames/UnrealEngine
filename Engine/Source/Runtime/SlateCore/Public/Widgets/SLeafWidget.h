// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;

template <>
struct TWidgetTypeTraits<class SLeafWidget>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Implements a leaf widget.
 *
 * A LeafWidget is a Widget that has no slots for children.
 * LeafWidgets are usually intended as building blocks for aggregate widgets.
 */
class SLeafWidget
	: public SWidget
{
public:
	SLATECORE_API SLeafWidget();

	SLATECORE_API virtual ~SLeafWidget();

	SLATECORE_API virtual void SetVisibility( TAttribute<EVisibility> InVisibility ) override final;

private:

	// Begin SWidget overrides

	/**
	 * Overwritten from SWidget.
	 *
	 * LeafWidgets provide a visual representation of themselves. They do so by adding DrawElement(s)
	 * to the OutDrawElements. DrawElements should have their positions set to absolute coordinates in
	 * Window space; for this purpose the Slate system provides the AllottedGeometry parameter.
	 * AllottedGeometry describes the space allocated for the visualization of this widget.
	 *
	 * Whenever possible, LeafWidgets should avoid dealing with layout properties. See TextBlock for an example.
	 */
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override = 0;
	
	/**
	 * Overwritten from SWidget.
	 *
	 * LeafWidgets should compute their DesiredSize based solely on their visual representation. There is no need to
	 * take child widgets into account as LeafWidgets have none by definition. For example, the TextBlock widget simply
	 * measures the area necessary to display its text with the given font and font size.
	 */
	virtual FVector2D ComputeDesiredSize(float) const override = 0;
	
	/**
	 * Overwritten from SWidget.
	 *
	 * Leaf widgets never have children.
	 */
	SLATECORE_API virtual FChildren* GetChildren() override;

	SLATECORE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
};
