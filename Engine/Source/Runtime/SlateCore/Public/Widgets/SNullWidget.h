// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Implements a widget that can be used as a placeholder.
 *
 * Widgets that support slots, such as SOverlay and SHorizontalBox should initialize
 * their slots' child widgets to SNullWidget if no user defined widget was provided.
 */
class SNullWidget
{
public:

	/**
	 * Returns a placeholder widget.
	 *
	 * @return The widget.
	 */
	static SLATECORE_API TSharedRef<class SWidget> NullWidget;
};
