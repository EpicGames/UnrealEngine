// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SlotBase.h"
#include "Widgets/SWidget.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"

#define UE_API SLATECORE_API

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;


template <>
struct TWidgetTypeTraits<class SOverlay>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * Implements an overlay widget.
 *
 * Overlay widgets allow for layering several widgets on top of each other.
 * Each slot of an overlay represents a layer that can contain one widget.
 * The slots will be rendered on top of each other in the order they are declared in code.
 *
 * Usage:
 *		SNew(SOverlay)
 *		+ SOverlay::Slot(SNew(SMyWidget1))
 *		+ SOverlay::Slot(SNew(SMyWidget2))
 *		+ SOverlay::Slot(SNew(SMyWidget3))
 *
 *		Note that SWidget3 will be drawn on top of SWidget2 and SWidget1.
 */
class SOverlay : public SPanel
{
	SLATE_DECLARE_WIDGET_API(SOverlay, SPanel, SLATECORE_API)
public:	

	/** A slot that support alignment of content and padding and z-order */
	class FOverlaySlot : public TBasicLayoutWidgetSlot<FOverlaySlot>
	{
	public:
		FOverlaySlot(int32 InZOrder)
			: TBasicLayoutWidgetSlot<FOverlaySlot>(HAlign_Fill, VAlign_Fill)
			, ZOrder(InZOrder)
		{ }

		SLATE_SLOT_BEGIN_ARGS(FOverlaySlot, TBasicLayoutWidgetSlot<FOverlaySlot>)
			SLATE_ARGUMENT(TOptional<int32>, ZOrder)
		SLATE_SLOT_END_ARGS()

		UE_API void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs);

		int32 GetZOrder() const
		{
			return ZOrder;
		}

		UE_API void SetZOrder(int32 InOrder);

	private:
		/**
		 * Slots with larger ZOrder values will draw above slots with smaller ZOrder values. Slots
		 * with the same ZOrder will simply draw in the order they were added.
		 */
		int32 ZOrder;
	};

	SLATE_BEGIN_ARGS( SOverlay )
	{
		_Visibility = EVisibility::SelfHitTestInvisible;
	}
		SLATE_SLOT_ARGUMENT( SOverlay::FOverlaySlot, Slots )
	SLATE_END_ARGS()

	SLATECORE_API SOverlay();
	SLATECORE_API ~SOverlay();

	/**
	 * Construct this widget.
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	SLATECORE_API void Construct( const FArguments& InArgs );

	/** Returns the number of child widgets */
	SLATECORE_API int32 GetNumWidgets() const;

	/**
	 * Removes a widget from this overlay.
	 *
	 * @param	Widget	The widget content to remove
	 */
	SLATECORE_API bool RemoveSlot( TSharedRef< SWidget > Widget );

	using FScopedWidgetSlotArguments = TPanelChildren<FOverlaySlot>::FScopedWidgetSlotArguments;
	/** Adds a slot at the specified location (ignores Z-order) */
	SLATECORE_API FScopedWidgetSlotArguments AddSlot(int32 ZOrder=INDEX_NONE);

	/** Returns true if there is a child slot with the specified z-order */
	SLATECORE_API bool HasSlotWithZOrder(int32 ZOrder) const;

	/** Removes a slot at the specified location */
	SLATECORE_API void RemoveSlot(int32 ZOrder=INDEX_NONE);

	/** Removes all children from the overlay */
	SLATECORE_API void ClearChildren();

	/**
	 * @param ZOrder Default z-order of the slot
	 * @return a new slot. Slots contain children for SOverlay
	 */
	static SLATECORE_API FOverlaySlot::FSlotArguments Slot(int32 ZOrder = 0);

	//~ Begin of SWidget interface
	SLATECORE_API virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	SLATECORE_API virtual FChildren* GetChildren() override;
	SLATECORE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	//~ End of SWidget interface

protected:
	//~ Begin SWidget overrides.
	SLATECORE_API virtual FVector2D ComputeDesiredSize(float) const override;
	//~ End SWidget overrides.

	/** Returns the index of the child slot at the specified z-order */
	SLATECORE_API int32 GetChildIndexByZOrder(int32 ZOrder) const;

protected:
	/** The SOverlay's slots; each slot contains a child widget. */
	TPanelChildren<FOverlaySlot> Children;
};

#undef UE_API
