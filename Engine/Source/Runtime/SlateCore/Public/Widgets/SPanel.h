// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"

class FArrangedChildren;
class FChildren;
class FPaintArgs;
class FSlateWindowElementList;

template <>
struct TWidgetTypeTraits<class SPanel>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A Panel arranges its child widgets on the screen.
 *
 * Each child widget should be stored in a Slot. The Slot describes how the individual child should be arranged with
 * respect to its parent (i.e. the Panel) and its peers Widgets (i.e. the Panel's other children.)
 * For a simple example see StackPanel.
 */
class SPanel
	: public SWidget
{
public:	

	/**
	 * Panels arrange their children in a space described by the AllottedGeometry parameter. The results of the arrangement
	 * should be returned by appending a FArrangedWidget pair for every child widget. See StackPanel for an example
	 *
	 * @param AllottedGeometry    The geometry allotted for this widget by its parent.
	 * @param ArrangedChildren    The array to which to add the WidgetGeometries that represent the arranged children.
	 */
	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override = 0;

	/**
	 * A Panel's desired size in the space required to arrange of its children on the screen while respecting all of
	 * the children's desired sizes and any layout-related options specified by the user. See StackPanel for an example.
	 *
	 * @return The desired size.
	 */
	virtual FVector2D ComputeDesiredSize(float) const override = 0;

	/**
	 * All widgets must provide a way to access their children in a layout-agnostic way.
	 * Panels store their children in Slots, which creates a dilemma. Most panels
	 * can store their children in a TPanelChildren<Slot>, where the Slot class
	 * provides layout information about the child it stores. In that case
	 * GetChildren should simply return the TPanelChildren<Slot>. See StackPanel for an example.
	 */
	virtual FChildren* GetChildren() override = 0;

public:

	/**
	 * Most panels do not create widgets as part of their implementation, so
	 * they do not need to implement a Construct()
	 */
	void Construct() { }

public:

	// SWidget overrides

	SLATECORE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

protected:

	/**
	 * Just like OnPaint, but takes already arranged children. Can be handy for writing custom SPanels.
	 */
	SLATECORE_API int32 PaintArrangedChildren( const FPaintArgs& Args, const FArrangedChildren& ArrangedChildren, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled  ) const;
	
protected:

	/** Hidden default constructor. */
	SLATECORE_API SPanel( );
	SLATECORE_API virtual ~SPanel( );

public:
	SLATECORE_API virtual void SetVisibility( TAttribute<EVisibility> InVisibility ) override final;
};
