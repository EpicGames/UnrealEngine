// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

/**
 * Use SUserWidget as a base class to build aggregate widgets that are not meant
 * to serve as low-level building blocks. Examples include: a main menu, a user card,
 * an info dialog for a selected object, a splash screen.
 *
 * See SUserWidgetExample
 *
 * SMyWidget.h
 * -----------
 * class SMyWidget : public SUserWidget
 * { 
 *   public:
 *   SLATE_USER_ARGS( SMyWidget )
 *	 {}
 *   SLATE_END_ARGS()
 *
 *   // MUST Provide this function for SNew to call!
 *   virtual void Construct( const FArguments& InArgs ) = 0;
 *
 *   virtual void DoSomething() = 0;
 * };
 *
 * SMyWidget.cpp
 * -------------
 * namespace Implementation 
 * {
 *   class SMyWidget : public ::SMyWidget
 *   {
 *     public:
 *     virtual void Construct( const FArguments& InArgs ) override
 *     {
 *        SUserWidget::Construct( SUserWidget::FArguments()
 *        [
 *           SNew(STextBlock)
 *           .Text( NSLOCTEXT("x", "x", "My Widget's Content") )
 *        ]
 *     }
 *     
 *     private:
 *     // Private implementation details can occur here
 *     // without ever leaking out into the .h file!
 *   }
 * }
 *
 * TSharedRef<SMyWidget> SMyWidget::New()
 * {
 *   return MakeShareable( new SMyWidget() );
 * }
 */
class SUserWidget : public SCompoundWidget
{
	public:
	struct FArguments : public TSlateBaseNamedArgs<SUserWidget>
	{
		typedef FArguments WidgetArgsType;
		FORCENOINLINE FArguments()
			: _Content()
			, _HAlign(HAlign_Fill)
			, _VAlign(VAlign_Fill)
		{}

		SLATE_DEFAULT_SLOT( FArguments, Content )
		SLATE_ARGUMENT( EHorizontalAlignment, HAlign )
		SLATE_ARGUMENT( EVerticalAlignment, VAlign )

	};
	
	protected:
	void Construct( const FArguments& InArgs )
	{
		this->ChildSlot
		.HAlign( InArgs._HAlign )
		.VAlign( InArgs._VAlign )
		[
			InArgs._Content.Widget
		];
	}

	protected:
	/** User widgets can be allocated explicitly in the C++  */
	void* operator new ( const size_t InSize )
	{
		return FMemory::Malloc(InSize);
	}

	public:
	// This exists to silence V1062 and provide implementation balance with the new above
	void operator delete (void* InMem)
	{
		FMemory::Free(InMem);
	}
};
