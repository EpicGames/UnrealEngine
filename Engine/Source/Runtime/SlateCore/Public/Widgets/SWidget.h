// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Misc/FrameValue.h"
#include "Stats/Stats.h"
#include "Styling/SlateColor.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "Layout/Clipping.h"
#include "Layout/Geometry.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/LayoutGeometry.h"
#include "Layout/Margin.h"
#include "Layout/FlowDirection.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Input/NavigationReply.h"
#include "Input/PopupMethodReply.h"
#include "Types/ISlateMetaData.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "Types/WidgetMouseEventsDelegate.h"
#include "Textures/SlateShaderResource.h"
#include "SlateGlobals.h"
#include "Types/PaintArgs.h"
#include "Types/SlateAttribute.h"
#include "Types/SlateVector2.h"
#include "FastUpdate/WidgetProxy.h"
#include "InvalidateWidgetReason.h"
#include "Widgets/SlateControlledConstruction.h"
#include "Widgets/Accessibility/SlateWidgetAccessibleTypes.h"
#include "WidgetPixelSnapping.h"
#include "Types/SlateAttributeMetaData.h"

class FActiveTimerHandle;
class FArrangedChildren;
class FChildren;
class FPaintArgs;
class FSlateWindowElementList;
class FSlotBase;
class FWeakWidgetPath;
class FWidgetPath;
class IToolTip;
class SWidget;
struct FSlateBaseNamedArgs;
struct FSlateBrush;
struct FSlatePaintElementLists;

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Widgets Created (Per Frame)"), STAT_SlateTotalWidgetsPerFrame, STATGROUP_Slate, SLATECORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("SWidget::Paint (Count)"), STAT_SlateNumPaintedWidgets, STATGROUP_Slate, SLATECORE_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("SWidget::Tick (Count)"), STAT_SlateNumTickedWidgets, STATGROUP_Slate, SLATECORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Execute Active Timers"), STAT_SlateExecuteActiveTimers, STATGROUP_Slate, SLATECORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Tick Widgets"), STAT_SlateTickWidgets, STATGROUP_Slate, SLATECORE_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("SlatePrepass"), STAT_SlatePrepass, STATGROUP_Slate, SLATECORE_API);

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
DECLARE_CYCLE_STAT_EXTERN(TEXT("SWidget MetaData"), STAT_SlateGetMetaData, STATGROUP_Slate, SLATECORE_API);
#endif

DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("Total Widgets"), STAT_SlateTotalWidgets, STATGROUP_SlateMemory, SLATECORE_API);
DECLARE_MEMORY_STAT_EXTERN(TEXT("SWidget Total Allocated Size"), STAT_SlateSWidgetAllocSize, STATGROUP_SlateMemory, SLATECORE_API);


/** Type Traits to cover the generic information of a Widget type **/
template <typename WidgetType>
struct TWidgetTypeTraits
{
	/** Returns true if a widget supports invalidation. **/
	static constexpr bool SupportsInvalidation() { return false; }
};


enum class EAccessibleType : uint8
{
	Main,
	Summary
};


/**
 * An FPopupLayer hosts the pop-up content which could be anything you want to appear on top of a widget.
 * The widget must understand how to host pop-ups to make use of this.
 */
class FPopupLayer : public TSharedFromThis<FPopupLayer>
{
public:
	FPopupLayer(const TSharedRef<SWidget>& InitHostWidget, const TSharedRef<SWidget>& InitPopupContent)
		: HostWidget(InitHostWidget)
		, PopupContent(InitPopupContent)
	{
	}

	virtual ~FPopupLayer() = default;
	
	virtual TSharedRef<SWidget> GetHost() { return HostWidget; }
	virtual TSharedRef<SWidget> GetContent() { return PopupContent; }
	virtual FSlateRect GetAbsoluteClientRect() = 0;

	virtual void Remove() = 0;

private:
	TSharedRef<SWidget> HostWidget;
	TSharedRef<SWidget> PopupContent;
};


/**
 * Performs the attribute assignment and invalidates the widget minimally based on what actually changed.  So if the boundness of the attribute didn't change
 * volatility won't need to be recalculated.  Returns true if the value changed.
 */
template<typename TargetValueType, typename SourceValueType>
static bool SetWidgetAttribute(SWidget& ThisWidget, TAttribute<TargetValueType>& TargetValue, const TAttribute<SourceValueType>& SourceValue, EInvalidateWidgetReason BaseInvalidationReason);

class IToolTip;

/**
 * HOW TO DEPRECATE SLATE_ATTRIBUTES
 * 
 * SLATE_ATTRIBUTE(ECheckBoxState, IsChecked)
 *
 * UE_DEPRECATED(4.xx, "Please use IsChecked(TAttribute<ECheckBoxState>)")
 * FArguments& IsChecked(bool InIsChecked)
 * {
 * 		_IsChecked = InIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
 * 		return Me();
 * }
 *
 * // This version would prevent ambiguous conversions.
 * FArguments& IsChecked(ECheckBoxState InIsChecked)
 * {
 *		_IsChecked = InIsChecked;
 * 		return Me();
 * }
 */


/**
 * Abstract base class for Slate widgets.
 *
 * STOP. DO NOT INHERIT DIRECTLY FROM WIDGET!
 *
 * Inheritance:
 *   Widget is not meant to be directly inherited. Instead consider inheriting from LeafWidget or Panel,
 *   which represent intended use cases and provide a succinct set of methods which to override.
 *
 *   SWidget is the base class for all interactive Slate entities. SWidget's public interface describes
 *   everything that a Widget can do and is fairly complex as a result.
 * 
 * Events:
 *   Events in Slate are implemented as virtual functions that the Slate system will call
 *   on a Widget in order to notify the Widget about an important occurrence (e.g. a key press)
 *   or querying the Widget regarding some information (e.g. what mouse cursor should be displayed).
 *
 *   Widget provides a default implementation for most events; the default implementation does nothing
 *   and does not handle the event.
 *
 *   Some events are able to reply to the system by returning an FReply, FCursorReply, or similar
 *   object. 
 */
class SWidget
	: public FSlateControlledConstruction,
	public TSharedFromThis<SWidget>		// Enables 'this->AsShared()'
{
	SLATE_DECLARE_WIDGET_API(SWidget, FSlateControlledConstruction, SLATECORE_API)

	friend class FWidgetProxy;
	friend class FSlateAttributeMetaData;
	friend class FSlateInvalidationRoot;
	friend class FSlateInvalidationWidgetList;
	friend class FSlateWindowElementList;
	friend class SWindow;
	friend class FSlateTrace;
	friend struct FSlateCachedElementList;
	template<class WidgetType, typename RequiredArgsPayloadType>
	friend struct TSlateDecl;

protected:
	/**
	 * A SlateAttribute that is member variable of a SWidget.
	 * @usage: TSlateAttribute<int32> MyAttribute1; TSlateAttribute<int32, EInvalidateWidgetReason::Paint> MyAttribute2; TSlateAttribute<int32, EInvalidateWidgetReason::Paint, TSlateAttributeComparePredicate<>> MyAttribute3;
	 */
	template<typename InObjectType, EInvalidateWidgetReason InInvalidationReasonValue = EInvalidateWidgetReason::None, typename InComparePredicate = TSlateAttributeComparePredicate<>>
	struct TSlateAttribute : public ::SlateAttributePrivate::TSlateMemberAttribute<
		InObjectType,
		typename std::conditional<InInvalidationReasonValue == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InInvalidationReasonValue>>::type,
		InComparePredicate>
	{
		using ::SlateAttributePrivate::TSlateMemberAttribute<
			InObjectType,
			typename std::conditional<InInvalidationReasonValue == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InInvalidationReasonValue>>::type,
			InComparePredicate>::TSlateMemberAttribute;
	};

	//~ Override for FText that use the TSlateAttributeFTextComparePredicate to compare FText
	template<>
	struct TSlateAttribute<FText, EInvalidateWidgetReason::None> : public ::SlateAttributePrivate::TSlateMemberAttribute<FText, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeFTextComparePredicate>
	{
		using ::SlateAttributePrivate::TSlateMemberAttribute<FText, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeFTextComparePredicate>::TSlateMemberAttribute;
	};

	//~ Override for FText that use the TSlateAttributeFTextComparePredicate to compare FText
	template<EInvalidateWidgetReason InInvalidationReasonValue>
	struct TSlateAttribute<FText, InInvalidationReasonValue> : public ::SlateAttributePrivate::TSlateMemberAttribute<
		FText,
		typename std::conditional<InInvalidationReasonValue == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InInvalidationReasonValue>>::type,
		TSlateAttributeFTextComparePredicate>
	{
		using ::SlateAttributePrivate::TSlateMemberAttribute<
			FText,
			typename std::conditional<InInvalidationReasonValue == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InInvalidationReasonValue>>::type,
			TSlateAttributeFTextComparePredicate>::TSlateMemberAttribute;
	};

	/**
	 * A SlateAttribute that is NOT a member variable of a SWidget.
	 * @usage: TSlateManagedAttribute<int32> MyAttribute1; TSlateManagedAttribute<int32, EInvalidateWidgetReason::Paint> MyAttribute2; TSlateManagedAttribute<int32, EInvalidateWidgetReason::Paint, TSlateAttributeComparePredicate<>> MyAttribute3;
	 */
	template<typename InObjectType, EInvalidateWidgetReason InInvalidationReasonValue = EInvalidateWidgetReason::None, typename InComparePredicate = TSlateAttributeComparePredicate<>>
	struct TSlateManagedAttribute : public ::SlateAttributePrivate::TSlateManagedAttribute<
		InObjectType,
		typename std::conditional<InInvalidationReasonValue == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InInvalidationReasonValue>>::type,
		InComparePredicate>
	{
		using ::SlateAttributePrivate::TSlateManagedAttribute<
			InObjectType,
			typename std::conditional<InInvalidationReasonValue == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InInvalidationReasonValue>>::type,
			InComparePredicate>::TSlateManagedAttribute;
	};

	//~ Override for FText that use the TSlateAttributeFTextComparePredicate to compare FText
	template<>
	struct TSlateManagedAttribute<FText, EInvalidateWidgetReason::None> : public ::SlateAttributePrivate::TSlateManagedAttribute<FText, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeFTextComparePredicate>
	{
		using ::SlateAttributePrivate::TSlateManagedAttribute<FText, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeFTextComparePredicate>::TSlateManagedAttribute;
	};

	//~ Override for FText that use the TSlateAttributeFTextComparePredicate to compare FText
	template<EInvalidateWidgetReason InInvalidationReasonValue>
	struct TSlateManagedAttribute<FText, InInvalidationReasonValue> : public ::SlateAttributePrivate::TSlateManagedAttribute<
		FText,
		typename std::conditional<InInvalidationReasonValue == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InInvalidationReasonValue>>::type,
		TSlateAttributeFTextComparePredicate>
	{
		using ::SlateAttributePrivate::TSlateManagedAttribute<
			FText,
			typename std::conditional<InInvalidationReasonValue == EInvalidateWidgetReason::None, ::SlateAttributePrivate::FSlateAttributeNoInvalidationReason, TSlateAttributeInvalidationReason<InInvalidationReasonValue>>::type,
			TSlateAttributeFTextComparePredicate>::TSlateManagedAttribute;
	};

	/** A Reference to a TSlateAttribute. */
	template<typename InObjectType, EInvalidateWidgetReason InInvalidationReasonValue = EInvalidateWidgetReason::None, typename InComparePredicate = TSlateAttributeComparePredicate<>>
	struct TSlateAttributeRef : public ::SlateAttributePrivate::TSlateMemberAttributeRef<TSlateAttribute<InObjectType, InInvalidationReasonValue, InComparePredicate>>
	{
		using ::SlateAttributePrivate::TSlateMemberAttributeRef<TSlateAttribute<InObjectType, InInvalidationReasonValue, InComparePredicate>>::TSlateMemberAttributeRef;
	};

	//~ Override for FText that use the TSlateAttributeFTextComparePredicate to compare FText
	template<>
	struct TSlateAttributeRef<FText, EInvalidateWidgetReason::None> : public ::SlateAttributePrivate::TSlateMemberAttributeRef<TSlateAttribute<FText>>
	{
		using ::SlateAttributePrivate::TSlateMemberAttributeRef<TSlateAttribute<FText>>::TSlateMemberAttributeRef;
	};

	//~ Override for FText that use the TSlateAttributeFTextComparePredicate to compare FText
	template<EInvalidateWidgetReason InInvalidationReasonValue>
	struct TSlateAttributeRef<FText, InInvalidationReasonValue> : public ::SlateAttributePrivate::TSlateMemberAttributeRef<TSlateAttribute<FText, InInvalidationReasonValue>>
	{
		using ::SlateAttributePrivate::TSlateMemberAttributeRef<TSlateAttribute<FText, InInvalidationReasonValue>>::TSlateMemberAttributeRef;
	};

public:
	//
	// GENERAL EVENTS
	//

	/**
	 * Called to tell a widget to paint itself (and it's children).
	 *
	 * The widget should respond by populating the OutDrawElements array with FDrawElements
	 * that represent it and any of its children.
	 *
	 * @param Args              All the arguments necessary to paint this widget (@todo umg: move all params into this struct)
	 * @param AllottedGeometry  The FGeometry that describes an area in which the widget should appear.
	 * @param MyCullingRect    The clipping rectangle allocated for this widget and its children.
	 * @param OutDrawElements   A list of FDrawElements to populate with the output.
	 * @param LayerId           The Layer onto which this widget should be rendered.
	 * @param InColorAndOpacity Color and Opacity to be applied to all the descendants of the widget being painted
	 * @param bParentEnabled	True if the parent of this widget is enabled.
	 * @return The maximum layer ID attained by this widget or any of its children.
	 */
	SLATECORE_API int32 Paint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;

	/**
	 * Ticks this widget with Geometry.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	SLATECORE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);

	//
	// KEY INPUT
	//

	/**
	 * Called when focus is given to this widget.  This event does not bubble.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param  InFocusEvent  The FocusEvent
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	SLATECORE_API virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent);

	/**
	 * Called when this widget loses focus.  This event does not bubble.
	 *
	 * @param InFocusEvent The FocusEvent
	 */
	SLATECORE_API virtual void OnFocusLost(const FFocusEvent& InFocusEvent);

	/** Called whenever a focus path is changing on all the widgets within the old and new focus paths */
	SLATECORE_API virtual void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent);

	/**
	 * Called after a character is entered while this widget has keyboard focus
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param  InCharacterEvent  Character event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	SLATECORE_API virtual FReply OnKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent);

	/**
	 * Called after a key is pressed when this widget or a child of this widget has focus
	 * If a widget handles this event, OnKeyDown will *not* be passed to the focused widget.
	 *
	 * This event is primarily to allow parent widgets to consume an event before a child widget processes
	 * it and it should be used only when there is no better design alternative.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param InKeyEvent  Key event
	 * @return Returns whether the event was handled, along with other possible actions
	 */
	SLATECORE_API virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/**
	 * Called after a key is pressed when this widget has focus (this event bubbles if not handled)
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param InKeyEvent  Key event
	 * @return Returns whether the event was handled, along with other possible actions
	 */
	SLATECORE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/**
	 * Called after a key is released when this widget has focus
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param InKeyEvent  Key event
	 * @return Returns whether the event was handled, along with other possible actions
	 */
	SLATECORE_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/**
	 * Called when an analog value changes on a button that supports analog
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param InAnalogInputEvent Analog input event
	 * @return Returns whether the event was handled, along with other possible actions
	 */
	SLATECORE_API virtual FReply OnAnalogValueChanged(const FGeometry& MyGeometry, const FAnalogInputEvent& InAnalogInputEvent);

	//
	// MOUSE INPUT
	//

	/**
	 * The system calls this method to notify the widget that a mouse button was pressed within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	SLATECORE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * Just like OnMouseButtonDown, but tunnels instead of bubbling.
	 * If this event is handled, OnMouseButtonDown will not be sent.
	 *
	 * Use this event sparingly as preview events generally make UIs more
	 * difficult to reason about.
	 */
	SLATECORE_API virtual FReply OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * The system calls this method to notify the widget that a mouse button was release within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	SLATECORE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * The system calls this method to notify the widget that a mouse moved within it. This event is bubbled.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 * @return Whether the event was handled along with possible requests for the system to take action.
	 */
	SLATECORE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * The system will use this event to notify a widget that the cursor has entered it. This event is uses a custom bubble strategy.
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param MouseEvent Information about the input event
	 */
	SLATECORE_API virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * The system will use this event to notify a widget that the cursor has left it. This event is uses a custom bubble strategy.
	 *
	 * @param MouseEvent Information about the input event
	 */
	SLATECORE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent);

	/**
	 * Called when the mouse wheel is spun. This event is bubbled.
	 *
	 * @param  MouseEvent  Mouse event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	SLATECORE_API virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/**
	 * The system asks each widget under the mouse to provide a cursor. This event is bubbled.
	 *
	 * @return FCursorReply::Unhandled() if the event is not handled; return FCursorReply::Cursor() otherwise.
	 */
	SLATECORE_API virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const;

	/**
	 * After OnCursorQuery has specified a cursor type the system asks each widget under the mouse to map that cursor to a widget. This event is bubbled.
	 *
	 * @return TOptional<TSharedRef<SWidget>>() if you don't have a mapping otherwise return the Widget to show.
	 */
	SLATECORE_API virtual TOptional<TSharedRef<SWidget>> OnMapCursor(const FCursorReply& CursorReply) const;

	/**
	 * Called when a mouse button is double clicked.  Override this in derived classes.
	 *
	 * @param  InMyGeometry  Widget geometry
	 * @param  InMouseEvent  Mouse button event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	SLATECORE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/**
	 * Called when Slate wants to visualize tooltip.
	 * If nobody handles this event, Slate will use default tooltip visualization.
	 * If you override this event, you should probably return true.
	 *
	 * @param  TooltipContent    The TooltipContent that I may want to visualize.
	 * @return true if this widget visualized the tooltip content; i.e., the event is handled.
	 */
	SLATECORE_API virtual bool OnVisualizeTooltip(const TSharedPtr<SWidget>& TooltipContent);

	/**
	 * Visualize a new pop-up if possible.  If it's not possible for this widget to host the pop-up
	 * content you'll get back an invalid pointer to the layer.  The returned FPopupLayer allows you
	 * to remove the pop-up when you're done with it
	 *
	 * @param PopupContent The widget to try and host overlaid on top of the widget.
	 *
	 * @return a valid FPopupLayer if this widget supported hosting it.  You can call Remove() on this to destroy the pop-up.
	 */
	SLATECORE_API virtual TSharedPtr<FPopupLayer> OnVisualizePopup(const TSharedRef<SWidget>& PopupContent);

	/**
	 * Called when Slate detects that a widget started to be dragged.
	 * Usage:
	 * A widget can ask Slate to detect a drag.
	 * OnMouseDown() reply with FReply::Handled().DetectDrag( SharedThis(this) ).
	 * Slate will either send an OnDragDetected() event or do nothing.
	 * If the user releases a mouse button or leaves the widget before
	 * a drag is triggered (maybe user started at the very edge) then no event will be
	 * sent.
	 *
	 * @param  InMyGeometry  Widget geometry
	 * @param  InMouseEvent  MouseMove that triggered the drag
	 */
	SLATECORE_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	//
	// DRAG AND DROP (DragDrop)
	//

	/**
	 * Called during drag and drop when the drag enters a widget.
	 *
	 * Enter/Leave events in slate are meant as lightweight notifications.
	 * So we do not want to capture mouse or set focus in response to these.
	 * However, OnDragEnter must also support external APIs (e.g. OLE Drag/Drop)
	 * Those require that we let them know whether we can handle the content
	 * being dragged OnDragEnter.
	 *
	 * The concession is to return a can_handled/cannot_handle
	 * boolean rather than a full FReply.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 *
	 * @return A reply that indicated whether the contents of the DragDropEvent can potentially be processed by this widget.
	 */
	SLATECORE_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);

	/**
	 * Called during drag and drop when the drag leaves a widget.
	 *
	 * @param DragDropEvent   The drag and drop event.
	 */
	SLATECORE_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent);

	/**
	 * Called during drag and drop when the the mouse is being dragged over a widget.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 * @return A reply that indicated whether this event was handled.
	 */
	SLATECORE_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);

	/**
	 * Called when the user is dropping something onto a widget; terminates drag and drop.
	 *
	 * @param MyGeometry      The geometry of the widget receiving the event.
	 * @param DragDropEvent   The drag and drop event.
	 * @return A reply that indicated whether this event was handled.
	 */
	SLATECORE_API virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent);

	//
	// TOUCH and GESTURES
	//

	/**
	 * Called when the user performs a gesture on trackpad. This event is bubbled.
	 *
	 * @param  GestureEvent  gesture event
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	SLATECORE_API virtual FReply OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent);

	/**
	 * Called when a touchpad touch is started (finger down)
	 *
	 * @param InTouchEvent	The touch event generated
	 */
	SLATECORE_API virtual FReply OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent);

	/**
	 * Called when a touchpad touch is moved  (finger moved)
	 *
	 * @param InTouchEvent	The touch event generated
	 */
	SLATECORE_API virtual FReply OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent);

	/**
	 * Called when a touchpad touch is ended (finger lifted)
	 *
	 * @param InTouchEvent	The touch event generated
	 */
	SLATECORE_API virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent);

	/**
	 * Called when a touchpad touch force changes
	 *
	 * @param InTouchEvent	The touch event generated
	 */
	SLATECORE_API virtual FReply OnTouchForceChanged(const FGeometry& MyGeometry, const FPointerEvent& TouchEvent);

	/**
	 * Called when a touchpad touch first moves after TouchStarted
	 *
	 * @param InTouchEvent	The touch event generated
	 */
	SLATECORE_API virtual FReply OnTouchFirstMove(const FGeometry& MyGeometry, const FPointerEvent& TouchEvent);

	/**
	 * Called when motion is detected (controller or device)
	 * e.g. Someone tilts or shakes their controller.
	 *
	 * @param InMotionEvent	The motion event generated
	 */
	SLATECORE_API virtual FReply OnMotionDetected(const FGeometry& MyGeometry, const FMotionEvent& InMotionEvent);

	/**
	 * Called to determine if we should render the focus brush.
	 *
	 * @param InFocusCause	The cause of focus
	 */
	SLATECORE_API virtual TOptional<bool> OnQueryShowFocus(const EFocusCause InFocusCause) const;

	/**
	 * Popups can manifest in a NEW OS WINDOW or via an OVERLAY in an existing window.
	 * This can be set explicitly on SMenuAnchor, or can be determined by a scoping widget.
	 * A scoping widget can reply to OnQueryPopupMethod() to drive all its descendants'
	 * poup methods.
	 *
	 * e.g. Fullscreen games cannot summon a new window, so game SViewports will reply with
	 *      EPopupMethod::UserCurrentWindow. This makes all the menu anchors within them
	 *      use the current window.
	 */
	SLATECORE_API virtual FPopupMethodReply OnQueryPopupMethod() const;

	SLATECORE_API virtual TOptional<FVirtualPointerPosition> TranslateMouseCoordinateForCustomHitTestChild(const SWidget& ChildWidget, const FGeometry& MyGeometry, const FVector2D ScreenSpaceMouseCoordinate, const FVector2D LastScreenSpaceMouseCoordinate) const;

	/**
	 * All the pointer (mouse, touch, stylus, etc.) events from this frame have been routed.
	 * This is a widget's chance to act on any accumulated data.
	 */
	SLATECORE_API virtual void OnFinishedPointerInput();

	/**
	 * All the key (keyboard, gamepay, joystick, etc.) input from this frame has been routed.
	 * This is a widget's chance to act on any accumulated data.
	 */
	SLATECORE_API virtual void OnFinishedKeyInput();

	/**
	 * Called when navigation is requested
	 * e.g. Left Joystick, Direction Pad, Arrow Keys can generate navigation events.
	 *
	 * @param InNavigationEvent	The navigation event generated
	 */
	SLATECORE_API virtual FNavigationReply OnNavigation(const FGeometry& MyGeometry, const FNavigationEvent& InNavigationEvent);

	/* 
	 * Helper function to preview Navigation Replies without an explicit Navigation Event
	 */
	SLATECORE_API FNavigationReply OnNavigation(const FGeometry& MyGeometry, const EUINavigation Type);

	/**
	 * Called when the mouse is moved over the widget's window, to determine if we should report whether
	 * OS-specific features should be active at this location (such as a title bar grip, system menu, etc.)
	 * Usually you should not need to override this function.
	 *
	 * @return	The window "zone" the cursor is over, or EWindowZone::Unspecified if no special behavior is needed
	 */
	SLATECORE_API virtual EWindowZone::Type GetWindowZoneOverride() const;

#if WITH_ACCESSIBILITY
	SLATECORE_API virtual TSharedRef<class FSlateAccessibleWidget> CreateAccessibleWidget();
#endif

public:
	//	
	// LAYOUT
	//

	bool NeedsPrepass() const { return bNeedsPrepass; }
	/** DEPRECATED version of SlatePrepass that assumes no scaling beyond AppScale*/
	//UE_DEPRECATED(4.20, "SlatePrepass requires a layout scale to be accurate.")
	SLATECORE_API void SlatePrepass();

	/**
	 * Descends to leaf-most widgets in the hierarchy and gathers desired sizes on the way up.
	 * i.e. Caches the desired size of all of this widget's children recursively, then caches desired size for itself.
	 */
	SLATECORE_API void SlatePrepass(float InLayoutScaleMultiplier);

	void SetCanTick(bool bInCanTick) { bInCanTick ? AddUpdateFlags(EWidgetUpdateFlags::NeedsTick) : RemoveUpdateFlags(EWidgetUpdateFlags::NeedsTick); }
	bool GetCanTick() const { return HasAnyUpdateFlags(EWidgetUpdateFlags::NeedsTick); }

	/** @return true if the widgets has any bound slate attribute. */
	bool HasRegisteredSlateAttribute() const { return bHasRegisteredSlateAttribute; }
	/** @return true if the widgets will update its registered slate attributes automatically or they need to be updated manually. */
	bool IsAttributesUpdatesEnabled() const { return bEnabledAttributesUpdate; }

	/** Update all the Slate Attributes of this widget. */
	void UpdateAllAttributes()
	{
		FSlateAttributeMetaData::UpdateAllAttributes(*this, FSlateAttributeMetaData::EInvalidationPermission::AllowInvalidation);
	}

	const FSlateWidgetPersistentState& GetPersistentState() const { return PersistentState; }
	const FWidgetProxyHandle GetProxyHandle() const { return FastPathProxyHandle; }

	/** @return the DesiredSize that was computed the last time CacheDesiredSize() was called. */
	SLATECORE_API UE::Slate::FDeprecateVector2DResult GetDesiredSize() const;

	SLATECORE_API void AssignParentWidget(TSharedPtr<SWidget> InParent);
	SLATECORE_API bool ConditionallyDetatchParentWidget(SWidget* InExpectedParent);

	/**  */
	virtual bool ValidatePathToChild(SWidget* InChild) { return true; }

	inline bool IsParentValid() const { return ParentWidgetPtr.IsValid(); }
	inline TSharedPtr<SWidget> GetParentWidget() const { return ParentWidgetPtr.Pin(); }

	inline TSharedPtr<SWidget> Advanced_GetPaintParentWidget() const { return PersistentState.PaintParent.Pin(); }

	/**
	 * Calculates what if any clipping state changes need to happen when drawing this widget.
	 * @return the culling rect that should be used going forward.
	 */
	SLATECORE_API FSlateRect CalculateCullingAndClippingRules(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, bool& bClipToBounds, bool& bAlwaysClip, bool& bIntersectClipBounds) const;

	/**
	 * @return the SlateRect that FHittestGrid would use to test for hit bounds
	 */
	SLATECORE_API virtual FSlateRect GetHitTestBoundingRect() const;

	bool HasAnyUpdateFlags(EWidgetUpdateFlags FlagsToCheck) const
	{
		return EnumHasAnyFlags(UpdateFlags, FlagsToCheck);
	}

	/** Whether the widget is a part of the project's content (as opposed to editor, chrome, diagnostic output).
	 * Dynamic: updated during pre-pass, inherited from parent but may be overwritten in CustomPrepass.
	 * Used to filter which widgets can be navigated to, which are considered by diagnostic tools.
	 */
	bool IsProjectContent() const
	{
		return bIsProjectContent;
	}

protected:
	void SetVolatilePrepass(bool bVolatile) { bVolatile ? AddUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePrepass) : RemoveUpdateFlags(EWidgetUpdateFlags::NeedsVolatilePrepass); }

	/**
	 * Call from CustomPrepass to change whether this widget and its children are a part of the project's content (as opposed to editor, chrome,
	 * diagnostic output).
	 */
	void SetIsProjectContent(bool bInIsProjectContent)
	{
		bIsProjectContent = bInIsProjectContent;
	}

	/**
	 * Call from CustomPrepass to change whether this widget's children are a part of the project's content.
	 */
	void SetIsProjectContentParent(bool bInIsProjectContent)
	{
		bIsProjectContentParent = bInIsProjectContent;
	}

	virtual bool CustomPrepass(float LayoutScaleMultiplier) { return false; }

	/**
	 * The system calls this method. It performs a breadth-first traversal of every visible widget and asks
	 * each widget to cache how big it needs to be in order to present all of its content.
	 */
	SLATECORE_API virtual void CacheDesiredSize(float InLayoutScaleMultiplier);

	/**
	 * Compute the ideal size necessary to display this widget. For aggregate widgets (e.g. panels) this size should include the
	 * size necessary to show all of its children. CacheDesiredSize() guarantees that the size of descendants is computed and cached
	 * before that of the parents, so it is safe to call GetDesiredSize() for any children while implementing this method.
	 *
	 * Note that ComputeDesiredSize() is meant as an aide to the developer. It is NOT meant to be very robust in many cases. If your
	 * widget is simulating a bouncing ball, you should just return a reasonable size; e.g. 160x160. Let the programmer set up a reasonable
	 * rule of resizing the bouncy ball simulation.
	 *
	 * @param  LayoutScaleMultiplier    This parameter is safe to ignore for almost all widgets; only really affects text measuring.
	 *
	 * @return The desired size.
	 */
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const = 0;


private:
	void SetFastPathProxyHandle(const FWidgetProxyHandle& Handle) { FastPathProxyHandle = Handle; }
	SLATECORE_API void SetFastPathProxyHandle(const FWidgetProxyHandle& Handle, FSlateInvalidationWidgetVisibility Visibility, bool bParentVolatile);
	SLATECORE_API void SetFastPathSortOrder(const FSlateInvalidationWidgetSortOrder SortOrder);

	SLATECORE_API void UpdateFastPathVisibility(FSlateInvalidationWidgetVisibility ParentVisibility, FHittestGrid* ParentHittestGrid);
	SLATECORE_API void UpdateFastPathWidgetRemoved(FHittestGrid* ParentHittestGrid);
	SLATECORE_API void UpdateFastPathVolatility(bool bParentVolatile);

	/**
	 * Explicitly set the desired size. This is highly advanced functionality that is meant
	 * to be used in conjunction with overriding CacheDesiredSize. Use ComputeDesiredSize() instead.
	 */
	void SetDesiredSize(const FVector2D& InDesiredSize)
	{
		DesiredSize = FVector2f(InDesiredSize);
		bDesiredSizeSet = true;
	}

#if STATS || ENABLE_STATNAMEDEVENTS
	SLATECORE_API void CreateStatID() const;
#endif

	void AddUpdateFlags(EWidgetUpdateFlags FlagsToAdd)
	{
		EWidgetUpdateFlags Previous = UpdateFlags;
		UpdateFlags |= FlagsToAdd;
		FastPathProxyHandle.UpdateWidgetFlags(this, Previous, UpdateFlags);
	}

	void RemoveUpdateFlags(EWidgetUpdateFlags FlagsToRemove)
	{
		EWidgetUpdateFlags Previous = UpdateFlags;
		UpdateFlags &= (~FlagsToRemove);
		FastPathProxyHandle.UpdateWidgetFlags(this, Previous, UpdateFlags);
	}

	SLATECORE_API void UpdateWidgetProxy(int32 NewLayerId, FSlateCachedElementsHandle& CacheHandle);

public:
#if UE_SLATE_TRACE_ENABLED
	uint8 Debug_GetWidgetInfoTraced() const 
	{ 
		return Debug_LastTraceInfoSent; 
	}

	void Debug_SetWidgetInfoTraced(uint8 InDebug_LastTraceInfoSent) const 
	{ 
		Debug_LastTraceInfoSent = InDebug_LastTraceInfoSent; 
	}
#endif // UE_SLATE_TRACE_ENABLED

#if WITH_SLATE_DEBUGGING
	uint32 Debug_GetLastPaintFrame() const 
	{ 
		return LastPaintFrame; 
	}
	uint32 Debug_GetLastPrepassFrame() const 
	{ 
		return LastPrepassFrame; 
	}
private:
	void Debug_UpdateLastPaintFrame() 
	{ 
		LastPaintFrame = GFrameNumber; 
	}
	void Debug_UpdateLastPrepassFrame() 
	{ 
		LastPrepassFrame = GFrameNumber; 
	}
#endif // WITH_SLATE_DEBUGGING

public:

	inline TStatId GetStatID(bool bForDeferredUse = false) const
	{
#if STATS
		// this is done to avoid even registering stats for a disabled group (unless we plan on using it later)
		if (bForDeferredUse || FThreadStats::IsCollectingData())
		{
			if (!StatID.IsValidStat())
			{
				CreateStatID();
			}
			return StatID;
		}
		return TStatId(); // not doing stats at the moment, or ever
#elif ENABLE_STATNAMEDEVENTS
		if (!StatID.IsValidStat() && (bForDeferredUse || GCycleStatsShouldEmitNamedEvents))
		{
			CreateStatID();
		}
		return StatID;
#else
		return TStatId(); // not doing stats at the moment, or ever
#endif
	}

	/** What is the Child's scale relative to this widget. */
	SLATECORE_API virtual float GetRelativeLayoutScale(const int32 ChildIndex, float LayoutScaleMultiplier) const;

	/**
	 * Non-virtual entry point for arrange children. ensures common work is executed before calling the virtual
	 * ArrangeChildren function.
	 * Compute the Geometry of all the children and add populate the ArrangedChildren list with their values.
	 * Each type of Layout panel should arrange children based on desired behavior.
	 * 
	 * Optionally, update the collapsed attributes (attributes that affect the visibility) of the children before executing the virtual ArrangeChildren function.
	 * The visibility attribute is updated once per frame (see SlatePrepass).
	 * Use the option when you are calling ArrangeChildren outside of the regular SWidget Paint/Tick.
	 *
	 * @param AllottedGeometry    The geometry allotted for this widget by its parent.
	 * @param ArrangedChildren    The array to which to add the WidgetGeometries that represent the arranged children.
	 * @param bUpdateAttributes   Update the collapsed attributes.
	 */
	SLATECORE_API void ArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, bool bUpdateAttributes = false) const;

	/**
	 * Returns the useful children (if any) of this widget. Some widget type may hide widget if they are needed by the system.
	 * Allows for iteration over the Widget's children regardless of how they are actually stored.
	 * @note Should be renamed to GetVisibleChildren (not ALL children will be returned in all cases).
	 */
	virtual FChildren* GetChildren() = 0;

	/**
	 * Returns the children (if any) of this widget that are used by the invalidation system.
	 * This is used by the FastPath system to generate the correct information for every widget.
	 * @note Prefer GetChildren. Some widget may hide widget from you.
	 * @note Should be name GetFastPathChildren or GetInvalidationChildren
	 */
	virtual FChildren* GetAllChildren() { return GetChildren(); }

#if WITH_SLATE_DEBUGGING
	/**
	 * Returns all Widgets, including widget hidden from the invalidation system.
	 * This is used by the WidgetReflector.
	 */
	virtual FChildren* Debug_GetChildrenForReflector() { return GetAllChildren(); }
#endif

	/**
	 * Checks is this widget can ever support keyboard focus.
	 *
	 * @return  True if this widget can ever support keyboard focus
	 */
	bool CanSupportFocus() const { return bCanSupportFocus; }

	/**
	 * Checks to see if this widget supports keyboard focus.  Override this in derived classes.
	 *
	 * @return  True if this widget can take keyboard focus
	 */
	SLATECORE_API virtual bool SupportsKeyboardFocus() const;

	/**
	 * Checks to see if this widget currently has the keyboard focus
	 *
	 * @return  True if this widget has keyboard focus
	 */
	SLATECORE_API virtual bool HasKeyboardFocus() const;

	/**
	 * Gets whether or not the specified users has this widget focused, and if so the type of focus.
	 *
	 * @return The optional will be set with the focus cause, if unset this widget doesn't have focus.
	 */
	SLATECORE_API TOptional<EFocusCause> HasUserFocus(int32 UserIndex) const;

	/**
	 * Gets whether or not any users have this widget focused, and if so the type of focus (first one found).
	 *
	 * @return The optional will be set with the focus cause, if unset this widget doesn't have focus.
	 */
	SLATECORE_API TOptional<EFocusCause> HasAnyUserFocus() const;

	/**
	 * Gets whether or not the specified users has this widget or any descendant focused.
	 *
	 * @return The optional will be set with the focus cause, if unset this widget doesn't have focus.
	 */
	SLATECORE_API bool HasUserFocusedDescendants(int32 UserIndex) const;

	/**
	 * @return Whether this widget has any descendants with keyboard focus
	 */
	SLATECORE_API bool HasFocusedDescendants() const;

	/**
	 * @return whether or not any users have this widget focused, or any descendant focused.
	 */
	SLATECORE_API bool HasAnyUserFocusOrFocusedDescendants() const;

	/**
	 * Checks to see if this widget is the current mouse captor
	 *
	 * @return  True if this widget has captured the mouse
	 */
	SLATECORE_API bool HasMouseCapture() const;

	/**
	 * Checks to see if this widget has mouse capture from the provided user.
	 *
	 * @return  True if this widget has captured the mouse
	 */
	SLATECORE_API bool HasMouseCaptureByUser(int32 UserIndex, TOptional<int32> PointerIndex = TOptional<int32>()) const;

public:
	/** Called when this widget had captured the mouse, but that capture has been revoked for some reason. */
	SLATECORE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent);

	/**
	 * Sets the enabled state of this widget
	 *
	 * @param InEnabledState	An attribute containing the enabled state or a delegate to call to get the enabled state.
	 */
	void SetEnabled(TAttribute<bool> InEnabledState)
	{
		EnabledStateAttribute.Assign(*this, MoveTemp(InEnabledState));
	}

	/** @return Whether or not this widget is enabled */
	inline bool IsEnabled() const
	{
		return EnabledStateAttribute.Get();
	}

	/** @return Is this widget interactive or not? Defaults to false */
	virtual bool IsInteractable() const
	{
		return false;
	}

	/** @return The tool tip associated with this widget; Invalid reference if there is not one */
	SLATECORE_API virtual TSharedPtr<IToolTip> GetToolTip();

	/** Called when a tooltip displayed from this widget is being closed */
	SLATECORE_API virtual void OnToolTipClosing();

	/**
	 * Sets whether this widget is a "tool tip force field".  That is, tool-tips should never spawn over the area
	 * occupied by this widget, and will instead be repelled to an outside edge
	 *
	 * @param	bEnableForceField	True to enable tool tip force field for this widget
	 */
	SLATECORE_API void EnableToolTipForceField(const bool bEnableForceField);

	/** @return True if a tool tip force field is active on this widget */
	bool HasToolTipForceField() const
	{
		return bToolTipForceFieldEnabled;
	}
	
	/** Sets a custom repel zone for the tooltip, bToolTipForceFieldEnabled must be true */
	SLATECORE_API void SetToolTipForceFieldExpansion(const TAttribute<TOptional<FSlateRect>>& InExpansionRect);
	
	/** @return FSlateRect that combines all FSlateToolTipForceFieldExpansionMetaData attribute values */
	SLATECORE_API TOptional<FSlateRect> GetToolTipForceFieldExpansion() const;

	/**
	 * @return True if this widget hovered
	 * @note IsHovered used to be virtual. Use SetHover to assign an attribute if you need to override the default behavior.
	 */
	bool IsHovered() const
	{
		return HoveredAttribute.Get();
	}

	/** @return True if this widget is directly hovered */
	SLATECORE_API bool IsDirectlyHovered() const;

protected:
	/**
	 * Set the hover state.
	 * Once set, the attribute that the ownership and SWidget code will not update the attribute value.
	 * You can return the control to the SWidget code by setting an empty TAttribute.
	 */
	void SetHover(TAttribute<bool> InHovered)
	{
		bIsHoveredAttributeSet = InHovered.IsSet();
		HoveredAttribute.Assign(*this, MoveTemp(InHovered));
	}

public:

	/**
	 * @return is this widget visible, hidden or collapsed.
	 * @note this widget can be visible but if a parent is hidden or collapsed, it would not show on screen.
	 */
	inline EVisibility GetVisibility() const { return VisibilityAttribute.Get(); }

	/** @param InVisibility  should this widget be */
	SLATECORE_API virtual void SetVisibility(TAttribute<EVisibility> InVisibility);

#if WITH_ACCESSIBILITY
	/**
	 * Get the text that should be reported to the user when attempting to access this widget.
	 *
	 * @param AccessibleType Whether the widget is being accessed directly or through a summary query.
	 * @return The text that should be conveyed to the user describing this widget.
	 */
	SLATECORE_API FText GetAccessibleText(EAccessibleType AccessibleType = EAccessibleType::Main) const;

	/**
	 * Traverse all child widgets and concat their results of GetAccessibleText(Summary).
	 *
	 * @return The combined text of all child widget's summary text.
	 */
	SLATECORE_API FText GetAccessibleSummary() const;

	/**
	 * Whether this widget is considered accessible or not. A widget is accessible if its behavior
	 * is set to something other than NotAccessible, and all of its parent widgets support accessible children.
	 *
	 * @return true if an accessible widget should be created for this widget.
	 */
	SLATECORE_API bool IsAccessible() const;

	/**
	 * Get the behavior describing how the accessible text of this widget should be retrieved.
	 *
	 * @param AccessibleType Whether the widget is being accessed directly or through a summary query.
	 * @return The accessible behavior of the widget.
	 */
	SLATECORE_API EAccessibleBehavior GetAccessibleBehavior(EAccessibleType AccessibleType = EAccessibleType::Main) const;

	/**
	 * Checks whether this widget allows its children to be accessible or not.
	 *
	 * @return true if children can be accessible.
	 */
	SLATECORE_API bool CanChildrenBeAccessible() const;

	/**
	 * Set a new accessible behavior, and if the behavior is custom, new accessible text to go along with it.
	 *
	 * @param InBehavior The new behavior for the widget. If the new behavior is custom, InText should also be set.
	 * @param InText, If the new behavior is custom, this will be the custom text assigned to the widget.
	 * @param AccessibleType Whether the widget is being accessed directly or through a summary query.
	 */
	SLATECORE_API void SetAccessibleBehavior(EAccessibleBehavior InBehavior, const TAttribute<FText>& InText = TAttribute<FText>(), EAccessibleType AccessibleType = EAccessibleType::Main);

	/**
	 * Sets whether children are allowed to be accessible or not.
	 * Warning: Calling this function after accessibility is enabled will cause the accessibility tree to become unsynced.
	 *
	 * @param InCanChildrenBeAccessible Whether children should be accessible or not.
	 */
	SLATECORE_API void SetCanChildrenBeAccessible(bool InCanChildrenBeAccessible);

	/**
	 * Assign AccessibleText with a default value that can be used when AccessibleBehavior is set to Auto or Custom.
	 *
	 * @param AccessibleType Whether the widget is being accessed directly or through a summary query.
	 */
	SLATECORE_API virtual TOptional<FText> GetDefaultAccessibleText(EAccessibleType AccessibleType = EAccessibleType::Main) const;
#endif

	/** Whether or not a widget is volatile and will update every frame without being invalidated */
	inline bool IsVolatile() const { return bCachedVolatile; }

	/**
	 * This widget is volatile because its parent or some ancestor is volatile
	 * @note only valid if the widget is contained by an InvalidationRoot (the proxy is valid).
	 */
	inline bool IsVolatileIndirectly() const { return bInheritedVolatility; }

	/**
	 * Should this widget always appear as volatile for any layout caching host widget.  A volatile
	 * widget's geometry and layout data will never be cached, and neither will any children.
	 * @param bForce should we force the widget to be volatile?
	 */
	inline void ForceVolatile(bool bForce)
	{
		if (bForceVolatile != bForce)
		{
			bForceVolatile = bForce;
			Invalidate(EInvalidateWidgetReason::PaintAndVolatility);
		}
	}

	inline bool ShouldInvalidatePrepassDueToVolatility() { return bVolatilityAlwaysInvalidatesPrepass; }

	/**
	 * Invalidates the widget from the view of a layout caching widget that may own this widget.
	 * will force the owning widget to redraw and cache children on the next paint pass.
	 */
	SLATECORE_API void Invalidate(EInvalidateWidgetReason InvalidateReason);

	/**
	 * Recalculates volatility of the widget and caches the result.  Should be called any time 
	 * anything examined by your implementation of ComputeVolatility is changed.
	 */
	inline void CacheVolatility()
	{
		bCachedVolatile = bForceVolatile || ComputeVolatility();
	}

	/**
	 * In fast path, if the widget is mark, do a full Prepass on its next update to calculate it's desired size.
	 * This does not invalidate the widget.
	 */
	void MarkPrepassAsDirty() { bNeedsPrepass = true; }

protected:

#if SLATE_CULL_WIDGETS
	/**
	 * Tests if an arranged widget should be culled.
	 * @param MyCullingRect the culling rect of the widget currently doing the culling.
	 * @param ArrangedChild the arranged widget in the widget currently attempting to cull children.
	 */
	SLATECORE_API bool IsChildWidgetCulled(const FSlateRect& MyCullingRect, const FArrangedWidget& ArrangedChild) const;
#else
	inline bool IsChildWidgetCulled(const FSlateRect&, const FArrangedWidget&) const { return false; }
#endif

protected:


	/**
	 * Called when a child is removed from the tree parent's widget tree either by removing it from a slot. This can also be called manually if you've got some non-slot based what of no longer reporting children
	 * An example of a widget that needs manual calling is SWidgetSwitcher.  It keeps all its children but only arranges and paints a single "active" one.  Once a child becomes inactive, its cached data should be removed.
	 */
	SLATECORE_API void InvalidateChildRemovedFromTree(SWidget& Child);

	/**
	 * Recalculates and caches volatility and returns 'true' if the volatility changed.
	 */
	inline bool Advanced_InvalidateVolatility()
	{
		const bool bWasDirectlyVolatile = IsVolatile();
		CacheVolatility();
		return bWasDirectlyVolatile != IsVolatile();
	}
public:

	/** @return the render opacity of the widget. */
	inline float GetRenderOpacity() const
	{
		return RenderOpacity;
	}

	/** @param InOpacity The opacity of the widget during rendering. */
	inline void SetRenderOpacity(float InRenderOpacity)
	{
		if(RenderOpacity != InRenderOpacity)
		{
			RenderOpacity = InRenderOpacity;
			Invalidate(EInvalidateWidgetReason::Paint);
		}
	}

	inline void SetTag(FName InTag)
	{
		Tag = InTag;
	}

	/** @return the render transform of the widget. */
	inline const TOptional<FSlateRenderTransform>& GetRenderTransform() const
	{
		return RenderTransformAttribute.Get();
	}

	inline TOptional<FSlateRenderTransform> GetRenderTransformWithRespectToFlowDirection() const
	{
		if (LIKELY(GSlateFlowDirection == EFlowDirection::LeftToRight))
		{
			return RenderTransformAttribute.Get();
		}
		else
		{
			// If we're going right to left, flip the X translation on render transforms.
			TOptional<FSlateRenderTransform> Transform = RenderTransformAttribute.Get();
			if (Transform.IsSet())
			{
				FVector2D Translation = Transform.GetValue().GetTranslation();
				Transform.GetValue().SetTranslation(FVector2D(-Translation.X, Translation.Y));
			}
			return Transform;
		}
	}

	inline UE::Slate::FDeprecateVector2DResult GetRenderTransformPivotWithRespectToFlowDirection() const
	{
		FVector2f TransformPivot = UE::Slate::CastToVector2f(RenderTransformPivotAttribute.Get());
		if (LIKELY(GSlateFlowDirection == EFlowDirection::LeftToRight))
		{
			return TransformPivot;
		}
		else
		{
			// If we're going right to left, flip the X's pivot mirrored about 0.5.
			TransformPivot.X = 0.5f + (0.5f - TransformPivot.X);
			return TransformPivot;
		}
	}

	/** @param InTransform the render transform to set for the widget (transforms from widget's local space). TOptional<> to allow code to skip expensive overhead if there is no render transform applied. */
	inline void SetRenderTransform(TAttribute<TOptional<FSlateRenderTransform>> InTransform)
	{
		RenderTransformAttribute.Assign(*this, MoveTemp(InTransform));
	}

	/** @return the pivot point of the render transform. */
	inline UE::Slate::FDeprecateVector2DResult GetRenderTransformPivot() const
	{
		FVector2f TransformPivot = UE::Slate::CastToVector2f(RenderTransformPivotAttribute.Get());
		return TransformPivot;
	}

	/** @param InTransformPivot Sets the pivot point of the widget's render transform (in normalized local space). */
	inline void SetRenderTransformPivot(TAttribute<FVector2D> InTransformPivot)
	{
		RenderTransformPivotAttribute.Assign(*this, MoveTemp(InTransformPivot));
	}

	/**
	 * Sets the clipping to bounds rules for this widget.
	 */
	SLATECORE_API void SetClipping(EWidgetClipping InClipping);

	/** @return The current clipping rules for this widget. */
	inline EWidgetClipping GetClipping() const
	{
		return Clipping;
	}
	
	/**
	* Sets the pixel snapping method for this widget.
	*/
	SLATECORE_API void SetPixelSnapping(EWidgetPixelSnapping InPixelSnappingMethod);

	/** @return The current pixel snapping rules for this widget. */
	inline EWidgetPixelSnapping GetPixelSnapping() const
	{
		return PixelSnappingMethod;
	}

	/**
	 * Sets an additional culling padding that is added to a widget to give more leeway when culling widgets.  Useful if 
	 * several child widgets have rendering beyond their bounds.
	 */
	inline void SetCullingBoundsExtension(const FMargin& InCullingBoundsExtension)
	{
		if (CullingBoundsExtension != InCullingBoundsExtension)
		{
			CullingBoundsExtension = InCullingBoundsExtension;
			// @todo - Fast path should this be Paint?
			Invalidate(EInvalidateWidgetReason::Layout);
		}
	}

	/** @return CullingBoundsExtension */
	inline FMargin GetCullingBoundsExtension() const
	{
		return CullingBoundsExtension;
	}

	/**
	 * Sets how content should flow in this panel, based on the current culture.  By default all panels inherit 
	 * the state of the widget above.  If they set a new flow direction it will be inherited down the tree.
	 */
	void SetFlowDirectionPreference(EFlowDirectionPreference InFlowDirectionPreference)
	{
		if (FlowDirectionPreference != InFlowDirectionPreference)
		{
			FlowDirectionPreference = InFlowDirectionPreference;
			Invalidate(EInvalidateWidgetReason::Paint);
		}
	}

	/** Gets the desired flow direction for the layout. */
	EFlowDirectionPreference GetFlowDirectionPreference() const
	{
		return FlowDirectionPreference;
	}

protected:
	/** Establishes a new flow direction potentially, if this widget has a particular preference for it and all its children. */
	EFlowDirection ComputeFlowDirection() const
	{
		switch (FlowDirectionPreference)
		{
		case EFlowDirectionPreference::Culture:
			return FLayoutLocalization::GetLocalizedLayoutDirection();
		case EFlowDirectionPreference::LeftToRight:
			return EFlowDirection::LeftToRight;
		case EFlowDirectionPreference::RightToLeft:
			return EFlowDirection::RightToLeft;
		}

		return GSlateFlowDirection;
	}

public:
	/** Set the tool tip that should appear when this widget is hovered. */
	SLATECORE_API void SetToolTipText(const TAttribute<FText>& ToolTipText);

	/** Set the tool tip that should appear when this widget is hovered. */
	SLATECORE_API void SetToolTipText( const FText& InToolTipText );

	/** Set the tool tip that should appear when this widget is hovered. */
	SLATECORE_API void SetToolTip(const TAttribute<TSharedPtr<IToolTip>>& InToolTip);

	/** Set the cursor that should appear when this widget is hovered  */
	SLATECORE_API void SetCursor( const TAttribute< TOptional<EMouseCursor::Type> >& InCursor );

protected:

	/** Used by Slate to set the runtime debug info about this widget. */
	SLATECORE_API void SetDebugInfo( const ANSICHAR* InType, const ANSICHAR* InFile, int32 OnLine, size_t InAllocSize );

	/** The cursor to show when the mouse is hovering over this widget. */
	SLATECORE_API virtual TOptional<EMouseCursor::Type> GetCursor() const;

public:
	/**
	 * Get the metadata of the type provided.
	 * @return the first metadata of the type supplied that we encounter
	 */
	template<typename MetaDataType>
	TSharedPtr<MetaDataType> GetMetaData() const
	{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
		SCOPE_CYCLE_COUNTER(STAT_SlateGetMetaData);
#endif
		for (const auto& MetaDataEntry : MetaData)
		{
			if (MetaDataEntry->IsOfType<MetaDataType>())
			{
				return StaticCastSharedRef<MetaDataType>(MetaDataEntry);
			}
		}
		return TSharedPtr<MetaDataType>();	
	}

	/**
	 * Get all metadata of the type provided.
	 * @return all the metadata found of the specified type.
	 */
	template<typename MetaDataType>
	TArray<TSharedRef<MetaDataType>> GetAllMetaData() const
	{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
		SCOPE_CYCLE_COUNTER(STAT_SlateGetMetaData);
#endif
		TArray<TSharedRef<MetaDataType>> FoundMetaData;
		for (const auto& MetaDataEntry : MetaData)
		{
			if (MetaDataEntry->IsOfType<MetaDataType>())
			{
				FoundMetaData.Add(StaticCastSharedRef<MetaDataType>(MetaDataEntry));
			}
		}
		return FoundMetaData;
	}

	/**
	 * Add metadata to this widget.
	 * @param AddMe the metadata to add to the widget.
	 */
	template<typename MetaDataType>
	void AddMetadata(const TSharedRef<MetaDataType>& AddMe)
	{
		AddMetadataInternal(AddMe);
	}

	/**
	 * Remove metadata to this widget.
	 * @returns Number of removed elements.
	 */
	template<typename MetaDataType>
	int32 RemoveMetaData(const TSharedRef<MetaDataType>& RemoveMe)
	{
		int32 Index = MetaData.Find(RemoveMe);
		if (Index == INDEX_NONE)
		{
			return 0;
		}

		checkf(Index != 0 || !HasRegisteredSlateAttribute(), TEXT("The first slot is reserved for SlateAttribute"));
		MetaData.RemoveAtSwap(Index, 1);

		OnRemovedMetadata(RemoveMe);
		return 1;
	}
	
	/**
	 * Finds or creates the metadata based on the type provided.
	 * @return the first metadata of the type supplied that we encounter or a newly created one
	 */
    template<typename MetaDataType>
    TSharedRef<MetaDataType> FindOrAddMetaData()
    {
    	if (TSharedPtr<MetaDataType> FoundMetaData = GetMetaData<MetaDataType>())
    	{
    		return FoundMetaData.ToSharedRef();
    	}

    	TSharedRef<MetaDataType> NewMetaData = MakeShared<MetaDataType>();
    	AddMetadataInternal(NewMetaData);
    	return NewMetaData;
    }

protected:
	virtual void OnAddedMetadata(const TSharedRef<ISlateMetaData>& AddedMetaData)
	{
	}

	virtual void OnRemovedMetadata(const TSharedRef<ISlateMetaData>& RemovedMetaData)
	{
	}

private:

	SLATECORE_API void AddMetadataInternal(const TSharedRef<ISlateMetaData>& AddMe);

	template<typename MetaDataType>
	int32 RemoveAllMetaData()
	{
		int32 NumBefore = MetaData.Num();
		for (int32 Index = NumBefore - 1; Index >= 0; --Index)
		{
			const auto& MetaDataEntry = MetaData[Index];
			if (MetaDataEntry->IsOfType<MetaDataType>())
			{
				checkf(Index != 0 || !HasRegisteredSlateAttribute(), TEXT("The first slot is reserved for SlateAttribute"));
				MetaData.RemoveAtSwap(Index);
				OnRemovedMetadata(MetaDataEntry);
			}
		}
		return NumBefore - MetaData.Num();
	}

	template<typename MetaDataType>
	bool RemoveMetaData()
	{
		for (int32 Index = MetaData.Num() - 1; Index >= 0; --Index)
		{
			const auto& MetaDataEntry = MetaData[Index];
			if (MetaDataEntry->IsOfType<MetaDataType>())
			{
				checkf(Index != 0 || !HasRegisteredSlateAttribute(), TEXT("The first slot is reserved for SlateAttribute"));
				MetaData.RemoveAtSwap(Index);
				OnRemovedMetadata(MetaDataEntry);
				return true;
			}
		}
		return false;
	}

public:

	/** See OnMouseButtonDown event */
	SLATECORE_API void SetOnMouseButtonDown(FPointerEventHandler EventHandler);

	/** See OnMouseButtonUp event */
	SLATECORE_API void SetOnMouseButtonUp(FPointerEventHandler EventHandler);

	/** See OnMouseMove event */
	SLATECORE_API void SetOnMouseMove(FPointerEventHandler EventHandler);

	/** See OnMouseDoubleClick event */
	SLATECORE_API void SetOnMouseDoubleClick(FPointerEventHandler EventHandler);

	/** See OnMouseEnter event */
	SLATECORE_API void SetOnMouseEnter(FNoReplyPointerEventHandler EventHandler);

	/** See OnMouseLeave event */
	SLATECORE_API void SetOnMouseLeave(FSimpleNoReplyPointerEventHandler EventHandler);

public:

	// Widget Inspector and debugging methods

	/** @return A String representation of the widget */
	SLATECORE_API virtual FString ToString() const;

	/** @return A String of the widget's type */
	SLATECORE_API FString GetTypeAsString() const;

	/** @return The widget's type as an FName ID */
	SLATECORE_API FName GetType() const;

	virtual bool IsTableView() const { return false; }

	/** @return A String of the widget's code location in readable format "BaseFileName(LineNumber)" */
	SLATECORE_API virtual FString GetReadableLocation() const;

	/** @return An FName of the widget's code location (full path with number == line number of the file) */
	SLATECORE_API FName GetCreatedInLocation() const;

	/** @return The name this widget was tagged with */
	SLATECORE_API virtual FName GetTag() const;

#if STATS
	size_t GetAllocSize() const { return AllocSize; }
#endif

#if UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
	/** @return The widget's id */
	uint64 GetId() const { return UniqueIdentifier; }
#endif

	/** @return the Foreground color that this widget sets; unset options if the widget does not set a foreground color */
	SLATECORE_API virtual FSlateColor GetForegroundColor() const;

	/** @return the Foreground color that this widget sets when this widget or any of its ancestors are disabled; unset options if the widget does not set a foreground color */
	SLATECORE_API virtual FSlateColor GetDisabledForegroundColor() const;

	//UE_DEPRECATED(4.23, "GetCachedGeometry has been deprecated, use GetTickSpaceGeometry instead")
	SLATECORE_API const FGeometry& GetCachedGeometry() const;

	/**
	 * Gets the last geometry used to Tick the widget.  This data may not exist yet if this call happens prior to
	 * the widget having been ticked/painted, or it may be out of date, or a frame behind.
	 *
	 * We recommend not to use this data unless there's no other way to solve your problem.  Normally in Slate we
	 * try and handle these issues by making a dependent widget part of the hierarchy, as to avoid frame behind
	 * or what are referred to as hysteresis problems, both caused by depending on geometry from the previous frame
	 * being used to advise how to layout a dependent object the current frame.
	 */
	SLATECORE_API const FGeometry& GetTickSpaceGeometry() const;

	/**
	 * Gets the last geometry used to Tick the widget.  This data may not exist yet if this call happens prior to
	 * the widget having been ticked/painted, or it may be out of date, or a frame behind.
 	 */
	SLATECORE_API const FGeometry& GetPaintSpaceGeometry() const;

	/** Returns the clipping state to clip this widget against its parent */
	const TOptional<FSlateClippingState>& GetCurrentClippingState() const { return PersistentState.InitialClipState; }

	/** Is this widget derivative of SWindow */
	virtual bool Advanced_IsWindow() const { return false; }
	virtual bool Advanced_IsInvalidationRoot() const { return false; }
	virtual const FSlateInvalidationRoot* Advanced_AsInvalidationRoot() const { return nullptr; }


#if UE_SLATE_WITH_DYNAMIC_INVALIDATION

	/** Returns true if this widget's class supports invalidation. This value never changes. */
	SLATECORE_API bool SupportsInvalidation() const;

	enum class EInvalidationStrategy : uint8
	{
		/** Use the last cached value unless it needs refreshing. */
		UseCachedValue,
		/** Check again if this widget and its direct children support invalidation. */
		RecomputeSelf,
	};

	enum class ECachedInvalidationResult : uint8
	{
		/** The last result has been invalidated and needs to be refreshed. This will happen in the next call to SupportsInvalidationRecursive. */
		NeedsRefresh,
		/** Used internally in SupportsInvalidationRecursive to avoid refreshing the same widget multiple times. */
		Refreshing,
		/** This widget and all its descendants support invalidation and have a valid parent. */
		SupportsInvalidation,
		/** The class of this widget does not support invalidation. This result never changes. */
		ClassNotSupportingInvalidation,
		/** Descendants of this widget are not supporting invalidation or have invalid parents. */
		ChildrenNotSupportingInvalidation,
		/**
		 * This widget currently has an invalid parent, meaning it is a child of widget A but its Parent points to another Widget B (or null).
		 * It is likely the case that this widget has been reused in multiple slots which is not allowed with invalidation.
		 */
		InvalidParent,
	};

	/**
	 * Returns true if this widget and all of its children support invalidation.
	 * This value is cached, and will be reused if InInvalidationStrategy is set to UseCachedValue.
	 */
	SLATECORE_API virtual bool SupportsInvalidationRecursive(EInvalidationStrategy InInvalidationStrategy) const;
private:
	/** Fast path to invalidate all ancestors when a descendant widget is not supporting invalidation. */
	void SetSelfAndParentsAsNotSupportingInvalidation();

	/** Sets this widget and its ancestors as needing to have their SupportsInvalidationRecursive refreshed. */
	void InvalidateCachedSupportsInvalidation();

	/** Store the value from the TWidgetTypeTraits<>::SupportsInvalidation just after the widget is created */
	SLATECORE_API void SetClassSupportsInvalidation(bool bClassSupportsInvalidation);

public:
	ECachedInvalidationResult Debug_GetCachedInvalidationResult() const { return CachedSupportsInvalidationRecursive; }
#endif // ~UE_SLATE_WITH_DYNAMIC_INVALIDATION

protected:
	/**
	 * Hidden default constructor.
	 *
	 * Use SNew(WidgetClassName) to instantiate new widgets.
	 *
	 * @see SNew
	 */
	SLATECORE_API SWidget();

	/** Construct a SWidget based on initial parameters. */
	SLATECORE_API void SWidgetConstruct(const FSlateBaseNamedArgs& Args);

	/** Is the widget construction completed (did we called and returned from the Construct() function) */
	bool IsConstructed() const { return bIsDeclarativeSyntaxConstructionCompleted; }

	/** 
	 * Find the geometry of a descendant widget. This method assumes that WidgetsToFind are a descendants of this widget.
	 * Note that not all widgets are guaranteed to be found; OutResult will contain null entries for missing widgets.
	 *
	 * @param MyGeometry      The geometry of this widget.
	 * @param WidgetsToFind   The widgets whose geometries we wish to discover.
	 * @param OutResult       A map of widget references to their respective geometries.
	 * @return True if all the WidgetGeometries were found. False otherwise.
	 */
	SLATECORE_API bool FindChildGeometries( const FGeometry& MyGeometry, const TSet< TSharedRef<SWidget> >& WidgetsToFind, TMap<TSharedRef<SWidget>, FArrangedWidget>& OutResult ) const;

	/**
	 * Actual implementation of FindChildGeometries.
	 *
	 * @param MyGeometry      The geometry of this widget.
	 * @param WidgetsToFind   The widgets whose geometries we wish to discover.
	 * @param OutResult       A map of widget references to their respective geometries.
	 */
	SLATECORE_API void FindChildGeometries_Helper( const FGeometry& MyGeometry, const TSet< TSharedRef<SWidget> >& WidgetsToFind, TMap<TSharedRef<SWidget>, FArrangedWidget>& OutResult ) const;

	/** 
	 * Find the geometry of a descendant widget. This method assumes that WidgetToFind is a descendant of this widget.
	 *
	 * @param MyGeometry   The geometry of this widget.
	 * @param WidgetToFind The widget whose geometry we wish to discover.
	 * @return the geometry of WidgetToFind.
	 */
	SLATECORE_API FGeometry FindChildGeometry( const FGeometry& MyGeometry, TSharedRef<SWidget> WidgetToFind ) const;

	/** @return The index of the child that the mouse is currently hovering */
	static SLATECORE_API int32 FindChildUnderMouse( const FArrangedChildren& Children, const FPointerEvent& MouseEvent );

	/** @return The index of the child that is under the specified position */
	static SLATECORE_API int32 FindChildUnderPosition(const FArrangedChildren& Children, const UE::Slate::FDeprecateVector2DParameter& ArrangedSpacePosition);

	/** 
	 * Determines if this widget should be enabled.
	 * 
	 * @param InParentEnabled	true if the parent of this widget is enabled
	 * @return true if the widget is enabled
	 */
	bool ShouldBeEnabled( bool InParentEnabled ) const
	{
		// This widget should be enabled if its parent is enabled and it is enabled
		return InParentEnabled && IsEnabled();
	}

	/** @return a brush to draw focus, nullptr if no focus drawing is desired */
	SLATECORE_API virtual const FSlateBrush* GetFocusBrush() const;

	/**
	 * Recomputes the volatility of the widget.  If you have additional state you automatically want to make
	 * the widget volatile, you should sample that information here.
	 */
	virtual bool ComputeVolatility() const { return false; }

	/**
	 * Protected static helper to allow widgets to access the visibility attribute of other widgets directly
	 * 
	 * @param Widget The widget to get the visibility attribute of
	 */
	static TAttribute<EVisibility> AccessWidgetVisibilityAttribute(const TSharedRef<SWidget>& Widget)
	{
		return Widget->VisibilityAttribute.ToAttribute(Widget.Get());
	}

	/**
	 * Called when clipping is changed.  Should be used to forward clipping states onto potentially
	 * hidden children that actually are responsible for clipping the content.
	 */
	SLATECORE_API virtual void OnClippingChanged();

private:

	/**
	 * The widget should respond by populating the OutDrawElements array with FDrawElements
	 * that represent it and any of its children. Called by the non-virtual OnPaint to enforce pre/post conditions
	 * during OnPaint.
	 *
	 * @param Args              All the arguments necessary to paint this widget (@todo umg: move all params into this struct)
	 * @param AllottedGeometry  The FGeometry that describes an area in which the widget should appear.
	 * @param MyCullingRect     The rectangle representing the bounds currently being used to completely cull widgets.  Unless IsChildWidgetCulled(...) returns true, you should paint the widget. 
	 * @param OutDrawElements   A list of FDrawElements to populate with the output.
	 * @param LayerId           The Layer onto which this widget should be rendered.
	 * @param InColorAndOpacity Color and Opacity to be applied to all the descendants of the widget being painted
	 * @param bParentEnabled	True if the parent of this widget is enabled.
	 * @return The maximum layer ID attained by this widget or any of its children.
	 */
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const = 0;

	/**
	 * Compute the Geometry of all the children and add populate the ArrangedChildren list with their values.
	 * Each type of Layout panel should arrange children based on desired behavior.
	 *
	 * @param AllottedGeometry    The geometry allotted for this widget by its parent.
	 * @param ArrangedChildren    The array to which to add the WidgetGeometries that represent the arranged children.
	 */
	virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const = 0;

	SLATECORE_API void Prepass_Internal(float LayoutScaleMultiplier);

protected:
	float GetPrepassLayoutScaleMultiplier() const
	{
		return bPrepassLayoutScaleMultiplierSet ? PrepassLayoutScaleMultiplierValue : 1.0f;
	}
	
	SLATECORE_API void Prepass_ChildLoop(float InLayoutScaleMultiplier, FChildren* MyChildren);

public:
	/**
	 * Registers an "active timer" delegate that will execute at some regular interval. TickFunction will not be called until the specified interval has elapsed once.
	 * A widget can register as many delegates as it needs. Be careful when registering to avoid duplicate active timers.
	 * 
	 * An active timer can be UnRegistered in one of three ways:
	 *   1. Call UnRegisterActiveTimer using the active timer handle that is returned here.
	 *   2. Have your delegate return EActiveTimerReturnType::Stop.
	 *   3. Destroying the widget
	 * 
	 * Active Timers
	 * --------------
	 * Slate may go to sleep when there is no user interaction for some time to save power.
	 * However, some UI elements may need to "drive" the UI even when the user is not providing any input
	 * (ie, animations, viewport rendering, async polling, etc). A widget notifies Slate of this by
	 * registering an "Active Timer" that is executed at a specified frequency to drive the UI.
	 * In this way, slate can go to sleep when there is no input and no active timer needs to fire.
	 * When any active timer needs to fire, all of Slate will do a Tick and Paint pass.
	 * 
	 * @param Period The time period to wait between each execution of the timer. Pass zero to fire the timer once per frame.
	 *                      If an interval is missed, the delegate is NOT called more than once.
	 * @param TimerFunction The active timer delegate to call every Period seconds.
	 * @return An active timer handle that can be used to UnRegister later.
	 */
	SLATECORE_API TSharedRef<FActiveTimerHandle> RegisterActiveTimer( float TickPeriod, FWidgetActiveTimerDelegate TickFunction );

	/**
	 * Unregisters an active timer handle. This is optional, as the delegate can UnRegister itself by returning EActiveTimerReturnType::Stop.
	 */
	SLATECORE_API void UnRegisterActiveTimer( const TSharedRef<FActiveTimerHandle>& ActiveTimerHandle );
	
	/** Does this widget have any active timers? */
	bool HasActiveTimers() const
	{
		return bHasActiveTimers;
	}

private:

	/** Iterates over the active timer handles on the widget and executes them if their interval has elapsed. */
	SLATECORE_API void ExecuteActiveTimers(double CurrentTime, float DeltaTime);

protected:
	/**
	 * Performs the attribute assignment and invalidates the widget minimally based on what actually changed.  So if the boundness of the attribute didn't change
	 * volatility won't need to be recalculated.  Returns true if the value changed.
	 */
	template<typename TargetValueType, typename SourceValueType>
	bool SetAttribute(TAttribute<TargetValueType>& TargetValue, const TAttribute<SourceValueType>& SourceValue, EInvalidateWidgetReason BaseInvalidationReason)
	{
		return SetWidgetAttribute(*this, TargetValue, SourceValue, BaseInvalidationReason);
	}

	/** @return an attribute reference of EnabledStateAttribute */
	TSlateAttributeRef<bool> GetEnabledStateAttribute() const { return TSlateAttributeRef<bool>(SharedThis(this), EnabledStateAttribute); }
	/** @return an attribute reference of HoveredAttribute */
	TSlateAttributeRef<bool> GetHoveredAttribute() const { return TSlateAttributeRef<bool>(SharedThis(this), HoveredAttribute); }
	/** @return an attribute reference of VisibilityAttribute */
	TSlateAttributeRef<EVisibility> GetVisibilityAttribute() const { return TSlateAttributeRef<EVisibility>(SharedThis(this), VisibilityAttribute); }
	/** @return an attribute reference of RenderTransformAttribute */
	TSlateAttributeRef<TOptional<FSlateRenderTransform>> GetRenderTransformAttribute() const { return TSlateAttributeRef<TOptional<FSlateRenderTransform>>(SharedThis(this), RenderTransformAttribute); }
	/** @return an attribute reference of RenderTransformPivotAttribute */
	TSlateAttributeRef<FVector2D> GetRenderTransformPivotAttribute() const { return TSlateAttributeRef<FVector2D>(SharedThis(this), RenderTransformPivotAttribute); }

protected:
	/** Dtor ensures that active timer handles are UnRegistered with the SlateApplication. */
	SLATECORE_API virtual ~SWidget();

private:
	/** Handle to the proxy when on the fast path */
	mutable FWidgetProxyHandle FastPathProxyHandle;

protected:
	/** Can the widget ever support keyboard focus */
	uint8 bCanSupportFocus : 1;

	/**
	 * Can the widget ever support children?  This will be false on SLeafWidgets, 
	 * rather than setting this directly, you should probably inherit from SLeafWidget.
	 */
	uint8 bCanHaveChildren : 1;

	/**
	  * Some widgets might be a complex hierarchy of child widgets you never see.  Some of those widgets
	  * would expose their clipping option normally, but may not personally be responsible for clipping
	  * so even though it may be set to clip, this flag is used to inform painting that this widget doesn't
	  * really do the clipping.
	  */
	uint8 bClippingProxy : 1;

private:
	/**
	 * Whether this widget is a "tool tip force field".  That is, tool-tips should never spawn over the area
	 * occupied by this widget, and will instead be repelled to an outside edge
	 */
	uint8 bToolTipForceFieldEnabled : 1;

	/** Should we be forcing this widget to be volatile at all times and redrawn every frame? */
	uint8 bForceVolatile : 1;

	/** The last cached volatility of this widget.  Cached so that we don't need to recompute volatility every frame. */
	uint8 bCachedVolatile : 1;

	/** If we're owned by a volatile widget, we need inherit that volatility and use as part of our volatility, but don't cache it. */
	uint8 bInheritedVolatility : 1;

	/** Are we currently updating the desired size? */
	uint8 bNeedsPrepass : 1;

	/** Is there at least one SlateAttribute currently registered. */
	uint8 bHasRegisteredSlateAttribute : 1;

	/** Are bound Slate Attributes will be updated once per frame. */
	uint8 bEnabledAttributesUpdate : 1;

	/** The SNew or SAssignedNew construction is completed. */
	uint8 bIsDeclarativeSyntaxConstructionCompleted : 1;

	/** Is the attribute IsHovered is set? */
	uint8 bIsHoveredAttributeSet : 1;

	/** Is there at least one ActiveTimer currently registered. */
	uint8 bHasActiveTimers : 1;

	/** Is the DesiredSize set. */
	uint8 bDesiredSizeSet : 1;

	/** Is the PrepassLayoutScaleMultiplier set. */
	uint8 bPrepassLayoutScaleMultiplierSet : 1;

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	template<class WidgetType, typename RequiredArgsPayloadType>
	friend struct TSlateDecl;

	// We need to keep this information for each widget as not all widgets have a class to store this information
	uint8 bSupportsInvalidation : 1 = false;
#endif

	uint8 bIsProjectContent : 1;
	uint8 bIsProjectContentParent : 1;

protected:
	uint8 bHasCustomPrepass : 1;

	uint8 bHasRelativeLayoutScale : 1;
	
	/** if this widget should always invalidate the prepass step when volatile */
	uint8 bVolatilityAlwaysInvalidatesPrepass : 1;

#if WITH_ACCESSIBILITY
	/** All variables surrounding how this widget is exposed to the platform's accessibility API. */
	uint8 bCanChildrenBeAccessible : 1;
	EAccessibleBehavior AccessibleBehavior;
	EAccessibleBehavior AccessibleSummaryBehavior;
#endif

#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
	/** The cached value of the last SupportsInvalidationRecursive call. */
	mutable ECachedInvalidationResult CachedSupportsInvalidationRecursive = ECachedInvalidationResult::ChildrenNotSupportingInvalidation;
#endif

	/**
	 * Set to true if all content of the widget should clip to the bounds of this widget.
	 */
	EWidgetClipping Clipping;
	
	/**
	 * When set to EPixelSnappingMethod::SnapToPixel, the widget is drawn at the nearest pixel. Will improve sharpness 
	 * but could show a stepping effect when moved in an animation.  By default everything in slate is Inherit, and the default
	 * state all things inherit is SnapToPixel.
	 */
	EWidgetPixelSnapping PixelSnappingMethod;

private:
	/** Flow direction preference */
	EFlowDirectionPreference FlowDirectionPreference;

	/** The different updates this widget needs next frame. */
	EWidgetUpdateFlags UpdateFlags;

	/** Is this widget visible, hidden or collapsed */
	TSlateAttribute<EVisibility> VisibilityAttribute;

	/** Whether or not this widget is enabled */
	TSlateAttribute<bool> EnabledStateAttribute;

	/** Whether or not this widget is hovered */
	TSlateAttribute<bool> HoveredAttribute;

	//~ See PrepassLayoutScaleMultiplier
	float PrepassLayoutScaleMultiplierValue;

	mutable FSlateWidgetPersistentState PersistentState;

	/** Stores the ideal size this widget wants to be. See bDesiredSizeSet. */
	FVector2f DesiredSize;

	/** Render transform pivot of this widget (in normalized local space) */
	TSlateAttribute<FVector2D> RenderTransformPivotAttribute;

	/** Render transform of this widget. TOptional<> to allow code to skip expensive overhead if there is no render transform applied. */
	TSlateAttribute<TOptional<FSlateRenderTransform>> RenderTransformAttribute;

protected:
	/**
	* Can be used to enlarge the culling bounds of this widget (pre-intersection), this can be useful if you've got
	* children that you know are using rendering transforms to render outside their standard bounds, if that happens
	* it's possible the parent might be culled before the descendant widget is entirely off screen.  For those cases,
	* you should extend the bounds of the culling area to add a bit more slack to how culling is performed to this panel.
	*/
	FMargin CullingBoundsExtension;

	/** The opacity of the widget. Automatically applied during rendering. */
	float RenderOpacity;

private:
	/** Metadata associated with this widget. */
	TArray<TSharedRef<ISlateMetaData>> MetaData;

	/** Pointer to this widgets parent widget.  If it is null this is a root widget or it is not in the widget tree */
	TWeakPtr<SWidget> ParentWidgetPtr;

	/** Tag for this widget */
	FName Tag;

	/** Debugging information on the type of widget we're creating for the Widget Reflector. */
	FName TypeOfWidget;

protected:
#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.5, "Direct access to PrepassLayoutScaleMultiplier is deprecated. Use the GetPrepassLayoutScaleMultiplier. The scale multitplier can only be set in the prepass.")
	TOptional<float> PrepassLayoutScaleMultiplier;
#endif

private: 
#if !UE_BUILD_SHIPPING
	/** Full file path (and line) in which this widget was created */
	FName CreatedInLocation;
#endif

#if WITH_SLATE_DEBUGGING
	/** The last time this widget got painted. */
	uint32 LastPaintFrame = 0;
	/** The last time this widget got prepassed. */
	uint32 LastPrepassFrame = 0;
	/** Flag to help detect when we access an invalid Widget. */
	uint8 Debug_DestroyedTag = 0xDC;
#endif // WITH_SLATE_DEBUGGING

#if UE_SLATE_TRACE_ENABLED
	/**
	 * If the widget info is sent when a trace is not active, it will be ignored.
	 * In this scenario, if the info is not re-sent the trace will not function correctly.
	 * Track which traces we have sent info for so we can re-send when needed.
	 */
	mutable uint8 Debug_LastTraceInfoSent = 0;
#endif // UE_SLATE_TRACE_ENABLED

#if UE_SLATE_WITH_WIDGET_UNIQUE_IDENTIFIER
	/** The widget's id */
	uint64 UniqueIdentifier;
#endif

#if STATS
	size_t AllocSize;
#endif

#if STATS || ENABLE_STATNAMEDEVENTS
	/** Stat id of this object, 0 if nobody asked for it yet */
	mutable TStatId				StatID;
#endif

#if ENABLE_STATNAMEDEVENTS
	mutable PROFILER_CHAR* StatIDStringStorage;
#endif
};

//=================================================================
// FGeometry Arranged Widget Inlined Functions
//=================================================================

inline FArrangedWidget FGeometry::MakeChild(const TSharedRef<SWidget>& ChildWidget, const UE::Slate::FDeprecateVector2DParameter& InLocalSize, const FSlateLayoutTransform& LayoutTransform) const
{
	// If there is no render transform set, use the simpler MakeChild call that doesn't bother concatenating the render transforms.
	// This saves a significant amount of overhead since every widget does this, and most children don't have a render transform.
	const TOptional<FSlateRenderTransform> RenderTransform = ChildWidget->GetRenderTransformWithRespectToFlowDirection();
	if (RenderTransform.IsSet() )
	{
		const FVector2f RenderTransformPivot = UE::Slate::CastToVector2f(ChildWidget->GetRenderTransformPivotWithRespectToFlowDirection());
		return FArrangedWidget(ChildWidget, MakeChild(InLocalSize, LayoutTransform, RenderTransform.GetValue(), RenderTransformPivot));
	}
	else
	{
		return FArrangedWidget(ChildWidget, MakeChild(InLocalSize, LayoutTransform));
	}
}

inline FArrangedWidget FGeometry::MakeChild(const TSharedRef<SWidget>& ChildWidget, const FLayoutGeometry& LayoutGeometry) const
{
	return MakeChild(ChildWidget, FVector2f(LayoutGeometry.GetSizeInLocalSpace()), LayoutGeometry.GetLocalToParentTransform());
}

inline FArrangedWidget FGeometry::MakeChild(const TSharedRef<SWidget>& ChildWidget, const UE::Slate::FDeprecateVector2DParameter& ChildOffset, const UE::Slate::FDeprecateVector2DParameter& InLocalSize, float ChildScale) const
{
	// Since ChildOffset is given as a LocalSpaceOffset, we MUST convert this offset into the space of the parent to construct a valid layout transform.
	// The extra TransformPoint below does this by converting the local offset to an offset in parent space.
	return MakeChild(ChildWidget, UE::Slate::CastToVector2f(InLocalSize), FSlateLayoutTransform(ChildScale, TransformPoint(ChildScale, UE::Slate::CastToVector2f(ChildOffset))));
}

template<typename TargetValueType, typename SourceValueType>
bool SetWidgetAttribute(SWidget& ThisWidget, TAttribute<TargetValueType>& TargetValue, const TAttribute<SourceValueType>& SourceValue, EInvalidateWidgetReason BaseInvalidationReason)
{
	if (!TargetValue.IdenticalTo(SourceValue))
	{
		const bool bWasBound = TargetValue.IsBound();
		const bool bBoundnessChanged = bWasBound != SourceValue.IsBound();
		TargetValue = SourceValue;

		EInvalidateWidgetReason InvalidateReason = BaseInvalidationReason;
		if (bBoundnessChanged)
		{
			InvalidateReason |= EInvalidateWidgetReason::PaintAndVolatility;
		}

		ThisWidget.Invalidate(InvalidateReason);
		return true;
	}

	return false;
}


#if UE_SLATE_WITH_DYNAMIC_INVALIDATION

inline const TCHAR* LexToString(SWidget::EInvalidationStrategy Strategy)
{
#define INVALIDATION_STRATEGY_LEXTOSTRING(Name) case SWidget::EInvalidationStrategy::Name: return TEXT(#Name);
	switch (Strategy)
	{
		INVALIDATION_STRATEGY_LEXTOSTRING(UseCachedValue)
		INVALIDATION_STRATEGY_LEXTOSTRING(RecomputeSelf)
	default:
		return TEXT("Unknown");
	}
#undef INVALIDATION_STRATEGY_LEXTOSTRING
}

inline const TCHAR* LexToString(SWidget::ECachedInvalidationResult Result)
{
#define INVALIDATION_RESULT_LEXTOSTRING(Name) case SWidget::ECachedInvalidationResult::Name: return TEXT(#Name);
	switch (Result)
	{
		INVALIDATION_RESULT_LEXTOSTRING(NeedsRefresh)
		INVALIDATION_RESULT_LEXTOSTRING(Refreshing)
		INVALIDATION_RESULT_LEXTOSTRING(SupportsInvalidation)
		INVALIDATION_RESULT_LEXTOSTRING(ClassNotSupportingInvalidation)
		INVALIDATION_RESULT_LEXTOSTRING(ChildrenNotSupportingInvalidation)
		INVALIDATION_RESULT_LEXTOSTRING(InvalidParent)
	default:
		return TEXT("Unknown");
	}
#undef INVALIDATION_RESULT_LEXTOSTRING
}

#endif // ~UE_SLATE_WITH_DYNAMIC_INVALIDATION