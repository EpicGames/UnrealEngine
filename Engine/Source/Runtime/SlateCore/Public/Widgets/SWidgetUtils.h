// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

#if STATS

/** Structure used to track time spent by a SWidget */
struct FScopeCycleCounterSWidget : public FCycleCounter
{
	/**
	 * Constructor, starts timing
	 */
	inline FScopeCycleCounterSWidget(const SWidget* Widget)
	{
		if (Widget)
		{
#if CPUPROFILERTRACE_ENABLED
			const bool bCpuChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel);
#else
			const bool bCpuChannelEnabled = false;
#endif
			bool bStarted = false;
			TStatId WidgetStatId = Widget->GetStatID(bCpuChannelEnabled);
			if (FThreadStats::IsCollectingData(WidgetStatId))
			{
				Start(WidgetStatId);
				bStarted = true;
			}

#if CPUPROFILERTRACE_ENABLED
			if (!bStarted && bCpuChannelEnabled && WidgetStatId.IsValidStat())
			{
				StartTrace(WidgetStatId.GetName(), WidgetStatId.GetStatDescriptionWIDE());
			}
#endif
		}
	}

	/**
	 * Updates the stat with the time spent
	 */
	inline ~FScopeCycleCounterSWidget()
	{
		Stop();
	}
};

#define SCOPE_CYCLE_SWIDGET(Object) \
	FScopeCycleCounterSWidget ANONYMOUS_VARIABLE(SlateWidgetCycleCount_) (Object);

#elif UE_USE_LIGHTWEIGHT_STATS

struct FScopeCycleCounterSWidget
{
	FScopeCycleCounter ScopeCycleCounter;
#if CPUPROFILERTRACE_ENABLED
	bool bPop = false;
#endif

	inline FScopeCycleCounterSWidget(const SWidget* Widget)
		: ScopeCycleCounter(Widget ? Widget->GetStatID().StatString : nullptr)
	{
#if CPUPROFILERTRACE_ENABLED
		if (GCycleStatsShouldEmitNamedEvents && UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel) && Widget)
		{
			const TStatId StatId = Widget->GetStatID();
			if (StatId.IsValidStat())
			{
				bPop = true;
				FCpuProfilerTrace::OutputBeginDynamicEvent(StatId.StatString);
			}
		}
#endif
	}

	inline ~FScopeCycleCounterSWidget()
	{
#if CPUPROFILERTRACE_ENABLED
		if (bPop)
		{
			FCpuProfilerTrace::OutputEndEvent();
		}
#endif
	}
};

#define SCOPE_CYCLE_SWIDGET(Object) \
	FScopeCycleCounterSWidget ANONYMOUS_VARIABLE(SlateWidgetCycleCount_) (Object);

#else

#define SCOPE_CYCLE_SWIDGET(Object)

#endif
