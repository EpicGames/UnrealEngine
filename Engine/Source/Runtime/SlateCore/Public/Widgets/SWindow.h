// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Attribute.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Layout/SlateRect.h"
#include "Layout/Visibility.h"
#include "Rendering/SlateLayoutTransform.h"
#include "Layout/Geometry.h"
#include "Input/CursorReply.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/GenericWindowDefinition.h"
#include "GenericPlatform/GenericWindow.h"
#include "Input/Reply.h"
#include "Rendering/RenderingCommon.h"
#include "Types/SlateStructs.h"
#include "Animation/CurveSequence.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SlotBase.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "FastUpdate/SlateInvalidationRoot.h"

class FActiveTimerHandle;
class FHittestGrid;
class FPaintArgs;
class FSlateWindowElementList;
class FWidgetPath;
class IWindowTitleBar;
class SPopupLayer;
class SWindow;
class SImage;

enum class EUpdateFastPathReason : uint8;

/** Notification that a window has been activated */
DECLARE_DELEGATE( FOnWindowActivated );
DECLARE_MULTICAST_DELEGATE( FOnWindowActivatedEvent );

/** Notification that a window has been deactivated */
DECLARE_DELEGATE( FOnWindowDeactivated );
DECLARE_MULTICAST_DELEGATE( FOnWindowDeactivatedEvent );

/** Notification that a window is about to be closed */
DECLARE_DELEGATE_OneParam( FOnWindowClosed, const TSharedRef<SWindow>& );
DECLARE_MULTICAST_DELEGATE_OneParam( FOnWindowClosedEvent, const TSharedRef<SWindow>& );

/** Notification that a window has been moved */
DECLARE_DELEGATE_OneParam( FOnWindowMoved, const TSharedRef<SWindow>& );

/** Override delegate for RequestDestroyWindow */
DECLARE_DELEGATE_OneParam( FRequestDestroyWindowOverride, const TSharedRef<SWindow>& );

/** Called when we need to switch game worlds for a window */
DECLARE_DELEGATE_RetVal_OneParam( int32, FOnSwitchWorldHack, int32 );


/** Enum to describe how to auto-center an SWindow */
enum class EAutoCenter : uint8
{
	/** Don't auto-center the window */
	None,

	/** Auto-center the window on the primary work area */
	PrimaryWorkArea,

	/** Auto-center the window on the preferred work area, determined using GetPreferredWorkArea() */
	PreferredWorkArea,
};


/** Enum to describe how windows are sized */
enum class ESizingRule : uint8
{
	/* The windows size fixed and cannot be resized **/
	FixedSize,

	/** The window size is computed from its content and cannot be resized by users */
	Autosized,

	/** The window can be resized by users */
	UserSized,
};

namespace SWindowDefs
{
	/** Height of a Slate window title bar, in pixels */
	inline constexpr float DefaultTitleBarSize = 34.0f;
}

/** Proxy structure to handle deprecated construction from bool */
struct FWindowTransparency
{
	FWindowTransparency(EWindowTransparency In) : Value(In) {}
	
	EWindowTransparency Value;
};

/**
 * Simple overlay layer to allow content to be laid out on a Window or similar widget.
 */
class FOverlayPopupLayer : public FPopupLayer
{
public:
	FOverlayPopupLayer(const TSharedRef<SWindow>& InitHostWindow, const TSharedRef<SWidget>& InitPopupContent, TSharedPtr<SOverlay> InitOverlay);

	virtual void Remove() override;
	virtual FSlateRect GetAbsoluteClientRect() override;

private:
	TSharedPtr<SWindow> HostWindow;
	TSharedPtr<SOverlay> Overlay;
};


/**
 * Popups, tooltips, drag and drop decorators all can be executed without creating a new window.
 * This slot along with the SWindow::AddPopupLayerSlot() API enabled it.
 */
struct FPopupLayerSlot : public TSlotBase<FPopupLayerSlot>
{
public:
	FPopupLayerSlot()
		: TSlotBase<FPopupLayerSlot>()
		, DesktopPosition_Attribute(FVector2D::ZeroVector)
		, WidthOverride_Attribute()
		, HeightOverride_Attribute()
		, Scale_Attribute(1.0f)
		, Clamp_Attribute(false)
		, ClampBuffer_Attribute(FVector2D::ZeroVector)
	{}

	SLATE_SLOT_BEGIN_ARGS(FPopupLayerSlot, TSlotBase<FPopupLayerSlot>)
		/** Pixel position in desktop space */
		SLATE_ATTRIBUTE(FVector2D, DesktopPosition)
		/** Width override in pixels */
		SLATE_ATTRIBUTE(float, WidthOverride)
		/** Width override in pixels */
		SLATE_ATTRIBUTE(float, HeightOverride)
		/** DPI scaling to be applied to the contents of this slot */
		SLATE_ATTRIBUTE(float, Scale)
		/** Should this slot be kept within the parent window */
		SLATE_ATTRIBUTE(bool, ClampToWindow)
		/** If this slot is kept within the parent window, how far from the edges should we clamp it */
		SLATE_ATTRIBUTE(FVector2D, ClampBuffer)
	SLATE_SLOT_END_ARGS()

	void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
	{
		if (InArgs._DesktopPosition.IsSet())
		{
			SetDesktopPosition(MoveTemp(InArgs._DesktopPosition));
		}
		if (InArgs._WidthOverride.IsSet())
		{
			SetWidthOverride(MoveTemp(InArgs._WidthOverride));
		}
		if (InArgs._HeightOverride.IsSet())
		{
			SetHeightOverride(MoveTemp(InArgs._HeightOverride));
		}
		if (InArgs._Scale.IsSet())
		{
			SetScale(MoveTemp(InArgs._Scale));
		}
		if (InArgs._ClampToWindow.IsSet())
		{
			SetClampToWindow(MoveTemp(InArgs._ClampToWindow));
		}
		if (InArgs._ClampBuffer.IsSet())
		{
			SetClampBuffer(MoveTemp(InArgs._ClampBuffer));
		}
		TSlotBase<FPopupLayerSlot>::Construct(SlotOwner, MoveTemp(InArgs));
	}

	/** Pixel position in desktop space */
	void SetDesktopPosition(TAttribute<FVector2D> InDesktopPosition)
	{
		DesktopPosition_Attribute = MoveTemp(InDesktopPosition);
	}

	/** Width override in pixels */
	void SetWidthOverride(TAttribute<float> InWidthOverride)
	{
		WidthOverride_Attribute = MoveTemp(InWidthOverride);
	}

	/** Width override in pixels */
	void SetHeightOverride(TAttribute<float> InHeightOverride)
	{
		HeightOverride_Attribute = MoveTemp(InHeightOverride);
	}

	/** DPI scaling to be applied to the contents of this slot */
	void SetScale(TAttribute<float> InScale)
	{
		Scale_Attribute = MoveTemp(InScale);
	}

	/** Should this slot be kept within the parent window */
	void SetClampToWindow(TAttribute<bool> InClamp_Attribute)
	{
		Clamp_Attribute = MoveTemp(InClamp_Attribute);
	}

	/** If this slot is kept within the parent window, how far from the edges should we clamp it */
	void SetClampBuffer(TAttribute<FVector2D> InClampBuffer_Attribute)
	{
		ClampBuffer_Attribute = MoveTemp(InClampBuffer_Attribute);
	}

private:
	/** SPopupLayer arranges FPopupLayerSlots, so it needs to know all about */
	friend class SPopupLayer;
	/** TPanelChildren need access to the Widget member */
	friend class TPanelChildren<FPopupLayerSlot>;

	TAttribute<FVector2D> DesktopPosition_Attribute;
	TAttribute<float> WidthOverride_Attribute;
	TAttribute<float> HeightOverride_Attribute;
	TAttribute<float> Scale_Attribute;
	TAttribute<bool> Clamp_Attribute;
	TAttribute<FVector2D> ClampBuffer_Attribute;
};


/**
 * SWindow is a platform-agnostic representation of a top-level window.
 */
class SWindow
	: public SCompoundWidget
	, public FSlateInvalidationRoot
{

public:

	SLATE_BEGIN_ARGS( SWindow )
		: _Type( EWindowType::Normal )
		, _Style( &FCoreStyle::Get().GetWidgetStyle<FWindowStyle>("Window") )
		, _Title()
		, _bDragAnywhere( false )
		, _AutoCenter( EAutoCenter::PreferredWorkArea )
		, _ScreenPosition( FVector2f::ZeroVector )
		, _ClientSize( FVector2f::ZeroVector )
		, _AdjustInitialSizeAndPositionForDPIScale(true)
		, _SupportsTransparency( EWindowTransparency::None )
		, _InitialOpacity( 1.0f )
		, _IsInitiallyMaximized( false )
		, _IsInitiallyMinimized(false)
		, _SizingRule( ESizingRule::UserSized )
		, _IsPopupWindow( false )
		, _IsTopmostWindow( false )
		, _FocusWhenFirstShown(true)
		, _FocusUserIndex( INDEX_NONE )
		, _ActivationPolicy( EWindowActivationPolicy::Always )
		, _UseOSWindowBorder( false )
		, _HasCloseButton( true )
		, _SupportsMaximize( true )
		, _SupportsMinimize( true )
		, _ShouldPreserveAspectRatio( false )
		, _CreateTitleBar( true )
		, _SaneWindowPlacement( true )
		, _LayoutBorder( _Style->BorderPadding )
		, _UserResizeBorder(FMargin(5, 5, 5, 5))
		, _bManualManageDPI( false )
		, _CloseButtonToolTipText(NSLOCTEXT("SWindow", "Window_DefaultCloseButtonToolTip", "Close"))
	{
	}

		/** Type of this window */
		SLATE_ARGUMENT( EWindowType, Type )

		/** Style used to draw this window */
		SLATE_STYLE_ARGUMENT( FWindowStyle, Style )

		/** Title of the window */
		SLATE_ATTRIBUTE( FText, Title )

		/** When true, the window can be dragged from anywhere. */
		SLATE_ARGUMENT( bool, bDragAnywhere )

		/** The windows auto-centering mode. If set to anything other than None, then the
			ScreenPosition value will be ignored */
		SLATE_ARGUMENT( EAutoCenter, AutoCenter )

		/** Screen-space position where the window should be initially located. */
		SLATE_ARGUMENT( UE::Slate::FDeprecateVector2DParameter, ScreenPosition )

		/** What the initial size of the window should be. */
		SLATE_ARGUMENT( UE::Slate::FDeprecateVector2DParameter, ClientSize )

		/** If the initial ClientSize and ScreenPosition arguments should be automatically adjusted to account for DPI scale */
		SLATE_ARGUMENT( bool, AdjustInitialSizeAndPositionForDPIScale )

		/** Should this window support transparency */
		SLATE_ARGUMENT( FWindowTransparency, SupportsTransparency )

		/** The initial opacity of the window */
		SLATE_ARGUMENT( float, InitialOpacity )

		/** Is the window initially maximized */
		SLATE_ARGUMENT( bool, IsInitiallyMaximized )
		
		/** Is the window initially minimized */
		SLATE_ARGUMENT(bool, IsInitiallyMinimized)

		/** How the window should be sized */
		SLATE_ARGUMENT( ESizingRule, SizingRule )

		/** True if this should be a 'pop-up' window */
		SLATE_ARGUMENT( bool, IsPopupWindow )

		/** True if this window should always be on top of all other windows */
		SLATE_ARGUMENT(bool, IsTopmostWindow)

		/** Should this window be focused immediately after it is shown? */
		SLATE_ARGUMENT( bool, FocusWhenFirstShown )

		/** The index of the user that should focus this window on activation. INDEX_NONE for all users */
		SLATE_ARGUMENT( int32, FocusUserIndex )

		/** When should this window be activated upon being shown? */
		SLATE_ARGUMENT( EWindowActivationPolicy, ActivationPolicy )

		/** Use the default os look for the border of the window */
		SLATE_ARGUMENT( bool, UseOSWindowBorder )

		/** Does this window have a close button? */
		SLATE_ARGUMENT( bool, HasCloseButton )

		/** Can this window be maximized? */
		SLATE_ARGUMENT( bool, SupportsMaximize )
		
		/** Can this window be minimized? */
		SLATE_ARGUMENT( bool, SupportsMinimize )

		/** Should this window preserve its aspect ratio when resized by user? */
		SLATE_ARGUMENT( bool, ShouldPreserveAspectRatio )

		/** The smallest width this window can be in Desktop Pixel Units. */
		SLATE_ARGUMENT( TOptional<float>, MinWidth )
		
		/** The smallest height this window can be in Desktop Pixel Units. */
		SLATE_ARGUMENT( TOptional<float>, MinHeight )
		
		/** The biggest width this window can be in Desktop Pixel Units. */
		SLATE_ARGUMENT( TOptional<float>, MaxWidth )

		/** The biggest height this window can be in Desktop Pixel Units. */
		SLATE_ARGUMENT( TOptional<float>, MaxHeight )

		/** True if we should initially create a traditional title bar area.  If false, the user must embed the title
			area content into the window manually, taking into account platform-specific considerations!  Has no
			effect for certain types of windows (popups, tool-tips, etc.) */
		SLATE_ARGUMENT( bool, CreateTitleBar )

		/** If the window appears off screen or is too large to safely fit this flag will force realistic 
			constraints on the window and bring it back into view. */
		SLATE_ARGUMENT( bool, SaneWindowPlacement )

		/** The padding around the edges of the window applied to it's content. */
		SLATE_ARGUMENT(FMargin, LayoutBorder)

		/** The margin around the edges of the window that will be detected as places the user can grab to resize the window. */
		SLATE_ARGUMENT(FMargin, UserResizeBorder)

		/** true if this window will self handle any eventual DPI adjustments */
		SLATE_ARGUMENT(bool, bManualManageDPI)

		SLATE_DEFAULT_SLOT( FArguments, Content )

		/** Optional tooltip for the close button, when HasCloseButton is true. */
		SLATE_ATTRIBUTE(FText, CloseButtonToolTipText)

		SLATE_END_ARGS()

	/**
	 * Default constructor. Use SNew(SWindow) instead.
	 */
	SLATECORE_API SWindow();
	SLATECORE_API ~SWindow();

public:

	SLATECORE_API void Construct(const FArguments& InArgs);

	/**
	 * Make a tool tip window
	 *
	 * @return The new SWindow
	 */
	static SLATECORE_API TSharedRef<SWindow> MakeToolTipWindow();

	/**
	 * Make cursor decorator window
	 *
	 * @return The new SWindow
	 */
	static SLATECORE_API TSharedRef<SWindow> MakeCursorDecorator();

	/**
	 * Make cursor decorator window with a non-default style
	 *
	 * @param InStyle The style to use for the cursor decorator
	 * @return The new SWindow
	 */
	static SLATECORE_API TSharedRef<SWindow> MakeStyledCursorDecorator(const FWindowStyle& InStyle);

	/**
	 * Make a notification window
	 *
	 * @return The new SWindow
	 */
	static SLATECORE_API TSharedRef<SWindow> MakeNotificationWindow();

	/**
	 * @param ContentSize      The size of content that we want to accommodate 
	 *
	 * @return The size of the window necessary to accommodate the given content */
	static SLATECORE_API UE::Slate::FDeprecateVector2DResult ComputeWindowSizeForContent( UE::Slate::FDeprecateVector2DParameter ContentSize );

	/**
	 * Grabs the window type
	 *
	 * @return The window's type
	 */
	EWindowType GetType() const
	{
		return Type;
	}

	/**
	 * Grabs the current window title
	 *
	 * @return The window's title
	 */
	FText GetTitle() const
	{
		return Title.Get();
	}

	/**
	 * Sets the current window title
	 *
	 * @param InTitle	The new title of the window
	 */
	void SetTitle( const FText& InTitle )
	{
		Title = InTitle;
		if (NativeWindow.IsValid())
		{
			NativeWindow->SetText( *InTitle.ToString() );
		}
	}

	/** Paint the window and all of its contents. Not the same as Paint(). */
	SLATECORE_API int32 PaintWindow( double CurrentTime, float DeltaTime, FSlateWindowElementList& OutDrawElements, const FWidgetStyle& InWidgetStyle, bool bParentEnabled );

	/**
	 * Returns the size of the title bar as a Slate size parameter.  Does not take into account application scale!
	 *
	 * @return  Title bar size
	 */
	SLATECORE_API FOptionalSize GetTitleBarSize() const;

	/** @return the desired size in desktop pixels */
	SLATECORE_API UE::Slate::FDeprecateVector2DResult GetDesiredSizeDesktopPixels() const;

	/**	@return The initially desired screen position of the slate window */
	SLATECORE_API UE::Slate::FDeprecateVector2DResult GetInitialDesiredSizeInScreen() const;

	/**	@return The initially desired size of the slate window */
	SLATECORE_API UE::Slate::FDeprecateVector2DResult GetInitialDesiredPositionInScreen() const;

	/** Get the Geometry that describes this window. Windows in Slate are unique in that they know their own geometry. */
	SLATECORE_API FGeometry GetWindowGeometryInScreen() const;

	/** @return The geometry of the window in window space (i.e. position and AbsolutePosition are 0) */
	SLATECORE_API FGeometry GetWindowGeometryInWindow() const;

	/** @return the transform from local space to screen space (desktop space). */
	SLATECORE_API FSlateLayoutTransform GetLocalToScreenTransform() const;

	/** @return the transform from local space to window space, which is basically desktop space without the offset. Essentially contains the DPI scale. */
	SLATECORE_API FSlateLayoutTransform GetLocalToWindowTransform() const;

	/** @return The position of the window in screen space */
	SLATECORE_API UE::Slate::FDeprecateVector2DResult GetPositionInScreen() const;

	/** @return the size of the window in screen pixels */
	SLATECORE_API UE::Slate::FDeprecateVector2DResult GetSizeInScreen() const;

	/** @return the rectangle of the window for its non-maximized state */
	SLATECORE_API FSlateRect GetNonMaximizedRectInScreen() const;

	/** @return Rectangle that this window occupies in screen space */
	SLATECORE_API FSlateRect GetRectInScreen() const;

	/** @return Rectangle of the window's usable client area in screen space. */
	SLATECORE_API FSlateRect GetClientRectInScreen() const;

	/** @return the size of the window's usable client area. */
	SLATECORE_API UE::Slate::FDeprecateVector2DResult GetClientSizeInScreen() const;

	/** @return a clipping rectangle that represents this window in Window Space (i.e. always starts at 0,0) */
	SLATECORE_API FSlateRect GetClippingRectangleInWindow() const;

	/** Returns the margins used for the window border. This varies based on whether it's maximized or not. */
	SLATECORE_API FMargin GetWindowBorderSize( bool bIncTitleBar = false ) const;

	/** Returns the margins used for the window border if it's not maximized */
	SLATECORE_API FMargin GetNonMaximizedWindowBorderSize() const;

	/** Relocate the window to a screenspace position specified by NewPosition */
	SLATECORE_API void MoveWindowTo( UE::Slate::FDeprecateVector2DParameter NewPosition );
	/** Relocate the window to a screenspace position specified by NewPosition and resize it to NewSize */
	SLATECORE_API void ReshapeWindow( UE::Slate::FDeprecateVector2DParameter NewPosition, UE::Slate::FDeprecateVector2DParameter NewSize );
	SLATECORE_API void ReshapeWindow( const FSlateRect& InNewShape );
	/**
	 * Resize the window to be dpi scaled NewClientSize immediately
	 *
	 * @param NewClientSize: Client size with DPI scaling already applied that does not include border or title bars.
	 */
	SLATECORE_API void Resize( UE::Slate::FDeprecateVector2DParameter NewClientSize );

	/** Returns the rectangle of the screen the window is associated with */
	SLATECORE_API FSlateRect GetFullScreenInfo() const;

	/** @return Returns true if the window is currently morphing to a new position, shape and/or opacity */
	SLATECORE_API bool IsMorphing() const;
	/** @return Returns true if the window is currently morphing and is morphing by size */
	SLATECORE_API bool IsMorphingSize() const;
	/** Animate the window to TargetOpacity and TargetPosition over a short period of time */
	SLATECORE_API void MorphToPosition( const FCurveSequence& Sequence, const float TargetOpacity, const UE::Slate::FDeprecateVector2DParameter& TargetPosition );
	/** Animate the window to TargetOpacity and TargetShape over a short period of time */
	SLATECORE_API void MorphToShape( const FCurveSequence& Sequence, const float TargetOpacity, const FSlateRect& TargetShape );
	/** Set a new morph shape and force the morph to run for at least one frame in order to reach that target */
	SLATECORE_API void UpdateMorphTargetShape( const FSlateRect& TargetShape );
	/** Set a new morph position and force the morph to run for at least one frame in order to reach that target */
	SLATECORE_API void UpdateMorphTargetPosition( const UE::Slate::FDeprecateVector2DParameter& TargetPosition );
	/** @return Returns the currently set morph target position */
	SLATECORE_API UE::Slate::FDeprecateVector2DResult GetMorphTargetPosition() const;
	/** @return Returns the currently set morph target shape */
	SLATECORE_API FSlateRect GetMorphTargetShape() const;

	/** Flashed the window, used for drawing attention to modal dialogs */
	SLATECORE_API void FlashWindow();

	/**
	 * Attempts to draw the user's attention to this window in whatever way is
	 * appropriate for the platform if this window is not the current active
	 *
	 * @param Parameters The parameters for bouncing. Depending on the
	 *        platform, not all parameters may be supported.
	 */
	SLATECORE_API void DrawAttention(const FWindowDrawAttentionParameters& Parameters);

	/** 
	 * Bring the window to the front 
	 *
	 * @param bForce	Forces the window to the top of the Z order, even if that means stealing focus from other windows
	 *					In general do not pass force in.  It can be useful for some window types, like game windows where not forcing it to the front
	 *					would cause mouse capture and mouse lock to happen but without the window visible
	 */
	SLATECORE_API void BringToFront( bool bForce = false );

	/** @hack Force a window to front even if a different application is in front. */
	SLATECORE_API void HACK_ForceToFront();

	/** Sets the actual screen position of the window. THIS SHOULD ONLY BE CALLED BY THE OS */
	SLATECORE_API void SetCachedScreenPosition(UE::Slate::FDeprecateVector2DParameter NewPosition);

	/**	Sets the actual size of the window. THIS SHOULD ONLY BE CALLED BY THE OS */
	SLATECORE_API void SetCachedSize(UE::Slate::FDeprecateVector2DParameter NewSize);

	SLATECORE_API TSharedPtr<FGenericWindow> GetNativeWindow();
	SLATECORE_API TSharedPtr<const FGenericWindow> GetNativeWindow() const ;

	/** Returns the DPI scale factor of the native window */
	SLATECORE_API float GetDPIScaleFactor() const;

	/** Overrides the DPI scale factor of the native window */
	SLATECORE_API void SetDPIScaleFactor(const float Factor);

	/** Will inform the native window that we need to handle any DPI changes from within the application */
	SLATECORE_API void SetManualManageDPIChanges(const bool bManualDPI);

	bool IsManualManageDPIChanges() const
	{
		return bManualManageDPI;
	}

	/** 
	 * Returns whether or not this window is a descendant of the specfied parent window
	 *
	 * @param ParentWindow	The window to check
	 * @return true if the window is a child of ParentWindow, false otherwise.
	 */
	SLATECORE_API bool IsDescendantOf( const TSharedPtr<SWindow>& ParentWindow ) const;

	/**
	 * Sets the native OS window associated with this SWindow
	 *
	 * @param InNativeWindow	The native window
	 */
	SLATECORE_API void SetNativeWindow( TSharedRef<FGenericWindow> InNativeWindow );
	
	/**
	 * Sets the widget content for this window
	 *
	 * @param	InContent	The widget to use as content for this window
	 */
	SLATECORE_API void SetContent( TSharedRef<SWidget> InContent );

	/**
	 * Gets the widget content for this window
	 *
	 * @return	The widget content for this window
	 */
	SLATECORE_API TSharedRef<SWidget> GetContent();

	/**
	 * Check whether we have a full window overlay, used to draw content over the entire window.
	 * 
	 * @return true if the window has an overlay
	 */
	SLATECORE_API bool HasOverlay() const;

	/**
	 * Adds content to draw on top of the entire window
	 *
	 * @param	InZOrder	Z-order to use for this widget
	 * @return The added overlay slot so that it can be configured and populated
	 */
	SLATECORE_API SOverlay::FScopedWidgetSlotArguments AddOverlaySlot( const int32 ZOrder = INDEX_NONE );

	/**
	 * Removes a widget that is being drawn over the entire window
	 *
	 * @param	InContent	The widget to remove
	 * @return	true if successful
	 */
	SLATECORE_API bool RemoveOverlaySlot(const TSharedRef<SWidget>& InContent);

	/**
	 * Visualize a new pop-up if possible.  If it's not possible for this widget to host the pop-up
	 * content you'll get back an invalid pointer to the layer.  The returned FPopupLayer allows you 
	 * to remove the pop-up when you're done with it
	 * 
	 * @param PopupContent The widget to try and host overlaid on top of the widget.
	 *
	 * @return a valid FPopupLayer if this widget supported hosting it.  You can call Remove() on this to destroy the pop-up.
	 */
	SLATECORE_API virtual TSharedPtr<FPopupLayer> OnVisualizePopup(const TSharedRef<SWidget>& PopupContent) override;

	using FScopedWidgetSlotArguments = TPanelChildren<FPopupLayerSlot>::FScopedWidgetSlotArguments;
	/** Return a new slot in the popup layer. Assumes that the window has a popup layer. */
	SLATECORE_API FScopedWidgetSlotArguments AddPopupLayerSlot();

	/** Counterpart to AddPopupLayerSlot */
	SLATECORE_API void RemovePopupLayerSlot( const TSharedRef<SWidget>& WidgetToRemove );

	/**
	 * Sets a widget to use as a full window overlay, or clears an existing widget if set.  When set, this widget will be drawn on top of all other window content.
	 *
	 * @param	InContent	The widget to use for full window overlay content, or nullptr for no overlay
	 */
	SLATECORE_API void SetFullWindowOverlayContent( TSharedPtr<SWidget> InContent );

	/**
	 * Begins a transition from showing regular window content to overlay content
	 * During the transition we show both sets of content
	 */
	SLATECORE_API void BeginFullWindowOverlayTransition();

	/**
	 * Ends a transition from showing regular window content to overlay content
	 * When this is called content occluded by the full window overlay(if there is one) will be physically hidden
	 */
	SLATECORE_API void EndFullWindowOverlayTransition();

	/**
	 * Checks to see if there is content assigned as a full window overlay
	 *
	 * @return	True if there is an overlay widget assigned
	 */
	SLATECORE_API bool HasFullWindowOverlayContent() const;

	/** Shows or hides native window buttons on platforms that use them */
	SLATECORE_API void SetNativeWindowButtonsVisibility(bool bVisible);

	/** @return should this window show up in the taskbar */
	SLATECORE_API bool AppearsInTaskbar() const;

	/** Gets the multicast delegate executed when the window is activated */
	FOnWindowActivatedEvent& GetOnWindowActivatedEvent() { return WindowActivatedEvent; }

	/** Gets the multicast delegate executed when the window is deactivated */
	FOnWindowDeactivatedEvent& GetOnWindowDeactivatedEvent() { return WindowDeactivatedEvent; }

	/** Sets the delegate to execute right before the window is closed */
	SLATECORE_API void SetOnWindowClosed( const FOnWindowClosed& InDelegate );

	/** Gets the multicast delegate to execute right before the window is closed */
	FOnWindowClosedEvent& GetOnWindowClosedEvent() { return WindowClosedEvent; }

	/** Sets the delegate to execute right after the window has been moved */
	SLATECORE_API void SetOnWindowMoved( const FOnWindowMoved& InDelegate);

	/** Sets the delegate to override RequestDestroyWindow */
	SLATECORE_API void SetRequestDestroyWindowOverride( const FRequestDestroyWindowOverride& InDelegate );

	/** Request that this window be destroyed. The window is not destroyed immediately. Instead it is placed in a queue for destruction on next Tick */
	SLATECORE_API void RequestDestroyWindow();

	/** Warning: use Request Destroy Window whenever possible!  This method destroys the window immediately! */
	SLATECORE_API void DestroyWindowImmediately();
 
	/** Calls the OnWindowClosed delegate when this window is about to be closed */
	SLATECORE_API void NotifyWindowBeingDestroyed();

	/** Make the window visible */
	SLATECORE_API void ShowWindow();

	/** Make the window invisible */
	SLATECORE_API void HideWindow();

	/**
	 * Enables or disables this window and all of its children
	 *
	 * @param bEnable	true to enable this window and its children false to diable this window and its children
	 */
	SLATECORE_API void EnableWindow( bool bEnable );

	/** Toggle window between window modes (fullscreen, windowed, etc) */
	SLATECORE_API void SetWindowMode( EWindowMode::Type WindowMode );

	/** @return The current window mode (fullscreen, windowed, etc) */
	EWindowMode::Type GetWindowMode() const { return NativeWindow->GetWindowMode(); }

	/** @return true if the window is visible, false otherwise*/
	SLATECORE_API bool IsVisible() const;

	/** @return true if the window is maximized, false otherwise*/
	SLATECORE_API bool IsWindowMaximized() const;

	/** @return true of the window is minimized (iconic), false otherwise */
	SLATECORE_API bool IsWindowMinimized() const;

	/** Maximize the window if bInitiallyMaximized is set */
	SLATECORE_API void InitialMaximize();

	/** Maximize the window if bInitiallyMinimized is set */
	SLATECORE_API void InitialMinimize();

	/**
	 * Sets the opacity of this window
	 *
	 * @param	InOpacity	The new window opacity represented as a floating point scalar
	 */
	SLATECORE_API void SetOpacity( const float InOpacity );

	/** @return the window's current opacity */
	SLATECORE_API float GetOpacity() const;

	/** @return the level of transparency supported by this window */
	SLATECORE_API EWindowTransparency GetTransparencySupport() const;

	/** @return A String representation of the widget */
	SLATECORE_API virtual FString ToString() const override;

	/**
	 * Sets a widget that should become focused when this window is next activated
	 *
	 * @param	InWidget	The widget to set focus to when this window is activated
	 */
	void SetWidgetToFocusOnActivate( TSharedPtr< SWidget > InWidget )
	{
		WidgetToFocusOnActivate = InWidget;
	}

	/**
	 * Returns widget last focused on deactivate
	 */
	TWeakPtr<SWidget> GetWidgetFocusedOnDeactivate()
	{
		return WidgetFocusedOnDeactivate;
	}

	/** @return the window activation policy used when showing the window */
	SLATECORE_API EWindowActivationPolicy ActivationPolicy() const;

	/** @return true if the window accepts input; false if the window is non-interactive */
	SLATECORE_API bool AcceptsInput() const;

	/** If true the window will accept input if window type supports it*/
	void SetAcceptsInput(bool bEnabled) { bAcceptsInput = bEnabled; }

	/** @return true if the user decides the size of the window */
	SLATECORE_API bool IsUserSized() const;

	/** @return true if the window is sized by the windows content */
	SLATECORE_API bool IsAutosized() const;

	/** Should this window automatically derive its size based on its content or be user-drive? */
	SLATECORE_API void SetSizingRule( ESizingRule InSizingRule );

	/** @return true if this is a vanilla window, or one being used for some special purpose: e.g. tooltip or menu */
	SLATECORE_API bool IsRegularWindow() const;

	/** @return true if the window should be on top of all other windows; false otherwise */
	SLATECORE_API bool IsTopmostWindow() const;

	/** @return True if we expect the window size to change frequently. See description of bSizeWillChangeOften member variable. */
	bool SizeWillChangeOften() const
	{
		return bSizeWillChangeOften;
	}

	bool ShouldPreserveAspectRatio() const
	{
		return bShouldPreserveAspectRatio;
	}

	/** @return Returns the configured expected maximum width of the window, or INDEX_NONE if not specified.  Can be used to optimize performance for window size animation */
	int32 GetExpectedMaxWidth() const
	{
		return ExpectedMaxWidth;
	}

	/** @return Returns the configured expected maximum height of the window, or INDEX_NONE if not specified.  Can be used to optimize performance for window size animation */
	int32 GetExpectedMaxHeight() const
	{
		return ExpectedMaxHeight;
	}

	/** @return true if the window is using the os window border instead of a slate created one */
	bool HasOSWindowBorder() const { return bHasOSWindowBorder; }

	/** @return true if mouse coordinates is within this window */
	SLATECORE_API bool IsScreenspaceMouseWithin(UE::Slate::FDeprecateVector2DParameter ScreenspaceMouseCoordinate) const;

	/** @return true if this is a user-sized window with a thick edge */
	SLATECORE_API bool HasSizingFrame() const;

	/** @return true if this window has a close button/box on the titlebar area */
	SLATECORE_API bool HasCloseBox() const;

	/** @return true if this window has a maximize button/box on the titlebar area */
	SLATECORE_API bool HasMaximizeBox() const;

	/** @return true if this window has a minimize button/box on the titlebar area */
	SLATECORE_API bool HasMinimizeBox() const;

	/** Set modal window related flags - called by Slate app code during FSlateApplication::AddModalWindow() */
	void SetAsModalWindow()
	{
		bIsModalWindow = true;
		bHasMaximizeButton = false;
		bHasMinimizeButton = false;
	}

	bool IsModalWindow()
	{
		return bIsModalWindow;
	}

	/** Set mirror window flag */
	void SetMirrorWindow(bool bSetMirrorWindow)
	{
		bIsMirrorWindow = bSetMirrorWindow;
	}
	
	void SetIsHDR(bool bHDR)
	{
		bIsHDR = bHDR;
	}

	bool GetIsHDR() const { return bIsHDR; }

	void SetViewportScaleUIOverride(float InViewportScaleUIOverride)
	{
		ViewportScaleUIOverride = InViewportScaleUIOverride;
	}

	void ResetViewportScaleUIOverride()
	{
		ViewportScaleUIOverride = -1.0f;
	}

	float GetViewportScaleUIOverride() const { return ViewportScaleUIOverride; }

	bool IsVirtualWindow() const { return bVirtualWindow; }

	bool IsMirrorWindow()
	{
		return bIsMirrorWindow;
	}

	void SetTitleBar( const TSharedPtr<IWindowTitleBar> InTitleBar )
	{
		TitleBar = InTitleBar;
	}

	TSharedPtr<IWindowTitleBar> GetTitleBar() const
	{
		return TitleBar;
	}

	// Events
	SLATECORE_API virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;

	/** The system will call this method to notify the window that it has been places in the foreground or background. */
	SLATECORE_API virtual bool OnIsActiveChanged( const FWindowActivateEvent& ActivateEvent );
	
	/** Windows functions */
	SLATECORE_API void Maximize();
	SLATECORE_API void Restore();
	SLATECORE_API void Minimize();

	/** Gets the current Window Zone that mouse position is over. */
	SLATECORE_API EWindowZone::Type GetCurrentWindowZone(UE::Slate::FDeprecateVector2DParameter LocalMousePosition);

	/** Used to store the zone where the mouse down event occurred during move / drag */
	EWindowZone::Type MoveResizeZone;
	UE::Slate::FDeprecateVector2DResult MoveResizeStart;
	FSlateRect MoveResizeRect;

	/** @return Gets the radius of the corner rounding of the window. */
	SLATECORE_API int32 GetCornerRadius();

	SLATECORE_API virtual bool SupportsKeyboardFocus() const override;

	bool IsDrawingEnabled() const { return bIsDrawingEnabled; }

	virtual bool Advanced_IsWindow() const override { return true; }
	SLATECORE_API virtual bool Advanced_IsInvalidationRoot() const override;
	SLATECORE_API virtual const FSlateInvalidationRoot* Advanced_AsInvalidationRoot() const override;

#if WITH_ACCESSIBILITY
	SLATECORE_API virtual TSharedRef<FSlateAccessibleWidget> CreateAccessibleWidget() override;
	SLATECORE_API virtual TOptional<FText> GetDefaultAccessibleText(EAccessibleType AccessibleType = EAccessibleType::Main) const override;
#endif
private:
	SLATECORE_API virtual FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent ) override;
	SLATECORE_API virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATECORE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	SLATECORE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	SLATECORE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/** The window's desired size takes into account the ratio between the slate units and the pixel size */
	SLATECORE_API virtual FVector2D ComputeDesiredSize(float) const override;
	SLATECORE_API virtual bool ComputeVolatility() const override;

	/** Resize using already dpi scaled window size including borders/title bar */
	SLATECORE_API void ResizeWindowSize( FVector2f NewWindowSize );

	SLATECORE_API void OnGlobalInvalidationToggled(bool bGlobalInvalidationEnabled);
public:
	/**
	 * For a given client size, calculate the window size required to accommodate any potential non-OS borders and title bars
	 *
	 * @param InClientSize: Client size with DPI scaling already applied
	 * @param DPIScale: Scale that will be applied for border and title. When not supplied detects DPIScale using native or initial position.
	 */
	SLATECORE_API UE::Slate::FDeprecateVector2DResult GetWindowSizeFromClientSize(UE::Slate::FDeprecateVector2DParameter InClientSize, TOptional<float> DPIScale = TOptional<float>());

	/** @return true if this window will be focused when it is first shown */
	inline bool IsFocusedInitially() const
	{
		return bFocusWhenFirstShown;
	}

	/** @return the list of this window's child windows */
	SLATECORE_API const TArray< TSharedRef<SWindow> >& GetChildWindows() const;
	
	/** @return the list of this window's child windows */
	SLATECORE_API TArray< TSharedRef<SWindow> >& GetChildWindows();

	/** Add ChildWindow as this window's child */
	SLATECORE_API void AddChildWindow( const TSharedRef<SWindow>& ChildWindow );

	/** @return the parent of this window; Invalid shared pointer if this window is not a child */
	SLATECORE_API TSharedPtr<SWindow> GetParentWindow() const;

	/** Look up the parent chain until we find the top-level window that owns this window */
	SLATECORE_API TSharedPtr<SWindow> GetTopmostAncestor();

	/** Remove DescendantToRemove from this window's children or their children. */
	SLATECORE_API bool RemoveDescendantWindow( const TSharedRef<SWindow>& DescendantToRemove );

	/** Sets the delegate to call when switching worlds in before ticking,drawing, or sending messages to widgets in this window */
	SLATECORE_API void SetOnWorldSwitchHack( FOnSwitchWorldHack& InOnWorldSwitchHack );

	/**
	 * Hack to switch worlds
	 *
	 * @param WorldId: User ID for a world that should be restored or -1 if no restore
	 * @param The ID of the world restore later
	 */
	SLATECORE_API int32 SwitchWorlds( int32 WorldId ) const;

	/** Is this window active? */
	SLATECORE_API bool IsActive() const;

	/** Are any of our child windows active? */
	SLATECORE_API bool HasActiveChildren() const;

	/** Are any of our parent windows active? */
	SLATECORE_API bool HasActiveParent() const;

	/**
	 * Sets whether or not the viewport size should be driven by the window's size.  If true, the two will be the same.  If false, an independent viewport size can be specified with SetIndependentViewportSize
	 */
	inline void SetViewportSizeDrivenByWindow(bool bDrivenByWindow)
	{
		if (bDrivenByWindow)
		{
			ViewportSize = FVector2f::ZeroVector;
		}
		else
		{
			ViewportSize = Size;
		}
	}
	
	/**
	 * Returns whether or not the viewport and window size should be linked together.  If false, the two can be independent in cases where it is needed (e.g. mirror mode window drawing)
	 */
	inline bool IsViewportSizeDrivenByWindow() const
	{
		return (ViewportSize.X == 0);
	}

	/**
	 * Returns the viewport size, taking into consideration if the window size should drive the viewport size 
	 */
	inline UE::Slate::FDeprecateVector2DResult GetViewportSize() const
	{
		return (ViewportSize.X != 0) ? ViewportSize : Size;
	}
	
	/**
	 * Sets the viewport size independently of the window size, if non-zero.
	 */
	inline void SetIndependentViewportSize(const UE::Slate::FDeprecateVector2DParameter& VP) 
	{
		ViewportSize = VP;
	}

	void SetViewport(TSharedRef<ISlateViewport> ViewportRef)
	{
		Viewport = ViewportRef;
	}

	void UnsetViewport(TSharedRef<ISlateViewport> ViewportRef)
	{
		if (ensure(Viewport == ViewportRef))
		{
			Viewport.Reset();
		}
	}

	TSharedPtr<ISlateViewport> GetViewport()
	{
		return Viewport.Pin();
	}

	/**
	 * Access the hittest acceleration data structure for this window.
	 * The grid is filled out every time the window is painted.
	 *
	 * @see FHittestGrid for more details.
	 */
	SLATECORE_API FHittestGrid& GetHittestGrid();

	/** Optional constraints on min and max sizes that this window can be. */
	SLATECORE_API FWindowSizeLimits GetSizeLimits() const;

	/** Set optional constraints on min and max sizes that this window can be. */
	SLATECORE_API void SetSizeLimits(const FWindowSizeLimits& InSizeLimits);

	SLATECORE_API void SetAllowFastUpdate(bool bInAllowFastUpdate);
public:

	//~ SWidget overrides
	SLATECORE_API virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/** Windows that are not hittestable should not show up in the hittest grid. */
	SLATECORE_API EVisibility GetWindowVisibility() const;

	/** Return the tooltip text that should be used for the window's Close button. */
	SLATECORE_API TAttribute<FText> GetWindowCloseButtonToolTipText() const;
protected:
	/**Returns swindow title bar widgets. */
	SLATECORE_API virtual TSharedRef<SWidget> MakeWindowTitleBar(const TSharedRef<SWindow>& Window, const TSharedPtr<SWidget>& CenterContent, EHorizontalAlignment CenterContentAlignment);
	/**Returns the alignment type for the titlebar's title text. */
	SLATECORE_API virtual EHorizontalAlignment GetTitleAlignment();

	/** Kick off a morph to whatever the target shape happens to be. */
	SLATECORE_API void StartMorph();

	SLATECORE_API virtual bool CustomPrepass(float LayoutScaleMultiplier) override;
protected:

	/** Type of the window */
	EWindowType Type;

	/** Title of the window, displayed in the title bar as well as potentially in the task bar (Windows platform) */
	TAttribute<FText> Title;

	/** Tooltip text displayed over the window's Close button (on Windows only). */
	TAttribute<FText> CloseButtonToolTipText;

	/** When true, grabbing anywhere on the window will allow it to be dragged. */
	bool bDragAnywhere;

	/** Current opacity of the window */
	float Opacity;

	/** How to size the window */
	ESizingRule SizingRule;

	/** How to auto center the window */
	EAutoCenter AutoCenterRule;

	/** Transparency setting for this window */
	EWindowTransparency TransparencySupport;

	/** True if this window has a title bar */
	bool bCreateTitleBar : 1;

	/** True if this is a pop up window */
	bool bIsPopupWindow : 1;

	/** True if this is a topmost window */
	bool bIsTopmostWindow : 1;

	/** True if we expect the size of this window to change often, such as if its animated, or if it recycled for tool-tips,
		and we'd like to avoid costly GPU buffer resizes when that happens.  Enabling this may incur memory overhead or
		other platform-specific side effects */
	bool bSizeWillChangeOften : 1;

	/** true if this window is maximized when its created */
	bool bInitiallyMaximized : 1;

	/** true if this window is minimized when its created */
	bool bInitiallyMinimized : 1;

	/** True if this window has been shown yet */
	bool bHasEverBeenShown : 1;

	/** Focus this window immediately as it is shown */
	bool bFocusWhenFirstShown : 1;

	/** True if this window displays the os window border instead of drawing one in slate */
	bool bHasOSWindowBorder : 1;

	/** True if this window is virtual and not directly rendered by slate application or the OS. */
	bool bVirtualWindow : 1;

	/** True if this window displays an enabled close button on the toolbar area */
	bool bHasCloseButton : 1;

	/** True if this window displays an enabled minimize button on the toolbar area */
	bool bHasMinimizeButton : 1;

	/** True if this window displays an enabled maximize button on the toolbar area */
	bool bHasMaximizeButton : 1;

	/** True if this window displays thick edge that can be used to resize the window */
	bool bHasSizingFrame : 1;
	
	/** True if the window is modal */
	bool bIsModalWindow : 1;

	/** True if the window is a mirror window for HMD content */
	bool bIsMirrorWindow : 1;

	/** True if the window should preserve its aspect ratio when resized by user */
	bool bShouldPreserveAspectRatio : 1;

	/** True if the window is being displayed on a HDR capable monitor */
	bool bIsHDR : 1;

	bool bManualManageDPI : 1;

	/** True if this window allows global invalidation of its contents */
	bool bAllowFastUpdate : 1;

	/** If true, window will accept user inputs */
	bool bAcceptsInput: 1;
	
	/** When should the window be activated upon being shown */
	EWindowActivationPolicy WindowActivationPolicy;

	/** Initial desired position of the window's content in screen space */
	UE::Slate::FDeprecateVector2DResult InitialDesiredScreenPosition;

	/** Initial desired size of the window's content in screen space */
	UE::Slate::FDeprecateVector2DResult InitialDesiredSize;

	/** Position of the window's content in screen space */
	UE::Slate::FDeprecateVector2DResult ScreenPosition;

	/** The position of the window before entering fullscreen */
	UE::Slate::FDeprecateVector2DResult PreFullscreenPosition;

	/** Size of the window's content area in screen space */
	UE::Slate::FDeprecateVector2DResult Size;

	/** Size of the viewport. If (0,0) then it is equal to Size */
	UE::Slate::FDeprecateVector2DResult ViewportSize;

	/** Pointer to the viewport registered with this window if any */
	TWeakPtr<ISlateViewport> Viewport;

	/** 
	 * Mix of aspect ratio + DPI + UI Zoom. Used in shaders / materials.
	 * Can be overriden by child widgets, is reset after draw every frame.
	 * Negative value indicates not used, in which case application level UI scale will be used.
	 */
	float ViewportScaleUIOverride;

	/** Size of this window's title bar.  Can be zero.  Set at construction and should not be changed afterwards. */
	float TitleBarSize;

	/** Utility for animating the window size. */
	struct FMorpher
	{
		FMorpher()
			: StartingMorphShape( FSlateRect(0,0,100,100) )
			, TargetMorphShape( FSlateRect(0,0,100,100) )
			, bIsActive(false)
			, bIsAnimatingWindowSize(false)
		{ }

		/** Initial window opacity */
		float StartingOpacity;
		/** Desired opacity of the window */
		float TargetOpacity;

		/** Initial size of the window (i.e. at the start of animation) */
		FSlateRect StartingMorphShape;
		/** Desired size of the window (i.e. at the end of the animation) */
		FSlateRect TargetMorphShape;
		
		/** Animation sequence to hold on to the Handle */
		FCurveSequence Sequence;

		/** True if this morph is currently active */
		bool bIsActive : 1;

		/** True if we're morphing size as well as position.  False if we're just morphing position */
		bool bIsAnimatingWindowSize : 1;

	} Morpher;

	/** Cached "zone" the cursor was over in the window the last time that someone called GetCurrentWindowZone() */
	EWindowZone::Type WindowZone;
	

	TSharedPtr<SWidget> TitleArea;
	SVerticalBox::FSlot* ContentSlot;

	/** Widget to transfer keyboard focus to when this window becomes active, if any.  This is used to
		restore focus to a widget after a popup has been dismissed. */
	TWeakPtr< SWidget > WidgetToFocusOnActivate;

	/** Widget that had keyboard focus when this window was last de-activated, if any.  This is used to
		restore focus to a widget after the window regains focus. */
	TWeakPtr< SWidget > WidgetFocusedOnDeactivate;

private:
	/** Style used to draw this window */
	const FWindowStyle* Style;

	const FSlateBrush* WindowBackground;

	TSharedPtr<SImage> WindowBackgroundImage;
	TSharedPtr<SImage> WindowBorder;
	TSharedPtr<SImage> WindowOutline;
	TSharedPtr<SWidget> ContentAreaVBox;
	EVisibility WindowContentVisibility;
protected:

	/** Min and Max values for Width and Height; all optional. */
	FWindowSizeLimits SizeLimits;

	/** The native window that is backing this Slate Window */
	TSharedPtr<FGenericWindow> NativeWindow;

	/** Each window has its own hittest grid for accelerated widget picking. */
	TUniquePtr<FHittestGrid> HittestGrid;
	
	/** Invoked when the window has been activated. */
	FOnWindowActivated OnWindowActivated;
	FOnWindowActivatedEvent WindowActivatedEvent;

	/** Invoked when the window has been deactivated. */
	FOnWindowDeactivated OnWindowDeactivated;
	FOnWindowDeactivatedEvent WindowDeactivatedEvent;

	/** Invoked when the window is about to be closed. */
	FOnWindowClosed OnWindowClosed;
	FOnWindowClosedEvent WindowClosedEvent;

	/** Invoked when the window is moved */
	FOnWindowMoved OnWindowMoved;

	/** Invoked when the window is requested to be destroyed. */
	FRequestDestroyWindowOverride RequestDestroyWindowOverride;

	/** Window overlay widget */
	TSharedPtr<SOverlay> WindowOverlay;	
	
	/**
	 * This layer provides mechanism for tooltips, drag-drop
	 * decorators, and popups without creating a new window.
	 */
	TSharedPtr<class SPopupLayer> PopupLayer;

	/** Full window overlay widget */
	TSharedPtr<SWidget> FullWindowOverlayWidget;

	/** When not null, this window will always appear on top of the parent and be closed when the parent is closed. */
	TWeakPtr<SWindow> ParentWindowPtr;

	/** Child windows of this window */
	TArray< TSharedRef<SWindow> > ChildWindows;
	
	/** World switch delegate */
	FOnSwitchWorldHack OnWorldSwitchHack;

	/** 
	 * Whether or not we should show content of the window which could be occluded by full screen window content. 
	 * This is used to hide content when there is a full screen overlay occluding it
	 */
	bool bShouldShowWindowContentDuringOverlay;

	/** The expected maximum width of the window.  May be used for performance optimization when bSizeWillChangeOften is set. */
	int32 ExpectedMaxWidth;

	/** The expected maximum height of the window.  May be used for performance optimization when bSizeWillChangeOften is set. */
	int32 ExpectedMaxHeight;

	/** The index of the user that should focus this window on activation. INDEX_NONE for all users */
	int32 FocusUserIndex;

	// The window title bar.
	TSharedPtr<IWindowTitleBar> TitleBar;

	// The padding for between the edges of the window and it's content
	FMargin LayoutBorder;

	// The margin around the edges of the window that will be detected as places the user can grab to resize the window. 
	FMargin UserResizeBorder;

	// Whether or not drawing is enabled for this window
	bool bIsDrawingEnabled;

protected:
	
	SLATECORE_API void ConstructWindowInternals();

	/** One-off active timer to trigger a the morph sequence to play */
	SLATECORE_API EActiveTimerReturnType TriggerPlayMorphSequence( double InCurrentTime, float InDeltaTime );

	SLATECORE_API void SetWindowBackground(const FSlateBrush* InWindowBackground);

	SLATECORE_API void UpdateWindowContentVisibility();

	//~ FSlateInvalidationRoot overrides
	SLATECORE_API virtual TSharedRef<SWidget> GetRootWidget() override;
	SLATECORE_API virtual int32 PaintSlowPath(const FSlateInvalidationContext& InvalidationContext) override;

public:

	/** Process the invalidation of the widget contained by the window in GlobalInvalidation. */
	SLATECORE_API void ProcessWindowInvalidation();

private:

	/** The handle to the active timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;
};

#if WITH_EDITOR

/**
 * Hack to switch worlds in a scope and switch back when we fall out of scope                   
 */
struct FScopedSwitchWorldHack
{
	SLATECORE_API FScopedSwitchWorldHack( const FWidgetPath& WidgetPath );

	FScopedSwitchWorldHack( TSharedPtr<SWindow> InWindow )
		: Window( InWindow )
		, WorldId( -1 )
	{
		if( Window.IsValid() )
		{
			WorldId = Window->SwitchWorlds( WorldId );
		}
	}

	~FScopedSwitchWorldHack()
	{
		if( Window.IsValid() )
		{
			Window->SwitchWorlds( WorldId );
		}
	}

private:

	// The window to switch worlds for.
	TSharedPtr<SWindow> Window;

	// The worldID serves as identification to the user about the world.  It can be anything although -1 is assumed to be always invalid.
	int32 WorldId;
};

#else

struct FScopedSwitchWorldHack
{
	inline FScopedSwitchWorldHack(const FWidgetPath& WidgetPath) { }
	inline FScopedSwitchWorldHack(TSharedPtr<SWindow> InWindow) { }
	inline ~FScopedSwitchWorldHack() = default;
};

#endif
