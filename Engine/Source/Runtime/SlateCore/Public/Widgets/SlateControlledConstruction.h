// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateGlobals.h"
#include "Types/SlateAttributeDescriptor.h"
#include "Templates/Identity.h"

enum class ESPMode : uint8;
class FSlateControlledConstruction;
namespace SharedPointerInternals
{
	template <typename ObjectType, ESPMode Mode>
	class TIntrusiveReferenceController;
}

/** */
#define SLATE_DECLARE_WIDGET(WidgetType, ParentType) \
	SLATE_DECLARE_WIDGET_API(WidgetType, ParentType, NO_API)

#define SLATE_DECLARE_WIDGET_API(WidgetType, ParentType, ModuleApiDefine) \
	private: \
		using Super = ParentType; \
		using ThisClass = WidgetType; \
		using PrivateThisType = WidgetType; \
		using PrivateParentType = ParentType; \
		template<class WidgetType, bool bIsUserWidget> \
		friend struct TWidgetAllocator; \
		template <typename ObjectType, ESPMode Mode> \
		friend class SharedPointerInternals::TIntrusiveReferenceController; /* Shouldn't really forward-declare this to allow MakeShared to work, but is for legacy reasons */ \
		static const FSlateWidgetClassData& GetPrivateWidgetClass() \
		{ \
			static FSlateWidgetClassData WidgetClassDataInstance = FSlateWidgetClassData(TIdentity<ParentType>(), #WidgetType, &WidgetType::PrivateRegisterAttributes); \
			return WidgetClassDataInstance; \
		} \
		static ModuleApiDefine void PrivateRegisterAttributes(FSlateAttributeInitializer&); \
	public: \
		static const FSlateWidgetClassData& StaticWidgetClass() { return GetPrivateWidgetClass(); } \
		virtual const FSlateWidgetClassData& GetWidgetClass() const override { return GetPrivateWidgetClass(); } \
	private:


#define SLATE_IMPLEMENT_WIDGET(WidgetType) \
	FSlateWidgetClassRegistration ClassRegistration__##WidgetType = WidgetType::StaticWidgetClass();

/** */
class FSlateWidgetClassData
{
private:
	friend FSlateControlledConstruction;

	FSlateWidgetClassData(FName InWidgetTypeName)
		: WidgetType(InWidgetTypeName)
	{}

public:
	template<typename InWidgetParentType>
	FSlateWidgetClassData(TIdentity<InWidgetParentType>, FName InWidgetTypeName, void(*AttributeInitializer)(FSlateAttributeInitializer&))
		: WidgetType(InWidgetTypeName)
	{
		// Initialize the parent class if it's not already done
		const FSlateWidgetClassData& ParentWidgetClassData = InWidgetParentType::StaticWidgetClass();
		// Initialize the attribute descriptor
		FSlateAttributeInitializer Initializer = {Descriptor, ParentWidgetClassData.GetAttributeDescriptor()};
		(*AttributeInitializer)(Initializer);
	}

	const FSlateAttributeDescriptor& GetAttributeDescriptor() const { return Descriptor; };
	FName GetWidgetType() const { return WidgetType; }

private:
	FSlateAttributeDescriptor Descriptor;
	FName WidgetType;
};

/** */
struct FSlateWidgetClassRegistration
{
	FSlateWidgetClassRegistration(const FSlateWidgetClassData&) {}
};

/** */
class FSlateControlledConstruction
{
public:
	FSlateControlledConstruction() = default;
	virtual ~FSlateControlledConstruction() = default;

public:
	static const FSlateWidgetClassData& StaticWidgetClass()
	{
		static FSlateWidgetClassData Instance = FSlateWidgetClassData(TEXT("FSlateControlledConstruction"));
		return Instance;
	}
	virtual const FSlateWidgetClassData& GetWidgetClass() const = 0;
	
private:
	/** UI objects cannot be copy-constructed */
	FSlateControlledConstruction(const FSlateControlledConstruction& Other) = delete;
	
	/** UI objects cannot be copied. */
	void operator= (const FSlateControlledConstruction& Other) = delete;

	/** Widgets should only ever be constructed via SNew or SAssignNew */
	void* operator new ( const size_t InSize )
	{
		return FMemory::Malloc(InSize);
	}

	/** Widgets should only ever be constructed via SNew or SAssignNew */
	void* operator new ( const size_t InSize, void* Addr )
	{
		return Addr;
	}

	template<class WidgetType, bool bIsUserWidget>
	friend struct TWidgetAllocator;

	/* Shouldn't really forward-declare this to allow MakeShared to work, but is for legacy reasons */
	template <typename ObjectType, ESPMode Mode>
	friend class SharedPointerInternals::TIntrusiveReferenceController;

public:
	void operator delete(void* mem)
	{
		FMemory::Free(mem);
	}
};
