// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "WidgetPixelSnapping.generated.h"

/**
 * The different states of pixel snapping a widget can be in.
 */
UENUM()
enum class EWidgetPixelSnapping : uint8
{
	/** Inherits the snapping method set by the parent widget. */
	Inherit,

	/** Draws the widget without snapping. Useful during animations or moving indicators. */
	Disabled,

	/** Draws the widget at the nearest pixel. Improves sharpness of widgets. */
	SnapToPixel
};
