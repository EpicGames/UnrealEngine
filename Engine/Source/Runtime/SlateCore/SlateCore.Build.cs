// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SlateCore : ModuleRules
{
	public SlateCore(ReadOnlyTargetRules Target) : base(Target)
	{
		StaticAnalyzerDisabledCheckers.Add("core.uninitialized.ArraySubscript");

		PrivateDefinitions.Add("UE_REPORT_SLATE_VECTOR_DEPRECATION=1");

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"InputCore",
				"Json",
				"TraceLog",
			});

		PrivateDependencyModuleNames.AddRange([
			"ImageCore",
		]);

		if (Target.bCompileAgainstApplicationCore)
		{
			PublicDependencyModuleNames.Add("ApplicationCore");
		}

		if (Target.Type != TargetType.Server)
		{
			if (Target.bCompileFreeType)
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "FreeType2");
				PublicDefinitions.Add("WITH_FREETYPE=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_FREETYPE=0");
			}

			if (Target.bCompileICU)
			{
				AddEngineThirdPartyPrivateStaticDependencies(Target, "ICU");
			}

			AddEngineThirdPartyPrivateStaticDependencies(Target, "HarfBuzz");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "Nanosvg");
		}
		else
		{
			PublicDefinitions.Add("WITH_FREETYPE=0");
			PublicDefinitions.Add("WITH_HARFBUZZ=0");
		}

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "XInput");
		}
	}
}
